// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

extern RENDERCORE_API void SuspendRenderThreadTimeout();
extern RENDERCORE_API void ResumeRenderThreadTimeout();
