// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR

#include "Async/Mutex.h"
#include "Templates/SharedPointer.h"
#include "Templates/UniquePtr.h"

enum EBloomMethod : int; // from Engine/Scene.h

/** 
  * Feedback from the renderer to the viewport, gathered while rendering. 
  * Used to communicate lightweight renderer state to the user in the editor.
  */
class FRenderViewportFeedback
{
public:
	// Bloom method that was used during rendering.
	EBloomMethod BloomMethod;
};

namespace UE::RenderViewportFeedback
{

class FCollector;

/** 
  * Datastructure that owns FRenderViewportFeedback on the gamethread side
  * and facilitates threadsafe updating from the renderer.
  */
class FReceiver : public TSharedFromThis<FReceiver, ESPMode::ThreadSafe>
{
	friend class FCollector;

public:
	RENDERCORE_API FReceiver();

	// Call from game thread only
	// Returns a copy, as the internal raw data may be updated at any time.
	RENDERCORE_API FRenderViewportFeedback Data() const;

	RENDERCORE_API TSharedPtr<FCollector> MakeCollector();

private:
	FRenderViewportFeedback InternalData;
	mutable UE::FMutex Mutex;
};

/** Helper class used on the render thread to gather data for the viewport. */
class FCollector
{
	friend class FReceiver;

public:
	FCollector(FReceiver& InReceiver);

	// Modify from render thread only
	FRenderViewportFeedback& Data()
	{
		return InternalData;
	}
	
	// Data collection finished, push the data to the receiver.
	RENDERCORE_API void EndFrameRenderThread();

private:
	// Weak pointer for in case the viewport is destroyed while the renderer is still active.
	TWeakPtr<FReceiver> Receiver;
	FRenderViewportFeedback InternalData;
};

};

#endif //WITH_EDITOR
