// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ShaderCompilerJobTypes.h"
#include "Containers/Array.h"

class FArchive;
template <typename FuncType> class TFunctionRef;

struct FSCWErrorCode
{
	enum ECode : int32
	{
		NotSet = -1,
		Success,
		GeneralCrash,
		BadShaderFormatVersion,
		BadInputVersion,
		BadSingleJobHeader,
		BadPipelineJobHeader,
		CantDeleteInputFile,
		CantSaveOutputFile,
		NoTargetShaderFormatsFound,
		CantCompileForSpecificFormat,
		CrashInsidePlatformCompiler,
		BadInputFile,
		MissingVirtualInputFile,
		OutOfMemory,
	};

	/**
	Sets the global SCW error code if it hasn't been set before.
	Call Reset first before setting a new value.
	Returns true on success, otherwise the error code has already been set.
	*/
	static RENDERCORE_API void Report(ECode Code, const FStringView& Info = {});

	/** Resets the global SCW error code to NotSet. */
	static RENDERCORE_API void Reset();

	/** Returns the global SCW error code. */
	static RENDERCORE_API ECode Get();

	/** Returns the global SCW error code information string. Empty string if not set. */
	static RENDERCORE_API const FString& GetInfo();

	/** Returns true if the SCW global error code has been set. Equivalent to 'Get() != NotSet'. */
	static RENDERCORE_API bool IsSet();
};

class FShaderCompileWorkerUtil
{
public:
	enum class EWriteTasksFlags : uint8
	{
		None = 0,
		CompressTaskFile = 1 << 0,
		SkipSource = 1 << 1,
	};

	enum class EReadTasksFlags : uint8
	{
		None = 0,
		WillRetry = 1 << 0,
	};

	RENDERCORE_API static void LogQueuedCompileJobs(const TArray<FShaderCommonCompileJobPtr>& QueuedJobs, int32 NumProcessedJobs);
	RENDERCORE_API static void DumpDebugCompileInput(FShaderCommonCompileJob& Job, FShaderDebugDataContext& Ctx);
	RENDERCORE_API static bool WriteTasks(const TArray<FShaderCommonCompileJobPtr>& QueuedJobs, FArchive& TransferFile, EWriteTasksFlags Flags = EWriteTasksFlags::None, TArray<FMemoryView>* OutCompressedSource = nullptr);
	RENDERCORE_API static FSCWErrorCode::ECode ReadTasks(const TArray<FShaderCommonCompileJobPtr>& QueuedJobs, FArchive& OutputFile, FShaderCompileWorkerDiagnostics* OutWorkerDiagnostics, EReadTasksFlags Flags = EReadTasksFlags::None);

	[[nodiscard]] RENDERCORE_API static const TArray<const IShaderFormat*>& GetShaderFormats();

	RENDERCORE_API static void ProcessInputFromArchive(
		FArchive* InputFilePtr,
		const FString& DumpDebugInfoPathOverride,
		TConstArrayView<FString> DebugSourceFiles,
		TFunctionRef<TArray<uint8> (int32)> LoadVirtualSourceFile,
		int32& OutNumProcessedJobs,
		TArray<FShaderCompileJob>& OutSingleJobs,
		TArray<FShaderPipelineCompileJob>& OutPipelineJobs,
		TArray<FString>& OutPipelineNames);

	RENDERCORE_API static void WriteToOutputArchive(
		FArchive& OutputFile,
		const FString& HostName,
		FShaderCompileWorkerDiagnostics& WorkerDiagnostics,
		int32 NumProcessedJobs,
		TArray<FShaderCompileJob>& SingleJobs,
		TArray<FShaderPipelineCompileJob>& PipelineJobs,
		TArray<FString>& PipelineNames);

	RENDERCORE_API static int64 WriteOutputFileHeader(
		FArchive& OutputFile,
		const FString& HostName,
		FShaderCompileWorkerDiagnostics& WorkerDiagnostics,
		int32 NumProcessedJobs,
		int32 CallstackLength,
		const TCHAR* Callstack,
		int32 ExceptionInfoLength,
		const TCHAR* ExceptionInfo);

	RENDERCORE_API static void UpdateFileSize(FArchive& OutputFile, int64 FileSizePosition);

private:
	static bool HandleWorkerCrash(
		const TArray<FShaderCommonCompileJobPtr>& QueuedJobs,
		FArchive& OutputFile,
		int32 OutputVersion,
		int64 FileSize,
		FSCWErrorCode::ECode ErrorCode,
		int32 NumProcessedJobs,
		int32 CallstackLength,
		int32 ExceptionInfoLength,
		int32 HostnameLength,
		bool bWillRetry);
};

ENUM_CLASS_FLAGS(FShaderCompileWorkerUtil::EWriteTasksFlags);
ENUM_CLASS_FLAGS(FShaderCompileWorkerUtil::EReadTasksFlags);
