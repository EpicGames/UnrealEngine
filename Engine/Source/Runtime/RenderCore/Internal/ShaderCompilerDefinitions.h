// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Misc/CoreMiscDefines.h"
#include "UObject/NameTypes.h"
#include "ShaderCore.h"
#include "ShaderParameterMetadata.h"

#define SHADER_COMPILER_FLOAT32_FORMAT_STRING TEXT("%#.9gf")

enum class EShaderCompilerDefineVariant : uint8
{
	None = 0,
	Integer,
	Unsigned,
	Float,
	String
};

/** Container for shader compiler definitions. */
class FShaderCompilerDefinitions
{
public:
	RENDERCORE_API FShaderCompilerDefinitions(bool bIncludeInitialDefines = false);
	RENDERCORE_API FShaderCompilerDefinitions(const FShaderCompilerDefinitions&);

	/** Value types supported:   bool, int32, uint32, float, const TCHAR*, FString& */
	template <typename ValueType>
	void SetDefine(FName Name, ValueType Value) { InternalSetValue(FindOrAddMapIndex(Name), Value); }
	void SetDefine(FName Name, const TCHAR* Value) { InternalSetValue(FindOrAddMapIndex(Name), Value); }
	void SetDefine(FName Name, const FString& Value) { InternalSetValue(FindOrAddMapIndex(Name), *Value); }

	template <typename ValueType>
	void SetDefine(FShaderCompilerDefineNameCache& Name, ValueType Value) { InternalSetValue(FindOrAddMapIndex(Name), Value); }
	void SetDefine(FShaderCompilerDefineNameCache& Name, const TCHAR* Value) { InternalSetValue(FindOrAddMapIndex(Name), Value); }
	void SetDefine(FShaderCompilerDefineNameCache& Name, const FString& Value) { InternalSetValue(FindOrAddMapIndex(Name), *Value); }

	int32 GetIntegerValue(FName Name) const
	{
		int32 Result = 0;
		int32 KeyIndex = FindMapIndex(Name, GetTypeHash(Name));
		if (KeyIndex != INDEX_NONE)
		{
			// For None, Integer, or Unsigned, return ValueInteger (None will have a default of zero)
			if (ValueTypes[KeyIndex] <= EShaderCompilerDefineVariant::Unsigned)
			{
				Result = Pairs[KeyIndex].ValueInteger;
			}
			else if (ValueTypes[KeyIndex] == EShaderCompilerDefineVariant::String)
			{
				Result = FCString::Atoi(*StringValues[Pairs[KeyIndex].ValueInteger]);
			}
			else
			{
				// EShaderCompilerDefineVariant::Float
				Result = (int32)Pairs[KeyIndex].ValueFloat;
			}
		}
		return Result;
	}

	int32 GetIntegerValue(FShaderCompilerDefineNameCache& NameCache, int32 ResultIfNotFound) const
	{
		int32 Result = ResultIfNotFound;
		int32 KeyIndex;

		// Check if this is an initial define, meaning it has a fixed map index.  If Name.MapIndex is INDEX_NONE
		// (MapIndex not initialized yet) or InitialDefineCount is zero (FShaderCompilerDefinitions constructed without
		// initial state), this condition will be false, and continue to the rest of the function.
		if ((uint32)NameCache.MapIndex < InitialDefineCount)
		{
			KeyIndex = NameCache.MapIndex;
		}
		else
		{
			KeyIndex = FindMapIndex(NameCache.Name, GetTypeHash(NameCache.Name));
		}

		if (KeyIndex != INDEX_NONE)
		{
			// Initialize MapIndex if necessary.  If the define is not an initial define, its index will be greater than
			// or equal to InitialDefineCount, indicating that it doesn't have a fixed map index.
			if (InitialDefineCount && NameCache.MapIndex == INDEX_NONE)
			{
				NameCache.MapIndex = KeyIndex;
			}

			EShaderCompilerDefineVariant ValueType = ValueTypes[KeyIndex];
			if (ValueType == EShaderCompilerDefineVariant::Integer ||
				ValueType == EShaderCompilerDefineVariant::Unsigned)
			{
				Result = Pairs[KeyIndex].ValueInteger;
			}
			else if (ValueType == EShaderCompilerDefineVariant::String)
			{
				Result = FCString::Atoi(*StringValues[Pairs[KeyIndex].ValueInteger]);
			}
			else if (ValueType == EShaderCompilerDefineVariant::Float)
			{
				// EShaderCompilerDefineVariant::Float
				Result = (int32)Pairs[KeyIndex].ValueFloat;
			}
		}
		return Result;
	}

	bool Contains(FName Name) const
	{
		int32 KeyIndex = FindMapIndex(Name, GetTypeHash(Name));
		return KeyIndex != INDEX_NONE && ValueTypes[KeyIndex] != EShaderCompilerDefineVariant::None;
	}

	FORCEINLINE int32 Num() const
	{
		return ValueCount;
	}

	void Empty()
	{
		if (ValueCount)
		{
			KeyHashTable.Free();
			Pairs.Empty();
			ValueTypes.Empty();
			StringValues.Empty();
			ValueCount = 0;

			// Re-copy initial defines if originally constructed with initial defines
			if (InitialDefineCount)
			{
				*this = *GInitialDefines;
			}
		}
	}

	void Merge(const FShaderCompilerDefinitions& Other)
	{
		for (FConstIterator OtherIt(Other); OtherIt; ++OtherIt)
		{
			int32 OtherIndex = OtherIt.GetIndex();
			switch (Other.ValueTypes[OtherIndex])
			{
			case EShaderCompilerDefineVariant::Integer:
				SetDefine(OtherIt.KeyFName(), Other.Pairs[OtherIndex].ValueInteger);
				break;
			case EShaderCompilerDefineVariant::Unsigned:
				SetDefine(OtherIt.KeyFName(), Other.Pairs[OtherIndex].ValueUnsigned);
				break;
			case EShaderCompilerDefineVariant::Float:
				SetDefine(OtherIt.KeyFName(), Other.Pairs[OtherIndex].ValueFloat);
				break;
			case EShaderCompilerDefineVariant::String:
				SetDefine(OtherIt.KeyFName(), OtherIt.Value());
				break;
			}
		}
	}

	FShaderCompilerDefinitions& operator=(const FShaderCompilerDefinitions& Other)
	{
		KeyHashTable = Other.KeyHashTable;
		Pairs = Other.Pairs;
		ValueTypes = Other.ValueTypes;
		StringValues = Other.StringValues;
		InitialDefineCount = Other.InitialDefineCount;
		ValueCount = Other.ValueCount;
		return *this;
	}

	friend FArchive& operator<<(FArchive& Ar, FShaderCompilerDefinitions& Defs)
	{
		if (Ar.IsSaving())
		{
			// Only write set values in the map
			Ar << Defs.ValueCount;
			for (FShaderCompilerDefinitions::FConstIterator DefineIt(Defs); DefineIt; ++DefineIt)
			{
				int32 Index = DefineIt.GetIndex();
				Ar << Defs.Pairs[Index];
				Ar << Defs.ValueTypes[Index];
			}
			Ar << Defs.StringValues;
		}
		else if (Ar.IsLoading())
		{
			Ar << Defs.ValueCount;
			for (uint32 ValueIndex = 0; ValueIndex < Defs.ValueCount; ValueIndex++)
			{
				FShaderCompilerDefinitions::FPairType Pair;
				Ar << Pair;
				int32 Index = Defs.FindOrAddMapIndex(Pair.Key);
				Defs.Pairs[Index] = Pair;
				Ar << Defs.ValueTypes[Index];
			}
			Ar << Defs.StringValues;
		}
		return Ar;
	}

	class FConstIterator
	{
	public:
		FORCEINLINE FConstIterator(const FShaderCompilerDefinitions& InDefines)
			: Defines(InDefines), Index(-1)
		{
			// NULL terminate these strings to start
			KeyStringBuffer[0] = 0;
			ValueStringBuffer[0] = 0;

			// Index set to -1 above, advance to first valid element
			++(*this);
		}

		/** conversion to "bool" returning true if the iterator is valid. */
		FORCEINLINE explicit operator bool() const
		{
			return Index < Defines.Pairs.Num();
		}
		/** inverse of the "bool" operator */
		FORCEINLINE bool operator !() const
		{
			return !(bool)*this;
		}

		FORCEINLINE FConstIterator& operator++()
		{
			Index++;

			// Skip over None values in the map
			int32 PairNum = Defines.Pairs.Num();
			while (Index < PairNum && Defines.ValueTypes[Index] == EShaderCompilerDefineVariant::None)
			{
				Index++;
			}
			return *this;
		}

		// Note that the output of Key() is transient, only valid until the iterator is incremented!
		FORCEINLINE const TCHAR* Key()
		{
			Defines.Pairs[Index].Key.ToString(KeyStringBuffer);
			return KeyStringBuffer;
		}

		FORCEINLINE const FName& KeyFName() const
		{
			return Defines.Pairs[Index].Key;
		}

		// Note that the output of Value() is transient, only valid until the iterator is incremented!
		FORCEINLINE const TCHAR* Value()
		{
			const TCHAR* Result;

			if (Defines.ValueTypes[Index] == EShaderCompilerDefineVariant::Integer)
			{
				int32 ValueInteger = Defines.Pairs[Index].ValueInteger;
				if (ValueInteger >= 0 && ValueInteger <= 9)
				{
					ValueStringBuffer[0] = (TCHAR)ValueInteger + '0';
					ValueStringBuffer[1] = 0;
				}
				else
				{
					FCString::Sprintf(ValueStringBuffer, TEXT("%d"), ValueInteger);
				}
				Result = ValueStringBuffer;
			}
			else if (Defines.ValueTypes[Index] == EShaderCompilerDefineVariant::Unsigned)
			{
				uint32 ValueUnsigned = Defines.Pairs[Index].ValueUnsigned;
				if (ValueUnsigned >= 0 && ValueUnsigned <= 9)
				{
					ValueStringBuffer[0] = (TCHAR)ValueUnsigned + '0';
					ValueStringBuffer[1] = 0;
				}
				else
				{
					FCString::Sprintf(ValueStringBuffer, TEXT("%u"), ValueUnsigned);
				}
				Result = ValueStringBuffer;
			}
			else if (Defines.ValueTypes[Index] == EShaderCompilerDefineVariant::Float)
			{
				// Make sure the printed value perfectly matches the given number
				FCString::Sprintf(ValueStringBuffer, SHADER_COMPILER_FLOAT32_FORMAT_STRING, Defines.Pairs[Index].ValueFloat);
				Result = ValueStringBuffer;
			}
			else
			{
				check(Defines.ValueTypes[Index] == EShaderCompilerDefineVariant::String);
				Result = *Defines.StringValues[Defines.Pairs[Index].ValueInteger];
			}

			return Result;
		}

		FORCEINLINE int32 GetIndex() const
		{
			return Index;
		}

	private:
		const FShaderCompilerDefinitions& Defines;
		int32 Index;
		TCHAR KeyStringBuffer[FName::StringBufferSize];
		TCHAR ValueStringBuffer[32];
	};


private:
	int32 FindMapIndex(FName Key, uint32 KeyHash) const
	{
		for (uint32 KeyIndex = KeyHashTable.First(KeyHash); KeyHashTable.IsValid(KeyIndex); KeyIndex = KeyHashTable.Next(KeyIndex))
		{
			if (Pairs[KeyIndex].Key == Key)
			{
				return KeyIndex;
			}
		}
		return INDEX_NONE;
	}

	int32 FindOrAddMapIndex(FName Name)
	{
		uint32 KeyHash = GetTypeHash(Name);
		int32 KeyIndex = FindMapIndex(Name, KeyHash);
		if (KeyIndex == INDEX_NONE)
		{
			KeyIndex = Pairs.Add(FPairType({ Name, {0} }));
			ValueTypes.Add(EShaderCompilerDefineVariant::None);
			KeyHashTable.Add(KeyHash, KeyIndex);
		}
		return KeyIndex;
	}

	int32 FindOrAddMapIndex(FShaderCompilerDefineNameCache& NameCache)
	{
		// Check if this is an initial define, meaning it has a fixed map index.  If Name.MapIndex is INDEX_NONE
		// (MapIndex not initialized yet) or InitialDefineCount is zero (FShaderCompilerDefinitions constructed without
		// initial state), this condition will be false, and continue to the rest of the function.
		if ((uint32)NameCache.MapIndex < InitialDefineCount)
		{
			return NameCache.MapIndex;
		}

		uint32 KeyHash = GetTypeHash(NameCache.Name);
		int32 KeyIndex = FindMapIndex(NameCache.Name, KeyHash);
		if (KeyIndex == INDEX_NONE)
		{
			KeyIndex = Pairs.Add(FPairType({ NameCache.Name, {0} }));
			ValueTypes.Add(EShaderCompilerDefineVariant::None);
			KeyHashTable.Add(KeyHash, KeyIndex);
		}

		// Initialize MapIndex if necessary.
		if (InitialDefineCount && NameCache.MapIndex == INDEX_NONE)
		{
			NameCache.MapIndex = KeyIndex;
		}
		return KeyIndex;
	}

	void InternalSetValue(int32 Index, const TCHAR* Value)
	{
		if (Value[0] >= '0' && Value[0] <= '9' && Value[1] == 0)
		{
			// If the string is a single digit integer, treat it as an integer.  Most usages of string define values are cases where
			// clients inadvertently pass bools as text, like TEXT("0") or TEXT("1"), and we can trivially optimize that.
			SetValueType(Index, EShaderCompilerDefineVariant::Integer);
			Pairs[Index].ValueInteger = Value[0] - '0';
		}
		else if (ValueTypes[Index] == EShaderCompilerDefineVariant::String)
		{
			// Value was already a string variant, overwrite it with new value, rather than allocating new string
			StringValues[Pairs[Index].ValueInteger] = Value;
		}
		else
		{
			// Set to string variant type, allocate and fill in the string
			SetValueType(Index, EShaderCompilerDefineVariant::String);
			Pairs[Index].ValueInteger = StringValues.Add(Value);
		}
	}

	void InternalSetValue(int32 Index, bool Value)
	{
		SetValueType(Index, EShaderCompilerDefineVariant::Integer);
		Pairs[Index].ValueInteger = Value ? 1 : 0;
	}

	void InternalSetValue(int32 Index, int32 Value)
	{
		SetValueType(Index, EShaderCompilerDefineVariant::Integer);
		Pairs[Index].ValueInteger = Value;
	}

	void InternalSetValue(int32 Index, uint32 Value)
	{
		SetValueType(Index, EShaderCompilerDefineVariant::Unsigned);
		Pairs[Index].ValueUnsigned = Value;
	}

	void InternalSetValue(int32 Index, float Value)
	{
		SetValueType(Index, EShaderCompilerDefineVariant::Float);
		Pairs[Index].ValueFloat = Value;
	}

	FORCEINLINE void SetValueType(int32 Index, EShaderCompilerDefineVariant InValueType)
	{
		EShaderCompilerDefineVariant& ValueType = ValueTypes[Index];
		if (ValueType == EShaderCompilerDefineVariant::None)
		{
			ValueCount++;
		}
		ValueType = InValueType;
	}

	/** Called from FShaderInitialDefinesInitializer */
	RENDERCORE_API static void InitializeInitialDefines(const FShaderCompilerDefinitions& InDefines);

	struct FPairType
	{
		FName Key;
		union
		{
			int32 ValueInteger;
			uint32 ValueUnsigned;
			float ValueFloat;
		};

		FORCEINLINE friend FArchive& operator<<(FArchive& Ar, FPairType& Pair)
		{
			// Avoid dynamic allocation when serializing keys.  We need to serialize the FName as a string, because the index in the FName is
			// non-deterministic (serialization generates hash values that need to be deterministic) and process specific (data needs to be
			// serialized to worker processes).  Compiler defines must be ANSI text, so serialize as such to save memory.
			TCHAR KeyBuffer[FName::StringBufferSize];
			ANSICHAR KeyBufferAnsi[FName::StringBufferSize];
			uint32 StringLength;

			if (Ar.IsLoading())
			{
				Ar << StringLength;
				checkf(StringLength < FName::StringBufferSize, TEXT("Trying to deserialize FName with length %d; buffer size is %d"), StringLength, FName::StringBufferSize);
				Ar.Serialize(KeyBufferAnsi, StringLength + 1);
				Pair.Key = FName(KeyBufferAnsi);
			}
			else
			{
				StringLength = Pair.Key.ToString(KeyBuffer);
				checkf(StringLength < FName::StringBufferSize, TEXT("FName %s length (%d) exceeds buffer size (%d)"), *Pair.Key.ToString(), StringLength, FName::StringBufferSize);
				for (uint32 CharIndex = 0; CharIndex < StringLength + 1; CharIndex++)
				{
					KeyBufferAnsi[CharIndex] = (ANSICHAR)KeyBuffer[CharIndex];
				}
				Ar << StringLength;
				Ar.Serialize(KeyBufferAnsi, StringLength + 1);
			}
			Ar << Pair.ValueInteger;
			return Ar;
		}
	};
	FHashTable KeyHashTable;
	TArray<FPairType> Pairs;
	TArray<EShaderCompilerDefineVariant> ValueTypes;
	TArray<FString> StringValues;
	uint32 InitialDefineCount;			// Number of items that came from GInitialDefines
	uint32 ValueCount;					// Number of valid values (ValueType != EShaderCompilerDefineVariant::None)

	RENDERCORE_API static FShaderCompilerDefinitions* GInitialDefines;

	friend struct FShaderInitialDefinesInitializer;
};
