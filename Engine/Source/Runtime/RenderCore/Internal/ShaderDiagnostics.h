// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "RHIShaderPlatform.h"
#include "ShaderCompilerJobTypes.h"

struct FShaderDiagnosticInfo
{
	TArray<FShaderCompileJob*> ErrorJobs;
	TArray<FString> UniqueErrors;
	TArray<FString> UniqueWarnings;
	TArray<EShaderPlatform> ErrorPlatforms;
	FString TargetShaderPlatformString;

	RENDERCORE_API FShaderDiagnosticInfo(const TArray<FShaderCommonCompileJobPtr>& Jobs);

private:
	void AddAndProcessErrorsForJob(FShaderCommonCompileJob& Job);
	int32 AddAndProcessErrorsForFailedJobFiltered(FShaderCompileJob& Job, const TCHAR* FilterMessage);
	void AddWarningsForJob(const FShaderCommonCompileJob& Job);

	TArray<uint32> UniqueErrorHashes;
};

RENDERCORE_API FString GetSingleJobCompilationDump(const FShaderCompileJob* SingleJob);
RENDERCORE_API bool IsShaderDevelopmentModeEnabled();
RENDERCORE_API bool ShouldShowShaderWarnings();

