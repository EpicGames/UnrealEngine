// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	EditorShaderCodeArchive.h: Declaration of FEditorShaderCodeArchive
=============================================================================*/

#pragma once

#include "Containers/Array.h"
#include "Containers/HashTable.h"
#include "Containers/Set.h"
#include "Containers/StringView.h"
#include "Memory/SharedBuffer.h"
#include "Misc/CoreMiscDefines.h"
#include "Misc/SecureHash.h"
#include "Serialization/Archive.h"
#include "ShaderCodeArchive.h"
#include "ShaderLibrary/ShaderCodeArchiveInternal.h"
#include "Templates/Function.h"
#include "Templates/UniquePtr.h"
#include "UObject/NameTypes.h"

#if WITH_EDITORONLY_DATA

class IShaderFormat;
#if WITH_EDITOR
namespace UE::Cook::Artifact { struct FUpdateOplogPackagesContext; }
#endif

/**
 * Shader code statistics structure used by FEditorShaderCodeArchive.
 */
struct FShaderCodeStats
{
	struct FPerTypeStats
	{
		int64 ShadersSize = 0;
		int32 NumShaders = 0;
	};

	int32 NumShaderMaps = 0;
	/** Maps from a shader type hash (see FShaderMapResourceCode::FShaderEditorOnlyDataEntry::ShaderTypeHashes) to its statistics. */
	TMap<uint64, FPerTypeStats> ShaderTypeStats;

	FORCEINLINE void AddEntry(uint64 InShaderTypeHash, uint32 InShaderCodeSize)
	{
		FShaderCodeStats::FPerTypeStats& OutTypeCodeStats = ShaderTypeStats.FindOrAdd(InShaderTypeHash);
		OutTypeCodeStats.NumShaders++;
		OutTypeCodeStats.ShadersSize += InShaderCodeSize;
	}
};

/**
 * Wrapper class to create chunks of serialized shaders and write them to a *.ushaderbytecode output file during cooking.
 * See also FEditorShaderStableInfo.
 */
class FEditorShaderCodeArchive
{
	using ECookShaderLibrarySource = ShaderCodeArchive::ECookShaderLibrarySource;
	using ESaveToDiskTarget = ShaderCodeArchive::ESaveToDiskTarget;
	using ESaveToDiskSortOrder = ShaderCodeArchive::ESaveToDiskSortOrder;

	/** Current library name, e.g. "Global" or "Lyra_Chunk0". */
	FString LibraryName;

	/** Combined shader format and platform name, e.g. "SF_VULKAN_SM6-VULKAN_SM6". */
	const FName FormatAndPlatformName;

	FSerializedShaderArchive SerializedShaders;

	/**
	 * The element at index N holds the ShaderCode for the element of SerializedShaders.ShaderEntries at index N.
	 * In MultiprocessCooking elements can be empty if they have been transferred to the Director (bHasCopiedAndCleared will be true in this case).
	 */
	TArray<FSharedBuffer> ShaderCode;

	/** A list of ShaderMaps that have not yet been copied to the CookDirector. Used only by MultiprocessCookWorkers. */
	TSet<int32> ShaderMapsToCopy;

	/** True if CopyToArchiveAndClear has been called, otherwise false. If false we avoid doing some tracking since it might never be used. */
	bool bHasCopiedAndCleared = false;

	const IShaderFormat* const Format = nullptr;

public:
	/** @InFormatAndPlatformName specifies the combined shader and platform format name, e.g. "SF_VULKAN_ES31_ANDROID-VULKAN_ES3_1_ANDROID". */
	RENDERCORE_API FEditorShaderCodeArchive(FName InFormatAndPlatformName);
	RENDERCORE_API ~FEditorShaderCodeArchive();

	/** Returns true if the IShaderFormat interface this code archive was created with supports native shader archives. */
	RENDERCORE_API bool SupportsShaderArchives() const;

	/** Resets the library name and flushes all serialized shaders and their shader code. */
	RENDERCORE_API void OpenLibrary(FString const& Name);

	/** Clears the library name and validates the specified library name was opened previously. */
	RENDERCORE_API void CloseLibrary(FString const& Name);

	/** Returns true if the serialized shaders contain the specified shader map hash. */
	FORCEINLINE bool HasShaderMap(const FShaderHash& Hash) const
	{
		return SerializedShaders.FindShaderMap(Hash) != INDEX_NONE;
	}

	/** Returns true if this archive has no serialized shaders. */
	FORCEINLINE bool IsEmpty() const
	{
		return SerializedShaders.GetNumShaders() == 0;
	}

	/** Adds and deduplicates shader code from the specified shader map resoure to the list of serialized shaders. */
	RENDERCORE_API int32 AddShaderCode(const FShaderMapResourceCode* Code, const FShaderMapAssetPaths& AssociatedAssets,
		ECookShaderLibrarySource CookSource, FShaderCodeStats* OutCodeStats = nullptr);

	/** Returns true if this archive has any data that can be copied in CopyToArchiveAndClear(). */
	RENDERCORE_API bool HasDataToCopy() const;

	/** Writes the specified transfer archive and transfer code with the compact binary writer.
	 * This is only a member function for consistency with FEditorShaderStableInfo. */
	RENDERCORE_API void CopyToCompactBinary(FCbWriter& Writer, FSerializedShaderArchive& TransferArchive, TArray<uint8>& TransferCode);

	/** Loads (LoadFromCompactBinary) and appends (AppendFromArchive) all serialized shaders from the specified compact binary field to this archive. */
	RENDERCORE_API bool AppendFromCompactBinary(FCbFieldView Field, ECookShaderLibrarySource CookSource, FShaderCodeStats* OutCodeStats = nullptr);

	/** Copies all serialized shaders to the target archive and its flat shader code array.
	 * This also flushes the shared buffer of all serialized shaders in this archive to release potentially large amounts of memory.
	 * If output parameter bOutRanOutOfRoom is true after this call, additional serialized shaders should be copied into the next target archive. */
	RENDERCORE_API void CopyToArchiveAndClear(FSerializedShaderArchive& TargetArchive, TArray<uint8>& TargetFlatShaderCode,
		bool& bOutRanOutOfRoom, int64 MaxShaderSize, int64 MaxShaderCount, ECookShaderLibrarySource CookSource);

	/** Appends all entries of the source archive to the list of serialized shaders in this archive, including copies of the shader code. */
	RENDERCORE_API bool AppendFromArchive(const FSerializedShaderArchive& SourceArchive, TConstArrayView64<uint8> SourceFlatShaderCode,
		ECookShaderLibrarySource CookSource, FShaderCodeStats* OutCodeStats = nullptr);

	/** Produces another archive that contains the code only for these assets and generates a chunked name */
	RENDERCORE_API TUniquePtr<FEditorShaderCodeArchive> CreateChunk(int32 ChunkId, const TSet<FName>& PackagesInChunk);

	/** Produces another archive that contains the code only for these assets and uses a custom chunk name. */
	RENDERCORE_API TUniquePtr<FEditorShaderCodeArchive> CreateNamedChunk(const FString& ChunkName, const TSet<FName>& PackagesInChunk, FShaderMapAssetAssociations::FShaderMapFilterFunction ShouldExcludeShaderMap = nullptr);

	/** Adds entries for the specified shader map from another archive with all its individual shaders and their code to this archive. */
	RENDERCORE_API int32 AddShaderCode(int32 OtherShaderMapIndex, const FEditorShaderCodeArchive& OtherArchive, ECookShaderLibrarySource CookSource);

	/** Shortcut to add shader code library from directory . */
	FORCEINLINE void AddShaderCodeLibraryFromDirectory(const FString& BaseDir, ECookShaderLibrarySource CookSource, FShaderCodeStats* OutCodeStats = nullptr)
	{
		AddShaderCodeLibraryByName(BaseDir, LibraryName, CookSource, OutCodeStats);
	}

	/** Reads all existing shader code entries from file and adds them to this library. This marks the end of populating this archive. */
	FORCEINLINE void FinishPopulate(const FString& OutputDir, ECookShaderLibrarySource CookSource, FShaderCodeStats* OutCodeStats = nullptr)
	{
		AddExistingShaderCodeLibrary(OutputDir, CookSource, OutCodeStats);
	}

	/** Saves this shader archive to disk:
	 * Shader entries and all their code to *.ushaderbytecode file,
	 * Asset-shader associations to *.assetinfo.json file,
	 * and shader type information to *.stinfo file. */
	RENDERCORE_API bool SaveToDisk(const FString& OutputDir, const FString& MetaOutputDir,
		ESaveToDiskTarget Target, ESaveToDiskSortOrder SortOrder,
		TArray<FString>* OutputFilenames = nullptr);

	/** Creates a native shader library for this archive, returns the output filenames,
	 * and deletes the shader code and pipeline files as they are no longer needed. */
	RENDERCORE_API bool PackageNativeShaderLibrary(const FString& ShaderCodeDir, TArray<FString>* OutputFilenames = nullptr);

	/** Prints the shader library statistics to the log. */
	RENDERCORE_API void DumpStatsAndDebugInfo();

	/** Invokes the specified callback for each asset that references the specified shader type hash.
	 * This can be slow and should only be used for debugging code or fallback conditions for assertions.
	 * The callback must return true to continue iterating. */
	RENDERCORE_API void ForEachAssetReferencingShaderType(uint64 InShaderTypeHash, FShaderMapAssetAssociations::FAssetFilterFunctionRef AssetReferenceCallback) const;

#if WITH_EDITOR
public:
	/** Resets bReferencedByOplog and bReferencedByStaging to false for all shadermap-asset associations. */
	void InitializeReferenceTracking();

	/** Called by the cooker to report which assets are referenced in the current cook or in incrementalcook oplog. */
	void UpdateOplogPackages(UE::Cook::Artifact::FUpdateOplogPackagesContext& Context);

	/** Called by the cooker to ask the archive to prune no-longer referenced shaders. */
	void Prune();
#endif // WITH_EDITOR

public:
	/** Loads all existing shader code libraries from the new meta data directory and patches their entries into this archive. */
	RENDERCORE_API static bool CreatePatchLibrary(FName FormatName, FString const& LibraryName, TArray<FString> const& OldMetaDataDirs, FString const& NewMetaDataDir, FString const& OutDir, bool bNativeFormat);

#if WITH_EDITOR
	TUniquePtr<FEditorShaderCodeArchive> CreateArchiveFromAssetsReferencedByStaging();
	TUniquePtr<FEditorShaderCodeArchive> CreateArchiveFromAssetsReferencedByOplog();
	TUniquePtr<FEditorShaderCodeArchive> CreateArchiveFromAssetsOnlyReferencedByOplog();
#endif

private:
	// Primary function to create a new shader code archive with a filtered set of assets under the condition specified by a function callback.
	TUniquePtr<FEditorShaderCodeArchive> CreateArchiveFromFilteredAssets(const FString& NewLibraryName,
		FShaderMapAssetAssociations::FAssetFilterFunctionRef ShouldKeepAsset,
		FShaderMapAssetAssociations::FShaderMapFilterFunction ShouldExcludeShaderMap = nullptr);

	// Calls AddShaderCode() for all shader maps from the new library unless they were already added in one of the old libraries.
	void MakePatchLibrary(TArray<FEditorShaderCodeArchive*> const& OldLibraries, FEditorShaderCodeArchive const& NewLibrary);

	struct FShaderCodeReadInfo
	{
		int64 ShaderCodeIndex;
		int64 Offset;
		int32 Size;
	};

	RENDERCORE_API void AddExistingShaderCodeLibrary(FString const& OutputDir, ECookShaderLibrarySource CookSource, FShaderCodeStats* OutCodeStats = nullptr);
	bool LoadExistingShaderCodeLibrary(FString const& MetaDataDir, ECookShaderLibrarySource CookSource);
	int32 PrepareAddShaderCode(int32 OtherShaderMapIndex, const FSerializedShaderArchive& OtherShaders, TArray<FShaderCodeReadInfo>& ReadInfo,
		ECookShaderLibrarySource CookSource, FShaderCodeStats* OutCodeStats = nullptr);

	enum class EShaderCodeReadStatus : uint8
	{
		OK,
		FileNotFound,
		MalformedArchive,
		VersionMismatch,
	};

	RENDERCORE_API EShaderCodeReadStatus AddShaderCodeLibraryByName(const FString& BaseDir, const FString& InLibraryName,
		ECookShaderLibrarySource CookSource, FShaderCodeStats* OutCodeStats = nullptr);
	EShaderCodeReadStatus ReadShaderCode(FArchive& Ar, TArray<FShaderCodeReadInfo>& ReadInfo);
	EShaderCodeReadStatus ReadShaderCodeBlock(FArchive& Ar, int64 StartOffset, const TArrayView<FShaderCodeReadInfo>& BlockReadInfo);

	void MarkShaderMapDirty(int32 ShaderMapIndex);
	void MarkAllShaderMapsDirty();

	// Wrapper to move all data from InArchive into this archive.
	// Constant members such as Format and such must be equal since they must never change.
	void AssignFrom(FEditorShaderCodeArchive& InArchive);

};

#endif // WITH_EDITORONLY_DATA
