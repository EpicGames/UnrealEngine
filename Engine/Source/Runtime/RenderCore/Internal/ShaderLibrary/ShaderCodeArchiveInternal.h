// Copyright Epic Games, Inc. All Rights Reserved.

/*=======================================================================================
	ShaderCodeArchiveInternal.h: Shared types and helpers for the shader code library
=========================================================================================*/

#pragma once

#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "Hash/ShaderHash.h"
#include "IO/IoChunkId.h"
#include "UObject/NameTypes.h"

namespace ShaderCodeArchive
{
	enum class ESaveToDiskTarget : uint8
	{
		Staging = 0,
		CookCache,
	};

	enum class ESaveToDiskSortOrder : uint8
	{
		/**
		 * Shaders of a cooked shader library are sorted by the order in which they are
		 * discovered during cook. This is better for performance when the library is loaded at runtime, since
		 * runtime load order is correlated with cooker load order. But the correlation is not perfect, and cooker
		 * load order is non-deterministic. This sort order is not beneficial at all if IoStore is used, because
		 * the libraries are sorted into a different sort order when IoStore puts them into containers.
		 */
		PackageLoad,
		/**
		 * Shaders of a cooked shader library are sorted by their hash values. This is useful for cook determinism:
		 * the bytes of the shader library in two different cooks are identical if shader inputs are unchanged between
		 * the cooks.
		 */
		ShaderHash,
	};
	
	inline FIoChunkId GetShaderCodeArchiveChunkId(const FString& LibraryName, FName FormatName)
	{
		FString Name = FString::Printf(TEXT("%s-%s"), *LibraryName, *FormatName.ToString());
		Name.ToLowerInline();
		uint64 Hash = FShaderHash::HashBuffer(reinterpret_cast<const char*>(*Name), Name.Len() * sizeof(TCHAR)).Hash;
		return CreateIoChunkId(Hash, 0, EIoChunkType::ShaderCodeLibrary);
	}

	template <int HashSize>
	FIoChunkId GetShaderCodeChunkId(const uint8(&HashBytes)[HashSize])
	{
		static_assert(HashSize >= 11); // ensure the hash is big enough
		uint8 Data[12];
		FMemory::Memcpy(Data, HashBytes, 11); // 11 bytes from given hash
		*reinterpret_cast<uint8*>(&Data[11]) = static_cast<uint8>(EIoChunkType::ShaderCode); // last byte is chunk type
		FIoChunkId ChunkId;
		ChunkId.Set(Data, 12);
		return ChunkId;
	}
	
	inline FIoChunkId GetLibraryScopedShaderGroupChunkId(const FIoChunkId& ContentChunkId, const FIoChunkId& LibraryChunkId)
	{
		uint8 Input[24];
		FMemory::Memcpy(Input,      ContentChunkId.GetData(), 12);
		FMemory::Memcpy(Input + 12, LibraryChunkId.GetData(), 12);
		const uint64 Hash = FShaderHash::HashBuffer(Input, sizeof(Input)).Hash;
		return CreateIoChunkId(Hash, 0, EIoChunkType::ShaderCode);
	}
} // namespace ShaderCodeArchive
