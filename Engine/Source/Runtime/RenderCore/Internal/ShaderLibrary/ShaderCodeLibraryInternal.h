// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ShaderCodeLibraryInternal.h: Extended functions to FShaderCodeLibrary for internal use only
=============================================================================*/

#pragma once

#include "CoreTypes.h"

class FShaderCodeLibraryInternal
{
public:
	/** Tries to find a new shader library instance for the specified resource.
		This must only be called when a shader library instance is being closed, when all the mutexes are already locked.
		See destructor of FShaderLibraryInstance. */
	static RENDERCORE_API bool MoveShaderMapResourceOwnership(class FShaderMapResource_SharedCode* Resource);
};
