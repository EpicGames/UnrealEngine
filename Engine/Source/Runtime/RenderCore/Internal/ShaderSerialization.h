// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Compression/CompressedBuffer.h"
#include "Memory/CompositeBuffer.h"
#include "Memory/SharedBuffer.h"
#include "Templates/Function.h"
#include "Templates/UniquePtr.h"
#include "UObject/NameTypes.h" 

class FMemoryReaderView;
class FMemoryWriter64;
class FShaderCodeResource;

#if WITH_EDITOR
namespace UE::DerivedData
{
	class FCacheRecord;
	struct FCacheKey;
}
#endif

/* Context object used for storing state and serialization parameterization related to shader/shadermap serialization. */
struct FShaderSerializeContext
{
	/* Default constructor, if called it's expected the archive pointer will be set by derived implementations. */
	FShaderSerializeContext() = default;

	/* Constructor which accepts an FArchive reference, used for simple serialization cases */
	FShaderSerializeContext(FArchive& InArchive) : Ar(&InArchive) {}

	virtual ~FShaderSerializeContext() = default;

	/* If this is overridden to return true, SerializeCode will be called to serialize shader code separately from the rest of the serialized object. */
	virtual bool EnableCustomCodeSerialize() { return false; }

	/* Optional function which must be implemented if EnableCustomCodeSerialize returns true; use to serialize shader code separately from the main object */
	virtual void SerializeCode(FShaderCodeResource& Resource, int32 Index) {};

	/* Optional function that can be used to reserve space for the given number of code objects by derived classes. */
	virtual void ReserveCode(int32 Count) {};

	/* Archive pointer which should be used for serializing the object, possibly excluding shader code if the EnableCustomCodeSerialize returns true */
	FArchive* Ar = nullptr;

	/* Flag indicating whether this serialization is a cooked load; used to change serialization behaviour for cooked data vs cached data */
	bool bLoadingCooked = false;

	/* FName of the asset which triggered the serialization, this is used only for diagnostic messages */
	FName SerializingAsset = NAME_None;

	/* Helper function to retrieve an FArchive reference; exists for convenience and validation purposes. */
	FArchive& GetMainArchive()
	{
		check(Ar);
		return *Ar;
	}
};


/* FShaderSerializeContext implementation used for saving to or loading from caches (either DDC or in-memory job cache) 
 * Note that this is just a base class of the below and should not be used directly.
 */
struct FShaderCacheSerializeContext : public FShaderSerializeContext
{
	FShaderCacheSerializeContext() : FShaderSerializeContext() 
	{
		ShaderCode = OwnedShaderCode;
		ShaderSymbols = OwnedShaderSymbols;

		checkf(ShaderCode.Num() == ShaderSymbols.Num(), TEXT("It is required to serialize a (possibly empty, but non-null) symbols buffer for every code buffer."));
	}

	virtual ~FShaderCacheSerializeContext() = default;

	/* Buffer which stores the main object data for a cache entry - i.e. a shadermap or job structure */
	FSharedBuffer ShaderObjectData;

	/* View on array of buffers which store the bytecode objects for the object, one per shader/stage.
	 * Note that this may or may not point to the OwnedShaderCode array below, depending on usage.
	 */
	TArrayView<FCompositeBuffer> ShaderCode;

	/* View on array of compressed buffers which store the symbols for the object, one per shader/stage.
	 * Note that this may or may not point to the OwnedShaderSymbols array below, depending on usage.
	 */
	TArrayView<FCompressedBuffer> ShaderSymbols;

	/* Array of code buffers actually owned by this context object; it is valid for this to be empty in cases
	 * where the array of ShaderCode buffers is stored externally.
	 */
	TArray<FCompositeBuffer> OwnedShaderCode;

	/* Array of symbol buffers actually owned by this context object; it is valid for this to be empty in cases
	 * where the array of ShaderSymbols buffers is stored externally.
	 */
	TArray<FCompressedBuffer> OwnedShaderSymbols;

	/* Subclasses of this type must implement custom code serialization function */
	virtual bool EnableCustomCodeSerialize() override { return true; }

	/* Get the total serialized size of data for this context; note that this will return 0 if called prior to the FSharedBuffers
	 * being set (this is done in the derived implementations, see below).
	 */
	int64 GetSerializedSize() const
	{
		int64 Size = 0u;
		if (ShaderObjectData)
		{
			Size += ShaderObjectData.GetSize();

			for (FCompositeBuffer& CodeBuf : ShaderCode)
			{
				Size += CodeBuf.GetSize();
			}
		}
		return Size;
	}

	/* Populates the given code array (transfering ownership) and resets the internal view to point to the new owning array's data */
	void MoveCode(TArray<FCompositeBuffer>& TargetCode, TArray<FCompressedBuffer>& TargetSymbols)
	{
		TargetCode = MoveTemp(OwnedShaderCode);
		ShaderCode = TargetCode;
		TargetSymbols = MoveTemp(OwnedShaderSymbols);
		ShaderSymbols = TargetSymbols;

		checkf(ShaderCode.Num() == ShaderSymbols.Num(), TEXT("It is required to serialize a (possibly empty, but non-null) symbols buffer for every code buffer."));
	}

	/* Returns true if there is valid serialized data referenced by this context. */
	bool HasData() const { return ShaderObjectData && !ShaderCode.IsEmpty(); }
};

/* Implementation of FShaderCacheSerializeContext used for saving data to caches. */
struct FShaderCacheSaveContext : public FShaderCacheSerializeContext
{
	/* Default constructor; sets up base class FArchive pointing to the owned memory writer. */
	RENDERCORE_API FShaderCacheSaveContext();

	virtual ~FShaderCacheSaveContext() = default;

	/* Converts the raw serialized object data into the ShaderObjectData FSharedBuffer.
	 * Note that this is called by BuildCacheRecord as well, calls subsequent to the first will have no effect.
	 */
	RENDERCORE_API void Finalize();

	/* Overridden code serialize implementation. */
	RENDERCORE_API virtual void SerializeCode(FShaderCodeResource& Resource, int32 Index) override;

	/* Overridden code reserve implementation. */
	RENDERCORE_API virtual void ReserveCode(int32 Count) override;

#if WITH_EDITOR
	/* Helper function to generate a DDC record from the data serialized using this context. Serialization must
	 * be executed prior to calling this function.
	 */
	RENDERCORE_API UE::DerivedData::FCacheRecord BuildCacheRecord(const UE::DerivedData::FCacheKey& Key);
#endif

	/* Call to reset internal state, this allows re-using internal allocations to store other objects (as an optimization). */
	RENDERCORE_API void Reset();

	/* Data that will be written to when serializing */
	TArray64<uint8> ShaderObjectRawData;

	/* FArchive that is passed to the base class, which writes to ShaderObjectRawData */
	TUniquePtr<FMemoryWriter64> Writer;
};

/* Implementation of FShaderCacheSerializeContext used for loading data from caches. */
struct FShaderCacheLoadContext : public FShaderCacheSerializeContext
{
	/*	Default constructor, use when array of code buffers will be allocated via ReadFromRecord */
	FShaderCacheLoadContext() = default;

	/* Constructor which references buffers (and an array of code buffers) owned elsewhere, does not allocate the OwnedShaderCode array */
	RENDERCORE_API FShaderCacheLoadContext(FSharedBuffer ShaderObjectData, TArrayView<FCompositeBuffer> CodeBuffers, TArrayView<FCompressedBuffer> SymbolBuffers);

	virtual ~FShaderCacheLoadContext() = default;

	/* Resets internal state to point to the given buffers and recreates the owned memoryreader */
	RENDERCORE_API void Reset(FSharedBuffer ShaderObjectData, TArrayView<FCompositeBuffer> CodeBuffers, TArrayView<FCompressedBuffer> SymbolBuffers);

	/* Call to reset reader to start position so the same load context can be used to populate multiple objects. */
	RENDERCORE_API void Reuse();

	/* Overridden code serialize implementation. */
	RENDERCORE_API virtual void SerializeCode(FShaderCodeResource& Resource, int32 Index) override;

#if WITH_EDITOR
	/* Helper function to populate the internal state (FSharedBuffers defined on FShaderCacheSerializeContext) from a DDC record. */
	RENDERCORE_API void ReadFromRecord(const UE::DerivedData::FCacheRecord& CacheRecord, bool bIsPersistent = false);
#endif

	/* FArchive passed to the base class; this will be used to read data from the FShaderCacheSerializeContext::ShaderObjectData shared buffer */
	TUniquePtr<FMemoryReaderView> Reader;
};

