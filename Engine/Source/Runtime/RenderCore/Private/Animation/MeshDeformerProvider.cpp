// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/MeshDeformerProvider.h"
#include "Features/IModularFeatures.h"

const FName IMeshDeformerProvider::ModularFeatureName(TEXT("MeshDeformer"));

IMeshDeformerProvider* IMeshDeformerProvider::Get()
{
	if (IsAvailable())
	{
		return &IModularFeatures::Get().GetModularFeature<IMeshDeformerProvider>(ModularFeatureName);
	}
	return nullptr;
}

bool IMeshDeformerProvider::IsAvailable()
{
	return IModularFeatures::Get().IsModularFeatureAvailable(ModularFeatureName);
}
