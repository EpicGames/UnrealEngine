// Copyright Epic Games, Inc. All Rights Reserved.

#include "BinkShaders.h"

IMPLEMENT_GLOBAL_SHADER(FBinkDrawVS, "/Engine/Private/Bink.usf", "BinkDrawVS", SF_Vertex);
IMPLEMENT_GLOBAL_SHADER(FBinkDrawYCbCrPS, "/Engine/Private/Bink.usf", "BinkDrawYCbCr", SF_Pixel);
IMPLEMENT_GLOBAL_SHADER(FBinkDrawICtCpPS, "/Engine/Private/Bink.usf", "BinkDrawICtCp", SF_Pixel);
