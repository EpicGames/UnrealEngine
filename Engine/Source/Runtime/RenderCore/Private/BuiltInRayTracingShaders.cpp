// Copyright Epic Games, Inc. All Rights Reserved.

#include "BuiltInRayTracingShaders.h"
#include "ShaderParameterUtils.h"
#include "PipelineStateCache.h"
#include "RendererInterface.h"

#if RHI_RAYTRACING

#include "RayTracingPayloadType.h"

uint32 GetRaytracingMaterialPayloadSizeFullySimplified()
{
	if (Substrate::IsSubstrateEnabled())
	{
		// All the data from FPackedMaterialClosestHitPayload except FSubstrateRaytracingPayload (see RayTracingCommon.ush)
		uint32 PayloadSizeBytes = 7 * sizeof(uint32);

		// The remaining data from FSubstrateRaytracingPayload.
		const bool bFullySimplifiedMaterial = true;	// This is needed because ERayTracingPayloadType::RayTracingMaterial will be fully simplified, see FShaderType::ModifyCompilationEnvironment.
		PayloadSizeBytes += Substrate::GetRayTracingMaterialPayloadSizeInBytes(bFullySimplifiedMaterial);

		return PayloadSizeBytes;
	}

	return 64u;
}

IMPLEMENT_RT_PAYLOAD_TYPE(ERayTracingPayloadType::Default, 24);
IMPLEMENT_RT_PAYLOAD_TYPE_FUNCTION(ERayTracingPayloadType::RayTracingMaterial, GetRaytracingMaterialPayloadSizeFullySimplified);

IMPLEMENT_GLOBAL_SHADER( FDefaultPayloadMS, "/Engine/Private/RayTracing/RayTracingBuiltInShaders.usf", "DefaultPayloadMS", SF_RayMiss);
IMPLEMENT_GLOBAL_SHADER( FPackedMaterialClosestHitPayloadMS, "/Engine/Private/RayTracing/RayTracingBuiltInShaders.usf", "PackedMaterialClosestHitPayloadMS", SF_RayMiss);
IMPLEMENT_GLOBAL_SHADER(FRayTracingDispatchDescCS, "/Engine/Private/RayTracing/RayTracingDispatchDesc.usf", "RayTracingDispatchDescCS", SF_Compute);


ERayTracingPayloadType FDefaultPayloadMS::GetRayTracingPayloadType(const int32 PermutationId)
{
	return ERayTracingPayloadType::Default;
}

ERayTracingPayloadType FPackedMaterialClosestHitPayloadMS::GetRayTracingPayloadType(const int32 PermutationId)
{
	return ERayTracingPayloadType::RayTracingMaterial;
}


void FRayTracingDispatchDescCS::Dispatch(FRHICommandList& RHICmdList, 
	const void* DispatchDescInput, uint32 DispatchDescSize, uint32 DispatchDescDimensionsOffset,
	FRHIShaderResourceView* DispatchDimensionsSRV, uint32 DimensionsBufferOffset,
	FRHIUnorderedAccessView* DispatchDescOutputUAV)
{
	const uint32 DispatchDescSizeDwords = DispatchDescSize / 4;
	const uint32 DispatchDescDimensionsOffsetDwords = DispatchDescDimensionsOffset / 4;

	checkf(DimensionsBufferOffset % 4 == 0, TEXT("Dispatch dimensions buffer offset must be DWORD-aligned"));
	const uint32 DimensionsBufferOffsetDwords = DimensionsBufferOffset / 4;

	check(DispatchDescSizeDwords <= DispatchDescMaxSizeDwords);

	TShaderMapRef<FRayTracingDispatchDescCS> ComputeShader(GetGlobalShaderMap(GMaxRHIFeatureLevel));
	FRHIComputeShader* ShaderRHI = ComputeShader.GetComputeShader();
	SetComputePipelineState(RHICmdList, ShaderRHI);

	static_assert(DispatchDescMaxSizeDwords % 4 == 0, "DispatchDescMaxSizeDwords must be a multiple of 4");
	static constexpr uint32 DispatchDescMaxSizeUint4s = DispatchDescMaxSizeDwords / 4;

	FUintVector4 DispatchDescData[DispatchDescMaxSizeUint4s] = {};
	FMemory::Memcpy(DispatchDescData, DispatchDescInput, DispatchDescSize);

	FRHIBatchedShaderParameters& BatchedParameters = RHICmdList.GetScratchShaderParameters();

	SetShaderValueArray(BatchedParameters, ComputeShader->DispatchDescInputParam, DispatchDescData, DispatchDescMaxSizeUint4s);
	SetShaderValue(BatchedParameters, ComputeShader->DispatchDescSizeDwordsParam, DispatchDescSizeDwords);
	SetShaderValue(BatchedParameters, ComputeShader->DispatchDescDimensionsOffsetDwordsParam, DispatchDescDimensionsOffsetDwords);
	SetShaderValue(BatchedParameters, ComputeShader->DimensionsBufferOffsetDwordsParam, DimensionsBufferOffsetDwords);

	SetSRVParameter(BatchedParameters, ComputeShader->DispatchDimensionsParam, DispatchDimensionsSRV);
	SetUAVParameter(BatchedParameters, ComputeShader->DispatchDescOutputParam, DispatchDescOutputUAV);
	RHICmdList.SetBatchedShaderParameters(ComputeShader.GetComputeShader(), BatchedParameters);

	RHICmdList.DispatchComputeShader(1, 1, 1);

	if (RHICmdList.NeedsShaderUnbinds())
	{
		FRHIBatchedShaderUnbinds& BatchedUnbinds = RHICmdList.GetScratchShaderUnbinds();

		UnsetSRVParameter(BatchedUnbinds, ComputeShader->DispatchDimensionsParam);
		UnsetUAVParameter(BatchedUnbinds, ComputeShader->DispatchDescOutputParam);
		RHICmdList.SetBatchedShaderUnbinds(ComputeShader.GetComputeShader(), BatchedUnbinds);
	}
}

#endif // RHI_RAYTRACING
