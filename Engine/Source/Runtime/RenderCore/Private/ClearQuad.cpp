// Copyright Epic Games, Inc. All Rights Reserved.

#include "ClearQuad.h"
#include "RenderUtils.h"
#include "Shader.h"
#include "RHIStaticStates.h"
#include "OneColorShader.h"
#include "PipelineStateCache.h"
#include "RendererInterface.h"
#include "Logging/LogMacros.h"
#include "RHIResourceUtils.h"

static const FVector4f GClearVertexBufferVertices[] =
{
	FVector4f(-1.0f, 1.0f, 0.0f, 1.0f),
	FVector4f(1.0f, 1.0f, 0.0f, 1.0f),
	FVector4f(-1.0f, -1.0f, 0.0f, 1.0f),
	FVector4f(1.0f, -1.0f, 0.0f, 1.0f),
};

void FClearVertexBuffer::InitRHI(FRHICommandListBase& RHICmdList)
{
	// create a static vertex buffer
	VertexBufferRHI = UE::RHIResourceUtils::CreateVertexBufferFromArray(RHICmdList, TEXT("FClearVertexBuffer"), EBufferUsageFlags::Static, MakeConstArrayView(GClearVertexBufferVertices));
}

TGlobalResource<FClearVertexBuffer> GClearVertexBuffer;

DEFINE_LOG_CATEGORY_STATIC(LogClearQuad, Log, Log)

template<EColorWriteMask ColorWriteMask>
static void ClearQuadSetup(FRHICommandList& RHICmdList, int32 NumClearColors, const FLinearColor* ClearColorArray, bool bClearDepth, float Depth, bool bClearStencil
	, uint32 Stencil, TFunction<void(FGraphicsPipelineStateInitializer&)> PSOModifier = nullptr
	, uint8 NumUintOutput = 0)
{
	if (UNLIKELY(!FApp::CanEverRender()))
	{
		return;
	}

	check(ColorWriteMask == CW_NONE || NumClearColors > 0);

	// Set new states
	FRHIBlendState* BlendStateRHI = TStaticBlendStateWriteMask<ColorWriteMask, ColorWriteMask, ColorWriteMask, ColorWriteMask, ColorWriteMask, ColorWriteMask, ColorWriteMask, ColorWriteMask>::GetRHI();
	
	FRHIDepthStencilState* DepthStencilStateRHI =
		(bClearDepth && bClearStencil)
			? TStaticDepthStencilState<
				true, CF_Always,
				true,CF_Always,SO_Replace,SO_Replace,SO_Replace,
				false,CF_Always,SO_Replace,SO_Replace,SO_Replace,
				0xff,0xff
				>::GetRHI()
			: bClearDepth
				? TStaticDepthStencilState<true, CF_Always>::GetRHI()
				: bClearStencil
					? TStaticDepthStencilState<
						false, CF_Always,
						true,CF_Always,SO_Replace,SO_Replace,SO_Replace,
						false,CF_Always,SO_Replace,SO_Replace,SO_Replace,
						0xff,0xff
						>::GetRHI()
					: TStaticDepthStencilState<false, CF_Always>::GetRHI();

	FGraphicsPipelineStateInitializer GraphicsPSOInit;
	RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);

	GraphicsPSOInit.RasterizerState = TStaticRasterizerState<FM_Solid, CM_None>::GetRHI();
	GraphicsPSOInit.BlendState = BlendStateRHI;
	GraphicsPSOInit.DepthStencilState = DepthStencilStateRHI;

	auto* ShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);

	// Set the new shaders
	TShaderMapRef<TOneColorVS<true> > VertexShader(ShaderMap);

	// Set the shader to write to the appropriate number of render targets
	// On AMD PC hardware, outputting to a color index in the shader without a matching render target set has a significant performance hit
	TOneColorPixelShaderMRT::FPermutationDomain PermutationVector;
	PermutationVector.Set<TOneColorPixelShaderMRT::TOneColorPixelShaderNumOutputs>(NumClearColors ? NumClearColors : 1);
	PermutationVector.Set<TOneColorPixelShaderMRT::TOneColorPixelShader128bitRT>(PlatformRequires128bitRT((EPixelFormat)GraphicsPSOInit.RenderTargetFormats[0]));
	PermutationVector.Set<TOneColorPixelShaderMRT::TOneColorPixelNumUintOutputs>(NumUintOutput);
	TShaderMapRef<TOneColorPixelShaderMRT > PixelShader(ShaderMap, PermutationVector);

	GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GetVertexDeclarationFVector4();
	GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
	GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
	GraphicsPSOInit.PrimitiveType = PT_TriangleStrip;

	if (PSOModifier)
	{
		PSOModifier(GraphicsPSOInit);
	}

	SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, Stencil);

	SetShaderParametersLegacyVS(RHICmdList, VertexShader, Depth);
	
	TOneColorPixelShaderMRT::FParameters PixelParameters;
	PixelShader->FillParameters(PixelParameters, ClearColorArray, NumClearColors);
	SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), PixelParameters);
}

void DrawClearQuadAlpha(FRHICommandList& RHICmdList, float Alpha)
{
	FLinearColor Color(0, 0, 0, Alpha);
	ClearQuadSetup<CW_ALPHA>(RHICmdList, 1, &Color, false, 0, false, 0);

	RHICmdList.SetStreamSource(0, GClearVertexBuffer.VertexBufferRHI, 0);
	RHICmdList.DrawPrimitive(0, 2, 1);
}

void DrawClearQuadMRT(FRHICommandList& RHICmdList, bool bClearColor, int32 NumClearColors, const FLinearColor* ClearColorArray, bool bClearDepth, float Depth, bool bClearStencil, uint32 Stencil)
{
	if (bClearColor)
	{
		ClearQuadSetup<CW_RGBA>(RHICmdList, NumClearColors, ClearColorArray, bClearDepth, Depth, bClearStencil, Stencil);
	}
	else
	{
		ClearQuadSetup<CW_NONE>(RHICmdList, NumClearColors, ClearColorArray, bClearDepth, Depth, bClearStencil, Stencil);
	}

	RHICmdList.SetStreamSource(0, GClearVertexBuffer.VertexBufferRHI, 0);
	RHICmdList.DrawPrimitive(0, 2, 1);
}

void DrawClearQuadMRTWithUints(FRHICommandList& RHICmdList, bool bClearColor, int32 NumClearColors, const FLinearColor* ClearColorArray, bool bClearDepth, float Depth, bool bClearStencil, uint32 Stencil, uint8 NumUintOutput)
{
	check(NumUintOutput <= NumClearColors);
	if (bClearColor)
	{
		ClearQuadSetup<CW_RGBA>(RHICmdList, NumClearColors, ClearColorArray, bClearDepth, Depth, bClearStencil, Stencil, nullptr , NumUintOutput);
	}
	else
	{
		ClearQuadSetup<CW_NONE>(RHICmdList, NumClearColors, ClearColorArray, bClearDepth, Depth, bClearStencil, Stencil, nullptr, NumUintOutput);
	}

	RHICmdList.SetStreamSource(0, GClearVertexBuffer.VertexBufferRHI, 0);
	RHICmdList.DrawPrimitive(0, 2, 1);
}

void DrawClearQuadMRT(FRHICommandList& RHICmdList, bool bClearColor, int32 NumClearColors, const FLinearColor* ClearColorArray, bool bClearDepth, float Depth, bool bClearStencil, uint32 Stencil, FClearQuadCallbacks ClearQuadCallbacks)
{
	if (bClearColor)
	{
		ClearQuadSetup<CW_RGBA>(RHICmdList, NumClearColors, ClearColorArray, bClearDepth, Depth, bClearStencil, Stencil, ClearQuadCallbacks.PSOModifier);
	}
	else
	{
		ClearQuadSetup<CW_NONE>(RHICmdList, NumClearColors, ClearColorArray, bClearDepth, Depth, bClearStencil, Stencil, ClearQuadCallbacks.PSOModifier);
	}

	if (ClearQuadCallbacks.PreClear)
	{
		ClearQuadCallbacks.PreClear(RHICmdList);
	}

	// Draw a fullscreen quad without a hole
	RHICmdList.SetStreamSource(0, GClearVertexBuffer.VertexBufferRHI, 0);
	RHICmdList.DrawPrimitive(0, 2, 1);

	if (ClearQuadCallbacks.PostClear)
	{
		ClearQuadCallbacks.PostClear(RHICmdList);
	}
}

void DrawClearQuadMRT(FRHICommandList& RHICmdList, bool bClearColor, int32 NumClearColors, const FLinearColor* ClearColorArray, bool bClearDepth, float Depth, bool bClearStencil, uint32 Stencil, FIntPoint ViewSize, FIntRect ExcludeRect)
{
	if (ExcludeRect.Min == FIntPoint::ZeroValue && ExcludeRect.Max == ViewSize)
	{
		// Early out if the entire surface is excluded
		return;
	}

	if (bClearColor)
	{
		ClearQuadSetup<CW_RGBA>(RHICmdList, NumClearColors, ClearColorArray, bClearDepth, Depth, bClearStencil, Stencil);
	}
	else
	{
		ClearQuadSetup<CW_NONE>(RHICmdList, NumClearColors, ClearColorArray, bClearDepth, Depth, bClearStencil, Stencil);
	}

	// Draw a fullscreen quad
	if (ExcludeRect.Width() > 0 && ExcludeRect.Height() > 0)
	{
		// with a hole in it
		FVector4f OuterVertices[4];
		OuterVertices[0].Set(-1.0f, 1.0f, Depth, 1.0f);
		OuterVertices[1].Set(1.0f, 1.0f, Depth, 1.0f);
		OuterVertices[2].Set(1.0f, -1.0f, Depth, 1.0f);
		OuterVertices[3].Set(-1.0f, -1.0f, Depth, 1.0f);

		float InvViewWidth = 1.0f / ViewSize.X;
		float InvViewHeight = 1.0f / ViewSize.Y;
		FVector4f FractionRect = FVector4f(ExcludeRect.Min.X * InvViewWidth, ExcludeRect.Min.Y * InvViewHeight, (ExcludeRect.Max.X - 1) * InvViewWidth, (ExcludeRect.Max.Y - 1) * InvViewHeight);

		FVector4f InnerVertices[4];
		InnerVertices[0].Set(FMath::Lerp(-1.0f, 1.0f, FractionRect.X), FMath::Lerp(1.0f, -1.0f, FractionRect.Y), Depth, 1.0f);
		InnerVertices[1].Set(FMath::Lerp(-1.0f, 1.0f, FractionRect.Z), FMath::Lerp(1.0f, -1.0f, FractionRect.Y), Depth, 1.0f);
		InnerVertices[2].Set(FMath::Lerp(-1.0f, 1.0f, FractionRect.Z), FMath::Lerp(1.0f, -1.0f, FractionRect.W), Depth, 1.0f);
		InnerVertices[3].Set(FMath::Lerp(-1.0f, 1.0f, FractionRect.X), FMath::Lerp(1.0f, -1.0f, FractionRect.W), Depth, 1.0f);

		const FVector4f Vertices[] =
		{
			OuterVertices[0],
			InnerVertices[0],
			OuterVertices[1],
			InnerVertices[1],
			OuterVertices[2],
			InnerVertices[2],
			OuterVertices[3],
			InnerVertices[3],
			OuterVertices[0],
			InnerVertices[0],
		};
		FBufferRHIRef VertexBufferRHI = UE::RHIResourceUtils::CreateVertexBufferFromArray(RHICmdList, TEXT("DrawClearQuadMRT"), EBufferUsageFlags::Volatile, MakeConstArrayView(Vertices));

		RHICmdList.SetStreamSource(0, VertexBufferRHI, 0);
		RHICmdList.DrawPrimitive(0, 8, 1);
	}
	else
	{
		// without a hole
		RHICmdList.SetStreamSource(0, GClearVertexBuffer.VertexBufferRHI, 0);
		RHICmdList.DrawPrimitive(0, 2, 1);
	}
}
