// Copyright Epic Games, Inc. All Rights Reserved.


#include "ClearReplacementShaders.h"
#include "ShaderParameterUtils.h"

IMPLEMENT_SHADER_TYPE4_WITH_TEMPLATE_PREFIX(template<>, RENDERCORE_API, FClearReplacementVS									, SF_Vertex);
IMPLEMENT_SHADER_TYPE4_WITH_TEMPLATE_PREFIX(template<>, RENDERCORE_API, FClearReplacementVS_Bounds							, SF_Vertex);
IMPLEMENT_SHADER_TYPE4_WITH_TEMPLATE_PREFIX(template<>, RENDERCORE_API, FClearReplacementVS_Depth							, SF_Vertex);
IMPLEMENT_SHADER_TYPE4_WITH_TEMPLATE_PREFIX(template<>, RENDERCORE_API, FClearReplacementPS									, SF_Pixel);
IMPLEMENT_SHADER_TYPE4_WITH_TEMPLATE_PREFIX(template<>, RENDERCORE_API, FClearReplacementPS_128								, SF_Pixel);
IMPLEMENT_SHADER_TYPE4_WITH_TEMPLATE_PREFIX(template<>, RENDERCORE_API, FClearReplacementPS_Zero							, SF_Pixel);
IMPLEMENT_SHADER_TYPE4_WITH_TEMPLATE_PREFIX(template<>, RENDERCORE_API, FClearReplacementCS_Buffer_Uint_Bounds				, SF_Compute);
IMPLEMENT_SHADER_TYPE4_WITH_TEMPLATE_PREFIX(template<>, RENDERCORE_API, FClearReplacementCS_Buffer_Uint4_Bounds				, SF_Compute);
IMPLEMENT_SHADER_TYPE4_WITH_TEMPLATE_PREFIX(template<>, RENDERCORE_API, FClearReplacementCS_Buffer_Uint4					, SF_Compute);
IMPLEMENT_SHADER_TYPE4_WITH_TEMPLATE_PREFIX(template<>, RENDERCORE_API, FClearReplacementCS_Buffer_Float_Bounds				, SF_Compute);
IMPLEMENT_SHADER_TYPE4_WITH_TEMPLATE_PREFIX(template<>, RENDERCORE_API, FClearReplacementCS_Buffer_Float4_Bounds			, SF_Compute);
IMPLEMENT_SHADER_TYPE4_WITH_TEMPLATE_PREFIX(template<>, RENDERCORE_API, FClearReplacementCS_LargeBuffer_Uint_Bounds			, SF_Compute);
IMPLEMENT_SHADER_TYPE4_WITH_TEMPLATE_PREFIX(template<>, RENDERCORE_API, FClearReplacementCS_LargeBuffer_Uint4_Bounds		, SF_Compute);
IMPLEMENT_SHADER_TYPE4_WITH_TEMPLATE_PREFIX(template<>, RENDERCORE_API, FClearReplacementCS_LargeBuffer_Uint4				, SF_Compute);
IMPLEMENT_SHADER_TYPE4_WITH_TEMPLATE_PREFIX(template<>, RENDERCORE_API, FClearReplacementCS_LargeBuffer_Float_Bounds		, SF_Compute);
IMPLEMENT_SHADER_TYPE4_WITH_TEMPLATE_PREFIX(template<>, RENDERCORE_API, FClearReplacementCS_LargeBuffer_Float4_Bounds		, SF_Compute);
IMPLEMENT_SHADER_TYPE4_WITH_TEMPLATE_PREFIX(template<>, RENDERCORE_API, FClearReplacementCS_StructuredBuffer_Uint_Bounds	, SF_Compute);
IMPLEMENT_SHADER_TYPE4_WITH_TEMPLATE_PREFIX(template<>, RENDERCORE_API, FClearReplacementCS_StructuredBuffer_Uint4_Bounds	, SF_Compute);
IMPLEMENT_SHADER_TYPE4_WITH_TEMPLATE_PREFIX(template<>, RENDERCORE_API, FClearReplacementCS_StructuredBuffer_Uint4			, SF_Compute);
IMPLEMENT_SHADER_TYPE4_WITH_TEMPLATE_PREFIX(template<>, RENDERCORE_API, FClearReplacementCS_StructuredBuffer_Float_Bounds	, SF_Compute);
IMPLEMENT_SHADER_TYPE4_WITH_TEMPLATE_PREFIX(template<>, RENDERCORE_API, FClearReplacementCS_StructuredBuffer_Float4_Bounds	, SF_Compute);
IMPLEMENT_SHADER_TYPE4_WITH_TEMPLATE_PREFIX(template<>, RENDERCORE_API, FClearReplacementCS_Texture3D_Float4_Bounds			, SF_Compute);
IMPLEMENT_SHADER_TYPE4_WITH_TEMPLATE_PREFIX(template<>, RENDERCORE_API, FClearReplacementCS_Texture2DArray_Float4_Bounds	, SF_Compute);
IMPLEMENT_SHADER_TYPE4_WITH_TEMPLATE_PREFIX(template<>, RENDERCORE_API, FClearReplacementCS_Texture2D_Float4_Bounds			, SF_Compute);
IMPLEMENT_SHADER_TYPE4_WITH_TEMPLATE_PREFIX(template<>, RENDERCORE_API, FClearReplacementCS_Buffer_Uint_Zero				, SF_Compute);
IMPLEMENT_SHADER_TYPE4_WITH_TEMPLATE_PREFIX(template<>, RENDERCORE_API, FClearReplacementCS_StructuredBuffer_Uint_Zero		, SF_Compute);
IMPLEMENT_SHADER_TYPE4_WITH_TEMPLATE_PREFIX(template<>, RENDERCORE_API, FClearReplacementCS_Texture2DArray_Uint_Zero		, SF_Compute);
IMPLEMENT_SHADER_TYPE4_WITH_TEMPLATE_PREFIX(template<>, RENDERCORE_API, FClearReplacementCS_Buffer_Uint						, SF_Compute);
IMPLEMENT_SHADER_TYPE4_WITH_TEMPLATE_PREFIX(template<>, RENDERCORE_API, FClearReplacementCS_LargeBuffer_Uint				, SF_Compute);
IMPLEMENT_SHADER_TYPE4_WITH_TEMPLATE_PREFIX(template<>, RENDERCORE_API, FClearReplacementCS_StructuredBuffer_Uint			, SF_Compute);
IMPLEMENT_SHADER_TYPE4_WITH_TEMPLATE_PREFIX(template<>, RENDERCORE_API, FClearReplacementCS_Texture2DArray_Uint				, SF_Compute);
IMPLEMENT_SHADER_TYPE4_WITH_TEMPLATE_PREFIX(template<>, RENDERCORE_API, FClearReplacementCS_Texture3D_Float4				, SF_Compute);
IMPLEMENT_SHADER_TYPE4_WITH_TEMPLATE_PREFIX(template<>, RENDERCORE_API, FClearReplacementCS_Texture2D_Float4				, SF_Compute);
IMPLEMENT_SHADER_TYPE4_WITH_TEMPLATE_PREFIX(template<>, RENDERCORE_API, FClearReplacementCS_Texture2DArray_Float4			, SF_Compute);
IMPLEMENT_SHADER_TYPE4_WITH_TEMPLATE_PREFIX(template<>, RENDERCORE_API, FClearReplacementCS_Texture3D_Uint4					, SF_Compute);
IMPLEMENT_SHADER_TYPE4_WITH_TEMPLATE_PREFIX(template<>, RENDERCORE_API, FClearReplacementCS_Texture2D_Uint4					, SF_Compute);
IMPLEMENT_SHADER_TYPE4_WITH_TEMPLATE_PREFIX(template<>, RENDERCORE_API, FClearReplacementCS_Texture2DArray_Uint4			, SF_Compute);
IMPLEMENT_SHADER_TYPE4_WITH_TEMPLATE_PREFIX(template<>, RENDERCORE_API, FClearReplacementCS_Texture3D_Uint4_Bounds			, SF_Compute);
IMPLEMENT_SHADER_TYPE4_WITH_TEMPLATE_PREFIX(template<>, RENDERCORE_API, FClearReplacementCS_Texture2D_Uint4_Bounds			, SF_Compute);
IMPLEMENT_SHADER_TYPE4_WITH_TEMPLATE_PREFIX(template<>, RENDERCORE_API, FClearReplacementCS_Texture2DArray_Uint4_Bounds		, SF_Compute);
IMPLEMENT_SHADER_TYPE4_WITH_TEMPLATE_PREFIX(template<>, RENDERCORE_API, FClearReplacementCS_Buffer_Sint4_Bounds				, SF_Compute);
IMPLEMENT_SHADER_TYPE4_WITH_TEMPLATE_PREFIX(template<>, RENDERCORE_API, FClearReplacementCS_LargeBuffer_Sint4_Bounds		, SF_Compute);
IMPLEMENT_SHADER_TYPE4_WITH_TEMPLATE_PREFIX(template<>, RENDERCORE_API, FClearReplacementCS_StructuredBuffer_Sint4_Bounds	, SF_Compute);
IMPLEMENT_SHADER_TYPE4_WITH_TEMPLATE_PREFIX(template<>, RENDERCORE_API, FClearReplacementCS_Texture3D_Sint4_Bounds			, SF_Compute);
IMPLEMENT_SHADER_TYPE4_WITH_TEMPLATE_PREFIX(template<>, RENDERCORE_API, FClearReplacementCS_Texture2D_Sint4_Bounds			, SF_Compute);
IMPLEMENT_SHADER_TYPE4_WITH_TEMPLATE_PREFIX(template<>, RENDERCORE_API, FClearReplacementCS_Texture2DArray_Sint4_Bounds		, SF_Compute);

void CreateClearReplacementShaders()
{
	ForceInitGlobalShaderType<FClearReplacementCS_Buffer_Uint_Bounds>();

	ForceInitGlobalShaderType<FClearReplacementCS_Buffer_Float_Bounds>();
	ForceInitGlobalShaderType<FClearReplacementCS_Buffer_Float4_Bounds>();
	ForceInitGlobalShaderType<FClearReplacementCS_LargeBuffer_Uint_Bounds>();
	ForceInitGlobalShaderType<FClearReplacementCS_LargeBuffer_Float_Bounds>();
	ForceInitGlobalShaderType<FClearReplacementCS_LargeBuffer_Float4_Bounds>();
	ForceInitGlobalShaderType<FClearReplacementCS_StructuredBuffer_Uint_Bounds>();
	ForceInitGlobalShaderType<FClearReplacementCS_StructuredBuffer_Float_Bounds>();
	ForceInitGlobalShaderType<FClearReplacementCS_StructuredBuffer_Float4_Bounds>();
	ForceInitGlobalShaderType<FClearReplacementCS_Texture3D_Float4_Bounds>();
	ForceInitGlobalShaderType<FClearReplacementCS_Texture2D_Float4_Bounds>();
	ForceInitGlobalShaderType<FClearReplacementCS_Texture2DArray_Float4_Bounds>();

	// Compute shaders for clearing each resource type. No bounds checks enabled.
	ForceInitGlobalShaderType<FClearReplacementCS_Buffer_Uint_Zero>();
	ForceInitGlobalShaderType<FClearReplacementCS_StructuredBuffer_Uint_Zero>();
	ForceInitGlobalShaderType<FClearReplacementCS_Texture2DArray_Uint_Zero>();
	ForceInitGlobalShaderType<FClearReplacementCS_Buffer_Uint>();
	ForceInitGlobalShaderType<FClearReplacementCS_LargeBuffer_Uint>();
	ForceInitGlobalShaderType<FClearReplacementCS_StructuredBuffer_Uint>();
	ForceInitGlobalShaderType<FClearReplacementCS_Texture2DArray_Uint>();

	ForceInitGlobalShaderType<FClearReplacementCS_Texture3D_Float4>();
	ForceInitGlobalShaderType<FClearReplacementCS_Texture2D_Float4>();
	ForceInitGlobalShaderType<FClearReplacementCS_Texture2DArray_Float4>();

	// Used by ClearUAV_T in ClearQuad.cpp
	ForceInitGlobalShaderType<FClearReplacementCS_Texture3D_Uint4>();
	ForceInitGlobalShaderType<FClearReplacementCS_Texture2D_Uint4>();
	ForceInitGlobalShaderType<FClearReplacementCS_Texture2DArray_Uint4>();
	ForceInitGlobalShaderType<FClearReplacementCS_Buffer_Uint4>();
	ForceInitGlobalShaderType<FClearReplacementCS_LargeBuffer_Uint4>();
	ForceInitGlobalShaderType<FClearReplacementCS_StructuredBuffer_Uint4>();

	ForceInitGlobalShaderType<FClearReplacementCS_Buffer_Uint4_Bounds>();
	ForceInitGlobalShaderType<FClearReplacementCS_LargeBuffer_Uint4_Bounds>();
	ForceInitGlobalShaderType<FClearReplacementCS_StructuredBuffer_Uint4_Bounds>();
	ForceInitGlobalShaderType<FClearReplacementCS_Texture3D_Uint4_Bounds>();
	ForceInitGlobalShaderType<FClearReplacementCS_Texture2D_Uint4_Bounds>();
	ForceInitGlobalShaderType<FClearReplacementCS_Texture2DArray_Uint4_Bounds>();

	ForceInitGlobalShaderType<FClearReplacementCS_Buffer_Sint4_Bounds>();
	ForceInitGlobalShaderType<FClearReplacementCS_LargeBuffer_Sint4_Bounds>();
	ForceInitGlobalShaderType<FClearReplacementCS_StructuredBuffer_Sint4_Bounds>();
	ForceInitGlobalShaderType<FClearReplacementCS_Texture3D_Sint4_Bounds>();
	ForceInitGlobalShaderType<FClearReplacementCS_Texture2D_Sint4_Bounds>();
	ForceInitGlobalShaderType<FClearReplacementCS_Texture2DArray_Sint4_Bounds>();
}
