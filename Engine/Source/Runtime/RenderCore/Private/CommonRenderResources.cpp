// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	DummyRenderResources.cpp: Implementations of frequently used render resources.
=============================================================================*/

#include "CommonRenderResources.h"
#include "Containers/DynamicRHIResourceArray.h"
#include "RHIResourceUtils.h"
#include "StereoRenderUtils.h"


TGlobalResource<FFilterVertexDeclaration, FRenderResource::EInitPhase::Pre> GFilterVertexDeclaration;
TGlobalResource<FEmptyVertexDeclaration, FRenderResource::EInitPhase::Pre> GEmptyVertexDeclaration;

TGlobalResource<FScreenRectangleVertexBuffer, FRenderResource::EInitPhase::Pre> GScreenRectangleVertexBuffer;
TGlobalResource<FScreenRectangleIndexBuffer, FRenderResource::EInitPhase::Pre> GScreenRectangleIndexBuffer;

IMPLEMENT_GLOBAL_SHADER(FScreenVertexShaderVS, "/Engine/Private/Tools/FullscreenVertexShader.usf", "MainVS", SF_Vertex);
IMPLEMENT_GLOBAL_SHADER(FInstancedScreenVertexShaderVS, "/Engine/Private/Tools/FullscreenVertexShader.usf", "MainVS", SF_Vertex);
IMPLEMENT_GLOBAL_SHADER(FMobileMultiViewVertexShaderVS, "/Engine/Private/Tools/FullscreenVertexShader.usf", "MainVS", SF_Vertex);
IMPLEMENT_GLOBAL_SHADER(FCopyRectPS, "/Engine/Private/ScreenPass.usf", "CopyRectPS", SF_Pixel);
IMPLEMENT_GLOBAL_SHADER(FCopyRectSrvPS, "/Engine/Private/ScreenPass.usf", "CopyRectPS", SF_Pixel);
IMPLEMENT_GLOBAL_SHADER(FImagePreTransformVS, "/Engine/Private/Tools/FullscreenVertexShader.usf", "MainForPreTransform", SF_Vertex);

bool FInstancedScreenVertexShaderVS::ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
{
	UE::StereoRenderUtils::FStereoShaderAspects Aspects(Parameters.Platform);
	return Aspects.IsInstancedStereoEnabled();
}

bool FMobileMultiViewVertexShaderVS::ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
{
	UE::StereoRenderUtils::FStereoShaderAspects Aspects(Parameters.Platform);
	return Aspects.IsMobileMultiViewEnabled();
}

static const FFilterVertex GScreenRectangleVertexBufferData[] =
{
	{ FVector4f(1, 1, 0, 1), FVector2f(1, 1) },
	{ FVector4f(0, 1, 0, 1), FVector2f(0, 1) },
	{ FVector4f(1, 0, 0, 1), FVector2f(1, 0) },
	{ FVector4f(0, 0, 0, 1), FVector2f(0, 0) },

	//The final two vertices are used for the triangle optimization (a single triangle spans the entire viewport )
	{ FVector4f(-1, 1, 0, 1), FVector2f(-1, 1) },
	{ FVector4f(1, -1, 0, 1), FVector2f(1, -1) },
};

void FScreenRectangleVertexBuffer::InitRHI(FRHICommandListBase& RHICmdList)
{
	// Create vertex buffer. Fill buffer with initial data upon creation
	VertexBufferRHI = UE::RHIResourceUtils::CreateVertexBufferFromArray(
		RHICmdList,
		TEXT("FScreenRectangleVertexBuffer"),
		EBufferUsageFlags::Static,
		MakeConstArrayView(GScreenRectangleVertexBufferData)
	);
}

static const uint16 GScreenRectangleIndexBufferData[] =
{
	0, 1, 2, 2, 1, 3,	// [0 .. 5]  Full screen quad with 2 triangles
	0, 4, 5,			// [6 .. 8]  Full screen triangle
	3, 2, 1				// [9 .. 11] Full screen rect defined with TL, TR, BL corners
};

void FScreenRectangleIndexBuffer::InitRHI(FRHICommandListBase& RHICmdList)
{
	// Create index buffer. Fill buffer with initial data upon creation
	IndexBufferRHI = UE::RHIResourceUtils::CreateIndexBufferFromArray(
		RHICmdList,
		TEXT("FScreenRectangleIndexBuffer"),
		EBufferUsageFlags::Static,
		MakeConstArrayView(GScreenRectangleIndexBufferData)
	);
}
