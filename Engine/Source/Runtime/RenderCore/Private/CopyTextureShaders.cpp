// Copyright Epic Games, Inc. All Rights Reserved.

#include "CopyTextureShaders.h"

IMPLEMENT_TYPE_LAYOUT(FCopyTextureCS);

#define IMPLEMENT_COPY_RESOURCE_SHADER(SrcType,DstType,ValueType)\
	typedef TCopyResourceCS<ECopyTextureResourceType::SrcType, ECopyTextureResourceType::DstType, ECopyTextureValueType::ValueType, 4> FCopyTextureCS_##SrcType##_##DstType##_##ValueType##4;\
	IMPLEMENT_SHADER_TYPE4_WITH_TEMPLATE_PREFIX(template<>, RENDERCORE_API, FCopyTextureCS_##SrcType##_##DstType##_##ValueType##4, SF_Compute);

#define IMPLEMENT_COPY_RESOURCE_SHADER_ALL_TYPES(SrcType,DstType)\
	IMPLEMENT_COPY_RESOURCE_SHADER(SrcType,DstType,Float)\
	IMPLEMENT_COPY_RESOURCE_SHADER(SrcType,DstType,Int32)\
	IMPLEMENT_COPY_RESOURCE_SHADER(SrcType,DstType,Uint32)

IMPLEMENT_COPY_RESOURCE_SHADER_ALL_TYPES(Texture2D     , Texture2D     );
IMPLEMENT_COPY_RESOURCE_SHADER_ALL_TYPES(Texture2D     , Texture2DArray);
IMPLEMENT_COPY_RESOURCE_SHADER_ALL_TYPES(Texture2D     , Texture3D     );
IMPLEMENT_COPY_RESOURCE_SHADER_ALL_TYPES(Texture2DArray, Texture2D     );
IMPLEMENT_COPY_RESOURCE_SHADER_ALL_TYPES(Texture2DArray, Texture2DArray);
IMPLEMENT_COPY_RESOURCE_SHADER_ALL_TYPES(Texture2DArray, Texture3D     );
IMPLEMENT_COPY_RESOURCE_SHADER_ALL_TYPES(Texture3D     , Texture2D     );
IMPLEMENT_COPY_RESOURCE_SHADER_ALL_TYPES(Texture3D     , Texture2DArray);
IMPLEMENT_COPY_RESOURCE_SHADER_ALL_TYPES(Texture3D     , Texture3D     );
