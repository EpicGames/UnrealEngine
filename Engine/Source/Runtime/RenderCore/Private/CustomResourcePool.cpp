// Copyright Epic Games, Inc. All Rights Reserved.

#include "CustomResourcePool.h"
#include "CoreGlobals.h"

ICustomResourcePool* GCustomResourcePool = nullptr;

void ICustomResourcePool::TickPoolElements(FRHICommandListImmediate& RHICmdList)
{
	check(IsInRenderingThread());
	if (GCustomResourcePool)
	{
		GCustomResourcePool->Tick(RHICmdList);
	}
}
