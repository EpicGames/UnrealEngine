// Copyright Epic Games, Inc. All Rights Reserved.

#include "DumpGPU.h"
#include "HAL/PlatformFileManager.h"
#include "HAL/PlatformMisc.h"
#include "HAL/PlatformOutputDevices.h"
#include "HAL/ThreadHeartBeat.h"
#include "Async/AsyncWork.h"
#include "Misc/App.h"
#include "Misc/FileHelper.h"
#include "Misc/WildcardString.h"
#include "Misc/OutputDeviceRedirector.h"
#include "Misc/CoreDelegates.h"
#include "RenderingThread.h"
#include "Runtime/Launch/Resources/Version.h"
#include "BuildSettings.h"
#include "Serialization/JsonTypes.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"
#include "GenericPlatform/GenericPlatformCrashContext.h"
#include "GenericPlatform/GenericPlatformDriver.h"
#include "RenderGraphUtils.h"
#include "GlobalShader.h"
#include "RHIUtilities.h"
#include "RHIValidation.h"
#include "RenderGraphPrivate.h"
#include "RenderThreadTimeoutControl.h"

DEFINE_LOG_CATEGORY_STATIC(LogDumpGPU, Log, All);

IDumpGPUUploadServiceProvider* IDumpGPUUploadServiceProvider::GProvider = nullptr;

FString IDumpGPUUploadServiceProvider::FDumpParameters::DumpServiceParametersFileContent() const
{
	TSharedPtr<FJsonObject> JsonObject = MakeShareable(new FJsonObject);
	JsonObject->SetStringField(TEXT("Type"), *Type);
	JsonObject->SetStringField(TEXT("Time"), *Time);
	JsonObject->SetStringField(TEXT("CompressionName"), *CompressionName.ToString());
	JsonObject->SetStringField(TEXT("CompressionFiles"), *CompressionFiles);

	FString OutputString;
	TSharedRef<TJsonWriter<TCHAR, TPrettyJsonPrintPolicy<TCHAR>>> Writer = TJsonWriterFactory<TCHAR, TPrettyJsonPrintPolicy<TCHAR>>::Create(&OutputString);
	FJsonSerializer::Serialize(JsonObject.ToSharedRef(), Writer);

	return OutputString;
}

bool IDumpGPUUploadServiceProvider::FDumpParameters::DumpServiceParametersFile() const
{
	FString FileName = LocalPath / kServiceFileName;
	FString OutputString = DumpServiceParametersFileContent();

	return FFileHelper::SaveStringToFile(OutputString, *FileName);
}

#if RDG_DUMP_RESOURCES

namespace
{

static TAutoConsoleVariable<FString> GDumpGPURootCVar(
	TEXT("r.DumpGPU.Root"),
	TEXT("*"),
	TEXT("Allows to filter the tree when using r.DumpGPU command, the pattern match is case sensitive."),
	ECVF_Default);

static TAutoConsoleVariable<int32> GDumpTextureCVar(
	TEXT("r.DumpGPU.Texture"), 2,
	TEXT("Whether to dump textures.\n")
	TEXT(" 0: Ignores all textures\n")
	TEXT(" 1: Dump only textures' descriptors\n")
	TEXT(" 2: Dump textures' descriptors and binaries (default)"),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> GDumpBufferCVar(
	TEXT("r.DumpGPU.Buffer"), 2,
	TEXT("Whether to dump buffer.\n")
	TEXT(" 0: Ignores all buffers\n")
	TEXT(" 1: Dump only buffers' descriptors\n")
	TEXT(" 2: Dump buffers' descriptors and binaries (default)"),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> GDumpMaxStagingSize(
	TEXT("r.DumpGPU.MaxStagingSize"), 64,
	TEXT("Maximum size of stating resource in MB (default=64)."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> GDumpGPUPassParameters(
	TEXT("r.DumpGPU.PassParameters"), 1,
	TEXT("Whether to dump the pass parameters."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> GDumpGPURedumpInputs(
	TEXT("r.DumpGPU.RedumpInputs"), 0,
	TEXT("Re-capture input resources at the point they are read, breaking linkage to any prior (stale) version.\n")
	TEXT("Useful when r.DumpGPU.Root excludes intermediate passes that modify a resource in-place. This would increase dumped memory footage.\n"),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<float> GDumpGPUDelay(
	TEXT("r.DumpGPU.Delay"), 0.0f,
	TEXT("Delay in seconds before dumping the frame."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> GDumpGPUFrameDelay(
	TEXT("r.DumpGPU.FrameDelay"), 0,
	TEXT("Delay in frames before dumping the frame."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> GDumpGPUFrameCount(
	TEXT("r.DumpGPU.FrameCount"), 1,
	TEXT("Number of consecutive frames to dump (default=1)."),
	ECVF_Default);

static TAutoConsoleVariable<int32> GDumpGPUCameraCut(
	TEXT("r.DumpGPU.CameraCut"), 0,
	TEXT("Whether to issue a camera cut on the first frame of the dump."),
	ECVF_Default);

static TAutoConsoleVariable<float> GDumpGPUFixedTickRate(
	TEXT("r.DumpGPU.FixedTickRate"), 0.0f,
	TEXT("Override the engine's tick rate to be fixed for every dumped frames (default=0)."),
	ECVF_Default);

static TAutoConsoleVariable<int32> GDumpGPUStream(
	TEXT("r.DumpGPU.Stream"), 0,
	TEXT("Asynchronously readback from GPU to disk.\n")
	TEXT(" 0: Synchronously copy from GPU to disk with extra carefulness to avoid OOM (default);\n")
	TEXT(" 1: Asynchronously copy from GPU to disk with dedicated staging resources pool. May run OOM. ")
	TEXT("Please consider using r.DumpGPU.Root to minimise amount of passes to stream and r.Test.SecondaryUpscaleOverride ")
	TEXT("to reduce resource size to minimise OOM and disk bandwidth bottleneck per frame."),
	ECVF_Default);

static TAutoConsoleVariable<int32> GDumpGPUDraws(
	TEXT("r.DumpGPU.Draws"), 0,
	TEXT("Whether to dump resource after each individual draw call (disabled by default)."),
	ECVF_Default);

static TAutoConsoleVariable<int32> GDumpGPUMask(
	TEXT("r.DumpGPU.Mask"), 1,
	TEXT("Whether to include GPU mask in the name of each Pass (has no effect unless system has multiple GPUs)."),
	ECVF_Default);

static TAutoConsoleVariable<int32> GDumpExploreCVar(
	TEXT("r.DumpGPU.Explore"), 1,
	TEXT("Whether to open file explorer to where the GPU dump on completion (enabled by default)."),
	ECVF_Default);

static TAutoConsoleVariable<int32> GDumpRenderingConsoleVariablesCVar(
	TEXT("r.DumpGPU.ConsoleVariables"), 1,
	TEXT("Whether to dump rendering console variables (enabled by default)."),
	ECVF_Default);

static TAutoConsoleVariable<int32> GDumpTestEnableDiskWrite(
	TEXT("r.DumpGPU.Test.EnableDiskWrite"), 1,
	TEXT("Main switch whether any files should be written to disk, used for r.DumpGPU automation tests to not fill up workers' hard drive."),
	ECVF_Default);

static TAutoConsoleVariable<int32> GDumpTestPrettifyResourceFileNames(
	TEXT("r.DumpGPU.Test.PrettifyResourceFileNames"), 0,
	TEXT("Whether the resource file names should include resource name. May increase the likelyness of running into Windows' filepath limit."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<FString> GDumpGPUDirectoryCVar(
	TEXT("r.DumpGPU.Directory"), TEXT(""),
	TEXT("Directory to dump to."),
	ECVF_Default);

static TAutoConsoleVariable<int32> GDumpGPUUploadCVar(
	TEXT("r.DumpGPU.Upload"), 1,
	TEXT("Allows to upload the GPU dump automatically if set-up."),
	ECVF_Default);

static TAutoConsoleVariable<int32> GDumpGPUUploadCompressResources(
	TEXT("r.DumpGPU.Upload.CompressResources"), 1,
	TEXT("Whether to compress resource binary.\n")
	TEXT(" 0: Disabled (default)\n")
	TEXT(" 1: Zlib\n")
	TEXT(" 2: GZip"),
	ECVF_Default);

// Although this cvar does not seams used in the C++ code base, it is dumped by DumpRenderingCVarsToCSV() and used by GPUDumpViewer.html.
static TAutoConsoleVariable<FString> GDumpGPUVisualizeResource(
	TEXT("r.DumpGPU.Viewer.Visualize"), TEXT(""),
	TEXT("Name of RDG output resource to automatically open in the dump viewer."),
	ECVF_Default);

static TAutoConsoleVariable<int32> GDumpGPUEnableLogWrite(
	TEXT("r.DumpGPU.EnableLogWrite"), 1,
	TEXT("Enables writing the log file to disk.\n")
	TEXT(" 0: Does not write log file to disk\n")
	TEXT(" 1: Logging writes are enabled (default)"),
	ECVF_Default);

} // namespace

#endif // RDG_DUMP_RESOURCES

namespace
{

class FDumpTextureCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FDumpTextureCS);
	SHADER_USE_PARAMETER_STRUCT(FDumpTextureCS, FGlobalShader);

	enum class ETextureType
	{
		Texture2DFloatNoMSAA,
		Texture2DUintNoMSAA,
		Texture2DDepthStencilNoMSAA,
		MAX
	};
	class FTextureTypeDim : SHADER_PERMUTATION_ENUM_CLASS("DIM_TEXTURE_TYPE", ETextureType);
	using FPermutationDomain = TShaderPermutationDomain<FTextureTypeDim>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_SRV(Texture2D, Texture)
		SHADER_PARAMETER_UAV(RWTexture2D, StagingOutput)
	END_SHADER_PARAMETER_STRUCT()
};

IMPLEMENT_GLOBAL_SHADER(FDumpTextureCS, "/Engine/Private/Tools/DumpTexture.usf", "MainCS", SF_Compute);

} // namespace

#if RDG_DUMP_RESOURCES

namespace
{

BEGIN_SHADER_PARAMETER_STRUCT(FDumpTexturePass, )
	SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, Texture)
	RDG_TEXTURE_ACCESS_DYNAMIC(TextureAccess)
END_SHADER_PARAMETER_STRUCT()

BEGIN_SHADER_PARAMETER_STRUCT(FDumpBufferPass, )
	RDG_BUFFER_ACCESS(Buffer, ERHIAccess::CopySrc)
END_SHADER_PARAMETER_STRUCT()

const FName GDumpGPUTextureFenceName("DumpGPU.TextureFence");
const FName GDumpGPUBufferFenceName("DumpGPU.BufferFence");

}

struct FUnifiedPassResourceFilter
{
	FWildcardString PassPattern;
	FWildcardString ResourcePattern;

	FUnifiedPassResourceFilter(const FString& InPassPattern, const FString& InResourcePattern) : PassPattern(InPassPattern), ResourcePattern(InResourcePattern) {}
};

class FRDGResourceDumpContext
{
public:
	static constexpr const TCHAR* kBaseDir = TEXT("Base/");
	static constexpr const TCHAR* kPassesDir = TEXT("Passes/");
	static constexpr const TCHAR* kResourcesDir = TEXT("Resources/");
	static constexpr const TCHAR* kStructuresDir = TEXT("Structures/");
	static constexpr const TCHAR* kStructuresMetadataDir = TEXT("StructuresMetadata/");

	bool bEnableDiskWrite = false;
	bool bUpload = false;
	bool bStream = false;
	float DeltaTime = 0.0f;
	int32 DumpedFrameId = 0;
	int32 FrameCount = 1;
	FName UploadResourceCompressionName;
	TArray<FUnifiedPassResourceFilter> UnifiedPassResourceFilter;
	FString DumpingDirectoryPath;
	FString Context;
	FDateTime Time;
	FGenericPlatformMemoryConstants MemoryConstants;
	FGenericPlatformMemoryStats MemoryStats;
	int32 ResourcesDumpPasses = 0;
	int32 ResourcesDumpExecutedPasses = 0;
	uint16 GraphBuilderIndex = 0;
	int32 PassesCount = 0;
	TMap<const FRDGResource*, const FRDGPass*> LastResourceVersion;
	TSet<const void*> IsDumpedToDisk;

	bool bOverrideFixedDeltaTime = false;
	double PreviousFixedDeltaTime = 0.0f;

	TAtomic<int32> MetadataFilesOpened = 0;
	TAtomic<int64> MetadataFilesWriteBytes = 0;
	TAtomic<int32> ParametersFilesOpened = 0;
	TAtomic<int64> ParametersFilesWriteBytes = 0;
	TAtomic<int32> ResourceBinaryFilesOpened = 0;
	TAtomic<int64> ResourceBinaryWriteBytes = 0;

	enum class ETimingBucket : uint8
	{
		RHIReadbackCommands,
		RHIReleaseResources,
		GPUWait,
		CPUPostProcessing,
		MetadataFileWrite,
		ParametersFileWrite,
		ResourceBinaryFileWrite,
		MAX
	};
	mutable TAtomic<double> TimingBucket[int32(ETimingBucket::MAX)];

	class FTimeBucketMeasure
	{
	public:
		FTimeBucketMeasure(FRDGResourceDumpContext* InDumpCtx, ETimingBucket InBucket)
			: DumpCtx(InDumpCtx)
			, Start(FPlatformTime::Cycles64())
			, Bucket(InBucket)
		{ }

		~FTimeBucketMeasure()
		{
			uint64 End = FPlatformTime::Cycles64();

			double Add = FPlatformTime::ToSeconds64(FMath::Max(End - Start, uint64(0)));
			double Old;
			do
			{
				Old = DumpCtx->TimingBucket[int32(Bucket)].Load();
			} while (!DumpCtx->TimingBucket[int32(Bucket)].CompareExchange(Old, Old + Add));
		}

	protected:
		FRDGResourceDumpContext* const DumpCtx;

	private:
		const uint64 Start;
		const ETimingBucket Bucket;
	};

	class FFileWriteCtx : public FTimeBucketMeasure
	{
	public:
		FFileWriteCtx(FRDGResourceDumpContext* InDumpCtx, ETimingBucket InBucket, int64 WriteSizeBytes, int32 FilesOpened = 1)
			: FTimeBucketMeasure(InDumpCtx, InBucket)
		{
			if (InBucket == ETimingBucket::MetadataFileWrite)
			{
				DumpCtx->MetadataFilesOpened += FilesOpened;
				DumpCtx->MetadataFilesWriteBytes += WriteSizeBytes;
			}
			else if (InBucket == ETimingBucket::ParametersFileWrite)
			{
				DumpCtx->ParametersFilesOpened += FilesOpened;
				DumpCtx->ParametersFilesWriteBytes += WriteSizeBytes;
			}
			else if (InBucket == ETimingBucket::ResourceBinaryFileWrite)
			{
				DumpCtx->ResourceBinaryFilesOpened += FilesOpened;
				DumpCtx->ResourceBinaryWriteBytes += WriteSizeBytes;
			}
			else
			{
				unimplemented();
			}
		}
	};

	bool bShowInExplore = false;

	FRDGResourceDumpContext()
	{
		for (int32 i = 0; i < int32(ETimingBucket::MAX); i++)
		{
			TimingBucket[i] = 0.0f;
		}
	}

	using FTaskCallback = TFunction<void()>;

	class FDumpGPUTask : public FNonAbandonableTask
	{
	public:
		FTaskCallback TaskCallback;

		FDumpGPUTask(FTaskCallback&& InTaskCallback)
			: TaskCallback(InTaskCallback)
		{ }

		void DoWork()
		{
			TaskCallback();
		}

		FORCEINLINE TStatId GetStatId() const
		{
			RETURN_QUICK_DECLARE_CYCLE_STAT(FDumpGPUTask, STATGROUP_ThreadPoolAsyncTasks);
		}
	};

	void KickOffAsyncTask(FTaskCallback&& Callback)
	{
		if (bStream)
		{
			(new FAutoDeleteAsyncTask<FDumpGPUTask>(MoveTemp(Callback)))->StartBackgroundTask();
		}
		else
		{
			Callback();
		}
	}

	IDumpGPUUploadServiceProvider::FDumpParameters GetDumpParameters() const
	{
		IDumpGPUUploadServiceProvider::FDumpParameters DumpServiceParameters;
		DumpServiceParameters.Type = TEXT("DumpGPU");
		DumpServiceParameters.LocalPath = DumpingDirectoryPath;
		DumpServiceParameters.Time = Time.ToString();
		return DumpServiceParameters;
	}

	FString GetDumpFullPath(const FString& DumpRelativeFileName) const
	{
		check(bEnableDiskWrite);
		return DumpingDirectoryPath / DumpRelativeFileName;
	}

	bool DumpStringToFile(FStringView OutputString, const FString& FileName, uint32 WriteFlags = FILEWRITE_None)
	{
		// Make it has if the write happened and was successful.
		if (!bEnableDiskWrite)
		{
			return true;
		}

		FString FullPath = GetDumpFullPath(FileName);

		FFileWriteCtx WriteCtx(this, ETimingBucket::MetadataFileWrite, OutputString.Len());
		return FFileHelper::SaveStringToFile(
			OutputString, *FullPath,
			FFileHelper::EEncodingOptions::ForceUTF8WithoutBOM, &IFileManager::Get(), WriteFlags);
	}

	bool DumpStatusToFile(FStringView StatusString)
	{
		UE_LOGF(LogDumpGPU, Display, "DumpGPU status = %ls", StatusString.GetData());
		return DumpStringToFile(StatusString, FString(FRDGResourceDumpContext::kBaseDir) / TEXT("Status.txt"));
	}

	bool DumpJsonToFile(const TSharedPtr<FJsonObject>& JsonObject, const FString& FileName, uint32 WriteFlags = FILEWRITE_None)
	{
		FString OutputString;
		TSharedRef<TJsonWriter<TCHAR, TPrettyJsonPrintPolicy<TCHAR>>> Writer = TJsonWriterFactory<TCHAR, TPrettyJsonPrintPolicy<TCHAR>>::Create(&OutputString);
		FJsonSerializer::Serialize(JsonObject.ToSharedRef(), Writer);

		return DumpStringToFile(OutputString, FileName, WriteFlags);
	}

	bool DumpBinaryToFile(TArrayView<const uint8> ArrayView, const FString& FileName, ETimingBucket Bucket)
	{
		// Make it has if the write happened and was successful.
		if (!bEnableDiskWrite)
		{
			return true;
		}

		FString FullPath = GetDumpFullPath(FileName);
		FFileWriteCtx WriteCtx(this, Bucket, ArrayView.Num());
		return FFileHelper::SaveArrayToFile(ArrayView, *FullPath);
	}

	bool DumpBinaryToFile(const uint8* Data, int64 DataByteSize, const FString& FileName, ETimingBucket Bucket)
	{
		// Make it has if the write happened and was successful.
		if (!bEnableDiskWrite)
		{
			return true;
		}

		FString FullPath = GetDumpFullPath(FileName);
		FFileWriteCtx WriteCtx(this, Bucket, DataByteSize);

		TUniquePtr<FArchive> Ar = TUniquePtr<FArchive>(IFileManager::Get().CreateFileWriter(*FullPath, /* WriteFlags = */ 0));
		if (!Ar)
		{
			return false;
		}
		Ar->Serialize((void*)Data, DataByteSize);

		// Always explicitly close to catch errors from flush/close
		Ar->Close();

		return !Ar->IsError() && !Ar->IsCriticalError();
	}

	bool DumpResourceBinaryToFile(const uint8* UncompressedData, int64 UncompressedSize, const FString& FileName)
	{
		return DumpBinaryToFile(UncompressedData, UncompressedSize, FileName, ETimingBucket::ResourceBinaryFileWrite);
	}

	bool IsUnsafeToDumpResource(SIZE_T ResourceByteSize, float DumpMemoryMultiplier) const
	{
		const uint64 AproximatedStagingMemoryRequired = uint64(double(ResourceByteSize) * DumpMemoryMultiplier);

		// Skip AvailableVirtual if it's not supported (reported as exactly 0)
		const uint64 MaxMemoryAvailable = (MemoryStats.AvailableVirtual > 0) ? FMath::Min(MemoryStats.AvailablePhysical, MemoryStats.AvailableVirtual) : MemoryStats.AvailablePhysical;

		return AproximatedStagingMemoryRequired > MaxMemoryAvailable;
	}

	template<typename T>
	FString PtrToString(const T* Ptr)
	{
		if (Ptr == nullptr)
		{
			return TEXT("00000000000000000000");
		}
		return FString::Printf(TEXT("%04u%016" UINT64_x_FMT), uint32(GraphBuilderIndex), static_cast<uint64>(reinterpret_cast<size_t>(Ptr)));
	}

	FString GetUniqueResourceName(const FRDGResource* Resource)
	{
		if (GDumpTestPrettifyResourceFileNames.GetValueOnRenderThread())
		{
			FString UniqueResourceName = FString::Printf(TEXT("%s.%s"), Resource->Name, *PtrToString(Resource));
			UniqueResourceName.ReplaceInline(TEXT("/"), TEXT(""));
			UniqueResourceName.ReplaceInline(TEXT("\\"), TEXT(""));
			return UniqueResourceName;
		}
		return PtrToString(Resource);
	}

	FString GetUniqueSubResourceName(const FRDGTextureSRVDesc& SubResourceDesc)
	{
		check(SubResourceDesc.NumMipLevels == 1);
		check(!SubResourceDesc.Texture->Desc.IsTextureArray() || SubResourceDesc.NumArraySlices == 1);

		FString UniqueResourceName = GetUniqueResourceName(SubResourceDesc.Texture);

		if (SubResourceDesc.Texture->Desc.IsTextureArray())
		{
			UniqueResourceName += FString::Printf(TEXT(".[%d]"), SubResourceDesc.FirstArraySlice);
		}

		if (SubResourceDesc.Format == PF_X24_G8)
		{
			return FString::Printf(TEXT("%s.stencil"), *UniqueResourceName);
		}

		return FString::Printf(TEXT("%s.mip%d"), *UniqueResourceName, SubResourceDesc.MipLevel);
	}

	void ReleaseRHIResources(FRHICommandListImmediate& RHICmdList)
	{
		FTimeBucketMeasure TimeBucketMeasure(this, ETimingBucket::RHIReleaseResources);

		// Flush the RHI resource memory so the readback memory can be fully reused in the next resource dump.
		{
			constexpr bool bFlushResources = true;
			RHICmdList.SubmitAndBlockUntilGPUIdle(bFlushResources);
		}
	}

	void UpdatePassProgress()
	{
		ResourcesDumpExecutedPasses++;

		if (ResourcesDumpExecutedPasses % 10 == 0)
		{
			UE_LOGF(LogDumpGPU, Display, "Dumped %d / %d resources", ResourcesDumpExecutedPasses, ResourcesDumpPasses);
		}
	}

	void DumpRenderingCVarsToCSV() const
	{
		FString FileName = GetDumpFullPath(FString(FRDGResourceDumpContext::kBaseDir) / TEXT("ConsoleVariables.csv"));

		TUniquePtr<FArchive> Ar = TUniquePtr<FArchive>(IFileManager::Get().CreateFileWriter(*FileName));
		auto OnConsoleVariable = [&Ar](const TCHAR* CVarName, IConsoleObject* ConsoleObj)
		{
			if (ConsoleObj->TestFlags(ECVF_Unregistered))
			{
				return;
			}

			const IConsoleVariable* CVar = ConsoleObj->AsVariable();
			if (!CVar)
			{
				return;
			}

			EConsoleVariableFlags CVarFlags = CVar->GetFlags();

			const TCHAR* Type = nullptr;
			if (CVar->IsVariableBool())
			{
				Type = TEXT("bool");
			}
			else if (CVar->IsVariableInt())
			{
				Type = TEXT("int32");
			}
			else if (CVar->IsVariableFloat())
			{
				Type = TEXT("float");
			}
			else if (CVar->IsVariableString())
			{
				Type = TEXT("FString");
			}
			else
			{
				return;
			}

			const TCHAR* SetByName = GetConsoleVariableSetByName(CVarFlags);

			FString Value = CVar->GetString();

			FString CSVLine = FString::Printf(TEXT("%s,%s,%s,%s\n"), CVarName, Type, SetByName, *Value);
			FStringView CSVLineView(CSVLine);

			auto Src = StringCast<ANSICHAR>(CSVLineView.GetData(), CSVLineView.Len());
			Ar->Serialize((ANSICHAR*)Src.Get(), Src.Length() * sizeof(ANSICHAR));
		};

		bool bSuccess = false;
		if (Ar)
		{
			{
				FString CSVLine = "CVar,Type,SetBy,Value\n";
				FStringView CSVLineView(CSVLine);

				auto Src = StringCast<ANSICHAR>(CSVLineView.GetData(), CSVLineView.Len());
				Ar->Serialize((ANSICHAR*)Src.Get(), Src.Length() * sizeof(ANSICHAR));
			}

			if (GDumpRenderingConsoleVariablesCVar.GetValueOnAnyThread() != 0)
			{
				IConsoleManager::Get().ForEachConsoleObjectThatStartsWith(FConsoleObjectVisitor::CreateLambda(OnConsoleVariable), TEXT(""));
			}
			else
			{
				IConsoleManager::Get().ForEachConsoleObjectThatStartsWith(FConsoleObjectVisitor::CreateLambda(OnConsoleVariable), TEXT("r.DumpGPU."));
			}
			// Always explicitly close to catch errors from flush/close
			Ar->Close();

			bSuccess = !Ar->IsError() && !Ar->IsCriticalError();
		}

		if (bSuccess)
		{
			UE_LOGF(LogDumpGPU, Display, "DumpGPU dumped rendering cvars to %ls.", *FileName);
		}
		else
		{
			UE_LOGF(LogDumpGPU, Error, "DumpGPU had a file error when dumping rendering cvars to %ls.", *FileName);
		}
	}

	template<typename T>
	bool IsDumped(const T* Ptr) const
	{
		return IsDumpedToDisk.Contains(static_cast<const void*>(Ptr));
	}

	template<typename T>
	void SetDumped(const T* Ptr)
	{
		check(!IsDumped(Ptr));
		if (IsDumpedToDisk.Num() % 1024 == 0)
		{
			IsDumpedToDisk.Reserve(IsDumpedToDisk.Num() + 1024);
		}
		IsDumpedToDisk.Add(static_cast<const void*>(Ptr));
	}

	bool HasResourceEverBeenDumped(const FRDGResource* Resource) const
	{
		return LastResourceVersion.Contains(Resource);
	}

	void RegisterResourceAsDumped(
		const FRDGResource* Resource,
		const FRDGPass* LastModifyingPass)
	{
		check(!LastResourceVersion.Contains(Resource));
		if (LastResourceVersion.Num() % 1024 == 0)
		{
			LastResourceVersion.Reserve(LastResourceVersion.Num() + 1024);
		}
		LastResourceVersion.Add(Resource, LastModifyingPass);
	}

	void UpdateResourceLastModifyingPass(
		const FRDGResource* Resource,
		const FRDGPass* LastModifyingPass)
	{
		check(LastResourceVersion.Contains(Resource));
		check(LastModifyingPass);
		LastResourceVersion[Resource] = LastModifyingPass;
	}

	FString ToJson(EPixelFormat Format)
	{
		FString PixelFormat = GPixelFormats[Format].Name;
		if (!PixelFormat.StartsWith(TEXT("PF_")))
		{
			PixelFormat = FString::Printf(TEXT("PF_%s"), GPixelFormats[Format].Name);
		}
		return PixelFormat;
	}

	TSharedPtr<FJsonObject> ToJson(const FShaderParametersMetadata::FMember& Member)
	{
		TSharedPtr<FJsonObject> JsonObject = MakeShareable(new FJsonObject);
		JsonObject->SetStringField(TEXT("Name"), Member.GetName());
		JsonObject->SetStringField(TEXT("ShaderType"), Member.GetShaderType());
		JsonObject->SetNumberField(TEXT("FileLine"), Member.GetFileLine());
		JsonObject->SetNumberField(TEXT("Offset"), Member.GetOffset());
		JsonObject->SetStringField(TEXT("BaseType"), GetUniformBufferBaseTypeString(Member.GetBaseType()));
		JsonObject->SetNumberField(TEXT("Precision"), Member.GetPrecision());
		JsonObject->SetNumberField(TEXT("NumRows"), Member.GetNumRows());
		JsonObject->SetNumberField(TEXT("NumColumns"), Member.GetNumColumns());
		JsonObject->SetNumberField(TEXT("NumElements"), Member.GetNumElements());
		JsonObject->SetStringField(TEXT("StructMetadata"), *PtrToString(Member.GetStructMetadata()));
		return JsonObject;
	}

	TSharedPtr<FJsonObject> ToJson(const FShaderParametersMetadata* Metadata)
	{
		TSharedPtr<FJsonObject> JsonObject = MakeShareable(new FJsonObject);
		JsonObject->SetStringField(TEXT("StructTypeName"), Metadata->GetStructTypeName());
		JsonObject->SetStringField(TEXT("ShaderVariableName"), Metadata->GetShaderVariableName());
		JsonObject->SetStringField(TEXT("FileName"), Metadata->GetFileName());
		JsonObject->SetNumberField(TEXT("FileLine"), Metadata->GetFileLine());
		JsonObject->SetNumberField(TEXT("Size"), Metadata->GetSize());
		//JsonObject->SetNumberField(TEXT("UseCase"), Metadata->GetUseCase());

		{
			TArray<TSharedPtr<FJsonValue>> Members;
			for (const FShaderParametersMetadata::FMember& Member : Metadata->GetMembers())
			{
				Members.Add(MakeShared<FJsonValueObject>(ToJson(Member)));
			}
			JsonObject->SetArrayField(TEXT("Members"), Members);
		}

		return JsonObject;
	}

	TSharedPtr<FJsonObject> ToJson(const FString& UniqueResourceName, const TCHAR* Name, const FRDGTextureDesc& Desc)
	{
		FString PixelFormat = ToJson(Desc.Format);

		int32 ResourceByteSize = Desc.Extent.X * Desc.Extent.Y * Desc.Depth * Desc.ArraySize * Desc.NumSamples * GPixelFormats[Desc.Format].BlockBytes;

		TSharedPtr<FJsonObject> JsonObject = MakeShareable(new FJsonObject);
		JsonObject->SetStringField(TEXT("Name"), Name);
		JsonObject->SetStringField(TEXT("UniqueResourceName"), UniqueResourceName);
		JsonObject->SetNumberField(TEXT("ByteSize"), ResourceByteSize);
		JsonObject->SetStringField(TEXT("Desc"), TEXT("FRDGTextureDesc"));
		JsonObject->SetStringField(TEXT("Type"), GetTextureDimensionString(Desc.Dimension));
		JsonObject->SetStringField(TEXT("Format"), *PixelFormat);
		JsonObject->SetNumberField(TEXT("ExtentX"), Desc.Extent.X);
		JsonObject->SetNumberField(TEXT("ExtentY"), Desc.Extent.Y);
		JsonObject->SetNumberField(TEXT("Depth"), Desc.Depth);
		JsonObject->SetNumberField(TEXT("ArraySize"), Desc.ArraySize);
		JsonObject->SetNumberField(TEXT("NumMips"), Desc.NumMips);
		JsonObject->SetNumberField(TEXT("NumSamples"), Desc.NumSamples);

		{
			TArray<TSharedPtr<FJsonValue>> FlagsNames;
			for (uint64 BitId = 0; BitId < uint64(8 * sizeof(ETextureCreateFlags)); BitId++)
			{
				ETextureCreateFlags Flag = ETextureCreateFlags(uint64(1) << BitId);
				if (EnumHasAnyFlags(Desc.Flags, Flag))
				{
					FlagsNames.Add(MakeShareable(new FJsonValueString(GetTextureCreateFlagString(Flag))));
				}
			}
			JsonObject->SetArrayField(TEXT("Flags"), FlagsNames);
		}

		return JsonObject;
	}

	TSharedPtr<FJsonObject> ToJson(const FString& UniqueResourceName, const TCHAR* Name, const FRDGBufferDesc& Desc, uint32 ResourceByteSize, uint32 ReservedSize)
	{
		TSharedPtr<FJsonObject> JsonObject = MakeShareable(new FJsonObject);
		JsonObject->SetStringField(TEXT("Name"), Name);
		JsonObject->SetStringField(TEXT("UniqueResourceName"), UniqueResourceName);
		JsonObject->SetNumberField(TEXT("ByteSize"), ResourceByteSize);
		JsonObject->SetStringField(TEXT("Desc"), TEXT("FRDGBufferDesc"));
		JsonObject->SetNumberField(TEXT("BytesPerElement"), Desc.BytesPerElement);
		JsonObject->SetNumberField(TEXT("NumElements"), Desc.NumElements);
		JsonObject->SetStringField(TEXT("Metadata"), *PtrToString(Desc.Metadata));

		if (EnumHasAnyFlags(Desc.Usage, EBufferUsageFlags::ReservedResource))
		{
			JsonObject->SetNumberField(TEXT("ReservedSize"), ReservedSize);
		}

		{
			TArray<TSharedPtr<FJsonValue>> UsageNames;
			for (uint64 BitId = 0; BitId < uint64(8 * sizeof(EBufferUsageFlags)); BitId++)
			{
				EBufferUsageFlags Flag = EBufferUsageFlags(uint64(1) << BitId);
				if (EnumHasAnyFlags(Desc.Usage, Flag))
				{
					UsageNames.Add(MakeShareable(new FJsonValueString(GetBufferUsageFlagString(Flag))));
				}
			}
			JsonObject->SetArrayField(TEXT("Usage"), UsageNames);
		}
		return JsonObject;
	}

	void Dump(const FShaderParametersMetadata* Metadata)
	{
		if (IsDumped(Metadata))
		{
			return;
		}

		TSharedPtr<FJsonObject> JsonObject = ToJson(Metadata);
		FString JsonPath = kStructuresMetadataDir / PtrToString(Metadata) + TEXT(".json");
		DumpJsonToFile(JsonObject, JsonPath);
		SetDumped(Metadata);

		// Dump dependencies
		Metadata->IterateStructureMetadataDependencies(
		[&](const FShaderParametersMetadata* Struct)
		{
			if (Struct)
			{
				Dump(Struct);
			}
		});
	}

	struct FTextureSubresourceDumpDesc
	{
		FIntPoint SubResourceExtent = FIntPoint(0, 0);
		SIZE_T ByteSize = 0;
		bool bPreprocessForStaging = false;
		FDumpTextureCS::ETextureType DumpTextureType = FDumpTextureCS::ETextureType::MAX;
		EPixelFormat PreprocessedPixelFormat;
		EPixelFormat PostprocessedPixelFormat;

		bool IsDumpSupported() const
		{
			return ByteSize != SIZE_T(0);
		}
	};
	
	FTextureSubresourceDumpDesc TranslateSubresourceDumpDesc(FRDGTextureSRVDesc SubresourceDesc)
	{
		check(SubresourceDesc.NumMipLevels == 1);
		check(!SubresourceDesc.Texture->Desc.IsTextureArray() || SubresourceDesc.NumArraySlices == 1);

		const FRDGTextureDesc& Desc = SubresourceDesc.Texture->Desc;

		FTextureSubresourceDumpDesc SubresourceDumpDesc;
		SubresourceDumpDesc.PostprocessedPixelFormat = Desc.Format;

		bool bIsUnsupported = false;

		// We support uncompressing certain BC formats, as the DumpGPU viewer doesn't support compressed formats.  Compression
		// is used by Lumen, so it's extremely useful to preview.  Additional code below selects the uncompressed format (BC4 is
		// scalar, so we use PF_R32_FLOAT, BC5 is dual channel, so it uses PF_G16R16F, etc).  If you add a format here, you must
		// also update Engine\Extras\GPUDumpViewer\GPUDumpViewer.html (see "translate_subresource_desc").
		if ((GPixelFormats[Desc.Format].BlockSizeX != 1 ||
			 GPixelFormats[Desc.Format].BlockSizeY != 1 ||
			 GPixelFormats[Desc.Format].BlockSizeZ != 1) &&
			!(Desc.Format == PF_BC4 ||
			  Desc.Format == PF_BC5 ||
			  Desc.Format == PF_BC6H ||
			  Desc.Format == PF_BC7))
		{
			bIsUnsupported = true;
		}

		if (!bIsUnsupported && Desc.IsTexture2D() && !Desc.IsMultisample())
		{
			SubresourceDumpDesc.SubResourceExtent.X = Desc.Extent.X >> SubresourceDesc.MipLevel;
			SubresourceDumpDesc.SubResourceExtent.Y = Desc.Extent.Y >> SubresourceDesc.MipLevel;

			if (IsUintFormat(Desc.Format) || IsSintFormat(Desc.Format))
			{
				SubresourceDumpDesc.DumpTextureType = FDumpTextureCS::ETextureType::Texture2DUintNoMSAA;
			}
			else
			{
				SubresourceDumpDesc.DumpTextureType = FDumpTextureCS::ETextureType::Texture2DFloatNoMSAA;
			}

			if (SubresourceDesc.Format == PF_X24_G8)
			{
				SubresourceDumpDesc.PostprocessedPixelFormat = PF_R8_UINT;
				SubresourceDumpDesc.DumpTextureType = FDumpTextureCS::ETextureType::Texture2DDepthStencilNoMSAA;
			}
			else if (Desc.Format == PF_DepthStencil)
			{
				SubresourceDumpDesc.PostprocessedPixelFormat = PF_R32_FLOAT;
				SubresourceDumpDesc.DumpTextureType = FDumpTextureCS::ETextureType::Texture2DFloatNoMSAA;
			}
			else if (Desc.Format == PF_ShadowDepth)
			{
				SubresourceDumpDesc.PostprocessedPixelFormat = PF_R32_FLOAT;
				SubresourceDumpDesc.DumpTextureType = FDumpTextureCS::ETextureType::Texture2DFloatNoMSAA;
			}
			else if (Desc.Format == PF_D24)
			{
				SubresourceDumpDesc.PostprocessedPixelFormat = PF_R32_FLOAT;
				SubresourceDumpDesc.DumpTextureType = FDumpTextureCS::ETextureType::Texture2DFloatNoMSAA;
			}
			else if (Desc.Format == PF_BC4)
			{
				SubresourceDumpDesc.PostprocessedPixelFormat = PF_R32_FLOAT;
				SubresourceDumpDesc.DumpTextureType = FDumpTextureCS::ETextureType::Texture2DFloatNoMSAA;
			}
			else if (Desc.Format == PF_BC5)
			{
				SubresourceDumpDesc.PostprocessedPixelFormat = PF_G16R16F;
				SubresourceDumpDesc.DumpTextureType = FDumpTextureCS::ETextureType::Texture2DFloatNoMSAA;
			}
			else if ((Desc.Format == PF_BC6H) || (Desc.Format == PF_BC7))
			{
				SubresourceDumpDesc.PostprocessedPixelFormat = PF_FloatRGBA;
				SubresourceDumpDesc.DumpTextureType = FDumpTextureCS::ETextureType::Texture2DFloatNoMSAA;
			}
		}

		// Whether the subresource need preprocessing pass before copy into staging.
		{
			// If need a pixel format conversion, use a pixel shader to do it.
			SubresourceDumpDesc.bPreprocessForStaging |= SubresourceDumpDesc.PostprocessedPixelFormat != Desc.Format;

			// If the texture has a mip chain, use pixel shader to correctly copy the right mip given RHI doesn't support copy from mip levels. Also on Mip 0 to avoid bugs on D3D11
			SubresourceDumpDesc.bPreprocessForStaging |= SubresourceDesc.Texture->Desc.NumMips > 1;

			// If the texture is an array, use pixel shader to correctly copy the right slice given RHI doesn't support copy from slices
			SubresourceDumpDesc.bPreprocessForStaging |= SubresourceDesc.Texture->Desc.IsTextureArray();

			// Reads the texture using a shader if it has SkipTracking to avoid transitioning it in anyway from its SRV read state.
			SubresourceDumpDesc.bPreprocessForStaging |= EnumHasAnyFlags(SubresourceDesc.Texture->Flags, ERDGTextureFlags::SkipTracking);
		}

		// Some RHIs (GL) only support 32Bit single channel images as CS output
		SubresourceDumpDesc.PreprocessedPixelFormat = SubresourceDumpDesc.PostprocessedPixelFormat;
		if (SubresourceDumpDesc.bPreprocessForStaging)
		{
			if (IsOpenGLPlatform(GMaxRHIShaderPlatform) &&
				GPixelFormats[SubresourceDumpDesc.PostprocessedPixelFormat].NumComponents == 1 &&
				GPixelFormats[SubresourceDumpDesc.PostprocessedPixelFormat].BlockBytes < 4)
			{
				SubresourceDumpDesc.PreprocessedPixelFormat = PF_R32_UINT;
			}
		}

		SubresourceDumpDesc.ByteSize = SIZE_T(SubresourceDumpDesc.SubResourceExtent.X) * SIZE_T(SubresourceDumpDesc.SubResourceExtent.Y) * SIZE_T(GPixelFormats[SubresourceDumpDesc.PreprocessedPixelFormat].BlockBytes);

		return SubresourceDumpDesc;
	}

	void PostProcessTexture(
		const FTextureSubresourceDumpDesc& SubresourceDumpDesc,
		void* Content,
		int32 RowPitchInPixels,
		int32 ColumnPitchInPixels,
		TArray64<uint8>& Array)
	{
		FTimeBucketMeasure TimeBucketMeasure(this, ETimingBucket::CPUPostProcessing);

		Array.SetNumUninitialized(SubresourceDumpDesc.ByteSize);

		SIZE_T BytePerPixel = SIZE_T(GPixelFormats[SubresourceDumpDesc.PreprocessedPixelFormat].BlockBytes);

		const uint8* SrcData = static_cast<const uint8*>(Content);

		for (int32 y = 0; y < SubresourceDumpDesc.SubResourceExtent.Y; y++)
		{
			// Keep the data to be top left corner.
			const uint8* SrcPos = SrcData + SIZE_T(y) * SIZE_T(RowPitchInPixels) * BytePerPixel;
			uint8* DstPos = (&Array[0]) + SIZE_T(y) * SIZE_T(SubresourceDumpDesc.SubResourceExtent.X) * BytePerPixel;

			FPlatformMemory::Memmove(DstPos, SrcPos, SIZE_T(SubresourceDumpDesc.SubResourceExtent.X) * BytePerPixel);
		}

		if (SubresourceDumpDesc.PreprocessedPixelFormat != SubresourceDumpDesc.PostprocessedPixelFormat)
		{
			// Convert 32Bit values back to 16 or 8bit
			const int32 DstPixelNumBytes = GPixelFormats[SubresourceDumpDesc.PostprocessedPixelFormat].BlockBytes;
			const uint32* SrcData32 = (const uint32*)Array.GetData();
			uint8* DstData8 = Array.GetData();
			uint16* DstData16 = (uint16*)Array.GetData();

			for (int32 Index = 0; Index < Array.Num() / 4; Index++)
			{
				uint32 Value32 = SrcData32[Index];
				if (DstPixelNumBytes == 2)
				{
					DstData16[Index] = (uint16)Value32;
				}
				else
				{
					DstData8[Index] = (uint8)Value32;
				}
			}

			Array.SetNum(Array.Num() / (4 / DstPixelNumBytes));
		}
	}

	enum class FStagingResourceStatus
	{
		Unused,
		RenderingIntermediaryOnly,
		Rendering,
		WritingToDisk,
		WritingToDiskComplete,
	};

	struct FStagingPoolEntry
	{
		FStagingResourceStatus Status = FStagingResourceStatus::Unused;

		FString DumpFilePath;
		int32 GPUIndex = 0;
		FGPUFenceRHIRef WriteToResourceComplete;
		FEvent* WriteToDiskComplete = nullptr;
	};

	struct FStagingTexturePoolEntry : public FStagingPoolEntry
	{
		FTextureSubresourceDumpDesc SubresourceDumpDesc;
		FRHITextureCreateDesc Desc;
		FTextureRHIRef Texture;
		FUnorderedAccessViewRHIRef UAV;
	};

	struct FStagingBufferPoolEntry : public FStagingPoolEntry
	{
		uint32 ByteSize = 0;
		FStagingBufferRHIRef StagingBuffer;
	};

	TArray<TUniquePtr<FStagingTexturePoolEntry>> StagingTexturePool;
	TArray<TUniquePtr<FStagingBufferPoolEntry>> StagingBufferPool;

	void CreateStagingTexture(FRHICommandListBase& RHICmdList, const FRHITextureCreateDesc& Desc, FTextureRHIRef* Texture)
	{
		check(IsInRenderingThread());
		(*Texture) = RHICmdList.CreateTexture(Desc);
		(*Texture)->DisableLifetimeExtension();
	}

	FShaderResourceViewRHIRef CreateSRVNoLifetimeExtension(FRHICommandList& RHICmdList, FRHITexture* Texture, FRHIViewDesc::FTextureSRV::FInitializer const& CreateInfo)
	{
		FShaderResourceViewRHIRef SRV = RHICmdList.CreateShaderResourceView(Texture, CreateInfo);
		SRV->DisableLifetimeExtension();
		return SRV;
	}

	FUnorderedAccessViewRHIRef CreateUAVNoLifetimeExtension(FRHICommandList& RHICmdList, FRHITexture* Texture, uint32 MipLevel)
	{
		FUnorderedAccessViewRHIRef UAV = RHICmdList.CreateUnorderedAccessView(
			Texture,
			FRHIViewDesc::CreateTextureUAV()
				.SetDimensionFromTexture(Texture)
				.SetMipLevel(uint8(MipLevel)));
		UAV->DisableLifetimeExtension();
		return UAV;
	}

	FStagingTexturePoolEntry& ReuseStagingTexture(
		FRHICommandListImmediate& RHICmdList,
		const FRHITextureCreateDesc& Desc,
		ERHIAccess Access,
		FTextureRHIRef* Texture,
		FUnorderedAccessViewRHIRef* UAV)
	{
		check(IsInRenderingThread());
		check(bStream);
		check(Access == ERHIAccess::CopyDest || Access == ERHIAccess::UAVCompute);

		for (TUniquePtr<FStagingTexturePoolEntry>& Entry : StagingTexturePool)
		{
			if (Entry->Desc != Desc)
			{
				continue;
			}

			if (Access == ERHIAccess::CopyDest)
			{
				if (Entry->Status == FStagingResourceStatus::Unused)
				{
					check(Entry->Texture.GetRefCount() == 1);
					*Texture = Entry->Texture;
					check(Entry->Texture.GetRefCount() == 2);
					Entry->Status = FStagingResourceStatus::Rendering;
					Entry->WriteToResourceComplete->Clear();
					return *Entry;
				}
			}
			else
			{
				if (Entry->Status == FStagingResourceStatus::RenderingIntermediaryOnly && Entry->Texture.GetRefCount() == 1)
				{
					*Texture = Entry->Texture;
					check(Entry->Texture.GetRefCount() == 2);
					check(!Entry->WriteToResourceComplete);
					return *Entry;
				}
			}
		}

		CreateStagingTexture(RHICmdList, Desc, /* out */ Texture);

		FStagingTexturePoolEntry* NewEntry = new FStagingTexturePoolEntry;
		NewEntry->Status = Access == ERHIAccess::CopyDest ? FStagingResourceStatus::Rendering : FStagingResourceStatus::RenderingIntermediaryOnly;
		NewEntry->Desc = Desc;
		NewEntry->Texture = *Texture;

		StagingTexturePool.Add(TUniquePtr<FStagingTexturePoolEntry>(NewEntry));

		RHICmdList.Transition(FRHITransitionInfo(*Texture, ERHIAccess::Unknown, Access));

		if (UAV)
		{
			check(Access == ERHIAccess::UAVCompute);
			NewEntry->UAV = CreateUAVNoLifetimeExtension(RHICmdList, *Texture, /* MipLevel = */ 0);
			*UAV = NewEntry->UAV;
		}
		else
		{
			check(Access == ERHIAccess::CopyDest);
			NewEntry->WriteToResourceComplete = RHICreateGPUFence(GDumpGPUTextureFenceName);
			NewEntry->WriteToResourceComplete->Clear();
		}

		return *NewEntry;
	}

	FStagingBufferPoolEntry& ReuseStagingBuffer(
		FRHICommandListImmediate& RHICmdList,
		uint32 ByteSize)
	{
		check(IsInRenderingThread());
		check(bStream);

		for (TUniquePtr<FStagingBufferPoolEntry>& Entry : StagingBufferPool)
		{
			if (Entry->ByteSize != ByteSize)
			{
				continue;
			}

			if (Entry->Status == FStagingResourceStatus::Unused)
			{
				Entry->Status = FStagingResourceStatus::Rendering;
				Entry->WriteToResourceComplete->Clear();
				return *Entry;
			}
		}

		FStagingBufferPoolEntry* NewEntry = new FStagingBufferPoolEntry;
		NewEntry->Status = FStagingResourceStatus::Rendering;
		NewEntry->ByteSize = ByteSize;
		NewEntry->StagingBuffer = RHICreateStagingBuffer();
		NewEntry->StagingBuffer->DisableLifetimeExtension();
		NewEntry->WriteToResourceComplete = RHICreateGPUFence(GDumpGPUBufferFenceName);
		NewEntry->WriteToResourceComplete->Clear();

		StagingBufferPool.Add(TUniquePtr<FStagingBufferPoolEntry>(NewEntry));

		return *NewEntry;
	}

	void DumpTextureSubResource(
		FRHICommandListImmediate& RHICmdList,
		const TCHAR* TextureDebugName,
		FRHITexture* Texture,
		FRHIShaderResourceView* SubResourceSRV,
		const FTextureSubresourceDumpDesc& SubresourceDumpDesc,
		const FString& DumpFilePath)
	{
		check(IsInRenderingThread());

		// Preprocess
		FTextureRHIRef StagingSrcTextureRef;
		FRHITexture* StagingSrcTexture;
		FTextureRHIRef StagingTexture;
		FStagingTexturePoolEntry* StagingTexturePoolEntry = nullptr;

		{
			FTimeBucketMeasure TimeBucketMeasure(this, ETimingBucket::RHIReadbackCommands);
			if (SubresourceDumpDesc.bPreprocessForStaging)
			{
				FUnorderedAccessViewRHIRef StagingOutput;
				{
					const FRHITextureCreateDesc Desc =
						FRHITextureCreateDesc::Create2D(TEXT("DumpGPU.PreprocessTexture"), SubresourceDumpDesc.SubResourceExtent, SubresourceDumpDesc.PreprocessedPixelFormat)
						.SetFlags(ETextureCreateFlags::UAV | ETextureCreateFlags::ShaderResource | ETextureCreateFlags::HideInVisualizeTexture);

					if (bStream)
					{
						ReuseStagingTexture(RHICmdList, Desc, ERHIAccess::UAVCompute, /* out */ &StagingSrcTextureRef, /* out */ &StagingOutput);
					}
					else
					{
						CreateStagingTexture(RHICmdList, Desc, /* out */ &StagingSrcTextureRef);

						RHICmdList.Transition(FRHITransitionInfo(StagingSrcTextureRef, ERHIAccess::Unknown, ERHIAccess::UAVCompute));
						StagingOutput = CreateUAVNoLifetimeExtension(RHICmdList, StagingSrcTextureRef, /* MipLevel = */ 0);
					}
					StagingSrcTexture = StagingSrcTextureRef;
				}

				FDumpTextureCS::FPermutationDomain PermutationVector;
				PermutationVector.Set<FDumpTextureCS::FTextureTypeDim>(SubresourceDumpDesc.DumpTextureType);
				TShaderMapRef<FDumpTextureCS> ComputeShader(GetGlobalShaderMap(GMaxRHIShaderPlatform), PermutationVector);

				FDumpTextureCS::FParameters ShaderParameters;
				ShaderParameters.Texture = SubResourceSRV;
				ShaderParameters.StagingOutput = StagingOutput;
				FComputeShaderUtils::Dispatch(
					RHICmdList,
					ComputeShader,
					ShaderParameters,
					FComputeShaderUtils::GetGroupCount(SubresourceDumpDesc.SubResourceExtent, 8));

				RHICmdList.Transition(FRHITransitionInfo(StagingSrcTexture, ERHIAccess::UAVCompute, ERHIAccess::CopySrc));
			}
			else
			{
				StagingSrcTexture = Texture;
			}

			// Copy the texture for CPU readback
			{
				const FRHITextureCreateDesc Desc =
					FRHITextureCreateDesc::Create2D(TEXT("DumpGPU.StagingTexture"), SubresourceDumpDesc.SubResourceExtent, SubresourceDumpDesc.PreprocessedPixelFormat)
					.SetFlags(ETextureCreateFlags::CPUReadback | ETextureCreateFlags::HideInVisualizeTexture);

				if (bStream)
				{
					StagingTexturePoolEntry = &ReuseStagingTexture(RHICmdList, Desc, ERHIAccess::CopyDest, /* out */ &StagingTexture, /* UAV = */ nullptr);
				}
				else
				{
					CreateStagingTexture(RHICmdList, Desc, /* out */ &StagingTexture);
					RHICmdList.Transition(FRHITransitionInfo(StagingTexture, ERHIAccess::Unknown, ERHIAccess::CopyDest));
				}

				// Transfer memory GPU -> CPU
				RHICmdList.CopyTexture(StagingSrcTexture, StagingTexture, {});

				RHICmdList.Transition(FRHITransitionInfo(StagingTexture, ERHIAccess::CopyDest, ERHIAccess::CPURead));
			}

			// Transition back the intermediary texture to UAVCompute for next ReuseStagingTexture().
			if (StagingSrcTextureRef && bStream)
			{
				RHICmdList.Transition(FRHITransitionInfo(StagingSrcTextureRef, ERHIAccess::CopySrc, ERHIAccess::UAVCompute));
			}
		}

		// jhoerner_todo 12/9/2021:  pick arbitrary GPU out of mask to avoid assert.  Eventually want to dump results for all GPUs, but
		// I need to understand how to modify the dumping logic, and this works for now (usually when debugging, the bugs happen on
		// secondary GPUs, so I figure the last index is most useful if we need to pick one).  I also would like the dump to include
		// information about the GPUMask for each pass, and perhaps have the dump include the final state of all external resources
		// modified by the graph (especially useful for MGPU, where we are concerned about cross-view or cross-frame state).
		uint32 GPUIndex = RHICmdList.GetGPUMask().GetLastIndex();

		if (bStream)
		{
			check(StagingTexturePoolEntry);
			// Write a fence to know when to transition to 
			RHICmdList.WriteGPUFence(StagingTexturePoolEntry->WriteToResourceComplete);
			StagingTexturePoolEntry->SubresourceDumpDesc = SubresourceDumpDesc;
			StagingTexturePoolEntry->DumpFilePath = DumpFilePath;
			StagingTexturePoolEntry->GPUIndex = GPUIndex;
		}
		else
		{
			// Submit to GPU and wait for completion.
			FGPUFenceRHIRef Fence = RHICreateGPUFence(GDumpGPUTextureFenceName);
			{
				FTimeBucketMeasure TimeBucketMeasure(this, ETimingBucket::GPUWait);

				Fence->Clear();
				RHICmdList.WriteGPUFence(Fence);
				RHICmdList.SubmitAndBlockUntilGPUIdle();
			}

			void* Content = nullptr;
			int32 RowPitchInPixels = 0;
			int32 ColumnPitchInPixels = 0;

			RHICmdList.MapStagingSurface(StagingTexture, Fence.GetReference(), Content, RowPitchInPixels, ColumnPitchInPixels, GPUIndex);

			if (Content)
			{
				TArray64<uint8> Array;
				PostProcessTexture(
					SubresourceDumpDesc,
					Content,
					RowPitchInPixels,
					ColumnPitchInPixels,
					/* out */ Array);

				RHICmdList.UnmapStagingSurface(StagingTexture, GPUIndex);

				bool bDumpSucceeded = DumpResourceBinaryToFile(Array.GetData(), Array.Num(), DumpFilePath);

				if (!bDumpSucceeded)
				{
					UE_LOGF(LogDumpGPU, Warning, "DumpResourceBinaryToFile() to dump texture %ls failed.", TextureDebugName);
				}
			}
			else
			{
				UE_LOGF(LogDumpGPU, Warning, "RHICmdList.MapStagingSurface() to dump texture %ls failed.", TextureDebugName);
			}
		}
	}

	void AddDumpTextureSubResourcePass(
		FRDGBuilder& GraphBuilder,
		TArray<TSharedPtr<FJsonValue>>& InputResourceNames,
		TArray<TSharedPtr<FJsonValue>>& OutputResourceNames,
		const FRDGPass* Pass,
		FRDGTextureSRVDesc SubresourceDesc,
		bool bRegisterSubResourceToPass,
		bool bIsOutputResource,
		bool bAllowDumpBinary)
	{
		int32 DumpTextureMode = GDumpTextureCVar.GetValueOnRenderThread();
		
		// We can't dump memoryless textures
		bool IsMemoryless = EnumHasAnyFlags(SubresourceDesc.Texture->Desc.Flags, TexCreate_Memoryless);
		
		if (DumpTextureMode == 0 || IsMemoryless)
		{
			return;
		}
 
		const FRDGTextureDesc& Desc = SubresourceDesc.Texture->Desc;
		const FString UniqueResourceSubResourceName = GetUniqueSubResourceName(SubresourceDesc);
		const FTextureSubresourceDumpDesc SubresourceDumpDesc = TranslateSubresourceDumpDesc(SubresourceDesc);

		if (bRegisterSubResourceToPass)
		{
			if (bIsOutputResource)
			{
				OutputResourceNames.AddUnique(MakeShareable(new FJsonValueString(UniqueResourceSubResourceName)));
			}
			else
			{
				InputResourceNames.AddUnique(MakeShareable(new FJsonValueString(UniqueResourceSubResourceName)));
			}
		}

		// Early return if this resource shouldn't be dumped.
		if (!bAllowDumpBinary || DumpTextureMode != 2)
		{
			return;
		}

		if (!SubresourceDumpDesc.IsDumpSupported())
		{
			return;
		}

		// Verify there is enough available memory to dump the resource.
		if (IsUnsafeToDumpResource(SubresourceDumpDesc.ByteSize, 2.2f + (SubresourceDumpDesc.bPreprocessForStaging ? 1.0f : 0.0f)))
		{
			UE_LOGF(LogDumpGPU, Warning, "Not dumping %ls because of insuficient memory available for staging texture.", SubresourceDesc.Texture->Name);
			return;
		}

		const FRDGViewableResource::FAccessModeState AccessModeState = SubresourceDesc.Texture->AccessModeState;

		const bool bIsExternalMode = AccessModeState.Mode == FRDGViewableResource::EAccessMode::External && !EnumHasAnyFlags(SubresourceDesc.Texture->Flags, ERDGTextureFlags::SkipTracking);
		if (bIsExternalMode)
		{
			GraphBuilder.UseInternalAccessMode(SubresourceDesc.Texture);
		}

		// Dump the resource's binary to a .bin file.
		{
			const FRDGPass* VersionPass = bIsOutputResource ? Pass : LastResourceVersion.FindRef(SubresourceDesc.Texture);
			FString DumpFilePath = kResourcesDir / FString::Printf(TEXT("%s.v%s.bin"), *UniqueResourceSubResourceName, *PtrToString(VersionPass));

			FDumpTexturePass* PassParameters = GraphBuilder.AllocParameters<FDumpTexturePass>();
			if (SubresourceDumpDesc.bPreprocessForStaging)
			{
				if (!(SubresourceDesc.Texture->Desc.Flags & TexCreate_ShaderResource))
				{
					UE_LOGF(LogDumpGPU, Warning, "Not dumping %ls because requires copy to staging texture using compute, but is missing TexCreate_ShaderResource.", *UniqueResourceSubResourceName);
					return;
				}

				PassParameters->Texture = GraphBuilder.CreateSRV(SubresourceDesc);
			}
			else
			{
				PassParameters->TextureAccess = FRDGTextureAccess(SubresourceDesc.Texture, ERHIAccess::CopySrc);
			}

			GraphBuilder.AddPass(
				RDG_EVENT_NAME("RDG DumpTexture(%s -> %s) %dx%d",
					SubresourceDesc.Texture->Name, *DumpFilePath,
					SubresourceDumpDesc.SubResourceExtent.X, SubresourceDumpDesc.SubResourceExtent.Y),
				PassParameters,
				(SubresourceDumpDesc.bPreprocessForStaging ? ERDGPassFlags::Compute : ERDGPassFlags::Copy) | ERDGPassFlags::NeverCull,
				[
					PassParameters, this, DumpFilePath,
					SubresourceDesc, SubresourceDumpDesc]
				(FRHICommandListImmediate& RHICmdList)
			{

				this->DumpTextureSubResource(
					RHICmdList,
					SubresourceDesc.Texture->Name,
					SubresourceDumpDesc.bPreprocessForStaging ? nullptr : SubresourceDesc.Texture->GetRHI(),
					SubresourceDumpDesc.bPreprocessForStaging ? PassParameters->Texture->GetRHI() : nullptr,
					SubresourceDumpDesc,
					DumpFilePath);
				if (!bStream)
				{
					this->ReleaseRHIResources(RHICmdList);
				}
				this->UpdatePassProgress();
			});

			ResourcesDumpPasses++;
		}

		if (bIsExternalMode)
		{
			GraphBuilder.UseExternalAccessMode(SubresourceDesc.Texture, AccessModeState.Access, AccessModeState.Pipelines);
		}
	}

	void AddDumpTextureAllSubResourcesPasses(
		FRDGBuilder& GraphBuilder,
		TArray<TSharedPtr<FJsonValue>>& InputResourceNames,
		TArray<TSharedPtr<FJsonValue>>& OutputResourceNames,
		const FRDGPass* Pass,
		const FRDGTextureSRVDesc& SubresourceRangeDesc,
		bool bRegisterSubResourceToPass,
		bool bIsOutputResource,
		bool bAllowDumpBinary)
	{
		if (SubresourceRangeDesc.Texture->Desc.IsTextureArray() && SubresourceRangeDesc.NumArraySlices != 1)
		{
			// If this is an array, recursively dump all subresource of each individual array slice.
			uint32 ArraySliceStart = SubresourceRangeDesc.FirstArraySlice;
			uint32 ArraySliceCount = (SubresourceRangeDesc.NumArraySlices > 0) ? SubresourceRangeDesc.NumArraySlices : (SubresourceRangeDesc.Texture->Desc.ArraySize - SubresourceRangeDesc.FirstArraySlice);

			for (uint32 ArraySlice = ArraySliceStart; ArraySlice < (ArraySliceStart + ArraySliceCount); ArraySlice++)
			{
				FRDGTextureSRVDesc SubresourceDesc = SubresourceRangeDesc;
				SubresourceDesc.FirstArraySlice = ArraySlice;
				SubresourceDesc.NumArraySlices = 1;
				SubresourceDesc.DimensionOverride = ETextureDimension::Texture2D;

				AddDumpTextureAllSubResourcesPasses(
					GraphBuilder,
					InputResourceNames,
					OutputResourceNames,
					Pass,
					SubresourceDesc,
					bRegisterSubResourceToPass,
					bIsOutputResource,
					bAllowDumpBinary);
			}
		}
		else if (SubresourceRangeDesc.Format == PF_X24_G8)
		{
			// Dump the stencil.
			AddDumpTextureSubResourcePass(
				GraphBuilder,
				InputResourceNames,
				OutputResourceNames,
				Pass,
				SubresourceRangeDesc,
				bRegisterSubResourceToPass,
				bIsOutputResource,
				bAllowDumpBinary);
		}
		else
		{
			for (int32 MipLevel = SubresourceRangeDesc.MipLevel; MipLevel < (SubresourceRangeDesc.MipLevel + SubresourceRangeDesc.NumMipLevels); MipLevel++)
			{
				FRDGTextureSRVDesc SubresourceDesc = SubresourceRangeDesc;
				SubresourceDesc.MipLevel = MipLevel;
				SubresourceDesc.NumMipLevels = 1;
				if(SubresourceRangeDesc.Texture->Desc.IsTextureArray())
				{
                	SubresourceDesc.DimensionOverride = ETextureDimension::Texture2D;
				}
                
				AddDumpTextureSubResourcePass(
					GraphBuilder,
					InputResourceNames,
					OutputResourceNames,
					Pass,
					SubresourceDesc,
					bRegisterSubResourceToPass,
					bIsOutputResource,
					bAllowDumpBinary);
			}
		}
	}

	void AddDumpTexturePasses(
		FRDGBuilder& GraphBuilder,
		TArray<TSharedPtr<FJsonValue>>& InputResourceNames,
		TArray<TSharedPtr<FJsonValue>>& OutputResourceNames,
		const FRDGPass* Pass,
		const FRDGTextureSRVDesc& SubresourceRangeDesc,
		bool bIsOutputResource)
	{
		if (!IsDumpingResource(SubresourceRangeDesc.Texture->Name, Pass))
		{
			return;
		}

		bool bAllowDumpBinary = true;
		if (!HasResourceEverBeenDumped(SubresourceRangeDesc.Texture))
		{
			RegisterResourceAsDumped(SubresourceRangeDesc.Texture, /* LastModifyingPass = */ bIsOutputResource ? Pass : nullptr);

			// Dump the descriptor of the resource
			{
				const FString UniqueResourceName = GetUniqueResourceName(SubresourceRangeDesc.Texture);
				TSharedPtr<FJsonObject> JsonObject = ToJson(UniqueResourceName, SubresourceRangeDesc.Texture->Name, SubresourceRangeDesc.Texture->Desc);
				DumpJsonToFile(JsonObject, FString(FRDGResourceDumpContext::kBaseDir) / TEXT("ResourceDescs.json"), FILEWRITE_Append);
			}

			// If the resource is input but has not been dumped ever before, make sure to dump all of its sub resources instead of just the sub resource described by SubresourceRangeDesc
			if (!bIsOutputResource)
			{
				// Dump the stencil if there is one
				if (IsStencilFormat(SubresourceRangeDesc.Texture->Desc.Format))
				{
					AddDumpTextureAllSubResourcesPasses(
						GraphBuilder,
						InputResourceNames,
						OutputResourceNames,
						Pass,
						FRDGTextureSRVDesc::CreateWithPixelFormat(SubresourceRangeDesc.Texture, PF_X24_G8),
						/* bRegisterSubResourceToPass = */ false,
						bIsOutputResource,
						/* bAllowDumpBinary = */ true);
				}

				// Dump all the remaining sub resource (mip levels, array slices...)
				AddDumpTextureAllSubResourcesPasses(
					GraphBuilder,
					InputResourceNames,
					OutputResourceNames,
					Pass,
					FRDGTextureSRVDesc::Create(SubresourceRangeDesc.Texture),
					/* bRegisterSubResourceToPass = */ false,
					bIsOutputResource,
					/* bAllowDumpBinary = */ true);

				// Still register the input subresources used without dump any binary.
				return AddDumpTextureAllSubResourcesPasses(
					GraphBuilder,
					InputResourceNames,
					OutputResourceNames,
					Pass,
					SubresourceRangeDesc,
					/* bRegisterSubResourceToPass = */ true,
					bIsOutputResource,
					/* bAllowDumpBinary = */ false);
			}
		}
		else if (bIsOutputResource)
		{
			UpdateResourceLastModifyingPass(SubresourceRangeDesc.Texture, Pass);
		}
		else
		{
			if (GDumpGPURedumpInputs.GetValueOnRenderThread() != 0)
			{
				// Re-dump under a unique version key (the consuming pass pointer). So if this texture has been dumped 
				// previously but modified by intermeddiate passes between then and now, We can still get the correct copy.
				UpdateResourceLastModifyingPass(SubresourceRangeDesc.Texture, Pass);
			}
			else
			{
				// The texture has already been dumped, so can early return given it is not being modified by this Pass.
				bAllowDumpBinary = false;
			}
		}

		return AddDumpTextureAllSubResourcesPasses(
			GraphBuilder,
			InputResourceNames,
			OutputResourceNames,
			Pass,
			SubresourceRangeDesc,
			/* bRegisterSubResourceToPass = */ true,
			bIsOutputResource,
			bAllowDumpBinary);
	}

	void AddDumpBufferPass(
		FRDGBuilder& GraphBuilder,
		TArray<TSharedPtr<FJsonValue>>& InputResourceNames,
		TArray<TSharedPtr<FJsonValue>>& OutputResourceNames,
		const FRDGPass* Pass,
		FRDGBuffer* Buffer,
		bool bIsOutputResource)
	{
		if (!IsDumpingResource(Buffer->Name, Pass))
		{
			return;
		}

		int32 DumpBufferMode = GDumpBufferCVar.GetValueOnRenderThread();

		if (DumpBufferMode == 0)
		{
			return;
		}

		FString UniqueResourceName = GetUniqueResourceName(Buffer);

		if (bIsOutputResource)
		{
			OutputResourceNames.AddUnique(MakeShareable(new FJsonValueString(UniqueResourceName)));
		}
		else
		{
			InputResourceNames.AddUnique(MakeShareable(new FJsonValueString(UniqueResourceName)));
		}

		const FRDGBufferDesc& Desc = Buffer->Desc;
		uint32 ByteSize = Desc.GetSize();
		uint32 ReservedSize = 0;

		// For reserved resources, only dump the committed portion. Currently reserved buffers are only used for no-copy
		// grow/shrink operations, so the committed region is contiguous and starts at offset 0. We don't need to add
		// handling for true sparse resources.
		if (EnumHasAnyFlags(Desc.Usage, EBufferUsageFlags::ReservedResource))
		{
			ReservedSize = ByteSize;
			// GetCommittedSize returns a 64-bit value, but this can never exceed the reserved size returned by Desc.GetSize(),
			// which is currently a 32-bit value. Therefore it's safe to truncate here.
			ByteSize = static_cast<uint32>(Buffer->GetCommittedSize());
		}

		bool bDumpResourceInfos = false;
		bool bDumpResourceBinary = bIsOutputResource;
		{
			if (!HasResourceEverBeenDumped(Buffer))
			{
				// First time we ever see this buffer, so dump it's info to disk
				bDumpResourceInfos = true;

				// If not an output, it might be a resource undumped by r.DumpGPU.Root or external texture so still dump it as v0.
				if (!bIsOutputResource)
				{
					bDumpResourceBinary = true;
				}

				RegisterResourceAsDumped(Buffer, bIsOutputResource ? Pass : nullptr);
			}
			else if (bIsOutputResource)
			{
				UpdateResourceLastModifyingPass(Buffer, Pass);
			}
		}

		// Dump the information of the buffer to json file.
		if (bDumpResourceInfos)
		{
			TSharedPtr<FJsonObject> JsonObject = ToJson(UniqueResourceName, Buffer->Name, Desc, ByteSize, ReservedSize);
			DumpJsonToFile(JsonObject, FString(FRDGResourceDumpContext::kBaseDir) / TEXT("ResourceDescs.json"), FILEWRITE_Append);

			if (Desc.Metadata && DumpBufferMode == 2)
			{
				Dump(Desc.Metadata);
			}
		}

		// Dump the resource's binary to a .bin file.
		if (bDumpResourceBinary && DumpBufferMode == 2)
		{
			if (ByteSize == 0)
			{
				// Reserved resource with nothing committed yet, nothing to dump.
				return;
			}

			const uint32 StagingResourceByteSize = bStream ? ByteSize : FMath::Min(ByteSize, GDumpMaxStagingSize.GetValueOnRenderThread() * 1024U * 1024U);

			FString DumpFilePath = kResourcesDir / FString::Printf(TEXT("%s.v%s.bin"), *UniqueResourceName, *PtrToString(bIsOutputResource ? Pass : nullptr));

			if (IsUnsafeToDumpResource(StagingResourceByteSize, 1.2f))
			{
				UE_LOGF(LogDumpGPU, Warning, "Not dumping %ls because of insuficient memory available for staging buffer.", *DumpFilePath);
				return;
			}

			// Verify the texture is able to do resource transitions.
			if (EnumHasAnyFlags(Buffer->Flags, ERDGBufferFlags::SkipTracking))
			{
				UE_LOGF(LogDumpGPU, Warning, "Not dumping %ls because has ERDGBufferFlags::SkipTracking.", Buffer->Name);
				return;
			}

			const FRDGViewableResource::FAccessModeState AccessModeState = Buffer->AccessModeState;

			if (AccessModeState.Mode == FRDGViewableResource::EAccessMode::External)
			{
				GraphBuilder.UseInternalAccessMode(Buffer);
			}

			FDumpBufferPass* PassParameters = GraphBuilder.AllocParameters<FDumpBufferPass>();
			PassParameters->Buffer = Buffer;

			GraphBuilder.AddPass(
				RDG_EVENT_NAME("RDG DumpBuffer(%s -> %s)", Buffer->Name, *DumpFilePath),
				PassParameters,
				ERDGPassFlags::Readback,
				[this, DumpFilePath, Buffer, ByteSize, StagingResourceByteSize](FRHICommandListImmediate& RHICmdList)
			{
				check(IsInRenderingThread());
				if (bStream)
				{
					check(ByteSize == StagingResourceByteSize);

					FStagingBufferPoolEntry& StagingBufferPoolEntry = ReuseStagingBuffer(RHICmdList, StagingResourceByteSize);
					StagingBufferPoolEntry.DumpFilePath = DumpFilePath;

					// Transfer memory GPU -> CPU
					{
						FTimeBucketMeasure TimeBucketMeasure(this, ETimingBucket::RHIReadbackCommands);
						RHICmdList.CopyToStagingBuffer(Buffer->GetRHI(), StagingBufferPoolEntry.StagingBuffer, /* Offset = */ 0, ByteSize);
						RHICmdList.WriteGPUFence(StagingBufferPoolEntry.WriteToResourceComplete);
					}
				}
				else // if (!bStream)
				{
					FStagingBufferRHIRef StagingBuffer;
					FGPUFenceRHIRef Fence;
					{
						FTimeBucketMeasure TimeBucketMeasure(this, ETimingBucket::RHIReadbackCommands);
						StagingBuffer = RHICreateStagingBuffer();
						StagingBuffer->DisableLifetimeExtension();

						Fence = RHICreateGPUFence(GDumpGPUBufferFenceName);
					}
				
					TUniquePtr<FArchive> Ar;
					if (bEnableDiskWrite)
					{
						FString FullPath = GetDumpFullPath(DumpFilePath);
						FFileWriteCtx WriteCtx(this, ETimingBucket::ResourceBinaryFileWrite, /* ByteSize = */ 0);

						Ar = TUniquePtr<FArchive>(IFileManager::Get().CreateFileWriter(*FullPath, /* WriteFlags = */ 0));
					}

					for (uint32 Offset = 0; Offset < ByteSize; Offset += StagingResourceByteSize)
					{
						uint32 CopyByteSize = FMath::Min(StagingResourceByteSize, ByteSize - Offset);

						// Transfer memory GPU -> CPU
						{
							FTimeBucketMeasure TimeBucketMeasure(this, ETimingBucket::RHIReadbackCommands);
							RHICmdList.CopyToStagingBuffer(Buffer->GetRHI(), StagingBuffer, Offset, CopyByteSize);
						}

						// Submit to GPU and wait for completion.
						{
							FTimeBucketMeasure TimeBucketMeasure(this, ETimingBucket::GPUWait);
							Fence->Clear();
							RHICmdList.WriteGPUFence(Fence);
							RHICmdList.SubmitAndBlockUntilGPUIdle();
						}

						void* Content = RHICmdList.LockStagingBuffer(StagingBuffer, Fence.GetReference(), 0, CopyByteSize);
						if (Content)
						{
							if (Ar)
							{
								FFileWriteCtx WriteCtx(this, ETimingBucket::ResourceBinaryFileWrite, CopyByteSize, /* OpenedFiles = */ 0);
								Ar->Serialize((void*)Content, CopyByteSize);
								Ar->Flush();
							}
							RHICmdList.UnlockStagingBuffer(StagingBuffer);
						}
						else
						{
							UE_LOGF(LogDumpGPU, Warning, "RHICmdList.LockStagingBuffer() to dump buffer %ls failed.", Buffer->Name);
							break;
						}
					}

					if (Ar)
					{
						FFileWriteCtx WriteCtx(this, ETimingBucket::ResourceBinaryFileWrite, /* ByteSize = */ 0, /* OpenedFiles = */ 0);
						// Always explicitly close to catch errors from flush/close
						Ar->Close();
						Ar = nullptr;
					}

					StagingBuffer = nullptr;
					Fence = nullptr;
					this->ReleaseRHIResources(RHICmdList);
				} // if (!bStream)

				this->UpdatePassProgress();
			});

			if (AccessModeState.Mode == FRDGViewableResource::EAccessMode::External)
			{
				GraphBuilder.UseExternalAccessMode(Buffer, AccessModeState.Access, AccessModeState.Pipelines);
			}

			ResourcesDumpPasses++;
		}
	}

	static void WaitDiskWriteAndReset(FStagingPoolEntry* Entry)
	{
		check(Entry->Status == FStagingResourceStatus::WritingToDisk || Entry->Status == FStagingResourceStatus::WritingToDiskComplete);
		check(Entry->WriteToDiskComplete);

		Entry->WriteToDiskComplete->Wait();
		FPlatformProcess::ReturnSynchEventToPool(Entry->WriteToDiskComplete);
		Entry->WriteToDiskComplete = nullptr;

		check(Entry->Status == FStagingResourceStatus::WritingToDiskComplete);
		Entry->Status = FStagingResourceStatus::Unused;
	}

	void LandCompletedResources(FRHICommandListImmediate& RHICmdList)
	{
		check(IsInRenderingThread());
		check(bStream);
		for (TUniquePtr<FStagingTexturePoolEntry>& Entry : StagingTexturePool)
		{
			if (Entry->Status == FStagingResourceStatus::Rendering)
			{
				// Try lock the staging surface.

				void* Content = nullptr;
				int32 RowPitchInPixels = 0;
				int32 ColumnPitchInPixels = 0;

				RHICmdList.MapStagingSurface(Entry->Texture, Entry->WriteToResourceComplete.GetReference(), Content, RowPitchInPixels, ColumnPitchInPixels, Entry->GPUIndex);

				if (!Content)
				{
					// NOP
				}
				else
				{
					check(Entry->WriteToDiskComplete == nullptr);
					Entry->WriteToDiskComplete = FPlatformProcess::GetSynchEventFromPool(true);
					Entry->Status = FStagingResourceStatus::WritingToDisk;

					FStagingTexturePoolEntry* Staging = Entry.Get();

					KickOffAsyncTask([this, Staging, Content, RowPitchInPixels, ColumnPitchInPixels](){
						check(Staging->WriteToDiskComplete);
						check(Staging->Status == FStagingResourceStatus::WritingToDisk);

						TArray64<uint8> Array;
						this->PostProcessTexture(
							Staging->SubresourceDumpDesc,
							Content,
							RowPitchInPixels,
							ColumnPitchInPixels,
							/* out */ Array);

						this->DumpResourceBinaryToFile(Array.GetData(), Array.Num(), Staging->DumpFilePath);

						Staging->Status = FStagingResourceStatus::WritingToDiskComplete;
						Staging->WriteToDiskComplete->Trigger();
					});
				}
			}
			else if (Entry->Status == FStagingResourceStatus::WritingToDiskComplete)
			{
				WaitDiskWriteAndReset(Entry.Get());

				RHICmdList.UnmapStagingSurface(Entry->Texture, Entry->GPUIndex);
				RHICmdList.Transition(FRHITransitionInfo(Entry->Texture, ERHIAccess::CPURead, ERHIAccess::CopyDest));
			}
		}

		for (TUniquePtr<FStagingBufferPoolEntry>& Entry : StagingBufferPool)
		{
			if (Entry->Status == FStagingResourceStatus::Rendering)
			{
				// Try lock the staging buffer.
				void* Content = RHICmdList.LockStagingBuffer(Entry->StagingBuffer, Entry->WriteToResourceComplete.GetReference(), 0, Entry->ByteSize);
				if (!Content)
				{
					// NOP
				}
				else
				{
					check(Entry->WriteToDiskComplete == nullptr);
					Entry->WriteToDiskComplete = FPlatformProcess::GetSynchEventFromPool(true);
					Entry->Status = FStagingResourceStatus::WritingToDisk;

					FStagingBufferPoolEntry* Staging = Entry.Get();

					KickOffAsyncTask([this, Staging, Content]() {
						check(Staging->WriteToDiskComplete);
						check(Staging->Status == FStagingResourceStatus::WritingToDisk);

						this->DumpResourceBinaryToFile(static_cast<const uint8*>(Content), Staging->ByteSize, Staging->DumpFilePath);

						Staging->Status = FStagingResourceStatus::WritingToDiskComplete;
						Staging->WriteToDiskComplete->Trigger();
					});
				}
			}
			else if (Entry->Status == FStagingResourceStatus::WritingToDiskComplete)
			{
				WaitDiskWriteAndReset(Entry.Get());

				RHICmdList.UnlockStagingBuffer(Entry->StagingBuffer);
				Entry->Status = FStagingResourceStatus::Unused;
			}
		}
	}

	void WaitAndReleaseStagingResources(FRHICommandListImmediate& RHICmdList)
	{
		check(IsInRenderingThread());
		check(bStream);

		RHICmdList.SubmitAndBlockUntilGPUIdle();

		LandCompletedResources(RHICmdList);

		for (TUniquePtr<FStagingTexturePoolEntry>& Entry : StagingTexturePool)
		{
			if (Entry->Status == FStagingResourceStatus::WritingToDisk || Entry->Status == FStagingResourceStatus::WritingToDiskComplete)
			{
				WaitDiskWriteAndReset(Entry.Get());

				RHICmdList.UnmapStagingSurface(Entry->Texture, Entry->GPUIndex);
			}

			check(Entry->WriteToDiskComplete == nullptr);
			check(Entry->Status == FStagingResourceStatus::Unused || Entry->Status == FStagingResourceStatus::RenderingIntermediaryOnly);
		}

		for (TUniquePtr<FStagingBufferPoolEntry>& Entry : StagingBufferPool)
		{
			if (Entry->Status == FStagingResourceStatus::WritingToDisk || Entry->Status == FStagingResourceStatus::WritingToDiskComplete)
			{
				WaitDiskWriteAndReset(Entry.Get());

				RHICmdList.UnlockStagingBuffer(Entry->StagingBuffer);
			}

			check(Entry->WriteToDiskComplete == nullptr);
			check(Entry->Status == FStagingResourceStatus::Unused);
		}

		RHICmdList.SubmitAndBlockUntilGPUIdle();

		// Releases all staging resources
		StagingTexturePool.Reset();
		StagingBufferPool.Reset();
		ReleaseRHIResources(RHICmdList);
	}

	// Look whether the pass matches matches r.DumpGPU.Root
	bool IsDumpingPass(const FRDGPass* Pass)
	{
		FString RootWildcardString = GDumpGPURootCVar.GetValueOnRenderThread();
		FWildcardString WildcardRootFilter(RootWildcardString);

		bool bDumpPass = (RootWildcardString == TEXT("*"));

		if (!bDumpPass)
		{
			bDumpPass = WildcardRootFilter.IsMatch(Pass->GetEventName().GetTCHAR());
		}

		FString PassPath = Pass->GetEventName().GetTCHAR();

#if RDG_EVENTS
		for (FRDGScope const* ParentScope = Pass->GetScope(); ParentScope; ParentScope = ParentScope->Parent)
		{
			if (FRDGScope_RHI const* RHIScope = ParentScope->Get<FRDGScope_RHI>())
			{
				FRHIBreadcrumb::FBuffer Buffer;
				bDumpPass |= WildcardRootFilter.IsMatch(RHIScope->GetTCHAR(Buffer));
				PassPath = RHIScope->GetTCHAR(Buffer) / PassPath;
			}
		}
#endif // RDG_EVENTS

		if (!UnifiedPassResourceFilter.IsEmpty())
		{
			bool bPathPassAllowed = false;

			for (const FUnifiedPassResourceFilter& Filter : UnifiedPassResourceFilter)
			{
				FWildcardString WildcardPassPathFilter(Filter.PassPattern);
				bPathPassAllowed |= WildcardPassPathFilter.IsMatch(*PassPath);
			}
			bDumpPass &= bPathPassAllowed;
		}

		return bDumpPass;
	}

	bool IsDumpingResource(const TCHAR* ResourceName, const FRDGPass* Pass) const
	{
		if (UnifiedPassResourceFilter.IsEmpty())
		{
			return true;
		}

		FString PassName = Pass->GetEventName().GetTCHAR(); 
		for (const FUnifiedPassResourceFilter& Filter : UnifiedPassResourceFilter)
		{
			if (Filter.PassPattern.IsMatch(PassName) && Filter.ResourcePattern.IsMatch(ResourceName))
			{
				return true;
			}
		}
        
		return false;
	}

	void OnCrash()
	{
		if (GLog)
		{
			GLog->Panic();
		}
		DumpStatusToFile(TEXT("crash"));
		FGenericCrashContext::DumpLog(DumpingDirectoryPath / FRDGResourceDumpContext::kBaseDir);
	}

	void Start();
	void Finish();
};

static FRDGResourceDumpContext* GRDGResourceDumpContext_GameThread = nullptr;
static FRDGResourceDumpContext* GRDGResourceDumpContext_RenderThread = nullptr;

static float GNextDumpingRemainingTime = -1.0f;
static int32 GNextDumpingRemaingFrames = -1;
static FRDGResourceDumpContext* GNextRDGResourceDumpContext = nullptr;


FString FRDGBuilder::BeginResourceDump(const TCHAR* Cmd, const TCHAR* Context)
{
	check(IsInGameThread());

	if (Cmd == nullptr)
	{
		Cmd = TEXT("");
	}
	if (Context == nullptr)
	{
		Context = TEXT("");
	}
	if (GRDGResourceDumpContext_GameThread || GNextRDGResourceDumpContext)
	{
		return FString();
	}

	TArray<FString> Tokens;
	TArray<FString> Switches;
	TMap<FString, FString> Parameters;
	{
		const TCHAR* LocalCmd = Cmd;

		FString NextToken;
		while (FParse::Token(LocalCmd, NextToken, false))
		{
			if (**NextToken == TCHAR('-'))
			{
				Switches.Add(NextToken.Mid(1));
			}
			else
			{
				Tokens.Add(NextToken);
			}
		}

		for (int32 SwitchIdx = Switches.Num() - 1; SwitchIdx >= 0; --SwitchIdx)
		{
			FString& Switch = Switches[SwitchIdx];
			TArray<FString> SplitSwitch;

			// Remove Bar=1 from the switch list and put it in params as {Bar,1}.
			// Note: Handle nested equality such as Bar="Key=Value"
			int32 AssignmentIndex = 0;
			if (Switch.FindChar(TEXT('='), AssignmentIndex))
			{
				Parameters.Add(Switch.Left(AssignmentIndex), Switch.RightChop(AssignmentIndex + 1).TrimQuotes());
				Switches.RemoveAt(SwitchIdx);
			}
		}
	}

	FRDGResourceDumpContext* NewResourceDumpContext;
	NewResourceDumpContext = new FRDGResourceDumpContext;

	NewResourceDumpContext->Time = FDateTime::Now();
	{
		FString CVarDirectoryPath = GDumpGPUDirectoryCVar.GetValueOnGameThread();
		FString EnvDirectoryPath = FPlatformMisc::GetEnvironmentVariable(TEXT("UE-DumpGPUPath"));

		FString DirectoryPath;
		if (!CVarDirectoryPath.IsEmpty())
		{
			DirectoryPath = CVarDirectoryPath;
		}
		else if (!EnvDirectoryPath.IsEmpty())
		{
			DirectoryPath = EnvDirectoryPath;
		}
		else
		{
			DirectoryPath = FPaths::ProjectSavedDir() / TEXT("GPUDumps/");
		}
		NewResourceDumpContext->DumpingDirectoryPath = DirectoryPath / FApp::GetProjectName() + TEXT("-") + FPlatformProperties::PlatformName() + TEXT("-") + NewResourceDumpContext->Time.ToString() + TEXT("/");
	}

	// Override dump directory CVar
	if (Parameters.Contains(TEXT("path")))
	{
		NewResourceDumpContext->DumpingDirectoryPath = Parameters[TEXT("path")];
	}

	if (Parameters.Contains(TEXT("unified_filter")))
	{
		// Example: "Temporal*@GBufferA|BasePass@SceneDepth*"
		TArray<FString> RawFilters;
		Parameters[TEXT("unified_filter")].ParseIntoArray(RawFilters, TEXT("|"));

		for (const FString& RawFilter : RawFilters)
		{
			FString PassStr, ResourceStr;
			// Split Resource and Passes into two variables in the same object
			if (RawFilter.Split(TEXT("@"), &PassStr, &ResourceStr))
			{
				NewResourceDumpContext->UnifiedPassResourceFilter.Emplace(PassStr, ResourceStr);
			}
			else
			{
				UE_LOGF(LogDumpGPU, Warning, "Invalid unified filter format: %ls. Expected Pass@Resource", *RawFilter);
			}
		}
	}

	NewResourceDumpContext->bStream = GDumpGPUStream.GetValueOnGameThread() != 0;
	NewResourceDumpContext->bEnableDiskWrite = GDumpTestEnableDiskWrite.GetValueOnGameThread() != 0;
	NewResourceDumpContext->DeltaTime = FApp::GetDeltaTime();
	NewResourceDumpContext->FrameCount = FMath::Max(GDumpGPUFrameCount.GetValueOnGameThread(), 1);
	NewResourceDumpContext->Context = Context;

	if (Switches.Contains(TEXT("upload")))
	{
		if (!IDumpGPUUploadServiceProvider::GProvider || !NewResourceDumpContext->bEnableDiskWrite)
		{
			UE_LOGF(LogDumpGPU, Warning, "DumpGPU upload services are not set up.");
		}
		else if (GDumpGPUUploadCVar.GetValueOnGameThread() == 0)
		{
			UE_LOGF(LogDumpGPU, Warning, "DumpGPU upload services are not available because r.DumpGPU.Upload=0.");
		}
		else
		{
			NewResourceDumpContext->bUpload = true;
		}
	}

	// Override frame count CVar, and just capture one frame.  Used for scene capture dumps, where they aren't going to run more than one frame.
	if (Switches.Contains(TEXT("oneframe")))
	{
		NewResourceDumpContext->FrameCount = 1;
	}

	if (NewResourceDumpContext->bUpload)
	{
		if (GDumpGPUUploadCompressResources.GetValueOnGameThread() == 1)
		{
			NewResourceDumpContext->UploadResourceCompressionName = NAME_Zlib;
		}
		else if (GDumpGPUUploadCompressResources.GetValueOnGameThread() == 2)
		{
			NewResourceDumpContext->UploadResourceCompressionName = NAME_Gzip;
		}
	}

	NewResourceDumpContext->bShowInExplore = NewResourceDumpContext->bEnableDiskWrite && GDumpExploreCVar.GetValueOnGameThread() != 0 && !NewResourceDumpContext->bUpload;

	// Override open explorer behavior
	if (Switches.Contains(TEXT("noexplorer")))
	{
		NewResourceDumpContext->bShowInExplore = false;
	}

	check(GNextRDGResourceDumpContext == nullptr);
	GNextRDGResourceDumpContext = NewResourceDumpContext;
	if (GDumpGPUDelay.GetValueOnGameThread() > 0.0f)
	{
		GNextDumpingRemainingTime = GDumpGPUDelay.GetValueOnGameThread();
		UE_LOGF(LogDumpGPU, Display, "DumpGPU to %ls armed for %d frames starting in %f seconds.", *NewResourceDumpContext->DumpingDirectoryPath, NewResourceDumpContext->FrameCount, GNextDumpingRemainingTime);
	}
	else if (GDumpGPUFrameDelay.GetValueOnGameThread() > 0)
	{
		GNextDumpingRemaingFrames = GDumpGPUFrameDelay.GetValueOnGameThread();
		UE_LOGF(LogDumpGPU, Display, "DumpGPU to %ls armed for %d frames starting in %d frames.", *NewResourceDumpContext->DumpingDirectoryPath, NewResourceDumpContext->FrameCount, GNextDumpingRemaingFrames);
	}
	else
	{
		UE_LOGF(LogDumpGPU, Display, "Dump to %ls armed for %d frames starting next frame.", *NewResourceDumpContext->DumpingDirectoryPath, NewResourceDumpContext->FrameCount);
	}

	if (NewResourceDumpContext->bEnableDiskWrite)
	{
		return NewResourceDumpContext->DumpingDirectoryPath;
	}
	return FString();
}

void FRDGResourceDumpContext::Start()
{
	check(IsInGameThread());
	check(GNextRDGResourceDumpContext == this);

	// Dumping resource may take a while and we don't want to get 'hang' crashes in the process.
	// We resume from EndResourceDump after the dump has finished.
	FThreadHeartBeat::Get().SuspendHeartBeat(true);
	SuspendRenderThreadTimeout();

	UE_LOGF(LogDumpGPU, Display, "DumpGPU to %ls starting this frame", *DumpingDirectoryPath);
	MemoryConstants = FPlatformMemory::GetConstants();
	MemoryStats = FPlatformMemory::GetStats();

	IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
	if (bEnableDiskWrite)
	{
		if (!PlatformFile.DirectoryExists(*DumpingDirectoryPath))
		{
			PlatformFile.CreateDirectoryTree(*DumpingDirectoryPath);
		}
		PlatformFile.CreateDirectoryTree(*(DumpingDirectoryPath / FRDGResourceDumpContext::kBaseDir));
		PlatformFile.CreateDirectoryTree(*(DumpingDirectoryPath / FRDGResourceDumpContext::kResourcesDir));

		DumpStringToFile(TEXT(""), FString(FRDGResourceDumpContext::kBaseDir) / TEXT("Passes.json"));
		DumpStringToFile(TEXT(""), FString(FRDGResourceDumpContext::kBaseDir) / TEXT("ResourceDescs.json"));
		DumpStringToFile(TEXT(""), FString(FRDGResourceDumpContext::kBaseDir) / TEXT("PassDrawCounts.json"));
	}

	// Dump status file and register NewResourceDumpContext to listen for OnShutdownAfterError to get a log of what happened with callstack.
	{
		DumpStatusToFile(TEXT("dumping"));
		FCoreDelegates::OnShutdownAfterError.AddRaw(this, &FRDGResourceDumpContext::OnCrash);
	}

	// Dump service parameters so GPUDumpViewer.html remain compatible when not using upload provider.
	if (bEnableDiskWrite)
	{
		GetDumpParameters().DumpServiceParametersFile();
	}

	if (GDumpGPUFixedTickRate.GetValueOnGameThread() > 0.0f && !FApp::UseFixedTimeStep())
	{
		bOverrideFixedDeltaTime = true;
		PreviousFixedDeltaTime = FApp::GetFixedDeltaTime();

		FApp::SetFixedDeltaTime(1.0f / GDumpGPUFixedTickRate.GetValueOnGameThread());
		FApp::SetUseFixedTimeStep(true);

		UE_LOGF(LogDumpGPU, Display, "DumpGPU overriding tick rate to %fs", float(FApp::GetFixedDeltaTime()));
	}

	// Output informations
	{
		const TCHAR* BranchName = BuildSettings::GetBranchName();
		const TCHAR* BuildDate = BuildSettings::GetBuildDate();
		const TCHAR* BuildVersion = BuildSettings::GetBuildVersion();

		FString BuildConfiguration = LexToString(FApp::GetBuildConfiguration());
		FString BuildTarget = LexToString(FApp::GetBuildTargetType());

		FString OSLabel, OSVersion;
		FPlatformMisc::GetOSVersions(OSLabel, OSVersion);
		if (!OSVersion.IsEmpty())
		{
			OSLabel = FString::Printf(TEXT("%s %s"), *OSLabel, *OSVersion);
		}

		FGPUDriverInfo GPUDriverInfo = FPlatformMisc::GetGPUDriverInfo(GRHIAdapterName);

		TSharedPtr<FJsonObject> JsonObject = MakeShareable(new FJsonObject);
		JsonObject->SetStringField(TEXT("Project"), FApp::GetProjectName());
		JsonObject->SetNumberField(TEXT("EngineMajorVersion"), ENGINE_MAJOR_VERSION);
		JsonObject->SetNumberField(TEXT("EngineMinorVersion"), ENGINE_MINOR_VERSION);
		JsonObject->SetNumberField(TEXT("EnginePatchVersion"), ENGINE_PATCH_VERSION);
		JsonObject->SetStringField(TEXT("BuildBranch"), BranchName ? BranchName : TEXT(""));
		JsonObject->SetStringField(TEXT("BuildDate"), BuildDate ? BuildDate : TEXT(""));
		JsonObject->SetStringField(TEXT("BuildVersion"), BuildVersion ? BuildVersion : TEXT(""));
		JsonObject->SetStringField(TEXT("BuildTarget"), BuildTarget);
		JsonObject->SetStringField(TEXT("BuildConfiguration"), BuildConfiguration);
		JsonObject->SetNumberField(TEXT("Build64Bits"), 1);
		JsonObject->SetStringField(TEXT("Platform"), FPlatformProperties::IniPlatformName());
		JsonObject->SetStringField(TEXT("OS"), OSLabel);
		JsonObject->SetStringField(TEXT("DeviceName"), FPlatformProcess::ComputerName());
		JsonObject->SetStringField(TEXT("CPUVendor"), FPlatformMisc::GetCPUVendor());
		JsonObject->SetStringField(TEXT("CPUBrand"), FPlatformMisc::GetCPUBrand());
		JsonObject->SetNumberField(TEXT("CPUNumberOfCores"), FPlatformMisc::NumberOfCores());
		JsonObject->SetNumberField(TEXT("CPUNumberOfCoresIncludingHyperthreads"), FPlatformMisc::NumberOfCoresIncludingHyperthreads());
		JsonObject->SetStringField(TEXT("GPUVendor"), RHIVendorIdToString());
		JsonObject->SetStringField(TEXT("GPUDeviceDescription"), GPUDriverInfo.DeviceDescription);
		JsonObject->SetStringField(TEXT("GPUDriverUserVersion"), GPUDriverInfo.UserDriverVersion);
		JsonObject->SetStringField(TEXT("GPUDriverInternalVersion"), GPUDriverInfo.GetUnifiedDriverVersion());
		JsonObject->SetStringField(TEXT("GPUDriverDate"), GPUDriverInfo.DriverDate);
		JsonObject->SetNumberField(TEXT("MemoryTotalPhysical"), MemoryConstants.TotalPhysical);
		JsonObject->SetNumberField(TEXT("MemoryPageSize"), MemoryConstants.PageSize);
		JsonObject->SetStringField(TEXT("RHI"), GDynamicRHI->GetName());
		JsonObject->SetStringField(TEXT("RHIMaxFeatureLevel"), LexToString(GMaxRHIFeatureLevel));
		JsonObject->SetStringField(TEXT("DumpTime"), Time.ToString());
		JsonObject->SetStringField(TEXT("DumpContext"), Context);
		{
			const FString LogSrcAbsolute = FPlatformOutputDevices::GetAbsoluteLogFilename();
			FString LogFilename = FPaths::GetCleanFilename(LogSrcAbsolute);
			JsonObject->SetStringField(TEXT("LogFilename"), LogFilename);
		}

		DumpJsonToFile(JsonObject, FString(FRDGResourceDumpContext::kBaseDir) / TEXT("Infos.json"));
	}

	// Dump the rendering cvars
	if (bEnableDiskWrite)
	{
		KickOffAsyncTask([this]() {
			this->DumpRenderingCVarsToCSV();
		});
	}

	// Copy the viewer
	if (bEnableDiskWrite)
	{
		KickOffAsyncTask([this, &PlatformFile]() {
			const TCHAR* OpenGPUDumpViewerBatName = TEXT("OpenGPUDumpViewer.bat");
			const TCHAR* OpenGPUDumpViewerShName = TEXT("OpenGPUDumpViewer.sh");

			const TCHAR* ViewerHTML = TEXT("GPUDumpViewer.html");
			FString DumpGPUViewerSourcePath = FPaths::EngineDir() + FString("Extras") / TEXT("GPUDumpViewer");

			PlatformFile.CopyFile(*(this->DumpingDirectoryPath / ViewerHTML), *(DumpGPUViewerSourcePath / ViewerHTML));
			PlatformFile.CopyFile(*(this->DumpingDirectoryPath / OpenGPUDumpViewerBatName), *(DumpGPUViewerSourcePath / OpenGPUDumpViewerBatName));
			PlatformFile.CopyFile(*(this->DumpingDirectoryPath / OpenGPUDumpViewerShName), *(DumpGPUViewerSourcePath / OpenGPUDumpViewerShName));
		});
	}

	GNextRDGResourceDumpContext = nullptr;
	GRDGResourceDumpContext_GameThread = this;

	ENQUEUE_RENDER_COMMAND(FStartGPUDump)(
		[this](FRHICommandListImmediate& ImmediateRHICmdList)
	{
		check(IsInRenderingThread());
		GRDGResourceDumpContext_RenderThread = this;

		ImmediateRHICmdList.SubmitAndBlockUntilGPUIdle();

		// Disable the validation for BUF_SourceCopy so that all buffers can be copied into staging buffer for CPU readback.
		#if ENABLE_RHI_VALIDATION
			GRHIValidateBufferSourceCopy = false;
		#endif
	});
}

void FRDGResourceDumpContext::Finish()
{
	check(IsInGameThread());
	check(GRDGResourceDumpContext_GameThread == this);
	
	// Wait all rendering commands are completed to finish with GRDGResourceDumpContext.
	{
		UE_LOGF(LogDumpGPU, Display, "Stalling game thread until render thread finishes to dump resources");

		ENQUEUE_RENDER_COMMAND(FEndGPUDump)(
			[this](FRHICommandListImmediate& ImmediateRHICmdList)
		{
			if (bStream)
			{
				WaitAndReleaseStagingResources(ImmediateRHICmdList);
			}

			ImmediateRHICmdList.SubmitAndBlockUntilGPUIdle();
			#if ENABLE_RHI_VALIDATION
				GRHIValidateBufferSourceCopy = true;
			#endif
		});

		FlushRenderingCommands();
	}

	// Log information about the dump.
	FString AbsDumpingDirectoryPath = IFileManager::Get().ConvertToAbsolutePathForExternalAppForRead(*DumpingDirectoryPath);
	{
		FDateTime Now = FDateTime::Now();
		double TotalDumpSeconds = (Now - Time).GetTotalSeconds();
		double RHIReadbackCommandsSeconds = TimingBucket[int32(FRDGResourceDumpContext::ETimingBucket::RHIReadbackCommands)];
		double GPUWaitSeconds = TimingBucket[int32(FRDGResourceDumpContext::ETimingBucket::GPUWait)];
		double CPUPostProcessingSeconds = TimingBucket[int32(FRDGResourceDumpContext::ETimingBucket::CPUPostProcessing)];
		double MetadataFileWriteSeconds = TimingBucket[int32(FRDGResourceDumpContext::ETimingBucket::MetadataFileWrite)];
		double ParametersFileWriteSeconds = TimingBucket[int32(FRDGResourceDumpContext::ETimingBucket::ParametersFileWrite)];
		double ResourceBinaryFileWriteSeconds = TimingBucket[int32(FRDGResourceDumpContext::ETimingBucket::ResourceBinaryFileWrite)];
		double RHIReleaseResourcesTimeSeconds = TimingBucket[int32(FRDGResourceDumpContext::ETimingBucket::RHIReleaseResources)];

		UE_LOGF(LogDumpGPU, Display, "Dumped %d resources in %.3f s to %ls", ResourcesDumpPasses, float(TotalDumpSeconds), *AbsDumpingDirectoryPath);
		UE_LOGF(LogDumpGPU, Display, "Dumped GPU readback commands: %.3f s", float(RHIReadbackCommandsSeconds));
		UE_LOGF(LogDumpGPU, Display, "Dumped GPU wait: %.3f s", float(GPUWaitSeconds));
		UE_LOGF(LogDumpGPU, Display, "Dumped CPU resource binary post processing: %.3f s", float(CPUPostProcessingSeconds));
		UE_LOGF(LogDumpGPU, Display, "Dumped metadata: %.3f MB in %d files under %.3f s at %.3f MB/s",
			float(MetadataFilesWriteBytes) / float(1024 * 1024),
			int32(MetadataFilesOpened),
			float(MetadataFileWriteSeconds),
			float(MetadataFilesWriteBytes) / (float(1024 * 1024) * float(MetadataFileWriteSeconds)));
		UE_LOGF(LogDumpGPU, Display, "Dumped parameters: %.3f MB in %d files under %.3f s at %.3f MB/s",
			float(ParametersFilesWriteBytes) / float(1024 * 1024),
			int32(ParametersFilesOpened),
			float(ParametersFileWriteSeconds),
			float(ParametersFilesWriteBytes) / (float(1024 * 1024) * float(ParametersFileWriteSeconds)));
		UE_LOGF(LogDumpGPU, Display, "Dumped resource binary: %.3f MB in %d files under %.3f s at %.3f MB/s",
			float(ResourceBinaryWriteBytes) / float(1024 * 1024),
			int32(ResourceBinaryFilesOpened),
			float(ResourceBinaryFileWriteSeconds),
			float(ResourceBinaryWriteBytes) / (float(1024 * 1024) * float(ResourceBinaryFileWriteSeconds)));
		UE_LOGF(LogDumpGPU, Display, "Dumped GPU readback resource release: %.3f s", float(RHIReleaseResourcesTimeSeconds));
	}

	// Dump the log into the dump directory.
	if (bEnableDiskWrite)
	{
		if (GLog)
		{
			GLog->FlushThreadedLogs();
			GLog->Flush();
		}
		if (GDumpGPUEnableLogWrite.GetValueOnGameThread())
		{
			FGenericCrashContext::DumpLog(DumpingDirectoryPath / FRDGResourceDumpContext::kBaseDir);
		}
	}

	// Update the dump status to OK and removes subscription to crashes
	{
		DumpStatusToFile(TEXT("ok"));
		FCoreDelegates::OnShutdownAfterError.RemoveAll(this);
	}

	if (bUpload && IDumpGPUUploadServiceProvider::GProvider)
	{
		IDumpGPUUploadServiceProvider::FDumpParameters DumpCompletedParameters = GetDumpParameters();

		// Compress the resource binary in background before uploading.
		if (!UploadResourceCompressionName.IsNone())
		{
			DumpCompletedParameters.CompressionName = UploadResourceCompressionName;
			DumpCompletedParameters.CompressionFiles = FWildcardString(TEXT("*.bin"));
		}

		IDumpGPUUploadServiceProvider::GProvider->UploadDump(DumpCompletedParameters);
	}

	#if PLATFORM_DESKTOP
	if (bShowInExplore)
	{
		FPlatformProcess::ExploreFolder(*AbsDumpingDirectoryPath);
	}
	#endif

	// Restore the engine tick rate to what it was
	if (bOverrideFixedDeltaTime)
	{
		FApp::SetFixedDeltaTime(PreviousFixedDeltaTime);
		FApp::SetUseFixedTimeStep(false);
	}

}

static const TCHAR* GetPassEventNameWithGPUMask(const FRDGPass* Pass, FString& OutNameStorage)
{
#if WITH_MGPU
	if ((GNumExplicitGPUsForRendering > 1) && GDumpGPUMask.GetValueOnRenderThread())
	{
		// Prepend GPU mask on the event name of each pass, so you can see which GPUs the pass ran on.  Putting the mask at the
		// front rather than the back makes all the masks line up, and easier to read (or ignore if you don't care about them).
		// Also, it's easy to globally search for passes with a particular GPU mask using name search in the dump browser.
		OutNameStorage = FString::Printf(TEXT("[%x] %s"), Pass->GetGPUMask().GetForDisplay(), Pass->GetEventName().GetTCHAR());
		return *OutNameStorage;
	}
	else
#endif  // WITH_MGPU
	{
		return Pass->GetEventName().GetTCHAR();
	}
}

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
void DumpReferencedStructs(const FShaderParametersMetadata* Metadata, const uint8* Content, uint32 ParentOffset,const FString& Prefix, TSharedPtr<FJsonObject> OutReferencedStructs)
{
	check(Metadata);
	check(Content);

	FRDGResourceDumpContext* ResourceDumpContext = GRDGResourceDumpContext_RenderThread;

	for (const FShaderParametersMetadata::FMember& Member : Metadata->GetMembers())
	{
		const FShaderParametersMetadata* MemberMetadata = Member.GetStructMetadata();
		if (!MemberMetadata)
		{
			// Basic types do not contain metadata and can be skipped
			continue;
		}

		FString MemberName = Prefix + Member.GetName();

		if (Member.GetBaseType() != EUniformBufferBaseType::UBMT_REFERENCED_STRUCT)
		{
			// Inlined structs can be skipped as they are contained in the pass content but since they can contain referenced structs we recursively call DumpReferencedStructs
			DumpReferencedStructs(MemberMetadata, Content, ParentOffset + Member.GetOffset(), MemberName + TEXT("."), OutReferencedStructs);
			continue;
		}

		const FUniformBufferBinding* UniformBufferBinding = reinterpret_cast<const FUniformBufferBinding*>(Content + ParentOffset + Member.GetOffset());
		if (!UniformBufferBinding)
		{
			// Skip referenced structs without a valid reference
			continue;
		}

		FRHIUniformBuffer* UniformBuffer = UniformBufferBinding->GetUniformBuffer();
		if (!UniformBuffer)
		{
			// Skip referenced structs without a valid buffer
			continue;
		}

		const uint8* ReferencedStructContent = (const uint8*)UE::RHI::UniformBufferContentMap::Get(UniformBuffer);
		// Record the mapping so the GPU dump viewer can link to this referenced struct's content
		if (ReferencedStructContent)
		{
			if (OutReferencedStructs)
			{
				TSharedPtr<FJsonObject> RefStructEntry = MakeShareable(new FJsonObject);
				RefStructEntry->SetStringField(TEXT("Content"), ResourceDumpContext->PtrToString(UniformBuffer));
				RefStructEntry->SetStringField(TEXT("Metadata"), ResourceDumpContext->PtrToString(MemberMetadata));
				OutReferencedStructs->SetObjectField(MemberName, RefStructEntry);
			}
		}

		if (!ResourceDumpContext->IsDumped(UniformBuffer))
		{
			ResourceDumpContext->SetDumped(UniformBuffer);
			ResourceDumpContext->Dump(MemberMetadata);

			if (ReferencedStructContent)
			{
				DumpReferencedStructs(MemberMetadata, ReferencedStructContent, 0, MemberName + TEXT("."), OutReferencedStructs);

				uint32 PassParametersByteSize = MemberMetadata->GetSize();
				if (PassParametersByteSize > 0)
				{
					TArrayView<const uint8> ArrayView(ReferencedStructContent, PassParametersByteSize);
					FString DumpFilePath = FRDGResourceDumpContext::kStructuresDir / ResourceDumpContext->PtrToString(UniformBuffer) + TEXT(".bin");
					ResourceDumpContext->DumpBinaryToFile(ArrayView, DumpFilePath, FRDGResourceDumpContext::ETimingBucket::ParametersFileWrite);
				}
			}
			else
			{
				UE_LOGF(LogDumpGPU, Warning, "Unable to dump referenced struct %ls as content could not be found. Please make sure to enable r.RHI.UniformBufferContentMap.Enable.", Member.GetShaderType());
			}
		}
	}
}
#endif

void FRDGBuilder::DumpNewGraphBuilder()
{
	FRDGResourceDumpContext* ResourceDumpContext = GRDGResourceDumpContext_RenderThread;
	if (!ResourceDumpContext)
	{
		return;
	}

	check(IsInRenderingThread());
	ResourceDumpContext->GraphBuilderIndex++;
	ResourceDumpContext->LastResourceVersion.Empty();
	ResourceDumpContext->IsDumpedToDisk.Empty();
}

void FRDGBuilder::DumpResourcePassOutputs(const FRDGPass* Pass)
{
	if (!AuxiliaryPasses.IsDumpAllowed())
	{
		return;
	}

	FRDGResourceDumpContext* ResourceDumpContext = GRDGResourceDumpContext_RenderThread;

	if (!ResourceDumpContext)
	{
		return;
	}

	check(IsInRenderingThread());
	if (!ResourceDumpContext->IsDumpingPass(Pass))
	{
		return;
	}

	RDG_RECURSION_COUNTER_SCOPE(AuxiliaryPasses.Dump);

	TArray<TSharedPtr<FJsonValue>> InputResourceNames;
	TArray<TSharedPtr<FJsonValue>> OutputResourceNames;
	Pass->GetParameters().Enumerate([&](FRDGParameter Parameter)
	{
		switch (Parameter.GetType())
		{
		case UBMT_RDG_TEXTURE:
		{
			if (FRDGTextureRef Texture = Parameter.GetAsTexture())
			{
				FRDGTextureSRVDesc TextureSubResource = FRDGTextureSRVDesc::Create(Texture);
				ResourceDumpContext->AddDumpTexturePasses(*this, InputResourceNames, OutputResourceNames, Pass, TextureSubResource, /* bIsOutputResource = */ false);
			}
		}
		break;
		case UBMT_RDG_TEXTURE_SRV:
		case UBMT_RDG_TEXTURE_NON_PIXEL_SRV:
		{
			if (FRDGTextureSRVRef SRV = Parameter.GetAsTextureSRV())
			{
				if (SRV->Desc.MetaData == ERHITextureMetaDataAccess::None)
				{
					ResourceDumpContext->AddDumpTexturePasses(*this, InputResourceNames, OutputResourceNames, Pass, SRV->Desc, /* bIsOutputResource = */ false);
				}
				else
				{
					UE_LOGF(LogDumpGPU, Warning, "Dumping texture %ls's meta data unsupported", SRV->Desc.Texture->Name);
				}
			}
		}
		break;
		case UBMT_RDG_TEXTURE_UAV:
		{
			if (FRDGTextureUAVRef UAV = Parameter.GetAsTextureUAV())
			{
				if (UAV->Desc.MetaData == ERHITextureMetaDataAccess::None)
				{
					FRDGTextureSRVDesc TextureSubResource = FRDGTextureSRVDesc::Create(UAV->Desc.Texture);
					TextureSubResource.MipLevel = UAV->Desc.MipLevel;
					TextureSubResource.NumMipLevels = 1;
					if (UAV->Desc.Texture->Desc.IsTextureArray())
					{
						TextureSubResource.FirstArraySlice = UAV->Desc.FirstArraySlice;
						TextureSubResource.NumArraySlices = UAV->Desc.NumArraySlices;
					}
					ResourceDumpContext->AddDumpTexturePasses(*this, InputResourceNames, OutputResourceNames, Pass, TextureSubResource, /* bIsOutputResource = */ true);
				}
				else
				{
					UE_LOGF(LogDumpGPU, Warning, "Dumping texture %ls's meta data unsupported", UAV->Desc.Texture->Name);
				}
			}
		}
		break;
		case UBMT_RDG_TEXTURE_ACCESS:
		{
			if (FRDGTextureAccess TextureAccess = Parameter.GetAsTextureAccess())
			{
				bool bIsOutputResource = IsWritableAccess(TextureAccess.GetAccess());

				FRDGTextureSRVDesc TextureSubResource = FRDGTextureSRVDesc::Create(TextureAccess);
				ResourceDumpContext->AddDumpTexturePasses(*this, InputResourceNames, OutputResourceNames, Pass, TextureSubResource, bIsOutputResource);
			}
		}
		break;
		case UBMT_RDG_TEXTURE_ACCESS_ARRAY:
		{
			const FRDGTextureAccessArray& TextureAccessArray = Parameter.GetAsTextureAccessArray();

			for (FRDGTextureAccess TextureAccess : TextureAccessArray)
			{
				bool bIsOutputResource = IsWritableAccess(TextureAccess.GetAccess());

				FRDGTextureSRVDesc TextureSubResource = FRDGTextureSRVDesc::Create(TextureAccess);
				ResourceDumpContext->AddDumpTexturePasses(*this, InputResourceNames, OutputResourceNames, Pass, TextureSubResource, bIsOutputResource);
			}
		}
		break;

		case UBMT_RDG_BUFFER_SRV:
		{
			if (FRDGBufferSRVRef SRV = Parameter.GetAsBufferSRV())
			{
				FRDGBufferRef Buffer = SRV->Desc.Buffer;
				ResourceDumpContext->AddDumpBufferPass(*this, InputResourceNames, OutputResourceNames, Pass, Buffer, /* bIsOutputResource = */ false);
			}
		}
		break;
		case UBMT_RDG_BUFFER_UAV:
		{
			if (FRDGBufferUAVRef UAV = Parameter.GetAsBufferUAV())
			{
				FRDGBufferRef Buffer = UAV->Desc.Buffer;
				ResourceDumpContext->AddDumpBufferPass(*this, InputResourceNames, OutputResourceNames, Pass, Buffer, /* bIsOutputResource = */ true);
			}
		}
		break;
		case UBMT_RDG_BUFFER_ACCESS:
		{
			if (FRDGBufferAccess BufferAccess = Parameter.GetAsBufferAccess())
			{
				bool bIsOutputResource = IsWritableAccess(BufferAccess.GetAccess());

				ResourceDumpContext->AddDumpBufferPass(*this, InputResourceNames, OutputResourceNames, Pass, BufferAccess, bIsOutputResource);
			}
		}
		break;
		case UBMT_RDG_BUFFER_ACCESS_ARRAY:
		{
			const FRDGBufferAccessArray& BufferAccessArray = Parameter.GetAsBufferAccessArray();

			for (FRDGBufferAccess BufferAccess : BufferAccessArray)
			{
				bool bIsOutputResource = IsWritableAccess(BufferAccess.GetAccess());

				ResourceDumpContext->AddDumpBufferPass(*this, InputResourceNames, OutputResourceNames, Pass, BufferAccess, bIsOutputResource);
			}
		}
		break;

		case UBMT_RENDER_TARGET_BINDING_SLOTS:
		{
			const FRenderTargetBindingSlots& RenderTargets = Parameter.GetAsRenderTargetBindingSlots();

			RenderTargets.Enumerate([&](FRenderTargetBinding RenderTarget)
			{
				FRDGTextureRef Texture = RenderTarget.GetTexture();
				FRDGTextureSRVDesc TextureSubResource = FRDGTextureSRVDesc::CreateForMipLevel(Texture, RenderTarget.GetMipIndex());

				if (Texture->Desc.IsTextureArray() && RenderTarget.GetArraySlice() != -1)
				{
					TextureSubResource.FirstArraySlice = RenderTarget.GetArraySlice();
					TextureSubResource.NumArraySlices = 1;
				}

				ResourceDumpContext->AddDumpTexturePasses(*this, InputResourceNames, OutputResourceNames, Pass, TextureSubResource, /* bIsOutputResource = */ true);
			});

			const FDepthStencilBinding& DepthStencil = RenderTargets.DepthStencil;

			if (FRDGTextureRef Texture = DepthStencil.GetTexture())
			{
				FExclusiveDepthStencil DepthStencilAccess = DepthStencil.GetDepthStencilAccess();

				if (DepthStencilAccess.IsUsingDepth())
				{
					FRDGTextureSRVDesc TextureSubResource = FRDGTextureSRVDesc::CreateForMipLevel(Texture, 0);
					if (Texture->Desc.IsTextureArray())
					{
						TextureSubResource.FirstArraySlice = 0;
						TextureSubResource.NumArraySlices = 1;
					}

					ResourceDumpContext->AddDumpTexturePasses(
						*this, InputResourceNames, OutputResourceNames, Pass, TextureSubResource,
						/* bIsOutputResource = */ DepthStencilAccess.IsDepthWrite());
				}

				if (DepthStencilAccess.IsUsingStencil())
				{
					FRDGTextureSRVDesc TextureSubResource = FRDGTextureSRVDesc::CreateWithPixelFormat(Texture, PF_X24_G8);
					if (Texture->Desc.IsTextureArray())
					{
						TextureSubResource.FirstArraySlice = 0;
						TextureSubResource.NumArraySlices = 1;
					}

					ResourceDumpContext->AddDumpTexturePasses(
						*this, InputResourceNames, OutputResourceNames, Pass, TextureSubResource,
						/* bIsOutputResource = */ DepthStencilAccess.IsStencilWrite());
				}
			}
		}
		break;
		}
	});

	// Dump the pass informations
	{
		TArray<TSharedPtr<FJsonValue>> ParentEventScopeNames;
		{
#if RDG_EVENTS
			FRDGScope const* ParentScope = Pass->GetScope();
			if (ParentScope)
			{
				// FRDGScopeState sets ERDGScopeMode::AllEventsAndPassNames when DumpGPU is active.
				ParentScope = ParentScope->Parent;
			}
			for (; ParentScope; ParentScope = ParentScope->Parent)
			{
				if (FRDGScope_RHI const* RHIScope = ParentScope->Get<FRDGScope_RHI>())
				{
					FRHIBreadcrumb::FBuffer Buffer;
					ParentEventScopeNames.Add(MakeShareable(new FJsonValueString(RHIScope->GetTCHAR(Buffer))));
				}
			}
#endif // RDG_EVENTS

			ParentEventScopeNames.Add(MakeShareable(new FJsonValueString(FString::Printf(TEXT("Frame %llu (Delta=%fs)"), GFrameCounterRenderThread, ResourceDumpContext->DeltaTime))));
		}

		FString EventNameStorage;

		TSharedPtr<FJsonObject> JsonObject = MakeShareable(new FJsonObject);
		JsonObject->SetStringField(TEXT("EventName"), GetPassEventNameWithGPUMask(Pass, EventNameStorage));
		JsonObject->SetStringField(TEXT("ParametersName"), Pass->GetParameters().GetLayout().GetDebugName());
		JsonObject->SetStringField(TEXT("Parameters"), ResourceDumpContext->PtrToString(Pass->GetParameters().GetContents()));
		JsonObject->SetStringField(TEXT("ParametersMetadata"), ResourceDumpContext->PtrToString(Pass->GetParameters().GetMetadata()));
		JsonObject->SetStringField(TEXT("Pointer"), ResourceDumpContext->PtrToString(Pass));
		JsonObject->SetNumberField(TEXT("Id"), ResourceDumpContext->PassesCount);
		JsonObject->SetArrayField(TEXT("ParentEventScopes"), ParentEventScopeNames);
		JsonObject->SetArrayField(TEXT("InputResources"), InputResourceNames);
		JsonObject->SetArrayField(TEXT("OutputResources"), OutputResourceNames);

		ResourceDumpContext->DumpJsonToFile(JsonObject, FString(FRDGResourceDumpContext::kBaseDir) / TEXT("Passes.json"), FILEWRITE_Append);
	}

	// Dump the pass' parameters
	if (GDumpGPUPassParameters.GetValueOnRenderThread() != 0)
	{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		TSharedPtr<FJsonObject> ReferencedStructsJson = MakeShareable(new FJsonObject);
#endif
		int32 PassParametersByteSize = 0;
		{
			const FShaderParametersMetadata* Metadata = Pass->GetParameters().GetMetadata();
			if (Metadata)
			{
				ResourceDumpContext->Dump(Metadata);
				PassParametersByteSize = Metadata->GetSize();
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
				if (const uint8* Content = Pass->GetParameters().GetContents())
				{
					DumpReferencedStructs(Metadata, Content, 0, TEXT(""), ReferencedStructsJson);
				}
#endif
			}
		}

		if (PassParametersByteSize == 0 && Pass->GetParameters().GetLayoutPtr())
		{
			PassParametersByteSize = Pass->GetParameters().GetLayout().ConstantBufferSize;
		}

		const uint8* PassParametersContent = Pass->GetParameters().GetContents();
		if (PassParametersContent && !ResourceDumpContext->IsDumped(PassParametersContent))
		{
			TArrayView<const uint8> ArrayView(PassParametersContent, PassParametersByteSize);
			FString DumpFilePath = FRDGResourceDumpContext::kStructuresDir / ResourceDumpContext->PtrToString(PassParametersContent) + TEXT(".bin");
			ResourceDumpContext->DumpBinaryToFile(ArrayView, DumpFilePath, FRDGResourceDumpContext::ETimingBucket::ParametersFileWrite);
			ResourceDumpContext->SetDumped(PassParametersContent);
		}
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		// Write referenced struct mapping for the viewer if any were found
		if (ReferencedStructsJson->Values.Num() > 0)
		{
			TSharedPtr<FJsonObject> ReferencedStructsFile = MakeShareable(new FJsonObject);
			ReferencedStructsFile->SetStringField(TEXT("Pass"), ResourceDumpContext->PtrToString(Pass));
			ReferencedStructsFile->SetObjectField(TEXT("ReferencedStructs"), ReferencedStructsJson);
			FString RefStructsPath = FString(FRDGResourceDumpContext::kBaseDir) / TEXT("ReferencedStructs.json");
			ResourceDumpContext->DumpJsonToFile(ReferencedStructsFile, RefStructsPath, FILEWRITE_Append);
		}
#endif

	}

	ResourceDumpContext->PassesCount++;
}

bool FRDGBuilder::IsDumpingFrame()
{
	return GRDGResourceDumpContext_RenderThread != nullptr;
}

namespace UE::RenderCore::DumpGPU
{

RENDERCORE_API FDumpScope::FDumpScope(bool bInCondition, const TCHAR* Cmd, const TCHAR* Context)
	: bCondition(bInCondition && !IsDumpingFrame())
{
	if (bCondition)
	{
		// Pass "-oneframe" to override CVar that could enable multiple frames of capture
		if (Cmd == nullptr)
		{
			Cmd = TEXT("");
		}

		const FString CmdWithOneFrame = FString::Printf(TEXT("-oneframe %s"), Cmd);

		FRDGBuilder::BeginResourceDump(*CmdWithOneFrame, Context);

		// Tick the DumpGPU system, which will start the dump
		TickEndFrame();
	}
}

void TickEndFrame()
{
	check(IsInGameThread());

	if (GRDGResourceDumpContext_GameThread)
	{
		check(GNextRDGResourceDumpContext == nullptr);

		GRDGResourceDumpContext_GameThread->DumpedFrameId++;
		check(GRDGResourceDumpContext_GameThread->DumpedFrameId <= GRDGResourceDumpContext_GameThread->FrameCount);

		if (GRDGResourceDumpContext_GameThread->DumpedFrameId < GRDGResourceDumpContext_GameThread->FrameCount)
		{
			int32 RemainingFrameCount = GRDGResourceDumpContext_GameThread->FrameCount - GRDGResourceDumpContext_GameThread->DumpedFrameId;
			ENQUEUE_RENDER_COMMAND(FDumpGPULogRemaingFrames)(
				[RemainingFrameCount](FRHICommandListImmediate& ImmediateRHICmdList)
			{
				UE_LOGF(LogDumpGPU, Display, "Remaining frames %d", RemainingFrameCount);

				if (GRDGResourceDumpContext_RenderThread->bStream)
				{
					GRDGResourceDumpContext_RenderThread->LandCompletedResources(ImmediateRHICmdList);
				}
			});
		}
		else
		{
			GRDGResourceDumpContext_GameThread->Finish();
			delete GRDGResourceDumpContext_GameThread;
			GRDGResourceDumpContext_GameThread = nullptr;
			GRDGResourceDumpContext_RenderThread = nullptr;

			// It matches SuspendHeartBeat from BeginResourceDump.
			FThreadHeartBeat::Get().ResumeHeartBeat(true);
			ResumeRenderThreadTimeout();
		}
	}
	else if (GNextRDGResourceDumpContext)
	{
		check(GRDGResourceDumpContext_GameThread == nullptr);

		if (GNextDumpingRemainingTime > 0.0f)
		{
			check(GNextRDGResourceDumpContext);
			GNextDumpingRemainingTime -= FApp::GetDeltaTime();
			if (GNextDumpingRemainingTime <= 0.0)
			{
				GNextDumpingRemainingTime = -1.0f;
				GNextRDGResourceDumpContext->Start();
			}
		}
		else if (GNextDumpingRemaingFrames > 0)
		{
			check(GNextRDGResourceDumpContext);
			GNextDumpingRemaingFrames -= 1;
			if (GNextDumpingRemaingFrames <= 0)
			{
				GNextDumpingRemaingFrames = -1;
				GNextRDGResourceDumpContext->Start();
			}
		}
		else
		{
			GNextRDGResourceDumpContext->Start();
		}
	}
}

bool IsDumpingFrame()
{
	check(IsInGameThread() || IsInRenderingThread());

	if (IsInGameThread())
	{
		return GRDGResourceDumpContext_GameThread != nullptr;
	}
	else if (IsInRenderingThread())
	{
		return GRDGResourceDumpContext_RenderThread != nullptr;
	}
	else
	{
		check(0);
	}
	return false;
}

bool ShouldCameraCut()
{
	check(IsInGameThread());
	if (GRDGResourceDumpContext_GameThread == nullptr)
	{
		return false;
	}

	if (GDumpGPUCameraCut.GetValueOnGameThread() == 0)
	{
		return false;
	}

	return GRDGResourceDumpContext_GameThread->DumpedFrameId == 0;
}

} // namespace UE::RenderCore::DumpGPU

#endif // RDG_DUMP_RESOURCES
