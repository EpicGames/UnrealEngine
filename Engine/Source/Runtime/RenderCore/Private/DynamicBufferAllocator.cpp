// Copyright Epic Games, Inc. All Rights Reserved.

/*==============================================================================
DynamicBufferAllocator.cpp: Classes for allocating transient rendering data.
==============================================================================*/

#include "DynamicBufferAllocator.h"
#include "Math/Float16.h"
#include "RenderResource.h"
#include "Misc/ScopeLock.h"
#include "RenderCore.h"

int32 GMaxReadBufferRenderingBytesAllocatedPerFrame = 32 * 1024 * 1024;

FAutoConsoleVariableRef CVarMaxReadBufferRenderingBytesAllocatedPerFrame(
	TEXT("r.ReadBuffer.MaxRenderingBytesAllocatedPerFrame"),
	GMaxReadBufferRenderingBytesAllocatedPerFrame,
	TEXT("The maximum number of transient rendering read buffer bytes to allocate before we start panic logging who is doing the allocations"));

// The allocator works by looking for the first free buffer that contains the required number of elements.  There is currently no trim so buffers stay in memory.
// To avoid increasing allocation sizes over multiple frames causing severe memory bloat (i.e. 100 elements, 1001 elements) we first align the required
// number of elements to GMinReadBufferRenderingBufferSize, we then take the max(aligned num, GMinReadBufferRenderingBufferSize)
int32 GMinReadBufferRenderingBufferSize = 256 * 1024;
FAutoConsoleVariableRef CVarMinReadBufferSize(
	TEXT("r.ReadBuffer.MinSize"),
	GMinReadBufferRenderingBufferSize,
	TEXT("The minimum size (in instances) to allocate in blocks for rendering read buffers. i.e. 256*1024 = 1mb for a float buffer"));

int32 GAlignReadBufferRenderingBufferSize = 64 * 1024;
FAutoConsoleVariableRef CVarAlignReadBufferSize(
	TEXT("r.ReadBuffer.AlignSize"),
	GAlignReadBufferRenderingBufferSize,
	TEXT("The alignment size (in instances) to allocate in blocks for rendering read buffers. i.e. 64*1024 = 256k for a float buffer"));

int32 GReadBufferD3D11StaticAllocation = true;
FAutoConsoleVariableRef CVarReadBufferD3D11StaticAllocation(
	TEXT("r.ReadBuffer.D3D11StaticAllocation"),
	GReadBufferD3D11StaticAllocation,
	TEXT("Whether to allocate the read buffer as Static instead of Dynamic in D3D11, to work around driver inefficiencies."));

struct FDynamicReadBufferPool
{
	/** List of vertex buffers. */
	TIndirectArray<FDynamicAllocReadBuffer> Buffers;
	/** The current buffer from which allocations are being made. */
	FDynamicAllocReadBuffer* CurrentBuffer;

	/** Default constructor. */
	FDynamicReadBufferPool()
		: CurrentBuffer(NULL)
	{
	}

	/** Destructor. */
	~FDynamicReadBufferPool()
	{
		int32 NumVertexBuffers = Buffers.Num();
		for (int32 BufferIndex = 0; BufferIndex < NumVertexBuffers; ++BufferIndex)
		{
			Buffers[BufferIndex].Release();
		}
	}
};



FGlobalDynamicReadBuffer::FGlobalDynamicReadBuffer()
	: TotalAllocatedSinceLastCommit(0)
{
	FloatBufferPool = new FDynamicReadBufferPool();
	Int32BufferPool = new FDynamicReadBufferPool();
	UInt32BufferPool = new FDynamicReadBufferPool();
	HalfBufferPool = new FDynamicReadBufferPool();
}

FGlobalDynamicReadBuffer::~FGlobalDynamicReadBuffer()
{
	Cleanup();
}

void FGlobalDynamicReadBuffer::Cleanup()
{
	if (FloatBufferPool)
	{
		delete FloatBufferPool;
		FloatBufferPool = nullptr;
	}

	if (Int32BufferPool)
	{
		delete Int32BufferPool;
		Int32BufferPool = nullptr;
	}

	if (UInt32BufferPool)
	{
		delete UInt32BufferPool;
		UInt32BufferPool = nullptr;
	}

	if (HalfBufferPool)
	{
		delete HalfBufferPool;
		HalfBufferPool = nullptr;
	}
}
void FGlobalDynamicReadBuffer::InitRHI(FRHICommandListBase&)
{
}

void FGlobalDynamicReadBuffer::ReleaseRHI()
{
	Cleanup();
}

template<EPixelFormat Format, typename Type>
FGlobalDynamicReadBuffer::FAllocation FGlobalDynamicReadBuffer::AllocateInternal(FDynamicReadBufferPool* BufferPool, uint32 Num)
{
	UE::TScopeLock ScopeLock(Mutex);
	FGlobalDynamicReadBuffer::FAllocation Allocation;

	if (!RHICmdList)
	{
		RHICmdList = new FRHICommandList(FRHIGPUMask::All());
		RHICmdList->SwitchPipeline(ERHIPipeline::Graphics);
	}

	uint32 SizeInBytes = sizeof(Type) * Num;
	FDynamicAllocReadBuffer* Buffer = BufferPool->CurrentBuffer;

	uint64 BufferAlignment = RHIGetMinimumAlignmentForBufferBackedSRV(Format);
	uint32 ByteOffset = Buffer == nullptr ? 0 : Align(Buffer->AllocatedByteCount, BufferAlignment);

	if (Buffer == nullptr || ByteOffset + SizeInBytes > Buffer->NumBytes)
	{
		// Find a buffer in the pool big enough to service the request.
		Buffer = nullptr;
		for (int32 BufferIndex = 0, NumBuffers = BufferPool->Buffers.Num(); BufferIndex < NumBuffers; ++BufferIndex)
		{
			FDynamicAllocReadBuffer& BufferToCheck = BufferPool->Buffers[BufferIndex];
			uint32 ByteOffsetToCheck = Align(BufferToCheck.AllocatedByteCount, BufferAlignment);
			if (ByteOffsetToCheck + SizeInBytes <= BufferToCheck.NumBytes)
			{
				Buffer = &BufferToCheck;
				break;
			}
		}

		// Create a new vertex buffer if needed.
		if (Buffer == nullptr)
		{
			const uint32 AlignedNum = FMath::DivideAndRoundUp(Num, (uint32)GAlignReadBufferRenderingBufferSize) * GAlignReadBufferRenderingBufferSize;
			const uint64 MaxViewDimensionForTypedBuffer = GRHIGlobals.MaxViewDimensionForTypedBuffer;
			// Cannot have the MinReadBufferRenderingBufferSize be greater than MaxViewDimensionForTypedBuffer for that platform
			const uint32 AdjustedMinReadBufferRenderingBufferSize = FMath::Min<uint32>(GMinReadBufferRenderingBufferSize, MaxViewDimensionForTypedBuffer);
			const uint32 NewBufferSize = FMath::Max(AlignedNum, AdjustedMinReadBufferRenderingBufferSize);
			if (UNLIKELY(NewBufferSize > MaxViewDimensionForTypedBuffer))
			{
				UE_LOGF(LogRendererCore, Warning, "FGlobalDynamicReadBuffer::AllocateInternal, trying to allocate a Buffer of size (in elements) %d that is greater than the platform max of %llu", NewBufferSize, MaxViewDimensionForTypedBuffer);
			}
			Buffer = new FDynamicAllocReadBuffer();
			BufferPool->Buffers.Add(Buffer);
			// Use static in D3D11 because we're running out of zeroed pages, and that causes the upload of this buffer to take a very long time. Static uses more memory and might be slightly less efficient
			// on the GPU, but it beats stalling for tens of milliseconds when committing the buffer.
			EBufferUsageFlags Flags = BUF_Dynamic;
			if (GReadBufferD3D11StaticAllocation && GDynamicRHI->GetInterfaceType() == ERHIInterfaceType::D3D11)
			{
				Flags = BUF_Static;
			}
			Buffer->Initialize(*RHICmdList, TEXT("FGlobalDynamicReadBuffer_AllocateInternal"), sizeof(Type), NewBufferSize, Format, Flags);
		}

		// Lock the buffer if needed.
		if (Buffer->MappedBuffer == nullptr)
		{
			Buffer->Lock(*RHICmdList);
		}

		// Remember this buffer, we'll try to allocate out of it in the future.
		BufferPool->CurrentBuffer = Buffer;
	}
	Buffer->AllocatedByteCount = Align(Buffer->AllocatedByteCount, BufferAlignment);

	check(Buffer != nullptr);
	checkf(Buffer->AllocatedByteCount + SizeInBytes <= Buffer->NumBytes, TEXT("Global dynamic read buffer allocation failed: BufferSize=%d AllocatedByteCount=%d SizeInBytes=%d"), Buffer->NumBytes, Buffer->AllocatedByteCount, SizeInBytes);
	Allocation.Buffer = Buffer->MappedBuffer + Buffer->AllocatedByteCount;
	Allocation.ReadBuffer = Buffer;
	Buffer->SubAllocations.Emplace(RHICmdList->CreateShaderResourceView(FShaderResourceViewInitializer(Buffer->Buffer, Format, Buffer->AllocatedByteCount, Num)));
	Allocation.SRV = Buffer->SubAllocations.Last();
	Buffer->AllocatedByteCount += SizeInBytes;

	return Allocation;
}

void FGlobalDynamicReadBuffer::IncrementTotalAllocations(uint32 Num)
{
	TotalAllocatedSinceLastCommit += Num;
	if (IsRenderAlarmLoggingEnabled())
	{
		UE_LOG(LogRendererCore, Warning, TEXT("FGlobalReadBuffer::AllocateInternal(%u), will have allocated %" SIZE_T_FMT " total this frame"), Num, TotalAllocatedSinceLastCommit);
	}
}

FGlobalDynamicReadBuffer::FAllocation FGlobalDynamicReadBuffer::AllocateFloat(uint32 Num)
{
	IncrementTotalAllocations(Num);
	return AllocateInternal<PF_R32_FLOAT, float>(FloatBufferPool, Num);
}

FGlobalDynamicReadBuffer::FAllocation FGlobalDynamicReadBuffer::AllocateHalf(uint32 Num)
{
	IncrementTotalAllocations(Num);
	return AllocateInternal<PF_R16F, FFloat16>(HalfBufferPool, Num);
}

FGlobalDynamicReadBuffer::FAllocation FGlobalDynamicReadBuffer::AllocateInt32(uint32 Num)
{
	IncrementTotalAllocations(Num);
	return AllocateInternal<PF_R32_SINT, int32>(Int32BufferPool, Num);
}

FGlobalDynamicReadBuffer::FAllocation FGlobalDynamicReadBuffer::AllocateUInt32(uint32 Num)
{
	IncrementTotalAllocations(Num);
	return AllocateInternal<PF_R32_UINT, uint32>(UInt32BufferPool, Num);
}

bool FGlobalDynamicReadBuffer::IsRenderAlarmLoggingEnabled() const
{
	return GMaxReadBufferRenderingBytesAllocatedPerFrame > 0 && TotalAllocatedSinceLastCommit >= (size_t)GMaxReadBufferRenderingBytesAllocatedPerFrame;
}

static void RemoveUnusedBuffers(FRHICommandListBase* RHICmdList, FDynamicReadBufferPool* BufferPool)
{
	extern int32 GGlobalBufferNumFramesUnusedThreshold;

	for (int32 BufferIndex = 0, NumBuffers = BufferPool->Buffers.Num(); BufferIndex < NumBuffers; ++BufferIndex)
	{
		FDynamicAllocReadBuffer& Buffer = BufferPool->Buffers[BufferIndex];
		if (Buffer.MappedBuffer != nullptr)
		{
			check(RHICmdList);
			Buffer.Unlock(*RHICmdList);
		}
		else if (GGlobalBufferNumFramesUnusedThreshold && !Buffer.AllocatedByteCount)
		{
			++Buffer.NumFramesUnused;
			if (Buffer.NumFramesUnused >= GGlobalBufferNumFramesUnusedThreshold)
			{
				// Remove the buffer, assumes they are unordered.
				Buffer.Release();
				BufferPool->Buffers.RemoveAtSwap(BufferIndex);
				--BufferIndex;
				--NumBuffers;
			}
		}
	}
}

void FGlobalDynamicReadBuffer::Commit(FRHICommandListImmediate& RHICmdListImmediate)
{
	UE::TScopeLock ScopeLock(Mutex);

	RemoveUnusedBuffers(RHICmdList, FloatBufferPool);
	FloatBufferPool->CurrentBuffer = nullptr;

	RemoveUnusedBuffers(RHICmdList, Int32BufferPool);
	Int32BufferPool->CurrentBuffer = nullptr;

	RemoveUnusedBuffers(RHICmdList, UInt32BufferPool);
	UInt32BufferPool->CurrentBuffer = nullptr;

	RemoveUnusedBuffers(RHICmdList, HalfBufferPool);
	HalfBufferPool->CurrentBuffer = nullptr;

	if (RHICmdList)
	{
		RHICmdList->FinishRecording();
		RHICmdListImmediate.QueueAsyncCommandListSubmit(RHICmdList);
		RHICmdList = nullptr;
	}

	TotalAllocatedSinceLastCommit = 0;
}
