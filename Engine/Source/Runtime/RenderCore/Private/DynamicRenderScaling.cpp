// Copyright Epic Games, Inc. All Rights Reserved.

#include "DynamicRenderScaling.h"
#include "RenderCore.h"

namespace
{

int32 GRDSGlobalListLinkSize = 0;

}

namespace DynamicRenderScaling
{

bool FHeuristicSettings::IsEnabled() const
{
	return BudgetMs > 0.0f;
}

float FHeuristicSettings::GetTargetedMs(float InBudgetMs) const
{
	check(IsEnabled());
	return InBudgetMs * (1.0f - TargetedHeadRoom);
}

float FHeuristicSettings::EstimateCostScale(float ResolutionFraction) const
{
	float Cost = ResolutionFraction;
	if (Model == EHeuristicModel::Linear)
	{
		Cost = ResolutionFraction;
	}
	else if (Model == EHeuristicModel::Quadratic)
	{
		Cost = ResolutionFraction * ResolutionFraction;
	}
	else
	{
		unimplemented();
	}
	return Cost;
}

float FHeuristicSettings::EstimateResolutionFactor(float TargetedMS, float TimingMs) const
{
	float S = 1.0f;
	if (Model == EHeuristicModel::Linear)
	{
		S = TargetedMS / TimingMs;
	}
	else if (Model == EHeuristicModel::Quadratic)
	{
		S = FMath::Sqrt(TargetedMS / TimingMs);
	}
	else
	{
		unimplemented();
	}

	return S;
}

float FHeuristicSettings::CorrectNewResolutionFraction(float CurrentResolutionFraction, float NewResolutionFraction, float ResolutionFractionScale) const
{
	// If scaling the resolution up, amortize to avoid oscillations.
	if (NewResolutionFraction > CurrentResolutionFraction)
	{
		NewResolutionFraction = FMath::Lerp(
			CurrentResolutionFraction,
			NewResolutionFraction,
			IncreaseAmortizationFactor);
	}

	if (FractionQuantization <= 0)
	{
		return FMath::Clamp(NewResolutionFraction, MinResolutionFraction * ResolutionFractionScale, MaxResolutionFraction * ResolutionFractionScale);
	}

	float Lerp = FMath::Clamp((NewResolutionFraction - MinResolutionFraction * ResolutionFractionScale) / (MaxResolutionFraction * ResolutionFractionScale - MinResolutionFraction * ResolutionFractionScale), 0.0f, 1.0f);
	int32 Quantization = FMath::RoundToZero(Lerp * FractionQuantization);
	float NewLerp = float(Quantization) / float(FractionQuantization);

	return FMath::Lerp(MinResolutionFraction, MaxResolutionFraction, Lerp) * ResolutionFractionScale;
}

bool FHeuristicSettings::DoesResolutionChangeEnough(float CurrentResolutionFraction, float NewResolutionFraction, bool bCanChangeResolution) const
{
	if (ChangeThreshold == 0.0f)
	{
		return true;
	}

	float IncreaseResolutionFractionThreshold = CurrentResolutionFraction * (1.0f + ChangeThreshold);
	float DecreaseResolutionFractionThreshold = CurrentResolutionFraction * (1.0f - ChangeThreshold);
	return bCanChangeResolution && (
		(NewResolutionFraction > IncreaseResolutionFractionThreshold) ||
		(NewResolutionFraction < DecreaseResolutionFractionThreshold) ||
		(NewResolutionFraction != CurrentResolutionFraction && NewResolutionFraction == MinResolutionFraction) ||
		(NewResolutionFraction != CurrentResolutionFraction && NewResolutionFraction == MaxResolutionFraction));
}

float FHeuristicSettings::EstimateTimeFactor(float CurrentResolutionFraction, float NewResolutionFraction) const
{
	float R = NewResolutionFraction / CurrentResolutionFraction;

	if (Model == EHeuristicModel::Linear)
	{
		return R;
	}
	else if (Model == EHeuristicModel::Quadratic)
	{
		return R * R;
	}
	else
	{
		unimplemented();
	}

	return 1.0f;
}

FBudget::FBudget(const TCHAR* InName, FHeuristicSettings (*InHeuristicSettingsGetter)(void))
	: Name(InName)
	, HeuristicSettingsGetter(InHeuristicSettingsGetter)
	, GlobalListLink(this)
{
	check(InName);

	AnsiName.SetNumZeroed(FCString::Strlen(InName) + 1);
	FCStringAnsi::Strncpy(AnsiName.GetData(), TCHAR_TO_ANSI(InName), AnsiName.Num());

	GlobalListLink.LinkHead(FBudget::GetGlobalList());
	BudgetId = GRDSGlobalListLinkSize;
	GRDSGlobalListLinkSize++;

#if !UE_BUILD_SHIPPING
	if (GRDSGlobalListLinkSize > TMap<float>::kInlineAllocatedBudgets)
	{
		UE_LOGF(LogRendererCore, Warning, "Consider bumping DynamicRenderScaling::TMap::kInlineAlloactedBudgets to %d.", GRDSGlobalListLinkSize);
	}
#endif

#if STATS
	StatId_TargetMs			= FDynamicStats::CreateStatIdDouble<STAT_GROUP_TO_FStatGroup(STATGROUP_RenderScaling)>(FString::Printf(TEXT("%s_TargetMs"), InName));
	StatId_MeasuredMs		= FDynamicStats::CreateStatIdDouble<STAT_GROUP_TO_FStatGroup(STATGROUP_RenderScaling)>(FString::Printf(TEXT("%s_MeasuredMs"), InName));
	StatId_MinScaling		= FDynamicStats::CreateStatIdDouble<STAT_GROUP_TO_FStatGroup(STATGROUP_RenderScaling)>(FString::Printf(TEXT("%s_MinFraction"), InName));
	StatId_MaxScaling		= FDynamicStats::CreateStatIdDouble<STAT_GROUP_TO_FStatGroup(STATGROUP_RenderScaling)>(FString::Printf(TEXT("%s_MaxFraction"), InName));
	StatId_CurrentScaling	= FDynamicStats::CreateStatIdDouble<STAT_GROUP_TO_FStatGroup(STATGROUP_RenderScaling)>(FString::Printf(TEXT("%s_Fraction"), InName));
#endif

	CachedSettings = (*HeuristicSettingsGetter)();
}

FBudget::~FBudget()
{
	GRDSGlobalListLinkSize--;
	GlobalListLink.Unlink();
}

// static
TLinkedList<FBudget*>*& FBudget::GetGlobalList()
{
	static TLinkedList<FBudget*>* GDynamicScalingBudgetsList = nullptr;
	return GDynamicScalingBudgetsList;
}

// static
int32 DynamicRenderScaling::FBudget::GetGlobalListSize()
{
	return GRDSGlobalListLinkSize;
}

void UpdateHeuristicsSettings()
{
	check(IsInRenderingThread());
	for (TLinkedList<DynamicRenderScaling::FBudget*>::TIterator BudgetIt(DynamicRenderScaling::FBudget::GetGlobalList()); BudgetIt; BudgetIt.Next())
	{
		DynamicRenderScaling::FBudget& Budget = **BudgetIt;
		Budget.CachedSettings = (*Budget.HeuristicSettingsGetter)();
		check(Budget.CachedSettings.Model != EHeuristicModel::Unknown);
	}
}

} // namespace DynamicRenderScaling
