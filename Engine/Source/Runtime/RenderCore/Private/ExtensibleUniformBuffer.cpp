// Copyright Epic Games, Inc. All Rights Reserved.

#include "ExtensibleUniformBuffer.h"
#include "RenderGraphBuilder.h"

/******************************/
/* Struct layout registration */
/******************************/

struct FExtensibleUniformBufferTypeRegistry::FMemberInfo
{
	int32 Id;
	int32 Offset;
	FString Name;
	const FShaderParametersMetadata* Metadata;
	FShaderParameterStructConstructor DefaultValueFactory;
};

class FExtensibleUniformBufferTypeRegistry::FImpl
{
public:
#if !UE_BUILD_SHIPPING
	inline static TArray<FExtensibleUniformBufferTypeRegistry::FImpl*> Instances {};
#endif

	TArray<FExtensibleUniformBufferTypeRegistry::FMemberInfo> MemberInfos{};
	FShaderParametersMetadata* StructMetaData = nullptr;

	const TCHAR* LayoutName = nullptr;
	const TCHAR* StructTypeName = nullptr;
	const TCHAR* ShaderVariableName = nullptr;
	const TCHAR* StaticSlotName = nullptr;
	const ANSICHAR* FileName = nullptr;
	int32 FileLine = 0;
	uint64 MaxStructSize = 0;
};

FExtensibleUniformBufferTypeRegistry::FExtensibleUniformBufferTypeRegistry(
	const TCHAR* InLayoutName,
	const TCHAR* InStructTypeName,
	const TCHAR* InShaderVariableName,
	const TCHAR* InStaticSlotName,
	const ANSICHAR* InFileName,
	const int32 InFileLine,
	const uint64 InMaxStructSize)
: Impl(new FImpl)
{
	Impl->LayoutName = InLayoutName;
	Impl->StructTypeName = InStructTypeName;
	Impl->ShaderVariableName = InShaderVariableName;
	Impl->StaticSlotName = InStaticSlotName;
	Impl->FileName = InFileName;
	Impl->FileLine = InFileLine;
	Impl->MaxStructSize = InMaxStructSize;

#if !UE_BUILD_SHIPPING
	FImpl::Instances.Add(Impl);
#endif
}

FExtensibleUniformBufferTypeRegistry::~FExtensibleUniformBufferTypeRegistry()
{
#if !UE_BUILD_SHIPPING
	FImpl::Instances.Remove(Impl);
#endif	
	delete Impl;
}

#if !UE_BUILD_SHIPPING
static FAutoConsoleCommand DumpExtensibleBufferInfoCmd(
	TEXT("r.PrintExtensibleUBs"),
	TEXT("Dumps layout of all extensible uniform buffers"),
	FConsoleCommandWithOutputDeviceDelegate::CreateLambda(
		[](FOutputDevice& Ar)
		{
			Ar.Log(TEXT("ExtensibleUniformBuffers:"));
			for (const FExtensibleUniformBufferTypeRegistry::FImpl* Registry : FExtensibleUniformBufferTypeRegistry::FImpl::Instances)
			{
				uint32 BytesUsed = 0;
				for (const FExtensibleUniformBufferTypeRegistry::FMemberInfo& Member : Registry->MemberInfos)
				{
					const FShaderParametersMetadata* Metadata = Member.Metadata;
					BytesUsed = FMath::Max(BytesUsed, Member.Offset + Member.Metadata->GetSize());
				}

				Ar.Logf(TEXT("  - TypeName: '%s'"), Registry->StructTypeName);
				Ar.Logf(TEXT("    LayoutName: '%s'"), Registry->LayoutName);
				Ar.Logf(TEXT("    ShaderVariableName: '%s'"), Registry->ShaderVariableName);
				Ar.Logf(TEXT("    StaticSlotName: '%s'"), Registry->StaticSlotName);
				Ar.Logf(TEXT("    FileName: '%s'"), StringCast<TCHAR>(Registry->FileName).Get());
				Ar.Logf(TEXT("    FileLine: %d"), Registry->FileLine);
				Ar.Logf(TEXT("    MaxStructSize: %d"), Registry->MaxStructSize);
				Ar.Logf(TEXT("    SizeUsed: %d"), BytesUsed);
				Ar.Logf(TEXT("    Members: %s"), TEXT(""));

				for (const FExtensibleUniformBufferTypeRegistry::FMemberInfo& Member : Registry->MemberInfos)
				{
					const FShaderParametersMetadata* Metadata = Member.Metadata;

					Ar.Logf(TEXT("      - Name: '%s'"), *Member.Name);
					Ar.Logf(TEXT("        TypeName: '%s'"), Metadata->GetStructTypeName());
					Ar.Logf(TEXT("        Id: %d"), Member.Id);
					Ar.Logf(TEXT("        Offset: %d"), Member.Offset);
					Ar.Logf(TEXT("        Size: %d"), Metadata->GetSize());
					Ar.Logf(TEXT("        FileName: '%s'"), StringCast<TCHAR>(Metadata->GetFileName()).Get());
					Ar.Logf(TEXT("        FileLine: %d"), Metadata->GetFileLine());
				}
			}
		})
);
#endif

FExtensibleUniformBufferTypeRegistry::FMemberId FExtensibleUniformBufferTypeRegistry::RegisterMember(
	const TCHAR* Name, const FShaderParametersMetadata& Metadata, FShaderParameterStructConstructor Factory)
{
	check(Impl->StructMetaData == nullptr);

	int NewMemberInfoIndex = Impl->MemberInfos.Emplace();
	FMemberInfo& NewMember = Impl->MemberInfos[NewMemberInfoIndex];
	NewMember.Id = NewMemberInfoIndex;
	NewMember.Name = Name;
	NewMember.Offset = -1;
	NewMember.Metadata = &Metadata;
	NewMember.DefaultValueFactory = Factory;
	return NewMemberInfoIndex;
}

void FExtensibleUniformBufferMemberRegistration::CommitAll()
{
	for (FExtensibleUniformBufferMemberRegistration* Instance : GetInstances())
	{
		Instance->Commit();
	}
	GetInstances().Empty();
}

static TArray<FExtensibleUniformBufferMemberRegistration*>* GExtensibleUniformBufferMemberRegistrationInstances = nullptr;
TArray<FExtensibleUniformBufferMemberRegistration*>& FExtensibleUniformBufferMemberRegistration::GetInstances()
{
	// This is not a static or static local to ensure this works correctly during static initialization
	if (GExtensibleUniformBufferMemberRegistrationInstances == nullptr)
	{
		GExtensibleUniformBufferMemberRegistrationInstances = new TArray<FExtensibleUniformBufferMemberRegistration*>();
	}
	return *GExtensibleUniformBufferMemberRegistrationInstances;
}

void FExtensibleUniformBufferMemberRegistration::Register(FExtensibleUniformBufferMemberRegistration& Entry)
{
	GetInstances().Add(&Entry);
}

FShaderParametersMetadata* FExtensibleUniformBufferTypeRegistry::GetStructMetadata()
{
	if (Impl->StructMetaData)
	{
		return Impl->StructMetaData;
	}

	FExtensibleUniformBufferMemberRegistration::CommitAll();

	// TODO: Could be nice to have more intelligent packing here, rather than just appending with padding
	TArray<FMemberInfo*> SortedMembers;
	SortedMembers.Reserve(Impl->MemberInfos.Num());
	for (FMemberInfo& Member : Impl->MemberInfos)
	{
		SortedMembers.Emplace(&Member);
	}
	Algo::SortBy(SortedMembers, &FMemberInfo::Name);

	FShaderParametersMetadataBuilder Builder{};

	for (FMemberInfo* Member : SortedMembers)
	{
		Member->Offset = Builder.AddNestedStruct(*Member->Name, Member->Metadata);
	}

	Impl->StructMetaData = Builder.Build(
		FShaderParametersMetadata::EUseCase::UniformBuffer,
		EUniformBufferBindingFlags::StaticAndShader,
		Impl->LayoutName,
		Impl->StructTypeName,
		Impl->ShaderVariableName,
		Impl->StaticSlotName,
		Impl->FileName,
		Impl->FileLine
	);
	checkf(Impl->StructMetaData->GetSize() <= Impl->MaxStructSize,
		TEXT("The total size of the extensible uniform buffer members exceeds the storage capacity of the parameter struct. Increase the capacity. Use 'r.PrintExtensibleUBs' for details."));
	return Impl->StructMetaData;
}

/**********/
/* Buffer */
/**********/

FExtensibleUniformBufferBase::FExtensibleUniformBufferBase(FParameterTypeAdaptor& InParameterTypeAdaptor)
	: ParameterTypeAdaptor(InParameterTypeAdaptor)
	, MemberHasBeenSet(false, ParameterTypeAdaptor.Registry.Impl->MemberInfos.Num())
{
	CachedData.Init(0, ParameterTypeAdaptor.StructMetadata.GetSize());
	check(Align(CachedData.GetData(), SHADER_PARAMETER_STRUCT_ALIGNMENT) == CachedData.GetData());
}

bool FExtensibleUniformBufferBase::Set(const FMemberId MemberId, const void* Value, const int32 ValueSize)
{
	const TArray<FRegistry::FMemberInfo>& MemberInfos = ParameterTypeAdaptor.Registry.Impl->MemberInfos;

	check(MemberId >= 0 && MemberId < MemberInfos.Num());
	const FRegistry::FMemberInfo& MemberInfo = MemberInfos[MemberId];
	check(MemberInfo.Metadata->GetSize() == ValueSize);
	check((MemberInfo.Offset + ValueSize) <= CachedData.Num());

	MemberHasBeenSet[MemberId] = true;

	uint8_t* MemberDataPtr = &CachedData[MemberInfo.Offset];
	if (FPlatformMemory::Memcmp(MemberDataPtr, Value, ValueSize) != 0)
	{
		bAnyMemberDirty = true;
		FPlatformMemory::Memcpy(MemberDataPtr, Value, ValueSize);
		return true;
	}
	return false;
}

const void* FExtensibleUniformBufferBase::Get(const FMemberId MemberId, const int32 ValueSize) const
{
	const TArray<FRegistry::FMemberInfo>& MemberInfos = ParameterTypeAdaptor.Registry.Impl->MemberInfos;

	check(MemberId >= 0 && MemberId < MemberInfos.Num());
	const FRegistry::FMemberInfo& MemberInfo = MemberInfos[MemberId];
	check(MemberInfo.Metadata->GetSize() == ValueSize);
	check((MemberInfo.Offset + ValueSize) <= CachedData.Num());
	
	if (!MemberHasBeenSet[MemberId])
	{
		return nullptr;
	}

	return &CachedData[MemberInfo.Offset];
}

const void* FExtensibleUniformBufferBase::GetOrDefault(const FMemberId MemberId, const int32 ValueSize, FRDGBuilder& GraphBuilder)
{
	const TArray<FRegistry::FMemberInfo>& MemberInfos = ParameterTypeAdaptor.Registry.Impl->MemberInfos;

	check(MemberId >= 0 && MemberId < MemberInfos.Num());
	const FRegistry::FMemberInfo& MemberInfo = MemberInfos[MemberId];
	check(MemberInfo.Metadata->GetSize() == ValueSize);
	check((MemberInfo.Offset + ValueSize) <= CachedData.Num());

	uint8* Offset = &CachedData[MemberInfo.Offset];
	if (!MemberHasBeenSet[MemberId])
	{
		check((MemberInfo.Offset + MemberInfo.Metadata->GetSize()) <= static_cast<uint32>(CachedData.Num()));
		MemberInfo.DefaultValueFactory(Offset, *MemberInfo.Metadata, GraphBuilder);
		MemberHasBeenSet[MemberId] = true;
	}

	return Offset;
}

FRDGUniformBuffer* FExtensibleUniformBufferBase::GetBufferGeneric(FRDGBuilder& GraphBuilder)
{
	check(IsInRenderingThread());

	const TArray<FRegistry::FMemberInfo>& MemberInfos = ParameterTypeAdaptor.Registry.Impl->MemberInfos;
	if (Buffer == nullptr || bAnyMemberDirty)
	{
		for (int MemberIndex = 0; MemberIndex < MemberInfos.Num(); ++MemberIndex)
		{
			const FRegistry::FMemberInfo& MemberInfo = MemberInfos[MemberIndex];
			if (!MemberHasBeenSet[MemberIndex])
			{
				// Never been populated, set up defaults, we defer this since we don't want to create redundant SRVs for the common case where it is not needed.
				uint8* Offset = &CachedData[MemberInfo.Offset];

				check((MemberInfo.Offset + MemberInfo.Metadata->GetSize()) <= static_cast<uint32>(CachedData.Num()));
				MemberInfo.DefaultValueFactory(Offset, *MemberInfo.Metadata, GraphBuilder);
				MemberHasBeenSet[MemberIndex] = true;
			}
		}
		// Create and copy cached parameters into the RDG-lifetime struct
		Buffer = ParameterTypeAdaptor.BuildRDGUniformBuffer(GraphBuilder, *this);

		bAnyMemberDirty = false;
	}
	return Buffer;
}

FRHIUniformBuffer* FExtensibleUniformBufferBase::GetBufferRHI(FRDGBuilder& GraphBuilder)
{
	// Ensure the buffer is prepped.
	GetBufferGeneric(GraphBuilder);
	return GraphBuilder.ConvertToExternalUniformBuffer(Buffer);
}

#if !UE_BUILD_SHIPPING
TArray<FExtensibleUniformBufferBase::FDebugMemberInfo> FExtensibleUniformBufferBase::GetDebugInfo() const
{
	TArray<FExtensibleUniformBufferBase::FDebugMemberInfo> DebugInfo;
	for (const auto& Member : ParameterTypeAdaptor.StructMetadata.GetMembers())
	{
		FExtensibleUniformBufferBase::FDebugMemberInfo Info;
		Info.ShaderType = Member.GetShaderType();
		Info.Name = Member.GetName();
		Info.Data = &CachedData[Member.GetOffset()];
		DebugInfo.Add(Info);
	}
	return DebugInfo;
}
#endif
