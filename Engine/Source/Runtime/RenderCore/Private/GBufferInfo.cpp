// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	GBufferInfo.cpp: Dynamic GBuffer implementation.
=============================================================================*/

#include "GBufferInfo.h"
#include "RenderUtils.h"
#include "HAL/IConsoleManager.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "RenderUtils.h"
#include "RHIGlobals.h"

// This is very ugly, but temporary. We can't include EngineTypes.h because this file is ShaderCore and that
// file is Engine. But since we will remove this flag anyways, we will copy/paste the enums for now, but remove
// them as soon as flexible GBuffer is in.
static const int32 EGBufferFormat_Force8BitsPerChannel = 0;
static const int32 EGBufferFormat_Default = 1;
/** Same as Default except normals are encoded at 16 bits per channel. */
static const int32 EGBufferFormat_HighPrecisionNormals = 3;
/** Forces all GBuffers to 16 bits per channel. Intended as profiling for best quality. */
static const int32 EGBufferFormat_Force16BitsPerChannel = 5;

static bool IsGBufferPackingEqual(const FGBufferPacking& Lhs, const FGBufferPacking& Rhs)
{
	return
		Lhs.bFull == Rhs.bFull &&
		Lhs.bIsValid == Rhs.bIsValid &&
		Lhs.TargetIndex == Rhs.TargetIndex &&
		Lhs.DstBitStart == Rhs.DstBitStart &&
		Lhs.DstChannelIndex == Rhs.DstChannelIndex &&
		Lhs.SrcBitStart == Rhs.SrcBitStart &&
		Lhs.SrcChannelIndex == Rhs.SrcChannelIndex &&
		Lhs.BitNum == Rhs.BitNum;
}

static bool IsGBufferItemEqual(const FGBufferItem& Lhs, const FGBufferItem& Rhs)
{
	for (int32 I = 0; I < FGBufferItem::MaxPacking; I++)
	{
		const bool bSame = IsGBufferPackingEqual(Lhs.Packing[I], Rhs.Packing[I]);
		if (!bSame)
		{
			return false;
		}
	}

	return Lhs.bIsValid == Rhs.bIsValid &&
		Lhs.bQuantizationBias == Rhs.bQuantizationBias &&
		Lhs.BufferSlot == Rhs.BufferSlot &&
		Lhs.Compression == Rhs.Compression &&
		Lhs.Checker == Rhs.Checker;
}

static bool IsMobileDeferred(const FGBufferParams& Params)
{
	return IsMobilePlatform(Params.ShaderPlatform) && IsMobileDeferredShadingEnabled(Params.ShaderPlatform);
}

bool RENDERCORE_API IsGBufferInfoEqual(const FGBufferInfo& Lhs, const FGBufferInfo& Rhs)
{
	if (Lhs.NumTargets != Rhs.NumTargets)
	{
		return false;
	}

	for (int32 I = 0; I < Lhs.NumTargets; I++)
	{
		if (!(Lhs.Targets[I] == Rhs.Targets[I]))
		{
			return false;
		}
	}

	for (int32 I = 0; I < GBS_Num; I++)
	{
		if (!IsGBufferItemEqual(Lhs.Slots[I], Rhs.Slots[I]))
		{
			return false;
		}
	}

	return true;
}


TArray < EGBufferSlot > FetchGBufferSlots(bool bHasVelocity, bool bHasTangent, bool bHasPrecShadowFactor)
{
	TArray < EGBufferSlot > NeededSlots;

	NeededSlots.Push(GBS_SceneColor);
	NeededSlots.Push(GBS_WorldNormal);
	NeededSlots.Push(GBS_PerObjectGBufferData);
	NeededSlots.Push(GBS_Metallic);
	NeededSlots.Push(GBS_Specular);
	NeededSlots.Push(GBS_Roughness);
	NeededSlots.Push(GBS_ShadingModelId);
	NeededSlots.Push(GBS_SelectiveOutputMask);
	NeededSlots.Push(GBS_BaseColor);

	// this is needed for all combinations, will have to split it later
	NeededSlots.Push(GBS_GenericAO);

	if (bHasVelocity)
	{
		NeededSlots.Push(GBS_Velocity);
	}
	if (bHasPrecShadowFactor)
	{
		NeededSlots.Push(GBS_PrecomputedShadowFactor);
	}
	if (bHasTangent)
	{
		NeededSlots.Push(GBS_WorldTangent);
		NeededSlots.Push(GBS_Anisotropy);
	}
	NeededSlots.Push(GBS_CustomData);

	return NeededSlots;
}


// finds the target by searching for the name. returns -1 if not found.
int32 FindGBufferTargetByName(const FGBufferInfo& GBufferInfo, const FString& Name)
{
	for (int32 I = 0; I < GBufferInfo.NumTargets; I++)
	{
		if (GBufferInfo.Targets[I].TargetName.Compare(Name) == 0)
		{
			return I;
		}
	}

	return -1;
}

FGBufferBinding FindGBufferBindingByName(const FGBufferInfo& GBufferInfo, const FString& Name, EShaderPlatform ShaderPlatform)
{
	const int32 Index = FindGBufferTargetByName(GBufferInfo, Name);

	FGBufferBinding Binding;

	if (Index >= 0)
	{
		const FGBufferTarget& Target = GBufferInfo.Targets[Index];

		EPixelFormat PixelFormat = PF_Unknown;

		switch (Target.TargetType)
		{
		case GBT_Unorm_16_16:
			PixelFormat = PF_G16R16;
			break;
		case GBT_Unorm_8_8_8_8:
			PixelFormat = PF_B8G8R8A8;
			break;
		case GBT_Unorm_11_11_10:
			PixelFormat = PF_FloatR11G11B10;
			break;
		case GBT_Unorm_10_10_10_2:
			PixelFormat = PF_A2B10G10R10;
			break;
		case GBT_Unorm_16_16_16_16:
			PixelFormat = PF_A16B16G16R16;
			break;
		case GBT_Float_16_16:
			PixelFormat = PF_G16R16F;
			break;
		case GBT_Float_16_16_16_16:
			PixelFormat = PF_FloatRGBA;
			break;
		case GBT_Float_32:
			PixelFormat = PF_R32_FLOAT;
			break;
		case GBT_Invalid:
		default:
			check(0);
			break;
		}

		Binding.Index = Index;
		Binding.Format = PixelFormat;
		Binding.Flags = TexCreate_ShaderResource | TexCreate_RenderTargetable;
		
		if (Target.bIsSrgb)
		{
			Binding.Flags |= TexCreate_SRGB;
		}

		if (DoesPlatformSupportNanite(ShaderPlatform, true))
		{
			Binding.Flags |= TexCreate_UAV;

			if (UseNaniteFastTileClear())
			{
				Binding.Flags |= TexCreate_DisableDCC;
			}
		}
	}

	return Binding;
}

/*
 * 4.25 Logic:
 *
 * if (SrcGlobal.GBUFFER_HAS_VELOCITY == 0 && SrcGlobal.GBUFFER_HAS_TANGENT == 0)
 *    0: Lighting
 *    1: GBufferA
 *    2: GBufferB
 *    3: GBufferC
 *    4: GBufferD
 *    if (GBUFFER_HAS_PRECSHADOWFACTOR)
 *        5: GBufferE
 * else if (SrcGlobal.GBUFFER_HAS_VELOCITY == 1 && SrcGlobal.GBUFFER_HAS_TANGENT == 0)
 *    0: Lighting
 *    1: GBufferA
 *    2: GBufferB
 *    3: GBufferC
 *    4: Velocity (NOTE!)
 *    5: GBufferD
 *    if (GBUFFER_HAS_PRECSHADOWFACTOR)
 *        6: GBufferE
 * else if (SrcGlobal.GBUFFER_HAS_VELOCITY == 0 && SrcGlobal.GBUFFER_HAS_TANGENT == 1)
 *    0: Lighting
 *    1: GBufferA
 *    2: GBufferB
 *    3: GBufferC
 *    4: GBufferF (NOTE!)
 *    5: GBufferD
 *    if (GBUFFER_HAS_PRECSHADOWFACTOR)
 *        6: GBufferE
 * else if (SrcGlobal.GBUFFER_HAS_VELOCITY == 1 && SrcGlobal.GBUFFER_HAS_TANGENT == 1)
 *    assert(0)
 *
*/

/*
 * LegacyFormatIndex: EGBufferFormat enum. Going forward, we will have better granularity on choosing precision, so thes
 *    are just being maintained for the transition.
 *
 * bUsesVelocityDepth: Normal velocity format is half16 for RG. But when enabled, it changes to half16 RGBA with
 *    alpha storing depth and blue unused.
 *
 */

FGBufferInfo RENDERCORE_API FetchLegacyGBufferInfo(const FGBufferParams& Params)
{
	FGBufferInfo Info = {};

	check(!Params.bHasVelocity || !Params.bHasTangent);

	bool bStaticLighting = Params.bHasPrecShadowFactor;

	int32 TargetLighting = 0;
	int32 TargetGBufferA = 1;
	int32 TargetGBufferB = 2;
	int32 TargetGBufferC = 3;
	int32 TargetGBufferD = -1;
	int32 TargetGBufferE = -1;
	int32 TargetGBufferF = -1;
	int32 TargetVelocity = -1;
	int32 TargetSeparatedMainDirLight = -1;

	// Substrate outputs material data through UAV. Only SceneColor, PrecalcShadow & Velocity data are still emitted through RenderTargets
	if (Substrate::IsSubstrateEnabled() && !Substrate::IsSubstrateBlendableGBufferEnabled(Params.ShaderPlatform))
	{
		TargetGBufferA = -1;
		TargetGBufferB = -1;
		TargetGBufferC = -1;

		Info.NumTargets = 1;
		if (Params.bHasVelocity)
		{
			TargetVelocity = Info.NumTargets++;
		}
		if (Params.bHasPrecShadowFactor)
		{
			TargetGBufferE = Info.NumTargets++;
		}
		if (Params.bHasSingleLayerWaterSeparatedMainLight)
		{
			TargetSeparatedMainDirLight = Info.NumTargets++;
		}

		// this value isn't correct, because it doesn't respect the scene color format cvar, but it's ignored anyways
		// so it's ok for now
		Info.Targets[TargetLighting].Init(GBT_Unorm_11_11_10, TEXT("Lighting"), false, true, true, true);
		Info.Slots[GBS_SceneColor] = FGBufferItem(GBS_SceneColor, GBC_Raw_Float_11_11_10, GBCH_Both);
		Info.Slots[GBS_SceneColor].Packing[0] = FGBufferPacking(TargetLighting, 0, 0);
		Info.Slots[GBS_SceneColor].Packing[1] = FGBufferPacking(TargetLighting, 1, 1);
		Info.Slots[GBS_SceneColor].Packing[2] = FGBufferPacking(TargetLighting, 2, 2);

		if (Params.bHasVelocity)
		{
			Info.Targets[TargetVelocity].Init(Params.bUsesVelocityDepth ? GBT_Unorm_16_16_16_16 : (IsAndroidOpenGLESPlatform(Params.ShaderPlatform) ? GBT_Float_16_16 : GBT_Unorm_16_16), TEXT("Velocity"), false, true, true, false); // Velocity
			Info.Slots[GBS_Velocity] = FGBufferItem(GBS_Velocity, Params.bUsesVelocityDepth ? GBC_Raw_Float_16_16_16_16 : GBC_Raw_Float_16_16, GBCH_Both);
			Info.Slots[GBS_Velocity].Packing[0] = FGBufferPacking(TargetVelocity, 0, 0);
			Info.Slots[GBS_Velocity].Packing[1] = FGBufferPacking(TargetVelocity, 1, 1);

			if (Params.bUsesVelocityDepth)
			{
				Info.Slots[GBS_Velocity].Packing[2] = FGBufferPacking(TargetVelocity, 2, 2);
				Info.Slots[GBS_Velocity].Packing[3] = FGBufferPacking(TargetVelocity, 3, 3);
			}
		}

		if (Params.bHasPrecShadowFactor)
		{
			Info.Targets[TargetGBufferE].Init(GBT_Unorm_8_8_8_8, TEXT("GBufferE"), false, true, true, false); // Precalc
			Info.Slots[GBS_PrecomputedShadowFactor] = FGBufferItem(GBS_PrecomputedShadowFactor, GBC_Raw_Unorm_8_8_8_8, GBCH_Both);
			Info.Slots[GBS_PrecomputedShadowFactor].Packing[0] = FGBufferPacking(TargetGBufferE, 0, 0);
			Info.Slots[GBS_PrecomputedShadowFactor].Packing[1] = FGBufferPacking(TargetGBufferE, 1, 1);
			Info.Slots[GBS_PrecomputedShadowFactor].Packing[2] = FGBufferPacking(TargetGBufferE, 2, 2);
			Info.Slots[GBS_PrecomputedShadowFactor].Packing[3] = FGBufferPacking(TargetGBufferE, 3, 3);
		}

		// Special water output
		if (Params.bHasSingleLayerWaterSeparatedMainLight)
		{
			Info.Slots[GBS_SeparatedMainDirLight] = FGBufferItem(GBS_SeparatedMainDirLight, GBC_Raw_Float_11_11_10, GBCH_Both);
			Info.Slots[GBS_SeparatedMainDirLight].Packing[0] = FGBufferPacking(TargetSeparatedMainDirLight, 0, 0);
			Info.Slots[GBS_SeparatedMainDirLight].Packing[1] = FGBufferPacking(TargetSeparatedMainDirLight, 1, 1);
			Info.Slots[GBS_SeparatedMainDirLight].Packing[2] = FGBufferPacking(TargetSeparatedMainDirLight, 2, 2);
		}

		return Info;
	}

	if (Params.bHasVelocity == 0 && Params.bHasTangent == 0)
	{
		Info.NumTargets = Params.bHasPrecShadowFactor ? 6 : 5;
	}
	else
	{
		Info.NumTargets = Params.bHasPrecShadowFactor ? 7 : 6;
	}

	// good to see the quality loss due to precision in the gbuffer
	const bool bHighPrecisionGBuffers = (Params.LegacyFormatIndex >= EGBufferFormat_Force16BitsPerChannel);
	// good to profile the impact of non 8 bit formats
	const bool bEnforce8BitPerChannel = (Params.LegacyFormatIndex == EGBufferFormat_Force8BitsPerChannel);

	EGBufferType        NormalGBufferFormatTarget  = bHighPrecisionGBuffers ? GBT_Float_16_16_16_16 : GBT_Unorm_10_10_10_2;
	EGBufferCompression NormalGBufferFormatChannel = bHighPrecisionGBuffers ? GBC_EncodeNormal_Normal_16_16_16 : GBC_EncodeNormal_Normal_10_10_10;

	if (bEnforce8BitPerChannel)
	{
		NormalGBufferFormatTarget  = GBT_Unorm_8_8_8_8;
		NormalGBufferFormatChannel = GBC_EncodeNormal_Normal_8_8_8;
	}
	else if (Params.LegacyFormatIndex == EGBufferFormat_HighPrecisionNormals)
	{
		NormalGBufferFormatTarget  = GBT_Float_16_16_16_16;
		NormalGBufferFormatChannel = GBC_EncodeNormal_Normal_16_16_16;
	}

	const EGBufferType        DiffuseAndSpecularGBufferFormat  = bHighPrecisionGBuffers ? GBT_Float_16_16_16_16 : GBT_Unorm_8_8_8_8;
	const EGBufferCompression DiffuseGBufferChannel = bHighPrecisionGBuffers ? GBC_Raw_Float_16_16_16 : GBC_Raw_Unorm_8_8_8;
	const EGBufferCompression SpecularGBufferChannel = bHighPrecisionGBuffers ? GBC_Raw_Float_16 : GBC_Raw_Unorm_8;

	Info.Targets[0].Init(GBT_Unorm_11_11_10,  TEXT("Lighting"), false,  true,  true,  true);
	Info.Targets[1].Init(NormalGBufferFormatTarget,TEXT("GBufferA"), false,  true,  true,  true);
	Info.Targets[2].Init(DiffuseAndSpecularGBufferFormat,   TEXT("GBufferB"), false,  true,  true,  true);
	
	const bool bLegacyAlbedoSrgb = true;
	Info.Targets[3].Init(DiffuseAndSpecularGBufferFormat,  TEXT("GBufferC"), bLegacyAlbedoSrgb && !bHighPrecisionGBuffers,  true,  true,  true);

	// This code should match TBasePassPS
	if (Params.bHasVelocity == 0 && Params.bHasTangent == 0)
	{
		TargetGBufferD = 4;
		Info.Targets[4].Init(GBT_Unorm_8_8_8_8,  TEXT("GBufferD"), false,  true,  true,  true);

		if (Params.bHasPrecShadowFactor)
		{
			TargetGBufferE = 5;
			Info.Targets[5].Init(GBT_Unorm_8_8_8_8, TEXT("GBufferE"), false, true, true, true);
		}
	}
	else if (Params.bHasVelocity)
	{
		TargetVelocity = 4;
		TargetGBufferD = 5;

		// note the false for use extra flags for velocity, not quite sure of all the ramifications, but this keeps it consistent with previous usage
		Info.Targets[4].Init(Params.bUsesVelocityDepth ? GBT_Unorm_16_16_16_16 : (IsAndroidOpenGLESPlatform(Params.ShaderPlatform) ? GBT_Float_16_16 : GBT_Unorm_16_16), TEXT("Velocity"), false, true, true, false);
		Info.Targets[5].Init(GBT_Unorm_8_8_8_8, TEXT("GBufferD"), false, true, true, true);

		if (Params.bHasPrecShadowFactor)
		{
			TargetGBufferE = 6;
			Info.Targets[6].Init(GBT_Unorm_8_8_8_8, TEXT("GBufferE"), false, true, true, false);
		}
	}
	else if (Params.bHasTangent)
	{
		TargetGBufferF = 4;
		TargetGBufferD = 5;
		Info.Targets[4].Init(GBT_Unorm_8_8_8_8,  TEXT("GBufferF"), false,  true,  true,  true);
		Info.Targets[5].Init(GBT_Unorm_8_8_8_8, TEXT("GBufferD"), false, true, true, true);
		if (Params.bHasPrecShadowFactor)
		{
			TargetGBufferE = 6;
			Info.Targets[6].Init(GBT_Unorm_8_8_8_8, TEXT("GBufferE"), false, true, true, true);
		}
	}
	else
	{
		// should never hit this path
		check(0);
	}

	// SLW never uses GBufferD (CustomData) and we want to tightly pack RTVs so there's space for UAVs after the RTV range (D3D11.0 only supports a total of 8 RTVs and UAVs and they must have non-overlapping ranges).
	TargetSeparatedMainDirLight = TargetGBufferD;

	// this value isn't correct, because it doesn't respect the scene color format cvar, but it's ignored anyways
	// so it's ok for now
	Info.Slots[GBS_SceneColor] = FGBufferItem(GBS_SceneColor, GBC_Raw_Float_11_11_10, GBCH_Both);
	Info.Slots[GBS_SceneColor].Packing[0] = FGBufferPacking(TargetLighting, 0, 0);
	Info.Slots[GBS_SceneColor].Packing[1] = FGBufferPacking(TargetLighting, 1, 1);
	Info.Slots[GBS_SceneColor].Packing[2] = FGBufferPacking(TargetLighting, 2, 2);

	Info.Slots[GBS_WorldNormal] = FGBufferItem(GBS_WorldNormal, NormalGBufferFormatChannel, GBCH_Both);
	Info.Slots[GBS_WorldNormal].Packing[0] = FGBufferPacking(TargetGBufferA, 0, 0);
	Info.Slots[GBS_WorldNormal].Packing[1] = FGBufferPacking(TargetGBufferA, 1, 1);
	Info.Slots[GBS_WorldNormal].Packing[2] = FGBufferPacking(TargetGBufferA, 2, 2);

	Info.Slots[GBS_PerObjectGBufferData] = FGBufferItem(GBS_PerObjectGBufferData, GBC_Raw_Unorm_2, GBCH_Both);
	Info.Slots[GBS_PerObjectGBufferData].Packing[0] = FGBufferPacking(TargetGBufferA, 0, 3);

#if 1
	Info.Slots[GBS_Metallic] = FGBufferItem(GBS_Metallic, SpecularGBufferChannel, GBCH_Both);
	Info.Slots[GBS_Metallic].Packing[0] = FGBufferPacking(TargetGBufferB, 0, 0);

	Info.Slots[GBS_Specular] = FGBufferItem(GBS_Specular, SpecularGBufferChannel, GBCH_Both);
	Info.Slots[GBS_Specular].Packing[0] = FGBufferPacking(TargetGBufferB, 0, 1);

	Info.Slots[GBS_Roughness] = FGBufferItem(GBS_Roughness, SpecularGBufferChannel, GBCH_Both);
	Info.Slots[GBS_Roughness].Packing[0] = FGBufferPacking(TargetGBufferB, 0, 2);
#else
	Info.Slots[GBS_Metallic] = FGBufferItem(GBS_Metallic, GBC_Packed_Quantized_4, GBCH_Both);
	Info.Slots[GBS_Metallic].Packing[0] = FGBufferPacking(TargetGBufferB, 0, 0, 0, 0, 4);

	Info.Slots[GBS_Specular] = FGBufferItem(GBS_Specular, GBC_Packed_Quantized_4, GBCH_Both);
	Info.Slots[GBS_Specular].Packing[0] = FGBufferPacking(TargetGBufferB, 0, 0, 0, 4, 4);

	Info.Slots[GBS_Roughness] = FGBufferItem(GBS_Roughness, GBC_Packed_Quantized_4, GBCH_Both);
	Info.Slots[GBS_Roughness].Packing[0] = FGBufferPacking(TargetGBufferB, 0, 1, 0, 0, 4);
#endif

	// pack it into bits [0:3] of alpha
	Info.Slots[GBS_ShadingModelId] = FGBufferItem(GBS_ShadingModelId, GBC_Bits_4, GBCH_Both);
	Info.Slots[GBS_ShadingModelId].Packing[0] = FGBufferPacking(TargetGBufferB, 0, 3, 0, 0, 4);

	// pack it into bits [4:7] of alpha
	Info.Slots[GBS_SelectiveOutputMask] = FGBufferItem(GBS_SelectiveOutputMask, GBC_Bits_4, GBCH_Both);
	Info.Slots[GBS_SelectiveOutputMask].Packing[0] = FGBufferPacking(TargetGBufferB, 0, 3, 0, 4, 4);

	{
#if 1
		EGBufferCompression BaseColorCompression = GBC_Invalid;

		Info.Slots[GBS_BaseColor] = FGBufferItem(GBS_BaseColor, DiffuseGBufferChannel, GBCH_Both);
		Info.Slots[GBS_BaseColor].Packing[0] = FGBufferPacking(TargetGBufferC, 0, 0);
		Info.Slots[GBS_BaseColor].Packing[1] = FGBufferPacking(TargetGBufferC, 1, 1);
		Info.Slots[GBS_BaseColor].Packing[2] = FGBufferPacking(TargetGBufferC, 2, 2);
#elif 0
		// pack it for funzies
		Info.Slots[GBS_BaseColor] = FGBufferItem(GBS_BaseColor, GBC_Packed_Color_5_6_5, GBCH_Both);
		Info.Slots[GBS_BaseColor].bQuantizationBias = true;
		Info.Slots[GBS_BaseColor].Packing[0] = FGBufferPacking(TargetGBufferB, 0, 0, 0, 0, 5);
		Info.Slots[GBS_BaseColor].Packing[1] = FGBufferPacking(TargetGBufferB, 1, 1, 0, 0, 6);
		Info.Slots[GBS_BaseColor].Packing[2] = FGBufferPacking(TargetGBufferB, 2, 2, 0, 0, 5);
#elif 0
		// pack it for funzies
		Info.Slots[GBS_BaseColor] = FGBufferItem(GBS_BaseColor, GBC_Packed_Color_5_6_5, GBCH_Both);
		Info.Slots[GBS_BaseColor].bQuantizationBias = true;
		Info.Slots[GBS_BaseColor].Packing[0] = FGBufferPacking(TargetGBufferC, 0, 0, 0, 0, 5);
		Info.Slots[GBS_BaseColor].Packing[1] = FGBufferPacking(TargetGBufferC, 1, 0, 0, 5, 3);
		Info.Slots[GBS_BaseColor].Packing[2] = FGBufferPacking(TargetGBufferC, 1, 1, 3, 5, 3);
		Info.Slots[GBS_BaseColor].Packing[3] = FGBufferPacking(TargetGBufferC, 2, 1, 0, 0, 5);
#elif 1
		// pack it for funzies
		Info.Slots[GBS_BaseColor] = FGBufferItem(GBS_BaseColor, GBC_Packed_Color_4_4_4_Sqrt, GBCH_Both);
		Info.Slots[GBS_BaseColor].bQuantizationBias = true;
		Info.Slots[GBS_BaseColor].Packing[0] = FGBufferPacking(TargetGBufferB, 0, 1, 0, 4, 4);
		Info.Slots[GBS_BaseColor].Packing[1] = FGBufferPacking(TargetGBufferB, 1, 2, 0, 0, 4);
		Info.Slots[GBS_BaseColor].Packing[2] = FGBufferPacking(TargetGBufferB, 2, 2, 0, 4, 4);
#else
		// pack it for funzies
		Info.Slots[GBS_BaseColor] = FGBufferItem(GBS_BaseColor, GBC_Packed_Color_3_3_2_Sqrt, GBCH_Both);
		Info.Slots[GBS_BaseColor].bQuantizationBias = true;
		Info.Slots[GBS_BaseColor].Packing[0] = FGBufferPacking(TargetGBufferC, 0, 0, 0, 0, 3);
		Info.Slots[GBS_BaseColor].Packing[1] = FGBufferPacking(TargetGBufferC, 1, 0, 0, 3, 3);
		Info.Slots[GBS_BaseColor].Packing[2] = FGBufferPacking(TargetGBufferC, 2, 0, 0, 6, 2);
#endif

		{
			Info.Slots[GBS_GenericAO] = FGBufferItem(GBS_GenericAO, GBC_Raw_Unorm_8, GBCH_Both);
			Info.Slots[GBS_GenericAO].Packing[0] = FGBufferPacking(TargetGBufferC, 0, 3);
		}
	}

	if (Params.bHasVelocity)
	{
		Info.Slots[GBS_Velocity] = FGBufferItem(GBS_Velocity, Params.bUsesVelocityDepth ? GBC_Raw_Float_16_16_16_16 : GBC_Raw_Float_16_16, GBCH_Both);
		Info.Slots[GBS_Velocity].Packing[0] = FGBufferPacking(TargetVelocity, 0, 0);
		Info.Slots[GBS_Velocity].Packing[1] = FGBufferPacking(TargetVelocity, 1, 1);

		if (Params.bUsesVelocityDepth)
		{
			Info.Slots[GBS_Velocity].Packing[2] = FGBufferPacking(TargetVelocity, 2, 2);
			Info.Slots[GBS_Velocity].Packing[3] = FGBufferPacking(TargetVelocity, 3, 3);
		}
	}
	if (Params.bHasPrecShadowFactor)
	{
		Info.Slots[GBS_PrecomputedShadowFactor] = FGBufferItem(GBS_PrecomputedShadowFactor, GBC_Raw_Unorm_8_8_8_8, GBCH_Both);
		Info.Slots[GBS_PrecomputedShadowFactor].Packing[0] = FGBufferPacking(TargetGBufferE, 0, 0);
		Info.Slots[GBS_PrecomputedShadowFactor].Packing[1] = FGBufferPacking(TargetGBufferE, 1, 1);
		Info.Slots[GBS_PrecomputedShadowFactor].Packing[2] = FGBufferPacking(TargetGBufferE, 2, 2);
		Info.Slots[GBS_PrecomputedShadowFactor].Packing[3] = FGBufferPacking(TargetGBufferE, 3, 3);
	}
	if (Params.bHasTangent)
	{
		Info.Slots[GBS_WorldTangent] = FGBufferItem(GBS_WorldTangent, GBC_Raw_Unorm_8_8_8, GBCH_Both);
		Info.Slots[GBS_WorldTangent].Packing[0] = FGBufferPacking(TargetGBufferF, 0, 0);
		Info.Slots[GBS_WorldTangent].Packing[1] = FGBufferPacking(TargetGBufferF, 1, 1);
		Info.Slots[GBS_WorldTangent].Packing[2] = FGBufferPacking(TargetGBufferF, 2, 2);

		Info.Slots[GBS_Anisotropy] = FGBufferItem(GBS_Anisotropy, GBC_Raw_Unorm_8, GBCH_Both);
		Info.Slots[GBS_Anisotropy].Packing[0] = FGBufferPacking(TargetGBufferF, 0, 3);
	}

	// GBufferD
	Info.Slots[GBS_CustomData] = FGBufferItem(GBS_CustomData, GBC_Raw_Unorm_8_8_8_8, GBCH_Both);
	Info.Slots[GBS_CustomData].Packing[0] = FGBufferPacking(TargetGBufferD, 0, 0);
	Info.Slots[GBS_CustomData].Packing[1] = FGBufferPacking(TargetGBufferD, 1, 1);
	Info.Slots[GBS_CustomData].Packing[2] = FGBufferPacking(TargetGBufferD, 2, 2);
	Info.Slots[GBS_CustomData].Packing[3] = FGBufferPacking(TargetGBufferD, 3, 3);

	// Special water output
	if (Params.bHasSingleLayerWaterSeparatedMainLight)
	{
		Info.Slots[GBS_SeparatedMainDirLight] = FGBufferItem(GBS_SeparatedMainDirLight, GBC_Raw_Float_11_11_10, GBCH_Both);
		Info.Slots[GBS_SeparatedMainDirLight].Packing[0] = FGBufferPacking(TargetSeparatedMainDirLight, 0, 0);
		Info.Slots[GBS_SeparatedMainDirLight].Packing[1] = FGBufferPacking(TargetSeparatedMainDirLight, 1, 1);
		Info.Slots[GBS_SeparatedMainDirLight].Packing[2] = FGBufferPacking(TargetSeparatedMainDirLight, 2, 2);
	}

	return Info;
}


FGBufferInfo RENDERCORE_API FetchMobileGBufferInfo(const FGBufferParams& Params)
{
	// for now, take desktop and trim out what we do not require.
	FGBufferInfo Info = FetchLegacyGBufferInfo(Params);
	
	// If we are using GL and don't have FBF support, use PLS
	bool bUsingPixelLocalStorage = IsAndroidOpenGLESPlatform(Params.ShaderPlatform) && GSupportsPixelLocalStorage && GSupportsShaderDepthStencilFetch;
	if (bUsingPixelLocalStorage)
	{
		Info.NumTargets = 1;
	}
	else
	{
		Info.NumTargets = 4;
		if (MobileUsesExtendedGBuffer(Params.ShaderPlatform))
		{
			Info.NumTargets++;
		}
		
		if (MobileRequiresSceneDepthAux(Params.ShaderPlatform))
		{
			// if used, mobile deferred is always F32.
			Info.Targets[Info.NumTargets].Init(GBT_Float_32, TEXT("DepthAux"), false, true, true, false);
			Info.NumTargets++;
		}
	}

	return Info;
}

FGBufferInfo RENDERCORE_API FetchFullGBufferInfo(const FGBufferParams& Params)
{
	// For now, we are only doing legacy. But next, we will have a switch between the old and new formats.
	if (IsMobileDeferred(Params))
	{
		return FetchMobileGBufferInfo(Params);
	}
	else
	{
		return FetchLegacyGBufferInfo(Params);
	}
}
