// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	GPUDebugCrashUtils.cpp: Utilities for crashing the GPU in various ways on purpose.
	=============================================================================*/

#include "GPUDebugCrashUtils.h"
#include "RenderGraphUtils.h"
#include "RenderResource.h"
#include "GlobalShader.h"
#include "RHIStaticStates.h"
#include "DataDrivenShaderPlatformInfo.h"

class FGPUDebugCrashUtilsCS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FGPUDebugCrashUtilsCS)
	SHADER_USE_PARAMETER_STRUCT(FGPUDebugCrashUtilsCS, FGlobalShader)

	class FPlatformBreakRequested : SHADER_PERMUTATION_BOOL("PLATFORM_BREAK_REQUESTED");
	class FAssertRequested : SHADER_PERMUTATION_BOOL("ASSERT_REQUESTED");
	class FHangRequested : SHADER_PERMUTATION_BOOL("HANG_REQUESTED");
	using FPermutationDomain = TShaderPermutationDomain<FPlatformBreakRequested, FAssertRequested, FHangRequested>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<uint>, PageFaultUAV)
	END_SHADER_PARAMETER_STRUCT()

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("THREADGROUPSIZE"), FComputeShaderUtils::kGolden2DGroupSize);
	}
};

IMPLEMENT_GLOBAL_SHADER(FGPUDebugCrashUtilsCS, "/Engine/Private/Tools/GPUDebugCrashUtils.usf", "MainCS", SF_Compute);

// create RDG pass for the shader/pass that's going to crash the GPU
void ScheduleGPUDebugCrash(FRDGBuilder& GraphBuilder)
{
	FGPUDebugCrashUtilsCS::FPermutationDomain PermutationVector;
	PermutationVector.Set<FGPUDebugCrashUtilsCS::FPlatformBreakRequested>(EnumHasAnyFlags(GRHIGlobals.TriggerGPUCrash, ERequestedGPUCrash::Type_PlatformBreak));
	PermutationVector.Set<FGPUDebugCrashUtilsCS::FAssertRequested>(EnumHasAnyFlags(GRHIGlobals.TriggerGPUCrash, ERequestedGPUCrash::Type_Assert));
	PermutationVector.Set<FGPUDebugCrashUtilsCS::FHangRequested>(EnumHasAnyFlags(GRHIGlobals.TriggerGPUCrash, ERequestedGPUCrash::Type_Hang));

	auto ComputeShader = GetGlobalShaderMap(GMaxRHIFeatureLevel)->GetShader<FGPUDebugCrashUtilsCS>(PermutationVector);
	FGPUDebugCrashUtilsCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FGPUDebugCrashUtilsCS::FParameters>();
	ETextureCreateFlags TexFlags = TexCreate_UAV | TexCreate_ShaderResource | TexCreate_NoFastClear | TexCreate_RenderTargetable;
	FRDGTextureDesc CreateInfo = FRDGTextureDesc::Create2D(
		FIntPoint(64, 64),
		EPixelFormat::PF_R32_UINT,
		FClearValueBinding::None,
		TexFlags);
	

	if (EnumHasAnyFlags(GRHIGlobals.TriggerGPUCrash, ERequestedGPUCrash::Type_PageFault))
	{
		CreateInfo.Flags |= TexCreate_Invalid;
	}

	FRDGTextureRef PageFaultUAV = GraphBuilder.CreateTexture(CreateInfo, TEXT("GPUDebugCrashUtils.PageFaultUAV"));
	PassParameters->PageFaultUAV = GraphBuilder.CreateUAV(PageFaultUAV);

	FString CrashTypeString = EnumHasAnyFlags(GRHIGlobals.TriggerGPUCrash, ERequestedGPUCrash::Type_PageFault) ? TEXT("PageFault") : TEXT("Hang");
	if (EnumHasAnyFlags(GRHIGlobals.TriggerGPUCrash, ERequestedGPUCrash::Type_PlatformBreak))
	{
		CrashTypeString = TEXT("PlatformBreak");
	}

	RDG_EVENT_SCOPE(GraphBuilder, "GPUDebugCrash_%s_%s",
		EnumHasAnyFlags(GRHIGlobals.TriggerGPUCrash, ERequestedGPUCrash::Queue_Compute) ? TEXT("ComputeQueue") : TEXT("DirectQueue"),
		*CrashTypeString);

	if(EnumHasAnyFlags(GRHIGlobals.TriggerGPUCrash, ERequestedGPUCrash::Type_CmdListCorruption))
	{ 
		GraphBuilder.AddPass(
			RDG_EVENT_NAME("GPUDebugCrashCmdCorruption"),
			ERDGPassFlags::None,
			[](FRDGAsyncTask, FRHICommandList& RHICmdList)
			{
				RHICmdList.GpuHangCommandListCorruption();
			});
	}

	FComputeShaderUtils::AddPass(
		GraphBuilder,
		RDG_EVENT_NAME("GPUDebugCrash"),
		(EnumHasAnyFlags(GRHIGlobals.TriggerGPUCrash, ERequestedGPUCrash::Queue_Compute) ? ERDGPassFlags::AsyncCompute : ERDGPassFlags::Compute) | ERDGPassFlags::NeverCull,
		ComputeShader,
		PassParameters,
		FIntVector(1, 1, 1)
	);
	GRHIGlobals.TriggerGPUCrash = ERequestedGPUCrash::None;
}
