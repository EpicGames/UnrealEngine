// Copyright Epic Games, Inc. All Rights Reserved.

#include "GPUMessaging.h"
#include "RenderResource.h"
#include "RenderGraphUtils.h"
#include "RHIGPUReadback.h"
#include "GlobalShader.h"

DEFINE_LOG_CATEGORY_STATIC(LogGPUMessaging, Log, All);

RDG_REGISTER_BLACKBOARD_STRUCT(GPUMessage::FParameters)

class FClearMessageBufferCS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FClearMessageBufferCS);
	SHADER_USE_PARAMETER_STRUCT(FClearMessageBufferCS, FGlobalShader)

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<uint>, GPUMessageDataBuffer)
	END_SHADER_PARAMETER_STRUCT()
};

IMPLEMENT_GLOBAL_SHADER(FClearMessageBufferCS, "/Engine/Private/GPUMessaging.usf", "ClearMessageBuffer", SF_Compute);

namespace GPUMessage
{

int32 GLogAllMessages = 0;
static FAutoConsoleVariableRef CVarLogAllMessages(
	TEXT("r.GPUMessage.LogAllMessages"),
	GLogAllMessages,
	TEXT("Log all messages to the console.\n")
	TEXT("0: Disabled\n")
	TEXT("1: Enabled\n"),
	ECVF_RenderThreadSafe
);

int32 GMaxBufferSize = 64;
static FAutoConsoleVariableRef CVarMaxBufferSize(
	TEXT("r.GPUMessage.MaxBufferSize"),
	GMaxBufferSize,
	TEXT("Specifies the maximum size of the GPU message buffer, in KiB.\n")
	TEXT("default: 64\n"),
	ECVF_RenderThreadSafe | ECVF_ReadOnly
);

class FSystem : public FRenderResource
{
public:
	// Need to guard access because it may happen during FScene initialization (Game Thread) while another scene is being rendered.
	FCriticalSection AccessCS;


	FSocket RegisterHandler(const TSharedPtr<FHandler>& Handler)
	{
		FScopeLock Lock(&AccessCS);
		const FMessageId MessageId(NextMessageId++);
		check(!MessageHandlers.Contains(MessageId));
		MessageHandlers.Add(MessageId, Handler);
		return FSocket(MessageId);
	}

	void RemoveHandler(FMessageId MessageId)
	{
		FScopeLock Lock(&AccessCS);
		check(MessageId.IsValid());
		check(MessageHandlers.Contains(MessageId));
		MessageHandlers.Remove(MessageId);
	}

	void BeginMessageScope(FRDGBuilder& GraphBuilder)
	{
		check(MessageBuffer == nullptr);
		FScopeLock Lock(&AccessCS);

		PollMessages();

		if (MessageHandlers.IsEmpty())
		{
			return;
		}

		const int32 MaxBufferSize = GetMaxBufferSize();

		FRDGBufferDesc Desc = FRDGBufferDesc::CreateStructuredDesc(sizeof(uint32), MaxBufferSize);
		Desc.Usage |= EBufferUsageFlags::SourceCopy;
		MessageBuffer = GraphBuilder.CreateBuffer(Desc, TEXT("GPUMessageManager.MessageBuffer"));

		FParameters& Parameters = GraphBuilder.Blackboard.Create<FParameters>();
		Parameters.GPUMessageDataBuffer = GraphBuilder.CreateUAV(MessageBuffer);
		Parameters.GPUMessageDataBufferSize = MaxBufferSize;

		TShaderMapRef<FClearMessageBufferCS> ComputeShader(GetGlobalShaderMap(GMaxRHIFeatureLevel));

		FClearMessageBufferCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FClearMessageBufferCS::FParameters>();
		PassParameters->GPUMessageDataBuffer = Parameters.GPUMessageDataBuffer;

		FComputeShaderUtils::AddPass(GraphBuilder, RDG_EVENT_NAME("ClearGPUMessageBuffer"), ComputeShader, PassParameters, FIntVector(1, 1, 1));
	}

	void EndMessageScope(FRDGBuilder& GraphBuilder)
	{
		FScopeLock Lock(&AccessCS);

		if (MessageHandlers.IsEmpty())
		{
			check(MessageBuffer == nullptr);
			return;
		}

		check(MessageBuffer != nullptr);

		FRHIGPUBufferReadback* GPUBufferReadback = nullptr;
		if (!MessageReadbackBuffersReady.IsEmpty())
		{
			GPUBufferReadback = MessageReadbackBuffersReady.Pop();
		}
		else
		{
			GPUBufferReadback = new FRHIGPUBufferReadback(TEXT("GPUMessageManager.Readback"));
		}

		AddEnqueueCopyPass(GraphBuilder, GPUBufferReadback, MessageBuffer, 0u);
		MessageReadbackBuffersInUse.Add(GPUBufferReadback);

		MessageBuffer = nullptr;
	}

private:
	void PollMessages()
	{
		for (int32 Index = 0; Index < MessageReadbackBuffersInUse.Num();)
		{
			FRHIGPUBufferReadback* GPUBufferReadback = MessageReadbackBuffersInUse[Index];

			if (GPUBufferReadback->IsReady())
			{
				const int32 MaxBufferSize = GetMaxBufferSize();

				const uint32* BufferPtr = (const uint32*)GPUBufferReadback->Lock(MaxBufferSize * sizeof(uint32));
				int32 ValidRangeEnd = (int32)BufferPtr[0] + 1;

				const int32 OriginalValidRangeEnd = ValidRangeEnd;
				ValidRangeEnd = FMath::Min(ValidRangeEnd, MaxBufferSize);
				ensureMsgf(ValidRangeEnd == OriginalValidRangeEnd, TEXT("GPU messages size %d exceeded maximum size %d. Results have been truncated."), OriginalValidRangeEnd, MaxBufferSize);

				if (GLogAllMessages != 0)
				{
					FString LogMessage;

					LogMessage.Appendf(TEXT("\nMessage Buffer[0] = %u;\n"), BufferPtr[0]);

					if (ValidRangeEnd != OriginalValidRangeEnd)
					{
						LogMessage.Appendf(TEXT("GPU messages size %d exceeded maximum size %d. Results have been truncated.\n"), OriginalValidRangeEnd, MaxBufferSize);
					}

					// Subtract 1 from ValidRangeEnd because we expect to have at least 2 ints available (message ID and payload size).
					for (int32 ReadOffset = 1; ReadOffset < ValidRangeEnd - 1; )
					{
						const FMessageId MessageId(BufferPtr[ReadOffset]);
						int32 PayloadNumUints = (int32)BufferPtr[ReadOffset + 1];

						const int32 OriginalPayloadNumUints = PayloadNumUints;
						PayloadNumUints = FMath::Min(PayloadNumUints, ValidRangeEnd - ReadOffset - 2);

						LogMessage.Appendf(TEXT("  Message[Offset: %d, ID: %d, PayloadSize: %d, "), ReadOffset, MessageId.GetIndexUnchecked(), OriginalPayloadNumUints);

						if (PayloadNumUints != OriginalPayloadNumUints)
						{
							LogMessage.Appendf(TEXT("!payload size %d exceeds buffer size (ReadOffset=%d, ValidRangeEnd=%d), truncated to %d!, "), OriginalPayloadNumUints, ReadOffset, ValidRangeEnd, PayloadNumUints);
						}

						if (auto Handler = MessageHandlers.Find(MessageId))
						{
							LogMessage.Appendf(TEXT("Name: %s]"), (*Handler)->GetName());
						}
						else
						{
							LogMessage.Append(TEXT("<Unknown>]"));
						}

						if (PayloadNumUints > 0)
						{
							LogMessage.Append(TEXT(" Payload: { "));
							for (int32 PayloadIndex = 0; PayloadIndex < PayloadNumUints; ++PayloadIndex)
							{
								LogMessage.Appendf(TEXT("%u%s"), BufferPtr[ReadOffset + 2 + PayloadIndex], PayloadIndex + 1 == PayloadNumUints ? TEXT("") : TEXT(", "));
							}
							LogMessage.Append(TEXT(" }"));
						}

						LogMessage.Append(TEXT("\n"));

						// Step past the message to the next
						ReadOffset += PayloadNumUints + 2;
					}

					UE_LOGF(LogGPUMessaging, Log, "%ls", *LogMessage);
				}

				int32 ReadOffset;
				for (ReadOffset = 1; ReadOffset < ValidRangeEnd - 1; )
				{
					const FMessageId MessageId(BufferPtr[ReadOffset]);
					int32 PayloadNumUints = (int32)BufferPtr[ReadOffset + 1];

					const int32 OriginalPayloadNumUints = PayloadNumUints;
					PayloadNumUints = FMath::Min(PayloadNumUints, ValidRangeEnd - ReadOffset - 2);
					ensureMsgf(PayloadNumUints == OriginalPayloadNumUints, TEXT("GPU message payload size %d exceeds buffer size (ReadOffset=%d, ValidRangeEnd=%d), truncated to %d."), OriginalPayloadNumUints, ReadOffset, ValidRangeEnd, PayloadNumUints);

					if (auto Handler = MessageHandlers.Find(MessageId))
					{
						(*Handler)->Execute(FReader(MessageId, PayloadNumUints, &BufferPtr[ReadOffset + 2]));
					}

					// Step past the message to the next
					ReadOffset += PayloadNumUints + 2;
				}

				ensureMsgf(ReadOffset == ValidRangeEnd, TEXT("Garbage at the end of the GPU message buffer, ReadOffset=%d, ValidRangeEnd=%d"), ReadOffset, ValidRangeEnd);

				GPUBufferReadback->Unlock();

				MessageReadbackBuffersInUse.RemoveAt(Index);
				MessageReadbackBuffersReady.Push(GPUBufferReadback);
			}
			else
			{
				++Index;
			}
		}
	}

	void ReleaseRHI() override
	{
		FScopeLock Lock(&AccessCS);

		check(MessageBuffer == nullptr);

		for (FRHIGPUBufferReadback* ReadbackBuffer : MessageReadbackBuffersInUse)
		{
			delete ReadbackBuffer;
		}
		MessageReadbackBuffersInUse.Empty();

		for (FRHIGPUBufferReadback* ReadbackBuffer : MessageReadbackBuffersReady)
		{
			delete ReadbackBuffer;
		}
		MessageReadbackBuffersReady.Empty();
	}

	int32 GetMaxBufferSize() const { return GMaxBufferSize * 1024; }

	TMap<FMessageId, TSharedPtr<FHandler>> MessageHandlers;

	TArray<FRHIGPUBufferReadback*, TInlineAllocator<8>> MessageReadbackBuffersInUse;
	TArray<FRHIGPUBufferReadback*, TInlineAllocator<8>> MessageReadbackBuffersReady;
	FRDGBuffer* MessageBuffer{};

	uint32 NextMessageId = 0;
};

static TGlobalResource<FSystem> GSystem;

void FSocket::Reset()
{
	if (MessageId.IsValid() && GSystem.IsInitialized())
	{
		GSystem.RemoveHandler(MessageId);
	}

	MessageId = FMessageId::Null;
}

bool FScope::bRecursionCheck = false;

FScope::FScope(FRDGBuilder& InGraphBuilder)
	: GraphBuilder(InGraphBuilder)
{
	checkf(!bRecursionCheck, TEXT("GPUMessage::FScope (GPU_MESSAGE_SCOPE) has already been pushed. Only one scope can be active at a time."));
	bRecursionCheck = true;
	GSystem.BeginMessageScope(InGraphBuilder);
}

FScope::~FScope()
{
	GSystem.EndMessageScope(GraphBuilder);
	bRecursionCheck = false;
}

FSocket RegisterHandler(const TSharedPtr<FHandler>& Handler)
{
	return GSystem.RegisterHandler(Handler);
}

FParameters GetShaderParameters(FRDGBuilder& GraphBuilder)
{
	const FParameters* Parameters = GraphBuilder.Blackboard.Get<FParameters>();
	checkf(Parameters, TEXT("Shader parameters not initialized. GPUMessage::GetShaderParameters may only be called within a GPU_MESSAGE_SCOPE."));
	return *Parameters;
}

} // GPUMessage