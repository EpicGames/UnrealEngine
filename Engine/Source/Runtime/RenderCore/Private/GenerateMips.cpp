// Copyright Epic Games, Inc. All Rights Reserved.

#include "GenerateMips.h"
#include "RenderGraphUtils.h"
#include "PipelineStateCache.h"
#include "GlobalShader.h"
#include "CommonRenderResources.h"
#include "RHIStaticStates.h"
#include "PixelShaderUtils.h"

#if PLATFORM_WINDOWS || PLATFORM_ANDROID || PLATFORM_LINUX
#include "IOpenGLDynamicRHI.h"
#endif

class FGenerateMipsCS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FGenerateMipsCS)
	SHADER_USE_PARAMETER_STRUCT(FGenerateMipsCS, FGlobalShader)

	class FGenMipsSRGB : SHADER_PERMUTATION_BOOL("GENMIPS_SRGB");
	class FGenMipsSwizzle : SHADER_PERMUTATION_BOOL("GENMIPS_SWIZZLE");
	using FPermutationDomain = TShaderPermutationDomain<FGenMipsSRGB, FGenMipsSwizzle>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(FVector2f, TexelSize)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, MipInSRV)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, MipOutUAV)
		SHADER_PARAMETER_SAMPLER(SamplerState, MipSampler)
	END_SHADER_PARAMETER_STRUCT()

	static void ModifyCompilationEnvironment(const FShaderPermutationParameters&, FShaderCompilerEnvironment& OutEnvironment)
	{
		OutEnvironment.SetDefine(TEXT("GENMIPS_COMPUTE"), 1);
	}
};

IMPLEMENT_GLOBAL_SHADER(FGenerateMipsCS, "/Engine/Private/ComputeGenerateMips.usf", "MainCS", SF_Compute);

class FGenerateMipsVS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FGenerateMipsVS);
	SHADER_USE_PARAMETER_STRUCT(FGenerateMipsVS, FGlobalShader);
	using FParameters = FEmptyShaderParameters;

	static void ModifyCompilationEnvironment(const FShaderPermutationParameters&, FShaderCompilerEnvironment& OutEnvironment)
	{
		OutEnvironment.SetDefine(TEXT("GENMIPS_COMPUTE"), 0);
	}
};

IMPLEMENT_GLOBAL_SHADER(FGenerateMipsVS, "/Engine/Private/ComputeGenerateMips.usf", "MainVS", SF_Vertex);

class FGenerateMipsPS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FGenerateMipsPS);
	SHADER_USE_PARAMETER_STRUCT(FGenerateMipsPS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(float, Level)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, MipInSRV)
		SHADER_PARAMETER_SAMPLER(SamplerState, MipSampler)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

	static void ModifyCompilationEnvironment(const FShaderPermutationParameters&, FShaderCompilerEnvironment& OutEnvironment)
	{
		OutEnvironment.SetDefine(TEXT("GENMIPS_COMPUTE"), 0);
	}
};

IMPLEMENT_GLOBAL_SHADER(FGenerateMipsPS, "/Engine/Private/ComputeGenerateMips.usf", "MainPS", SF_Pixel);

// Determine the indirect dispatch based on conditions
class FBuildIndirectDispatchArgsBufferCS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FBuildIndirectDispatchArgsBufferCS)
	SHADER_USE_PARAMETER_STRUCT(FBuildIndirectDispatchArgsBufferCS, FGlobalShader)

		BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(FIntPoint, TextureSize)
		SHADER_PARAMETER(uint32, Offset)
		SHADER_PARAMETER(uint32, NumMips)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint32>, ConditionBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint32>, RWIndirectDispatchArgsBuffer)
		END_SHADER_PARAMETER_STRUCT()

	static void ModifyCompilationEnvironment(const FShaderPermutationParameters&, FShaderCompilerEnvironment& OutEnvironment)
	{
		OutEnvironment.SetDefine(TEXT("GENMIPS_COMPUTE"), 1);
	}
};

IMPLEMENT_GLOBAL_SHADER(FBuildIndirectDispatchArgsBufferCS, "/Engine/Private/ComputeGenerateMips.usf", "BuildIndirectDispatchArgsCS", SF_Compute);

class FGenerateMipsIndirectCS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FGenerateMipsIndirectCS)
	SHADER_USE_PARAMETER_STRUCT(FGenerateMipsIndirectCS, FGlobalShader)

		class FGenMipsSRGB : SHADER_PERMUTATION_BOOL("GENMIPS_SRGB");
	class FGenMipsSwizzle : SHADER_PERMUTATION_BOOL("GENMIPS_SWIZZLE");
	using FPermutationDomain = TShaderPermutationDomain<FGenMipsSRGB, FGenMipsSwizzle>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(FVector2f, TexelSize)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, MipInSRV)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, MipOutUAV)
		SHADER_PARAMETER_SAMPLER(SamplerState, MipSampler)
		RDG_BUFFER_ACCESS(IndirectDispatchArgsBuffer, ERHIAccess::IndirectArgs)
		END_SHADER_PARAMETER_STRUCT()

	static void ModifyCompilationEnvironment(const FShaderPermutationParameters&, FShaderCompilerEnvironment& OutEnvironment)
	{
		OutEnvironment.SetDefine(TEXT("GENMIPS_COMPUTE"), 1);
	}
};

IMPLEMENT_GLOBAL_SHADER(FGenerateMipsIndirectCS, "/Engine/Private/ComputeGenerateMips.usf", "MainCS", SF_Compute);

BEGIN_SHADER_PARAMETER_STRUCT(FGenerateMipsRHIImplParameters, )
	RDG_TEXTURE_ACCESS(Texture, ERHIAccess::CopyDest)
END_SHADER_PARAMETER_STRUCT()

void FGenerateMips::ExecuteRaster(FRDGBuilder& GraphBuilder, ERHIFeatureLevel::Type FeatureLevel, FRDGTextureRef Texture, FRHISamplerState* Sampler)
{
	check(Texture);
	check(Sampler);

	const FRDGTextureDesc& TextureDesc = Texture->Desc;

	auto ShaderMap = GetGlobalShaderMap(FeatureLevel);
	TShaderMapRef<FGenerateMipsVS> VertexShader(ShaderMap);
	TShaderMapRef<FGenerateMipsPS> PixelShader(ShaderMap);

	int32 SliceCount = 1;

	FRDGTextureSRVDesc SRVDesc(Texture);
	SRVDesc.NumMipLevels = 1;

	if (TextureDesc.Dimension == ETextureDimension::TextureCube)
	{
		SliceCount = ECubeFace::CubeFace_MAX;

		SRVDesc.DimensionOverride = ETextureDimension::Texture2DArray;
		SRVDesc.NumArraySlices = 1;
	}

	// Loop through each level of the mips that require creation and add a dispatch pass per level.
	for (uint8 MipLevel = 1, MipCount = TextureDesc.NumMips; MipLevel < MipCount; ++MipLevel)
	{
		const uint32 InputMipLevel = MipLevel - 1;

		const FIntPoint DestTextureSize(
			FMath::Max(TextureDesc.Extent.X >> MipLevel, 1),
			FMath::Max(TextureDesc.Extent.Y >> MipLevel, 1));

		SRVDesc.MipLevel = InputMipLevel;

		for (int32 SliceIndex = 0; SliceIndex < SliceCount; ++SliceIndex)
		{
			SRVDesc.FirstArraySlice = SliceIndex;

			FGenerateMipsPS::FParameters* PassParameters = GraphBuilder.AllocParameters<FGenerateMipsPS::FParameters>();
			PassParameters->Level = InputMipLevel;
			PassParameters->MipInSRV = GraphBuilder.CreateSRV(SRVDesc);
			PassParameters->MipSampler = Sampler;
			PassParameters->RenderTargets[0] = FRenderTargetBinding(Texture, ERenderTargetLoadAction::ELoad, MipLevel, SliceCount > 1 ? SliceIndex : INDEX_NONE);

			GraphBuilder.AddPass(
				RDG_EVENT_NAME("GenerateMips DestMipLevel=%d Slice=%d", MipLevel, SliceIndex),
				PassParameters,
				ERDGPassFlags::Raster,
				[VertexShader, PixelShader, PassParameters, DestTextureSize](FRDGAsyncTask, FRHICommandList& RHICmdList)
			{
				RHICmdList.SetViewport(0.0f, 0.0f, 0.0f, (float)DestTextureSize.X, (float)DestTextureSize.Y, 1.0f);

				FGraphicsPipelineStateInitializer GraphicsPSOInit;
				GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();
				GraphicsPSOInit.RasterizerState = TStaticRasterizerState<>::GetRHI();
				GraphicsPSOInit.BlendState = TStaticBlendStateWriteMask<CW_RGBA, CW_NONE, CW_NONE, CW_NONE, CW_NONE, CW_NONE, CW_NONE, CW_NONE>::GetRHI();
				GraphicsPSOInit.PrimitiveType = PT_TriangleStrip;
				GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GFilterVertexDeclaration.VertexDeclarationRHI;
				GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
				GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
				RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);
				SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);
				SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), *PassParameters);

				FPixelShaderUtils::DrawFullscreenTriangle(RHICmdList, 1);
			});
		}
	}
}

void FGenerateMips::ExecuteCompute(FRDGBuilder& GraphBuilder, ERHIFeatureLevel::Type FeatureLevel, FRDGTextureRef Texture, FRHISamplerState* Sampler)
{
	check(Texture);
	check(Sampler);

	const FRDGTextureDesc& TextureDesc = Texture->Desc;

	// Select compute shader variant (normal vs. sRGB etc.)
	// Note: Floating-point platform textures cannot be sRGB-encoded (invalid condition). We exclude them here for safety.
	bool bMipsSRGB = EnumHasAnyFlags(TextureDesc.Flags, TexCreate_SRGB) && !IsFloatFormat(TextureDesc.Format);
	const bool bMipsSwizzle = false; 

	FGenerateMipsCS::FPermutationDomain PermutationVector;
	PermutationVector.Set<FGenerateMipsCS::FGenMipsSRGB>(bMipsSRGB);
	PermutationVector.Set<FGenerateMipsCS::FGenMipsSwizzle>(bMipsSwizzle);
	TShaderMapRef<FGenerateMipsCS> ComputeShader(GetGlobalShaderMap(FeatureLevel), PermutationVector);

	int32 SliceCount = 1;

	FRDGTextureSRVDesc SRVDesc(Texture);
	FRDGTextureUAVDesc UAVDesc(Texture);
	SRVDesc.NumMipLevels = 1;

	if (TextureDesc.Dimension == ETextureDimension::TextureCube)
	{
		SliceCount = ECubeFace::CubeFace_MAX;

		SRVDesc.DimensionOverride = ETextureDimension::Texture2DArray;
		SRVDesc.NumArraySlices = 1;
		UAVDesc.DimensionOverride = ETextureDimension::Texture2DArray;
		UAVDesc.NumArraySlices = 1;
	}

	// Loop through each level of the mips that require creation and add a dispatch pass per level.
	for (uint8 MipLevel = 1, MipCount = TextureDesc.NumMips; MipLevel < MipCount; ++MipLevel)
	{
		const FIntPoint DestTextureSize(
			FMath::Max(TextureDesc.Extent.X >> MipLevel, 1),
			FMath::Max(TextureDesc.Extent.Y >> MipLevel, 1));

		SRVDesc.MipLevel = (int8)(MipLevel - 1);
		UAVDesc.MipLevel = (int8)(MipLevel);

		for (int32 SliceIndex = 0; SliceIndex < SliceCount; ++SliceIndex)
		{
			SRVDesc.FirstArraySlice = SliceIndex;
			UAVDesc.FirstArraySlice = SliceIndex;

			FGenerateMipsCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FGenerateMipsCS::FParameters>();
			PassParameters->TexelSize = FVector2f(1.0f / DestTextureSize.X, 1.0f / DestTextureSize.Y);
			PassParameters->MipInSRV = GraphBuilder.CreateSRV(SRVDesc);
			PassParameters->MipOutUAV = GraphBuilder.CreateUAV(UAVDesc);
			PassParameters->MipSampler = Sampler;

			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("GenerateMips DestMipLevel=%d Slice=%d", MipLevel, SliceIndex),
				ComputeShader,
				PassParameters,
				FComputeShaderUtils::GetGroupCount(DestTextureSize, FComputeShaderUtils::kGolden2DGroupSize));
		}
	}
}

void FGenerateMips::ExecuteCompute(
	FRDGBuilder& GraphBuilder,
	ERHIFeatureLevel::Type FeatureLevel,
	FRDGTextureRef Texture,
	FRHISamplerState* Sampler,
	FRDGBufferRef ConditionBuffer,
	uint32 Offset)
{
	check(Texture);
	check(Sampler);

	FGlobalShaderMap* ShaderMap = GetGlobalShaderMap(FeatureLevel);

	const FRDGTextureDesc& TextureDesc = Texture->Desc;

	FRDGBufferRef IndirectDispatchArgsBuffer = GraphBuilder.CreateBuffer(
		FRDGBufferDesc::CreateIndirectDesc<FRHIDispatchIndirectParameters>(FMath::Max(TextureDesc.NumMips - 1,1)),
		TEXT("IndirectDispatchArgsBuffer"));
	{
		// build the indirect dispatch arguments buffer ( compute the group count on GPU conditionally)
		FBuildIndirectDispatchArgsBufferCS::FParameters* PassParameters = 
			GraphBuilder.AllocParameters<FBuildIndirectDispatchArgsBufferCS::FParameters>();
		PassParameters->TextureSize = TextureDesc.Extent;
		PassParameters->Offset = Offset;
		PassParameters->NumMips = TextureDesc.NumMips;
		PassParameters->ConditionBuffer = GraphBuilder.CreateSRV(ConditionBuffer, EPixelFormat::PF_R32_UINT);
		PassParameters->RWIndirectDispatchArgsBuffer = GraphBuilder.CreateUAV(IndirectDispatchArgsBuffer, EPixelFormat::PF_R32_UINT);

		TShaderMapRef<FBuildIndirectDispatchArgsBufferCS> ComputeShader(ShaderMap);

		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("GenerateMips BuildIndirectArgs(Mips=%d)", TextureDesc.NumMips),
			ComputeShader,
			PassParameters,
			FIntVector(FMath::DivideAndRoundUp(TextureDesc.NumMips - 1,FComputeShaderUtils::kGolden2DGroupSize), 1, 1));
	}

	// Select compute shader variant (normal vs. sRGB etc.)
	bool bMipsSRGB = EnumHasAnyFlags(TextureDesc.Flags, TexCreate_SRGB);
	const bool bMipsSwizzle = false;

	FGenerateMipsIndirectCS::FPermutationDomain PermutationVector;
	PermutationVector.Set<FGenerateMipsIndirectCS::FGenMipsSRGB>(bMipsSRGB);
	PermutationVector.Set<FGenerateMipsIndirectCS::FGenMipsSwizzle>(bMipsSwizzle);
	TShaderMapRef<FGenerateMipsIndirectCS> ComputeShader(ShaderMap, PermutationVector);

	// Loop through each level of the mips that require creation and add a dispatch pass per level.
	for (uint8 MipLevel = 1, MipCount = TextureDesc.NumMips; MipLevel < MipCount; ++MipLevel)
	{
		const FIntPoint DestTextureSize(
			FMath::Max(TextureDesc.Extent.X >> MipLevel, 1),
			FMath::Max(TextureDesc.Extent.Y >> MipLevel, 1));

		FGenerateMipsIndirectCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FGenerateMipsIndirectCS::FParameters>();
		PassParameters->TexelSize = FVector2f(1.0f / DestTextureSize.X, 1.0f / DestTextureSize.Y);
		PassParameters->MipInSRV = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::CreateForMipLevel(Texture, MipLevel - 1));
		PassParameters->MipOutUAV = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(Texture, MipLevel));
		PassParameters->MipSampler = Sampler;
		PassParameters->IndirectDispatchArgsBuffer = IndirectDispatchArgsBuffer;

		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("GenerateMips DestMipLevel=%d", MipLevel),
			ComputeShader,
			PassParameters,
			IndirectDispatchArgsBuffer, sizeof(FRHIDispatchIndirectParameters) * (MipLevel - 1));
	}
}

bool FGenerateMips::WillFormatSupportCompute(EPixelFormat InPixelFormat)
{
	return UE::PixelFormat::HasCapabilities(InPixelFormat, EPixelFormatCapabilities::TypedUAVStore);
}

void FGenerateMips::Execute(FRDGBuilder& GraphBuilder, ERHIFeatureLevel::Type FeatureLevel, FRDGTextureRef Texture, FGenerateMipsParams Params, EGenerateMipsPass Pass)
{
	if (Texture->Desc.NumMips > 1)
	{
		FSamplerStateInitializerRHI SamplerInit(Params.Filter, Params.AddressU, Params.AddressV, Params.AddressW);
		FSamplerStateRHIRef Sampler = *GraphBuilder.AllocObject<FSamplerStateRHIRef>(RHICreateSamplerState(SamplerInit));
		Execute(GraphBuilder, FeatureLevel, Texture, Sampler, Pass);
	}
}

void FGenerateMips::Execute(FRDGBuilder& GraphBuilder, ERHIFeatureLevel::Type FeatureLevel, FRDGTextureRef Texture, FRHISamplerState* Sampler, EGenerateMipsPass Pass)
{
#if PLATFORM_WINDOWS || PLATFORM_ANDROID || PLATFORM_LINUX
	if (RHIGetInterfaceType() == ERHIInterfaceType::OpenGL)
	{
		// Special case for OpenGL. We can't use the above compute/pixel shaders due to lack of proper SRV support.
		FGenerateMipsRHIImplParameters* PassParameters = GraphBuilder.AllocParameters<FGenerateMipsRHIImplParameters>();
		PassParameters->Texture = Texture;

		GraphBuilder.AddPass(RDG_EVENT_NAME("GenerateMips - OpenGL"), PassParameters, ERDGPassFlags::Copy,
			[Texture](FRDGAsyncTask, FRHICommandList& RHICmdList)
		{
			RHICmdList.EnqueueLambda(TEXT("GenerateMips - OpenGL"), [Texture = Texture->GetRHI()](FRHICommandList&)
			{
				GetIOpenGLDynamicRHI()->RHIGenerateMips(Texture);
			});
		});
	}
	else
#endif
	{
		if (Pass == EGenerateMipsPass::AutoDetect)
		{
			// Use compute when the given texture has a UAV-compatible format and UAV create flag, otherwise fallback to raster.
			Pass = WillFormatSupportCompute(Texture->Desc.Format) && EnumHasAnyFlags(Texture->Desc.Flags, ETextureCreateFlags::UAV) 
				? EGenerateMipsPass::Compute
				: EGenerateMipsPass::Raster;
		}

		if (Pass == EGenerateMipsPass::Compute)
		{
			ensureMsgf(EnumHasAllFlags(Texture->Desc.Flags, ETextureCreateFlags::UAV | ETextureCreateFlags::ShaderResource),
				TEXT("Texture must be created with ETextureCreateFlags::UAV and ETextureCreateFlags::ShaderResource to be used in compute-based mip generation."));

			ExecuteCompute(GraphBuilder, FeatureLevel, Texture, Sampler);
		}
		else
		{
			ensureMsgf(EnumHasAllFlags(Texture->Desc.Flags, ETextureCreateFlags::RenderTargetable | ETextureCreateFlags::ShaderResource),
				TEXT("Texture must be created with ETextureCreateFlags::RenderTargetable and ETextureCreateFlags::ShaderResource to be used in raster-based mip generation."));

			ExecuteRaster(GraphBuilder, FeatureLevel, Texture, Sampler);
		}
	}
}
