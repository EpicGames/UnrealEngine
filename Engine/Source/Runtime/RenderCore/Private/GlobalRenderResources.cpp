// Copyright Epic Games, Inc. All Rights Reserved.

#include "GlobalRenderResources.h"
#include "RenderGraphUtils.h"
#include "Containers/ResourceArray.h"
#include "RenderCore.h"
#include "RenderUtils.h"
#include "RHIResourceUtils.h"
#include "Algo/Reverse.h"
#include "Async/Mutex.h"

// The maximum number of transient vertex buffer bytes to allocate before we start panic logging who is doing the allocations
int32 GMaxVertexBytesAllocatedPerFrame = 32 * 1024 * 1024;

FAutoConsoleVariableRef CVarMaxVertexBytesAllocatedPerFrame(
	TEXT("r.MaxVertexBytesAllocatedPerFrame"),
	GMaxVertexBytesAllocatedPerFrame,
	TEXT("The maximum number of transient vertex buffer bytes to allocate before we start panic logging who is doing the allocations"));

int32 GGlobalBufferNumFramesUnusedThreshold = 30;
FAutoConsoleVariableRef CVarGlobalBufferNumFramesUnusedThreshold(
	TEXT("r.NumFramesUnusedBeforeReleasingGlobalResourceBuffers"),
	GGlobalBufferNumFramesUnusedThreshold,
	TEXT("Number of frames after which unused global resource allocations will be discarded. Set 0 to ignore. (default=30)"));

//////////////////////////////////////////////////////////////////////////////////////////////////////////////////////


// Bulk data interface for providing a single color used to initialize a volume texture.
struct FColorBulkData : public FResourceBulkDataInterface
{
	FColorBulkData(uint8 Alpha) : Color(0, 0, 0, Alpha) { }
	FColorBulkData(int32 R, int32 G, int32 B, int32 A) : Color(R, G, B, A) {}
	FColorBulkData(FColor InColor) : Color(InColor) { }

	virtual const void* GetResourceBulkData() const override { return &Color; }
	virtual uint32 GetResourceBulkDataSize() const override { return sizeof(Color); }
	virtual void Discard() override { }

	FColor Color;
};

/**
 * A solid-colored 1x1 texture.
 */
template <int32 R, int32 G, int32 B, int32 A>
class FColoredTexture : public FTextureWithSRV
{
public:
	// FResource interface.
	virtual void InitRHI(FRHICommandListBase& RHICmdList) override
	{
		// BGRA typed UAV is unsupported per D3D spec, use RGBA here.
		const FRHITextureCreateDesc Desc =
			FRHITextureCreateDesc::Create2D(GetTextureName(), 1, 1, PF_R8G8B8A8)
			.SetFlags(ETextureCreateFlags::ShaderResource)
			.SetClassName(TEXT("FColoredTexture"))
			.SetInitActionInitializer();

		FRHITextureInitializer Initializer = RHICmdList.CreateTextureInitializer(Desc);

		const FColor ColorValue(R, G, B, A);
		Initializer.GetTexture2DSubresource(0).WriteColor(ColorValue);

		// Create the texture RHI.
		TextureRHI = Initializer.Finalize();

		// Create the sampler state RHI resource.
		FSamplerStateInitializerRHI SamplerStateInitializer(SF_Point, AM_Wrap, AM_Wrap, AM_Wrap);
		SamplerStateRHI = GetOrCreateSamplerState(SamplerStateInitializer);

		// Create a view of the texture
		ShaderResourceViewRHI = RHICmdList.CreateShaderResourceView(TextureRHI, FRHIViewDesc::CreateTextureSRV().SetDimensionFromTexture(TextureRHI));
	}

	virtual uint32 GetSizeX() const override { return 1; }
	virtual uint32 GetSizeY() const override { return 1; }
	virtual const TCHAR* GetTextureName() const { return TEXT("ColoredTexture"); }
};

class FEmptyVertexBuffer : public FVertexBufferWithSRV
{
public:
	virtual void InitRHI(FRHICommandListBase& RHICmdList) override
	{
		// Create the texture RHI.
		const FRHIBufferCreateDesc CreateDesc =
			FRHIBufferCreateDesc::CreateVertex(TEXT("EmptyVertexBuffer"), 16)
			.AddUsage(BUF_Static | BUF_ShaderResource | BUF_UnorderedAccess)
			.DetermineInitialState();

		VertexBufferRHI = RHICmdList.CreateBuffer(CreateDesc);

		// Create a view of the buffer
		ShaderResourceViewRHI = RHICmdList.CreateShaderResourceView(
			VertexBufferRHI,
			FRHIViewDesc::CreateBufferSRV()
				.SetType(FRHIViewDesc::EBufferType::Typed)
				.SetFormat(PF_R32_UINT));
		UnorderedAccessViewRHI = RHICmdList.CreateUnorderedAccessView(
			VertexBufferRHI,
			FRHIViewDesc::CreateBufferUAV()
				.SetType(FRHIViewDesc::EBufferType::Typed)
				.SetFormat(PF_R32_UINT));
	}
};

class FEmptyUInt4VertexBuffer : public FVertexBufferWithSRV
{
public:
	virtual void InitRHI(FRHICommandListBase& RHICmdList) override
	{
		const FRHIBufferCreateDesc CreateDesc =
			FRHIBufferCreateDesc::CreateVertex(TEXT("EmptyUInt4VertexBuffer"), sizeof(FUintVector4))
			.AddUsage(BUF_Static | BUF_ShaderResource | BUF_UnorderedAccess)
			.DetermineInitialState();
		
		const FUintVector4 InitialData(ForceInitToZero);
		VertexBufferRHI = UE::RHIResourceUtils::CreateBufferWithValue(RHICmdList, CreateDesc, InitialData);

		ShaderResourceViewRHI = RHICmdList.CreateShaderResourceView(
			VertexBufferRHI,
			FRHIViewDesc::CreateBufferSRV()
				.SetType(FRHIViewDesc::EBufferType::Typed)
				.SetFormat(PF_R32G32B32A32_UINT));

		UnorderedAccessViewRHI = RHICmdList.CreateUnorderedAccessView(
			VertexBufferRHI,
			FRHIViewDesc::CreateBufferUAV()
				.SetType(FRHIViewDesc::EBufferType::Typed)
				.SetFormat(PF_R32G32B32A32_UINT));
	}
};

class FEmptyStructuredBuffer : public FVertexBufferWithSRV
{
public:
	virtual void InitRHI(FRHICommandListBase& RHICmdList) override
	{
		const FRHIBufferCreateDesc CreateDesc =
			FRHIBufferCreateDesc::CreateStructured(TEXT("EmptyStructuredBuffer"), sizeof(uint32) * 4, sizeof(uint32))
			.AddUsage(BUF_Static | BUF_ShaderResource | BUF_UnorderedAccess)
			.DetermineInitialState();

		// Create the buffer RHI.
		VertexBufferRHI = RHICmdList.CreateBuffer(CreateDesc);

		// Create a view of the buffer
		ShaderResourceViewRHI = RHICmdList.CreateShaderResourceView(VertexBufferRHI, FRHIViewDesc::CreateBufferSRV().SetTypeFromBuffer(VertexBufferRHI));
		UnorderedAccessViewRHI = RHICmdList.CreateUnorderedAccessView(VertexBufferRHI, FRHIViewDesc::CreateBufferUAV().SetTypeFromBuffer(VertexBufferRHI));
	}
};

class FBlackFloat4StructuredBufferWithSRV : public FVertexBufferWithSRV
{
public:
	virtual void InitRHI(FRHICommandListBase& RHICmdList) override
	{
		const FRHIBufferCreateDesc CreateDesc =
			FRHIBufferCreateDesc::CreateStructured(TEXT("BlackFloat4StructuredBuffer"), sizeof(FVector4f), sizeof(FVector4f))
			.AddUsage(BUF_Static | BUF_ShaderResource | BUF_UnorderedAccess)
			.SetInitialState(ERHIAccess::SRVMask);

		const FVector4f InitialData(0.0f, 0.0f, 0.0f, 0.0f);

		// Create the buffer RHI.
		VertexBufferRHI = UE::RHIResourceUtils::CreateBufferWithValue(RHICmdList, CreateDesc, InitialData);

		// Create a view of the buffer
		ShaderResourceViewRHI = RHICmdList.CreateShaderResourceView(VertexBufferRHI, FRHIViewDesc::CreateBufferSRV().SetTypeFromBuffer(VertexBufferRHI));
		UnorderedAccessViewRHI = RHICmdList.CreateUnorderedAccessView(VertexBufferRHI, FRHIViewDesc::CreateBufferUAV().SetTypeFromBuffer(VertexBufferRHI));
	}
};

class FBlackFloat4VertexBuffer : public FVertexBufferWithSRV
{
public:
	virtual void InitRHI(FRHICommandListBase& RHICmdList) override
	{
		const FRHIBufferCreateDesc CreateDesc =
			FRHIBufferCreateDesc::CreateVertex(TEXT("BlackFloat4VertexBuffer"), sizeof(FVector4f))
			.AddUsage(BUF_Static | BUF_ShaderResource | BUF_UnorderedAccess)
			.SetInitialState(ERHIAccess::SRVMask);

		const FVector4f InitialData(0.0f, 0.0f, 0.0f, 0.0f);

		// Create the texture RHI.
		VertexBufferRHI = UE::RHIResourceUtils::CreateBufferWithValue(RHICmdList, CreateDesc, InitialData);

		// Create a view of the buffer
		ShaderResourceViewRHI = RHICmdList.CreateShaderResourceView(
			VertexBufferRHI,
			FRHIViewDesc::CreateBufferSRV()
				.SetType(FRHIViewDesc::EBufferType::Typed)
				.SetFormat(PF_A32B32G32R32F));
		UnorderedAccessViewRHI = RHICmdList.CreateUnorderedAccessView(
			VertexBufferRHI, FRHIViewDesc::CreateBufferUAV()
				.SetType(FRHIViewDesc::EBufferType::Typed)
				.SetFormat(PF_A32B32G32R32F));
	}
};


class FDummyTransitionTexture : public FTexture
{
public:
	virtual void InitRHI(FRHICommandListBase& RHICmdList) override
	{
		const FRHITextureCreateDesc Desc =
			FRHITextureCreateDesc::Create2D(TEXT("DummyTransitionTexture"), 1, 1, PF_R8G8B8A8)
			.SetFlags(ETextureCreateFlags::ShaderResource)
			.SetClassName(TEXT("FDummyTransitionTexture"))
			.SetInitActionInitializer();

		FRHITextureInitializer Initializer = RHICmdList.CreateTextureInitializer(Desc);

		const FColor ColorValue(0, 0, 0, 255);
		Initializer.GetTexture2DSubresource(0).WriteColor(ColorValue);

		// Create the texture RHI.
		TextureRHI = Initializer.Finalize();

		// Create the sampler state RHI resource.
		FSamplerStateInitializerRHI SamplerStateInitializer(SF_Point, AM_Wrap, AM_Wrap, AM_Wrap);
		SamplerStateRHI = GetOrCreateSamplerState(SamplerStateInitializer);
	}

	virtual uint32 GetSizeX() const override { return 1; }
	virtual uint32 GetSizeY() const override { return 1; }
};

class FWhiteTexture : public FColoredTexture<255, 255, 255, 255>
{
	virtual const TCHAR* GetTextureName() const override { return TEXT("FWhiteTexture"); }
};

class FBlackTexture : public FColoredTexture<0, 0, 0, 0>
{
	virtual const TCHAR* GetTextureName() const override { return TEXT("FBlackTexture"); }
};

class FBlackTextureWithSRV : public FColoredTexture<0, 0, 0, 255>
{
public:
	virtual void InitRHI(FRHICommandListBase& RHICmdList) override
	{
		FColoredTexture::InitRHI(RHICmdList);
		FRHITextureReference::DefaultTexture = TextureRHI;
	}

	virtual void ReleaseRHI() override
	{
		FRHITextureReference::DefaultTexture.SafeRelease();
		FColoredTexture::ReleaseRHI();
	}

	virtual const TCHAR* GetTextureName() const override { return TEXT("FBlackTextureWithSRV"); }
};

FTextureWithSRV* GWhiteTextureWithSRV = new TGlobalResource<FWhiteTexture, FRenderResource::EInitPhase::Pre>;
FTextureWithSRV* GBlackTextureWithSRV = new TGlobalResource<FBlackTextureWithSRV, FRenderResource::EInitPhase::Pre>();
FTextureWithSRV* GTransparentBlackTextureWithSRV = new TGlobalResource<FBlackTexture, FRenderResource::EInitPhase::Pre>;
FTexture* GDummyTransitionTexture = new TGlobalResource<FDummyTransitionTexture, FRenderResource::EInitPhase::Pre>;

FTexture* GWhiteTexture = GWhiteTextureWithSRV;
FTexture* GBlackTexture = GBlackTextureWithSRV;
FTexture* GTransparentBlackTexture = GTransparentBlackTextureWithSRV;

FVertexBufferWithSRV* GEmptyVertexBufferWithUAV = new TGlobalResource<FEmptyVertexBuffer, FRenderResource::EInitPhase::Pre>;
FVertexBufferWithSRV* GEmptyVertexBufferUInt4WithUAV = new TGlobalResource<FEmptyUInt4VertexBuffer, FRenderResource::EInitPhase::Pre>;
FVertexBufferWithSRV* GEmptyStructuredBufferWithUAV = new TGlobalResource<FEmptyStructuredBuffer, FRenderResource::EInitPhase::Pre>;
FVertexBufferWithSRV* GBlackFloat4StructuredBufferWithSRV = new TGlobalResource<FBlackFloat4StructuredBufferWithSRV, FRenderResource::EInitPhase::Pre>;
FVertexBufferWithSRV* GBlackFloat4VertexBufferWithSRV = new TGlobalResource<FBlackFloat4VertexBuffer, FRenderResource::EInitPhase::Pre>;

class FWhiteVertexBuffer : public FVertexBufferWithSRV
{
public:
	virtual void InitRHI(FRHICommandListBase& RHICmdList) override
	{
		const FRHIBufferCreateDesc CreateDesc =
			FRHIBufferCreateDesc::CreateVertex(TEXT("WhiteVertexBuffer"), sizeof(FVector4f))
			.AddUsage(BUF_Static | BUF_ShaderResource)
			.DetermineInitialState();

		const FVector4f InitialData(1.0f, 1.0f, 1.0f, 1.0f);

		// Create the buffer RHI.
		VertexBufferRHI = UE::RHIResourceUtils::CreateBufferWithValue(RHICmdList, CreateDesc, InitialData);

		// Create a view of the buffer
		ShaderResourceViewRHI = RHICmdList.CreateShaderResourceView(
			VertexBufferRHI,
			FRHIViewDesc::CreateBufferSRV()
				.SetType(FRHIViewDesc::EBufferType::Typed)
				.SetFormat(PF_A32B32G32R32F));
	}
};

FVertexBufferWithSRV* GWhiteVertexBufferWithSRV = new TGlobalResource<FWhiteVertexBuffer, FRenderResource::EInitPhase::Pre>;

class FBlackVertexBuffer : public FVertexBufferWithSRV
{
public:
	virtual void InitRHI(FRHICommandListBase& RHICmdList) override
	{
		const FRHIBufferCreateDesc CreateDesc =
			FRHIBufferCreateDesc::CreateVertex(TEXT("BlackVertexBuffer"), sizeof(FVector4f))
			.AddUsage(BUF_Static | BUF_ShaderResource)
			.DetermineInitialState();

		const FVector4f InitialData(0.0f, 0.0f, 0.0f, 0.0f);

		// Create the buffer RHI.
		VertexBufferRHI = UE::RHIResourceUtils::CreateBufferWithValue(RHICmdList, CreateDesc, InitialData);

		// Create a view of the buffer
		ShaderResourceViewRHI = RHICmdList.CreateShaderResourceView(
			VertexBufferRHI,
			FRHIViewDesc::CreateBufferSRV()
				.SetType(FRHIViewDesc::EBufferType::Typed)
				.SetFormat(PF_A32B32G32R32F));
	}
};

FVertexBufferWithSRV* GBlackVertexBufferWithSRV = new TGlobalResource<FBlackVertexBuffer, FRenderResource::EInitPhase::Pre>;

class FWhiteVertexBufferWithRDG : public FBufferWithRDG
{
public:
	virtual void InitRHI(FRHICommandListBase& RHICmdList) override
	{
		if (!Buffer.IsValid())
		{
			Buffer = AllocatePooledBuffer(FRDGBufferDesc::CreateBufferDesc(sizeof(FVector4f), 1), TEXT("WhiteVertexBufferWithRDG"));

			FVector4f* BufferData = (FVector4f*)RHICmdList.LockBuffer(Buffer->GetRHI(), 0, sizeof(FVector4f), RLM_WriteOnly);
			*BufferData = FVector4f(1.0f, 1.0f, 1.0f, 1.0f);
			RHICmdList.UnlockBuffer(Buffer->GetRHI());
		}
	}
};

FBufferWithRDG* GWhiteVertexBufferWithRDG = new TGlobalResource<FWhiteVertexBufferWithRDG, FRenderResource::EInitPhase::Pre>();

/**
 * A class representing a 1x1x1 black volume texture.
 */
template <EPixelFormat PixelFormat, uint8 Alpha>
class FBlackVolumeTexture : public FTexture
{
public:
	virtual void InitRHI(FRHICommandListBase& RHICmdList) override
	{
		FColorBulkData BulkData(Alpha);

		FRHITextureCreateDesc Desc =
			GSupportsTexture3D
			? FRHITextureCreateDesc::Create3D(TEXT("BlackVolumeTexture3D"), 1, 1, 1, PixelFormat)
			: FRHITextureCreateDesc::Create2D(TEXT("BlackVolumeTexture2D"), 1, 1, PixelFormat);

		Desc.SetFlags(ETextureCreateFlags::ShaderResource)
			.SetClassName(TEXT("FBlackVolumeTexture"))
			.SetInitActionBulkData(&BulkData);

		TextureRHI = RHICmdList.CreateTexture(Desc);

		// Create the sampler state.
		FSamplerStateInitializerRHI SamplerStateInitializer(SF_Point, AM_Wrap, AM_Wrap, AM_Wrap);
		SamplerStateRHI = GetOrCreateSamplerState(SamplerStateInitializer);
	}

	virtual uint32 GetSizeX() const override { return 1; }
	virtual uint32 GetSizeY() const override { return 1; }
};

/** Global black volume texture resource. */
FTexture* GBlackVolumeTexture = new TGlobalResource<FBlackVolumeTexture<PF_B8G8R8A8, 0>, FRenderResource::EInitPhase::Pre>();
FTexture* GBlackAlpha1VolumeTexture = new TGlobalResource<FBlackVolumeTexture<PF_B8G8R8A8, 255>, FRenderResource::EInitPhase::Pre>();

/** Global black volume texture resource. */
FTexture* GBlackUintVolumeTexture = new TGlobalResource<FBlackVolumeTexture<PF_R8G8B8A8_UINT, 0>, FRenderResource::EInitPhase::Pre>();

class FBlackArrayTexture : public FTexture
{
public:
	virtual void InitRHI(FRHICommandListBase& RHICmdList) override
	{
		const FRHITextureCreateDesc Desc =
			FRHITextureCreateDesc::Create2DArray(TEXT("BlackArrayTexture"), 1, 1, 1, PF_B8G8R8A8)
			.SetFlags(ETextureCreateFlags::ShaderResource)
			.SetClassName(TEXT("FBlackArrayTexture"))
			.SetInitActionInitializer();

		FRHITextureInitializer Initializer = RHICmdList.CreateTextureInitializer(Desc);

		const FColor ColorValue(0, 0, 0, 0);
		Initializer.GetTexture2DSubresource(0).WriteColor(ColorValue);

		// Create the texture RHI.
		TextureRHI = Initializer.Finalize();

		// Create the sampler state RHI resource.
		FSamplerStateInitializerRHI SamplerStateInitializer(SF_Point, AM_Wrap, AM_Wrap, AM_Wrap);
		SamplerStateRHI = GetOrCreateSamplerState(SamplerStateInitializer);
	}

	virtual uint32 GetSizeX() const override { return 1; }
	virtual uint32 GetSizeY() const override { return 1; }
};

FTexture* GBlackArrayTexture = new TGlobalResource<FBlackArrayTexture, FRenderResource::EInitPhase::Pre>;

//
// FMipColorTexture implementation
//

/**
 * A texture that has a different solid color in each mip-level
 */
class FMipColorTexture : public FTexture
{
public:
	enum
	{
		NumMips = 12
	};
	static const FColor MipColors[NumMips];

	// FResource interface.
	virtual void InitRHI(FRHICommandListBase& RHICmdList) override
	{
		// Create the texture RHI.
		int32 TextureSize = 1 << (NumMips - 1);

		const FRHITextureCreateDesc Desc =
			FRHITextureCreateDesc::Create2D(TEXT("FMipColorTexture"), TextureSize, TextureSize, PF_B8G8R8A8)
			.SetNumMips(NumMips)
			.SetFlags(ETextureCreateFlags::ShaderResource)
			.SetClassName(TEXT("FMipColorTexture"))
			.SetInitActionInitializer();

		FRHITextureInitializer Initializer = RHICmdList.CreateTextureInitializer(Desc);

		// Write the contents of the texture.
		int32 Size = TextureSize;
		for (int32 MipIndex = 0; MipIndex < NumMips; ++MipIndex)
		{
			const FRHITextureSubresourceInitializer Subresource = Initializer.GetTexture2DSubresource(MipIndex);

			FColor* DestBuffer = reinterpret_cast<FColor*>(Subresource.Data);
			uint64 DestBufferOffset = 0;

			for (int32 Y = 0; Y < Size; ++Y)
			{
				for (int32 X = 0; X < Size; ++X)
				{
					DestBuffer[X + DestBufferOffset] = MipColors[NumMips - 1 - MipIndex];
				}
				DestBufferOffset += Subresource.Stride / sizeof(FColor);
				check(DestBufferOffset <= Subresource.Size);
			}

			Size >>= 1;
		}

		TextureRHI = Initializer.Finalize();

		// Create the sampler state RHI resource.
		FSamplerStateInitializerRHI SamplerStateInitializer(SF_Point, AM_Wrap, AM_Wrap, AM_Wrap);
		SamplerStateRHI = GetOrCreateSamplerState(SamplerStateInitializer);
	}

	/** Returns the width of the texture in pixels. */
	virtual uint32 GetSizeX() const override
	{
		int32 TextureSize = 1 << (NumMips - 1);
		return TextureSize;
	}

	/** Returns the height of the texture in pixels. */
	// PVS-Studio notices that the implementation of GetSizeX is identical to this one
	// and warns us. In this case, it is intentional, so we disable the warning:
	virtual uint32 GetSizeY() const override //-V524
	{
		int32 TextureSize = 1 << (NumMips - 1);
		return TextureSize;
	}
};

const FColor FMipColorTexture::MipColors[NumMips] =
{
	FColor(80,  80,  80, 0),		// Mip  0: 1x1			(dark grey)
	FColor(200, 200, 200, 0),		// Mip  1: 2x2			(light grey)
	FColor(200, 200,   0, 0),		// Mip  2: 4x4			(medium yellow)
	FColor(255, 255,   0, 0),		// Mip  3: 8x8			(yellow)
	FColor(160, 255,  40, 0),		// Mip  4: 16x16		(light green)
	FColor(0, 255,   0, 0),		// Mip  5: 32x32		(green)
	FColor(0, 255, 200, 0),		// Mip  6: 64x64		(cyan)
	FColor(0, 170, 170, 0),		// Mip  7: 128x128		(light blue)
	FColor(60,  60, 255, 0),		// Mip  8: 256x256		(dark blue)
	FColor(255,   0, 255, 0),		// Mip  9: 512x512		(pink)
	FColor(255,   0,   0, 0),		// Mip 10: 1024x1024	(red)
	FColor(255, 130,   0, 0),		// Mip 11: 2048x2048	(orange)
};

FTexture* GMipColorTexture = new FMipColorTexture;
int32 GMipColorTextureMipLevels = FMipColorTexture::NumMips;

// 4: 8x8 cubemap resolution, shader needs to use the same value as preprocessing
const uint32 GDiffuseConvolveMipLevel = 4;

/** A solid color cube texture. */
class FSolidColorTextureCube : public FTexture
{
public:
	FSolidColorTextureCube(const FColor& InColor)
		: bInitToZero(false)
		, PixelFormat(PF_B8G8R8A8)
		, ColorData(InColor.DWColor())
	{}

	FSolidColorTextureCube(EPixelFormat InPixelFormat)
		: bInitToZero(true)
		, PixelFormat(InPixelFormat)
		, ColorData(0)
	{}

	// FRenderResource interface.
	virtual void InitRHI(FRHICommandListBase& RHICmdList) override
	{
		// Create the texture RHI.
		const FRHITextureCreateDesc CreateDesc =
			FRHITextureCreateDesc::CreateCube(TEXT("SolidColorCube"), 1, PixelFormat)
			.SetFlags(ETextureCreateFlags::ShaderResource)
			.SetClassName(TEXT("FSolidColorTextureCube"))
			.SetInitialState(ERHIAccess::SRVMask)
			.SetInitActionInitializer();

		FRHITextureInitializer Initializer = RHICmdList.CreateTextureInitializer(CreateDesc);

		for (uint32 FaceIndex = 0; FaceIndex < 6; FaceIndex++)
		{
			FRHITextureSubresourceInitializer Subresource = Initializer.GetTextureCubeSubresource(FaceIndex, 0);
			if (bInitToZero)
			{
				FMemory::Memzero(Subresource.Data, GPixelFormats[PixelFormat].BlockBytes);
			}
			else
			{
				FMemory::Memcpy(Subresource.Data, &ColorData, sizeof(ColorData));
			}
		}

		TextureRHI = Initializer.Finalize();

		// Create the sampler state RHI resource.
		FSamplerStateInitializerRHI SamplerStateInitializer(SF_Point, AM_Wrap, AM_Wrap, AM_Wrap);
		SamplerStateRHI = GetOrCreateSamplerState(SamplerStateInitializer);
	}

	virtual uint32 GetSizeX() const override { return 1; }
	virtual uint32 GetSizeY() const override { return 1; }

private:
	const bool bInitToZero;
	const EPixelFormat PixelFormat;
	const uint32 ColorData;
};

/** A white cube texture. */
FTexture* GWhiteTextureCube = new TGlobalResource<FSolidColorTextureCube, FRenderResource::EInitPhase::Pre>(FColor::White);

/** A black cube texture. */
FTexture* GBlackTextureCube = new TGlobalResource<FSolidColorTextureCube, FRenderResource::EInitPhase::Pre>(FColor::Black);

/** A black cube texture. */
FTexture* GBlackTextureDepthCube = new TGlobalResource<FSolidColorTextureCube, FRenderResource::EInitPhase::Pre>(PF_ShadowDepth);

class FBlackCubeArrayTexture : public FTexture
{
public:
	// FResource interface.
	virtual void InitRHI(FRHICommandListBase& RHICmdList) override
	{
		const FRHITextureCreateDesc CreateDesc =
			FRHITextureCreateDesc::CreateCubeArray(TEXT("BlackCubeArray"), 1, 1, PF_B8G8R8A8)
			.SetFlags(ETextureCreateFlags::ShaderResource)
			.SetClassName(TEXT("FBlackCubeArrayTexture"))
			.SetInitialState(ERHIAccess::SRVMask)
			.SetInitActionInitializer();

		FRHITextureInitializer Initializer = RHICmdList.CreateTextureInitializer(CreateDesc);

		for (uint32 FaceIndex = 0; FaceIndex < 6; FaceIndex++)
		{
			FRHITextureSubresourceInitializer Subresource = Initializer.GetTextureCubeSubresource(FaceIndex, 0);

			// Note: alpha is used by reflection environment to say how much of the foreground texture is visible, so 0 says it is completely invisible
			Subresource.WriteColor(FColor(0, 0, 0, 0));
		}

		// Create the texture RHI.
		TextureRHI = Initializer.Finalize();

		// Create the sampler state RHI resource.
		FSamplerStateInitializerRHI SamplerStateInitializer(SF_Point, AM_Wrap, AM_Wrap, AM_Wrap);
		SamplerStateRHI = GetOrCreateSamplerState(SamplerStateInitializer);
	}

	virtual uint32 GetSizeX() const override { return 1; }
	virtual uint32 GetSizeY() const override { return 1; }
};
FTexture* GBlackCubeArrayTexture = new TGlobalResource<FBlackCubeArrayTexture, FRenderResource::EInitPhase::Pre>;

/**
 * A UINT 1x1 texture.
 */
class FBlackUintTexture : public FTextureWithSRV
{
public:
	// FResource interface.
	virtual void InitRHI(FRHICommandListBase& RHICmdList) override
	{
		const FRHITextureCreateDesc CreateDesc =
			FRHITextureCreateDesc::Create2D(TEXT("BlackUintTexture"), 1, 1, PF_R32G32B32A32_UINT)
			.SetFlags(ETextureCreateFlags::ShaderResource)
			.SetClassName(TEXT("FBlackUintTexture"))
			.SetInitActionInitializer();

		FRHITextureInitializer Initializer = RHICmdList.CreateTextureInitializer(CreateDesc);

		FRHITextureSubresourceInitializer Subresource = Initializer.GetTextureCubeSubresource(0, 0);
		FMemory::Memzero(Subresource.Data, Subresource.Size);

		TextureRHI = Initializer.Finalize();

		// Create the sampler state RHI resource.
		FSamplerStateInitializerRHI SamplerStateInitializer(SF_Point, AM_Wrap, AM_Wrap, AM_Wrap);
		SamplerStateRHI = GetOrCreateSamplerState(SamplerStateInitializer);

		// Create a view of the texture
		ShaderResourceViewRHI = RHICmdList.CreateShaderResourceView(TextureRHI, FRHIViewDesc::CreateTextureSRV().SetDimensionFromTexture(TextureRHI));
	}

	virtual uint32 GetSizeX() const override { return 1; }
	virtual uint32 GetSizeY() const override { return 1; }
};

FTexture* GBlackUintTexture = new TGlobalResource<FBlackUintTexture, FRenderResource::EInitPhase::Pre>;


//////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// FNullColorVertexBuffer

FNullColorVertexBuffer::FNullColorVertexBuffer() = default;
FNullColorVertexBuffer::~FNullColorVertexBuffer() = default;

void FNullColorVertexBuffer::InitRHI(FRHICommandListBase& RHICmdList)
{
	// create a static vertex buffer
	const FRHIBufferCreateDesc CreateDesc =
		FRHIBufferCreateDesc::CreateVertex<uint32>(TEXT("FNullColorVertexBuffer"), 4)
		.AddUsage(EBufferUsageFlags::Static | EBufferUsageFlags::ShaderResource)
		.SetInitialState(ERHIAccess::VertexOrIndexBuffer | ERHIAccess::SRVMask);

	const uint32 DWWhite = FColor(255, 255, 255, 255).DWColor();
	const uint32 ColorValues[4] = { DWWhite, DWWhite, DWWhite, DWWhite };

	VertexBufferRHI = UE::RHIResourceUtils::CreateBufferWithArray(RHICmdList, CreateDesc, ColorValues);
	VertexBufferSRV = RHICmdList.CreateShaderResourceView(
		VertexBufferRHI,
		FRHIViewDesc::CreateBufferSRV()
			.SetType(FRHIViewDesc::EBufferType::Typed)
			.SetFormat(PF_R8G8B8A8));
}

void FNullColorVertexBuffer::ReleaseRHI()
{
	VertexBufferSRV.SafeRelease();
	FVertexBuffer::ReleaseRHI();
}

/** The global null color vertex buffer, which is set with a stride of 0 on meshes without a color component. */
TGlobalResource<FNullColorVertexBuffer, FRenderResource::EInitPhase::Pre> GNullColorVertexBuffer;

//////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// FNullVertexBuffer

FNullVertexBuffer::FNullVertexBuffer() = default;
FNullVertexBuffer::~FNullVertexBuffer() = default;

void FNullVertexBuffer::InitRHI(FRHICommandListBase& RHICmdList)
{
	// create a static vertex buffer
	const FRHIBufferCreateDesc CreateDesc =
		FRHIBufferCreateDesc::CreateVertex<FVector3f>(TEXT("FNullVertexBuffer"), 1)
		.AddUsage(EBufferUsageFlags::Static | EBufferUsageFlags::ShaderResource)
		.SetInitialState(ERHIAccess::VertexOrIndexBuffer | ERHIAccess::SRVMask);

	const FVector3f InitialValue(0.0f);

	VertexBufferRHI = UE::RHIResourceUtils::CreateBufferWithValue(RHICmdList, CreateDesc, InitialValue);
	VertexBufferSRV = RHICmdList.CreateShaderResourceView(
		VertexBufferRHI,
		FRHIViewDesc::CreateBufferSRV()
			.SetType(FRHIViewDesc::EBufferType::Typed)
			.SetFormat(PF_R8G8B8A8));
}

void FNullVertexBuffer::ReleaseRHI()
{
	VertexBufferSRV.SafeRelease();
	FVertexBuffer::ReleaseRHI();
}

/** The global null vertex buffer, which is set with a stride of 0 on meshes */
TGlobalResource<FNullVertexBuffer, FRenderResource::EInitPhase::Pre> GNullVertexBuffer;

//////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// FScreenSpaceVertexBuffer

static const FVector2f GScreenSpaceVertexBufferData[4] =
{
	FVector2f(-1,-1),
	FVector2f(-1,+1),
	FVector2f(+1,-1),
	FVector2f(+1,+1),
};

void FScreenSpaceVertexBuffer::InitRHI(FRHICommandListBase& RHICmdList)
{
	// create a static vertex buffer
	VertexBufferRHI = UE::RHIResourceUtils::CreateVertexBufferFromArray(RHICmdList, TEXT("FScreenSpaceVertexBuffer"), EBufferUsageFlags::Static, MakeConstArrayView(GScreenSpaceVertexBufferData));
}

TGlobalResource<FScreenSpaceVertexBuffer, FRenderResource::EInitPhase::Pre> GScreenSpaceVertexBuffer;

//////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// FTileVertexDeclaration

FTileVertexDeclaration::FTileVertexDeclaration() = default;
FTileVertexDeclaration::~FTileVertexDeclaration() = default;

void FTileVertexDeclaration::InitRHI(FRHICommandListBase& RHICmdList)
{
	FVertexDeclarationElementList Elements;
	uint16 Stride = sizeof(FVector2f);
	Elements.Add(FVertexElement(0, 0, VET_Float2, 0, Stride, false));
	VertexDeclarationRHI = RHICreateVertexDeclaration(Elements);
}

void FTileVertexDeclaration::ReleaseRHI()
{
	VertexDeclarationRHI.SafeRelease();
}

TGlobalResource<FTileVertexDeclaration, FRenderResource::EInitPhase::Pre> GTileVertexDeclaration;

//////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// FCubeIndexBuffer

void FCubeIndexBuffer::InitRHI(FRHICommandListBase& RHICmdList)
{
	// create a static index buffer
	IndexBufferRHI = UE::RHIResourceUtils::CreateIndexBufferFromArray(RHICmdList, TEXT("FCubeIndexBuffer"), EBufferUsageFlags::Static, MakeConstArrayView(GCubeIndices));
}

TGlobalResource<FCubeIndexBuffer, FRenderResource::EInitPhase::Pre> GCubeIndexBuffer;

//////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// FTwoTrianglesIndexBuffer

static const uint16 GTwoTrianglesIndexBufferData[6] = { 0, 1, 3, 0, 3, 2 };

void FTwoTrianglesIndexBuffer::InitRHI(FRHICommandListBase& RHICmdList)
{
	// create a static index buffer
	IndexBufferRHI = UE::RHIResourceUtils::CreateIndexBufferFromArray(RHICmdList, TEXT("FTwoTrianglesIndexBuffer"), EBufferUsageFlags::Static, MakeConstArrayView(GTwoTrianglesIndexBufferData));
}

TGlobalResource<FTwoTrianglesIndexBuffer, FRenderResource::EInitPhase::Pre> GTwoTrianglesIndexBuffer;

void FEmptyResourceCollection::InitRHI(FRHICommandListBase& RHICmdList)
{
	ResourceCollection = RHICmdList.CreateResourceCollection({});
}

void FEmptyResourceCollection::ReleaseRHI()
{
	ResourceCollection.SafeRelease();
}

TGlobalResource<FEmptyResourceCollection, FRenderResource::EInitPhase::Pre> GEmptyResourceCollection;

/*------------------------------------------------------------------------------
	FGlobalDynamicVertexBuffer implementation.
------------------------------------------------------------------------------*/

/**
 * An individual dynamic vertex buffer.
 */
template <typename BufferType>
class TDynamicBuffer final : public BufferType
{
public:
	/** The aligned size of all dynamic vertex buffers. */
	enum { ALIGNMENT = (1 << 16) }; // 64KB
	/** Pointer to the vertex buffer mapped in main memory. */
	uint8* MappedBuffer;
	/** Size of the vertex buffer in bytes. */
	uint32 BufferSize;
	/** Number of bytes currently allocated from the buffer. */
	uint32 AllocatedByteCount;
	/** Stride of the buffer in bytes. */
	uint32 Stride;
	/** Last render thread frame this resource was used in. */
	uint64 LastUsedFrame = 0;

	/** Default constructor. */
	explicit TDynamicBuffer(uint32 InMinBufferSize, uint32 InStride)
		: MappedBuffer(NULL)
		, BufferSize(FMath::Max<uint32>(Align(InMinBufferSize, ALIGNMENT), ALIGNMENT))
		, AllocatedByteCount(0)
		, Stride(InStride)
	{
	}

	/**
	 * Locks the vertex buffer so it may be written to.
	 */
	void Lock(FRHICommandListBase& RHICmdList)
	{
		check(MappedBuffer == NULL);
		check(AllocatedByteCount == 0);
		check(IsValidRef(BufferType::GetRHI()));
		MappedBuffer = (uint8*)RHICmdList.LockBuffer(BufferType::GetRHI(), 0, BufferSize, RLM_WriteOnly);
	}

	/**
	 * Unocks the buffer so the GPU may read from it.
	 */
	void Unlock(FRHICommandListBase& RHICmdList)
	{
		check(MappedBuffer != NULL);
		check(IsValidRef(BufferType::GetRHI()));
		RHICmdList.UnlockBuffer(BufferType::GetRHI());
		MappedBuffer = NULL;
		AllocatedByteCount = 0;
	}

	// FRenderResource interface.
	virtual void InitRHI(FRHICommandListBase& RHICmdList) override
	{
		check(!IsValidRef(BufferType::GetRHI()));

		const FRHIBufferCreateDesc CreateDesc =
			FRHIBufferCreateDesc::Create(TEXT("FDynamicBuffer"), BufferSize, Stride, Stride == 0 ? EBufferUsageFlags::VertexBuffer : EBufferUsageFlags::IndexBuffer)
			.AddUsage(EBufferUsageFlags::ShaderResource)
			.SetInitialState(ERHIAccess::VertexOrIndexBuffer | ERHIAccess::SRVCompute);

		BufferType::SetRHI(RHICmdList.CreateBuffer(CreateDesc));
		MappedBuffer = NULL;
		AllocatedByteCount = 0;
	}

	virtual void ReleaseRHI() override
	{
		BufferType::ReleaseRHI();
		MappedBuffer = NULL;
		AllocatedByteCount = 0;
	}
};

/**
 * A pool of dynamic buffers.
 */
template <typename DynamicBufferType>
struct TDynamicBufferPool : public FRenderResource
{
	UE::FMutex Mutex;
	TArray<DynamicBufferType*> LiveList;
	TArray<DynamicBufferType*> FreeList;
	TArray<DynamicBufferType*> LockList;
	TArray<DynamicBufferType*> ReclaimList;
	std::atomic_uint32_t TotalAllocatedMemory{0};
	uint32 CurrentCycle = 0;

	DynamicBufferType* Acquire(FRHICommandListBase& RHICmdList, uint32 SizeInBytes, uint32 Stride)
	{
		const uint32 MinimumBufferSize = 65536u;
		SizeInBytes = FMath::Max(SizeInBytes, MinimumBufferSize);

		DynamicBufferType* FoundBuffer = nullptr;
		bool bInitializeBuffer = false;

		{
			UE::TScopeLock Lock(Mutex);

			// Traverse the free list like a stack, starting from the top, so recently reclaimed items are allocated first.
			for (int32 Index = FreeList.Num() - 1; Index >= 0; --Index)
			{
				DynamicBufferType* Buffer = FreeList[Index];

				if (SizeInBytes <= Buffer->BufferSize && Buffer->Stride == Stride)
				{
					FreeList.RemoveAt(Index, EAllowShrinking::No);
					FoundBuffer = Buffer;
					break;
				}
			}

			if (!FoundBuffer)
			{
				FoundBuffer = new DynamicBufferType(SizeInBytes, Stride);
				LiveList.Emplace(FoundBuffer);
				bInitializeBuffer = true;
			}

			check(FoundBuffer);
		}

		if (IsRenderAlarmLoggingEnabled())
		{
			UE_LOGF(LogRendererCore, Warning, "FGlobalDynamicVertexBuffer::Allocate(%u), will have allocated %u total this frame", SizeInBytes, TotalAllocatedMemory.load());
		}

		if (bInitializeBuffer)
		{
			FoundBuffer->InitResource(RHICmdList);
			TotalAllocatedMemory += FoundBuffer->BufferSize;
		}

		FoundBuffer->Lock(RHICmdList);
		FoundBuffer->LastUsedFrame = GFrameCounterRenderThread;

		return FoundBuffer;
	}

	void Forfeit(FRHICommandListBase& RHICmdList, TConstArrayView<DynamicBufferType*> BuffersToForfeit)
	{
		if (!BuffersToForfeit.IsEmpty())
		{
			for (DynamicBufferType* Buffer : BuffersToForfeit)
			{
				Buffer->Unlock(RHICmdList);
			}

			UE::TScopeLock Lock(Mutex);
			ReclaimList.Append(BuffersToForfeit);
		}
	}

	void GarbageCollect()
	{
		UE::TScopeLock Lock(Mutex);
		FreeList.Append(ReclaimList);
		ReclaimList.Reset();

		for (int32 Index = 0; Index < LiveList.Num(); ++Index)
		{
			DynamicBufferType* Buffer = LiveList[Index];

			if (GGlobalBufferNumFramesUnusedThreshold > 0 && Buffer->LastUsedFrame + GGlobalBufferNumFramesUnusedThreshold <= GFrameCounterRenderThread)
			{
				TotalAllocatedMemory -= Buffer->BufferSize;
				Buffer->ReleaseResource();
				LiveList.RemoveAt(Index, EAllowShrinking::No);
				FreeList.Remove(Buffer);
				delete Buffer;
			}
		}
	}

	bool IsRenderAlarmLoggingEnabled() const
	{
		return GMaxVertexBytesAllocatedPerFrame > 0 && TotalAllocatedMemory >= (size_t)GMaxVertexBytesAllocatedPerFrame;
	}

private:
	void ReleaseRHI() override
	{
		check(LockList.IsEmpty());
		check(FreeList.Num() == LiveList.Num());

		for (DynamicBufferType* Buffer : LiveList)
		{
			TotalAllocatedMemory -= Buffer->BufferSize;
			Buffer->ReleaseResource();
			delete Buffer;
		}
		LiveList.Empty();
		FreeList.Empty();
	}
};

static TGlobalResource<TDynamicBufferPool<FDynamicVertexBuffer>, FRenderResource::EInitPhase::Pre> GDynamicVertexBufferPool;

FGlobalDynamicVertexBuffer::FAllocation FGlobalDynamicVertexBuffer::Allocate(uint32 SizeInBytes)
{
	checkf(RHICmdList, TEXT("FGlobalDynamicVertexBuffer was not initialized prior to calling Allocate."));

	FAllocation Allocation;

	if (VertexBuffers.IsEmpty() || VertexBuffers.Last()->AllocatedByteCount + SizeInBytes > VertexBuffers.Last()->BufferSize)
	{
		VertexBuffers.Emplace(GDynamicVertexBufferPool.Acquire(*RHICmdList, SizeInBytes, 0));
	}

	FDynamicVertexBuffer* VertexBuffer = VertexBuffers.Last();

	checkf(VertexBuffer->AllocatedByteCount + SizeInBytes <= VertexBuffer->BufferSize, TEXT("Global vertex buffer allocation failed: BufferSize=%d AllocatedByteCount=%d SizeInBytes=%d"), VertexBuffer->BufferSize, VertexBuffer->AllocatedByteCount, SizeInBytes);
	Allocation.Buffer = VertexBuffer->MappedBuffer + VertexBuffer->AllocatedByteCount;
	Allocation.VertexBuffer = VertexBuffer;
	Allocation.VertexOffset = VertexBuffer->AllocatedByteCount;
	VertexBuffer->AllocatedByteCount += Align(SizeInBytes, RHI_RAW_VIEW_ALIGNMENT);
	return Allocation;
}

bool FGlobalDynamicVertexBuffer::IsRenderAlarmLoggingEnabled() const
{
	return GDynamicVertexBufferPool.IsRenderAlarmLoggingEnabled();
}

void FGlobalDynamicVertexBuffer::Commit()
{
	if (RHICmdList)
	{
		GDynamicVertexBufferPool.Forfeit(*RHICmdList, VertexBuffers);
		VertexBuffers.Reset();
	}
}

static TGlobalResource<TDynamicBufferPool<FDynamicIndexBuffer>, FRenderResource::EInitPhase::Pre> GDynamicIndexBufferPool;

FGlobalDynamicIndexBuffer::FAllocation FGlobalDynamicIndexBuffer::Allocate(uint32 NumIndices, uint32 IndexStride)
{
	checkf(RHICmdList, TEXT("FGlobalDynamicIndexBuffer was not initialized prior to calling Allocate."));

	FAllocation Allocation;

	if (IndexStride != 2 && IndexStride != 4)
	{
		return Allocation;
	}

	const uint32 SizeInBytes = NumIndices * IndexStride;

	TArray<FDynamicIndexBuffer*>& IndexBuffers = (IndexStride == 2)
		? IndexBuffers16
		: IndexBuffers32;

	if (IndexBuffers.IsEmpty() || IndexBuffers.Last()->AllocatedByteCount + SizeInBytes > IndexBuffers.Last()->BufferSize)
	{
		IndexBuffers.Emplace(GDynamicIndexBufferPool.Acquire(*RHICmdList, SizeInBytes, IndexStride));
	}

	FDynamicIndexBuffer* IndexBuffer = IndexBuffers.Last();

	checkf(IndexBuffer->AllocatedByteCount + SizeInBytes <= IndexBuffer->BufferSize, TEXT("Global index buffer allocation failed: BufferSize=%d BufferStride=%d AllocatedByteCount=%d SizeInBytes=%d"), IndexBuffer->BufferSize, IndexBuffer->Stride, IndexBuffer->AllocatedByteCount, SizeInBytes);
	Allocation.Buffer = IndexBuffer->MappedBuffer + IndexBuffer->AllocatedByteCount;
	Allocation.IndexBuffer = IndexBuffer;
	Allocation.FirstIndex = IndexBuffer->AllocatedByteCount / IndexStride;
	IndexBuffer->AllocatedByteCount += Align(SizeInBytes, RHI_RAW_VIEW_ALIGNMENT);
	return Allocation;
}

void FGlobalDynamicIndexBuffer::Commit()
{
	if (RHICmdList)
	{
		GDynamicIndexBufferPool.Forfeit(*RHICmdList, IndexBuffers16);
		GDynamicIndexBufferPool.Forfeit(*RHICmdList, IndexBuffers32);
		IndexBuffers16.Reset();
		IndexBuffers32.Reset();
	}
}

namespace GlobalDynamicBuffer
{
	void GarbageCollect()
	{
		GDynamicVertexBufferPool.GarbageCollect();
		GDynamicIndexBufferPool.GarbageCollect();
	}
}