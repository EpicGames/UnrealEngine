// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	GlobalShader.cpp: Global shader implementation.
=============================================================================*/

#include "GlobalShader.h"

#include "Containers/StaticBitArray.h"
#include "Interfaces/ITargetPlatform.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/CoreMisc.h"
#include "Misc/Paths.h"
#include "Serialization/MemoryWriter.h"
#include "ShaderCodeLibrary.h"
#include "ShaderSerialization.h"
#include "Tasks/Task.h"

/** The global shader map. */
FGlobalShaderMap* GGlobalShaderMap[SP_NumPlatforms] = {};

IMPLEMENT_TYPE_LAYOUT(FGlobalShader);
IMPLEMENT_TYPE_LAYOUT(FGlobalShaderMapContent);

IMPLEMENT_SHADER_TYPE(,FNULLPS,TEXT("/Engine/Private/NullPixelShader.usf"),TEXT("Main"),SF_Pixel);

/** A per-platform map of global shader defines that come from config .ini */
class FGlobalShaderConfigDefines
{
public:

	static void ApplyConfigDefines(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FName ShaderFormat;
		if (const FPlatformInfo* Platform = GetPlatformInfoAndErrorCheck(Parameters.Platform, ShaderFormat))
		{
			if (const TArray<FDefine>* ShaderDefines = Platform->ShaderDefines.Find(Parameters.GlobalShaderName))
			{			
				for (const FDefine& Define : *ShaderDefines)
				{
					if (Define.IsRelevant(ShaderFormat, Parameters.PermutationId))
					{
						OutEnvironment.SetDefineAndCompileArgument(*Define.Name, Define.Value);
					}
				}
			}
		}
	}

	static void AppendKeyString(FString& KeyString, FName ShaderName, int32 PermutationID, FName IniPlatform, EShaderPlatform ShaderPlatform)
	{
		FName ShaderFormat;
		if (const FPlatformInfo* Platform = GetPlatformInfoAndErrorCheck(ShaderPlatform, ShaderFormat))
		{
			if (const TArray<FDefine>* ShaderDefines = Platform->ShaderDefines.Find(ShaderName))
			{
				TArray<FDefine> FilteredDefines;
				for (const FDefine& Define : *ShaderDefines)
				{
					if (Define.IsRelevant(ShaderFormat, PermutationID))
					{
						FilteredDefines.Add(Define);
					}
				}

				if (FilteredDefines.Num() > 0)
				{
					// Sort and hash the applicable defines
					FilteredDefines.Sort();

					FSHA1 HashState;
					for (const FDefine& Define : FilteredDefines)
					{
						HashState.UpdateWithString(*Define.Name, Define.Name.Len());
						HashState.UpdateWithString(*Define.Value, Define.Value.Len());

						const uint32 ShaderFormatHash = GetTypeHash(Define.ShaderFormat);
						HashState.Update((const uint8*)&ShaderFormatHash, sizeof(ShaderFormatHash));

						HashState.Update((const uint8*)&Define.PermutationID, sizeof(Define.PermutationID));
					}

					KeyString += HashState.Finalize().ToString();
				}
			}
		}
	}

private:
	struct FDefine
	{
		FString Name;
		FString Value;
		FName ShaderFormat = NAME_None;
		int32 PermutationID = INDEX_NONE;

		bool IsRelevant(FName InShaderFormat, int32 InPermutationID) const
		{
			if (ShaderFormat != NAME_None && ShaderFormat != InShaderFormat)
			{
				// This define is inappropriate for the target shader format
				return false;
			}

			if (PermutationID != INDEX_NONE && PermutationID != InPermutationID)
			{
				// This define is inappropriate for the permutation being compiled
				return false;
			}

			return true;
		}

		int32 Compare(const FDefine& Other) const
		{
			int Cmp = Name.Compare(Other.Name, ESearchCase::IgnoreCase);
			if (Cmp == 0)
			{
				Cmp = Value.Compare(Other.Value, ESearchCase::CaseSensitive);
				if (Cmp == 0)
				{
					Cmp = ShaderFormat.Compare(Other.ShaderFormat);
					if (Cmp == 0)
					{
						Cmp = Other.PermutationID - PermutationID;
					}
				}
			}

			return Cmp;
		}

		bool operator==(const FDefine& Other) const
		{
			return Name.Equals(Other.Name, ESearchCase::IgnoreCase)
				&& Value == Other.Value
				&& ShaderFormat == Other.ShaderFormat
				&& PermutationID == Other.PermutationID;
		}

		bool operator!=(const FDefine& Other) const
		{
			return !(*this == Other);
		}

		bool operator<(const FDefine& Other) const
		{
			return Compare(Other) < 0;
		}
	};

	using FShaderDefines = TMap<FName, TArray<FDefine>>;

	struct FPlatformInfo
	{
		FName Name;
		bool bInitialized = false;
		FShaderDefines ShaderDefines;
	};
	
	static TArray<FPlatformInfo> ConfigDefines;
	static TStaticArray<const FPlatformInfo*, EShaderPlatform::SP_NumPlatforms> PerPlatformConfigs;
	static TStaticBitArray<EShaderPlatform::SP_NumPlatforms> ErrorCheckedPlatforms;

	static const FPlatformInfo* GetPlatformInfoAndErrorCheck(EShaderPlatform ShaderPlatform, FName& OutShaderFormat)
	{
		OutShaderFormat = LegacyShaderPlatformToShaderFormat(ShaderPlatform);
		
		const TArray<ITargetPlatform*> AllPlatforms = GetTargetPlatformManagerRef().GetTargetPlatforms();
		if (AllPlatforms.Num() != ConfigDefines.Num())
		{
			// if the number of platforms has changed since this was last called, we need to resize the array 
			// and reset PerPlatformConfigs since the pointers will now be invalid.
			ConfigDefines.SetNum(AllPlatforms.Num());
			for (uint16 SpIndex = 0; SpIndex < EShaderPlatform::SP_NumPlatforms; ++SpIndex)
			{
				PerPlatformConfigs[SpIndex] = nullptr;
			}
		}

		if (PerPlatformConfigs[ShaderPlatform] != nullptr)
		{
			return PerPlatformConfigs[ShaderPlatform];
		}

		// Search for all target platforms that support this shader platform
		TArray<const ITargetPlatform*, TInlineAllocator<16>> IniPlatforms;
		for(const ITargetPlatform* TP : AllPlatforms)
		{
			check(TP);
			TArray<FName> PlatformShaderFormats;
			TP->GetAllPossibleShaderFormats(PlatformShaderFormats);
			if (PlatformShaderFormats.Contains(OutShaderFormat))
			{
				IniPlatforms.AddUnique(TP);
			}
		}
		
		if (IniPlatforms.Num() == 0)
		{
			// No found platforms that support this shader format
			return nullptr;
		}
		
		// first add all platforms to populate the array
		for (int32 PlatformIndex = 0; PlatformIndex < IniPlatforms.Num(); ++PlatformIndex)
		{
			int32 PlatformOrdinal = IniPlatforms[PlatformIndex]->GetPlatformOrdinal();
			if (ConfigDefines.Num() < PlatformOrdinal + 1)
			{
				ConfigDefines.SetNum(PlatformOrdinal + 1);
			}
			FPlatformInfo& Platform = ConfigDefines[PlatformOrdinal];

			if (!Platform.bInitialized)
			{
				InitializePlatform(Platform, IniPlatforms[PlatformIndex]);
				check(Platform.bInitialized);
			}
		}

		if (!ErrorCheckedPlatforms[ShaderPlatform])
		{
			ErrorCheckedPlatforms[ShaderPlatform] = true;

			if (IniPlatforms.Num() > 1)
			{
				// This shader platform is shared by multiple target platforms that can be configured independently. We need to make sure all config defines
				// match up, and that no platform-specific defines exist that might introduce shader compiler output that diverges between target platforms
				const FPlatformInfo& Platform0 = ConfigDefines[IniPlatforms[0]->GetPlatformOrdinal()];
				check(Platform0.bInitialized);

				for (int32 PlatformIndex = 1; PlatformIndex < IniPlatforms.Num(); ++PlatformIndex)
				{
					const FPlatformInfo& OtherPlatform = ConfigDefines[IniPlatforms[PlatformIndex]->GetPlatformOrdinal()];
					check(OtherPlatform.bInitialized);

					ErrorCheckPlatformsForShaderFormat(Platform0, OtherPlatform, OutShaderFormat);
				}
			}
		}
		
		const FPlatformInfo& Platform = ConfigDefines[IniPlatforms[0]->GetPlatformOrdinal()];
		PerPlatformConfigs[ShaderPlatform] = &Platform;
		return &Platform;
	}

	static void InitializePlatform(FPlatformInfo& Platform, const ITargetPlatform* TP)
	{
		Platform.Name = FName(TP->IniPlatformName());
		if (FConfigCacheIni* ConfigCache = FConfigCacheIni::ForPlatform(Platform.Name))
		{
			TArray<FString> DefineStrings;
			ConfigCache->GetArray(TEXT("GlobalShaderDefines"), TEXT("Definitions"), DefineStrings, GEngineIni);

			for (const FString& DefineString : DefineStrings)
			{				
				FString ShaderName, DefineName;
				if (FParse::Value(*DefineString, TEXT("Shader="), ShaderName) &&
					FParse::Value(*DefineString, TEXT("Define="), DefineName))
				{
					FName ShaderFormat = NAME_None;
					FParse::Value(*DefineString, TEXT("ShaderFormat="), ShaderFormat);

					int32 PermutationID = INDEX_NONE;
					FParse::Value(*DefineString, TEXT("PermutationID="), PermutationID);

					FString DefineValue;
					int32 EqualsIndex = INDEX_NONE;
					if (DefineName.FindChar(TCHAR('='), EqualsIndex))
					{
						DefineValue = DefineName.Mid(EqualsIndex + 1);
						DefineName.MidInline(0, EqualsIndex, EAllowShrinking::No);
					}

					const FShaderType* ShaderType = FindShaderTypeByName(ShaderName);
					if (ShaderType == nullptr || ShaderType->GetGlobalShaderType() == nullptr)
					{
						// This global shader doesn't actually exist
						UE_LOGF(LogShaders, Warning, "Global shader definition '%ls' found in engine config for global shader '%ls', which does not exist", *DefineName, *ShaderName);
						continue;
					}

					TArray<FDefine>& ShaderDefines = Platform.ShaderDefines.FindOrAdd(FName(ShaderName));
					FDefine* Define = ShaderDefines.FindByPredicate(
						[&](const FDefine& Define)
						{
							return Define.Name.Equals(DefineName, ESearchCase::IgnoreCase) &&
								Define.ShaderFormat == ShaderFormat &&
								Define.PermutationID == PermutationID;
						});

					if (!Define)
					{
						Define = &ShaderDefines.AddDefaulted_GetRef();
						Define->Name = DefineName;
						Define->ShaderFormat = ShaderFormat;
						Define->PermutationID = PermutationID;
					}

					Define->Value = DefineValue;
				}
			}
		}
		Platform.bInitialized = true;
	}

	static void ErrorCheckPlatformsForShaderFormat(const FPlatformInfo& PlatformA, const FPlatformInfo& PlatformB, FName ShaderFormat)
	{
		auto FilterShaderDefines = [ShaderFormat](const FShaderDefines& ShaderDefines) -> FShaderDefines
		{
			FShaderDefines FilteredShaderDefines;

			for (const auto& ShaderIt : ShaderDefines)
			{
				TArray<FDefine> FilteredDefines = ShaderIt.Value.FilterByPredicate(
					[ShaderFormat](const FDefine& Define)
					{
						return Define.ShaderFormat == NAME_None || Define.ShaderFormat == ShaderFormat;
					});

				if (FilteredDefines.Num() > 0)
				{
					// Sort the filtered defines for every shader for the purposes of comparison in error checking
					FilteredDefines.Sort();
					FilteredShaderDefines.Add(ShaderIt.Key, MoveTemp(FilteredDefines));
				}
			}

			return MoveTemp(FilteredShaderDefines);
		};

		FShaderDefines FilteredDefinesA = FilterShaderDefines(PlatformA.ShaderDefines);
		FShaderDefines FilteredDefinesB = FilterShaderDefines(PlatformB.ShaderDefines);

		TSet<FName> ShadersWithError;

		// Check if there are any keys in A that aren't in B, and vice versa
		{
			TSet<FName> ShaderNamesA, ShaderNamesB;
			FilteredDefinesA.GetKeys(ShaderNamesA);
			FilteredDefinesB.GetKeys(ShaderNamesB);

			ShadersWithError.Append(ShaderNamesA.Difference(ShaderNamesB));
			ShadersWithError.Append(ShaderNamesB.Difference(ShaderNamesA));
		}

		// Check if the defines (that are relevant to this shader format) are different between A or B
		for (const auto& ShaderIt : FilteredDefinesA)
		{
			const TArray<FDefine>& DefinesA = ShaderIt.Value;
			const TArray<FDefine>* DefinesB = FilteredDefinesB.Find(ShaderIt.Key);

			if (DefinesB && DefinesA != *DefinesB)
			{
				ShadersWithError.Add(ShaderIt.Key);
			}
		}

		if (ShadersWithError.Num() > 0)
		{
			FString ShaderListString = *FString::JoinBy(ShadersWithError, TEXT("\n    "), [](const FName& A) { return A.ToString(); });
			UE_LOGF(LogShaders, Fatal, "It has been detected that one or more global shaders are configured differently between platform '%ls' and '%ls' for "
				"shader format '%ls'. This is unsupported, and could result in data mismatches in the Derived Data Cache. Please check that all entries "
				"of '[GlobalShaderDefines]` in the various engine config .ini files are the same for these shaders on all platforms that use this shader "
				"model.\n\nGlobal shaders with errors:\n    %ls\n",
				*PlatformA.Name.ToString(), *PlatformB.Name.ToString(), *ShaderFormat.ToString(), *ShaderListString);
		}
	}
};

TArray<FGlobalShaderConfigDefines::FPlatformInfo> FGlobalShaderConfigDefines::ConfigDefines;
TStaticArray<const FGlobalShaderConfigDefines::FPlatformInfo*, EShaderPlatform::SP_NumPlatforms> FGlobalShaderConfigDefines::PerPlatformConfigs;
TStaticBitArray<EShaderPlatform::SP_NumPlatforms> FGlobalShaderConfigDefines::ErrorCheckedPlatforms;

/** Used to identify the global shader map in compile queues. */
const int32 GlobalShaderMapId = 0;

FGlobalShaderMapId::FGlobalShaderMapId(EShaderPlatform Platform, const ITargetPlatform* TargetPlatform)
{
	if (TargetPlatform == nullptr)
	{
		TargetPlatform = GetTargetPlatformManagerRef().GetRunningTargetPlatform();
		check(TargetPlatform);
	}

	ShaderPlatform = Platform;
	IniPlatformName = FName(TargetPlatform->IniPlatformName());

	LayoutParams.InitializeForPlatform(TargetPlatform);
	const EShaderPermutationFlags PermutationFlags = GetShaderPermutationFlags();
	TArray<FShaderType*> ShaderTypes;
	TArray<const FShaderPipelineType*> ShaderPipelineTypes;

	for (TLinkedList<FShaderType*>::TIterator ShaderTypeIt(FShaderType::GetTypeList()); ShaderTypeIt; ShaderTypeIt.Next())
	{
		FGlobalShaderType* GlobalShaderType = ShaderTypeIt->GetGlobalShaderType();
		if (!GlobalShaderType)
		{
			continue;
		}

		bool bList = false;
		for (int32 PermutationId = 0; PermutationId < GlobalShaderType->GetPermutationCount(); PermutationId++)
		{
			if (GlobalShaderType->ShouldCompilePermutation(Platform, PermutationId, PermutationFlags))
			{
				bList = true;
				break;
			}
		}

		if (bList)
		{
			ShaderTypes.Add(GlobalShaderType);
		}
	}

	const TArray<FShaderPipelineType*>& SortedMaterialPipelineTypes = FShaderPipelineType::GetSortedTypes(FShaderType::EShaderTypeForDynamicCast::Global);

	for (FShaderPipelineType* Pipeline : SortedMaterialPipelineTypes)
	{
		check(Pipeline);
		if (Pipeline->IsGlobalTypePipeline())
		{
			int32 NumStagesNeeded = 0;
			auto& StageTypes = Pipeline->GetStages();
			for (const FShaderType* Shader : StageTypes)
			{
				const FGlobalShaderType* GlobalShaderType = Shader->GetGlobalShaderType();
				if (GlobalShaderType->ShouldCompilePermutation(Platform, /* PermutationId = */ 0, PermutationFlags))
				{
					++NumStagesNeeded;
				}
				else
				{
					break;
				}
			}

			if (NumStagesNeeded == StageTypes.Num())
			{
				ShaderPipelineTypes.Add(Pipeline);
			}
		}
	}

	// Individual shader dependencies
	ShaderTypes.Sort(FCompareShaderTypes());
	for (int32 TypeIndex = 0; TypeIndex < ShaderTypes.Num(); TypeIndex++)
	{
		FShaderType* ShaderType = ShaderTypes[TypeIndex];
		FShaderTypeDependency Dependency(ShaderType, Platform);

		const TCHAR* ShaderFilename = ShaderType->GetShaderFilename();

		TArray<FShaderTypeDependency>& Dependencies = ShaderFilenameToDependenciesMap.FindOrAdd(ShaderFilename);

		Dependencies.Add(Dependency);
	}

	for (int32 TypeIndex = 0; TypeIndex < ShaderPipelineTypes.Num(); TypeIndex++)
	{
		const FShaderPipelineType* Pipeline = ShaderPipelineTypes[TypeIndex];
		FShaderPipelineTypeDependency Dependency(Pipeline, Platform);
		ShaderPipelineTypeDependencies.Add(Dependency);
	}
}

void FGlobalShaderMapId::AppendKeyString(FString& KeyString, const TArray<FShaderTypeDependency>& Dependencies) const
{
#if WITH_EDITOR

	LayoutParams.AppendKeyString(KeyString);

	{
		const FSHAHash LayoutHash = GetShaderTypeLayoutHash(StaticGetTypeLayoutDesc<FGlobalShaderMapContent>(), LayoutParams);
		KeyString.AppendChar('_');
		LayoutHash.AppendString(KeyString);
		KeyString.AppendChar('_');
	}

	const FName PreferredShaderCodeFormat = FDataDrivenShaderPlatformInfo::GetPreferredShaderCodeFormat(ShaderPlatform);
	if (PreferredShaderCodeFormat != NAME_None)
	{
		KeyString.Append(PreferredShaderCodeFormat.ToString());
		KeyString.AppendChar('_');
	}

	AppendKeyStringShaderDependencies(Dependencies, ShaderPipelineTypeDependencies, TConstArrayView<FVertexFactoryTypeDependency>(), LayoutParams, KeyString);

	// Extra logic, just for Global shaders
	for (const FShaderTypeDependency& ShaderTypeDependency : Dependencies)
	{
		const FShaderType* ShaderType = FindShaderTypeByName(ShaderTypeDependency.ShaderTypeName);
		// Add the config define hash, if any defines exist in config
		FGlobalShaderConfigDefines::AppendKeyString(KeyString, ShaderType->GetFName(), ShaderTypeDependency.PermutationId, IniPlatformName, ShaderPlatform);
	}

#endif // WITH_EDITOR
}

bool FGlobalShaderMapId::WithEditorOnly() const
{
	return LayoutParams.WithEditorOnly();
}

bool FGlobalShaderType::ShouldCompilePipeline(const FShaderPipelineType* ShaderPipelineType, EShaderPlatform Platform, EShaderPermutationFlags Flags)
{
	for (const FShaderType* ShaderType : ShaderPipelineType->GetStages())
	{
		const FGlobalShaderPermutationParameters Parameters(ShaderType->GetFName(), Platform, kUniqueShaderPermutationId, Flags);
		checkSlow(ShaderType->GetGlobalShaderType());
		if (!ShaderType->ShouldCompilePermutation(Parameters))
		{
			return false;
		}
	}
	return true;
}

bool FGlobalShaderType::ForEachCompiledPermutation(EShaderPlatform Platform, EShaderPermutationFlags PermutationFlags, TFunction<bool(int32)> Function)
{
	bool bSuccess = true;
	if (FDataDrivenShaderPlatformInfo::GetSupportsSpecializationConstants(Platform) && (GetSpecializationCount() > 0))
	{
		// If the shader type includes specialization, collapse the permutations into valid IDs
		TSet<int32> UnspecializedIDs;
		UnspecializedIDs.Reserve(GetPermutationCount());
		for (int32 PermutationId = 0; PermutationId < GetPermutationCount(); PermutationId++)
		{
			if (ShouldCompilePermutation(Platform, PermutationId, PermutationFlags))
			{
				UnspecializedIDs.Add(GetUnspecializedId(PermutationId));
			}
		}
		for (int32 PermutationId : UnspecializedIDs)
		{
			if (!Function(PermutationId))
			{
				bSuccess = false;
				break;
			}
		}
	}
	else
	{
		for (int32 PermutationId = 0; PermutationId < GetPermutationCount(); PermutationId++)
		{
			if (ShouldCompilePermutation(Platform, PermutationId, PermutationFlags))
			{
				if (!Function(PermutationId))
				{
					bSuccess = false;
					break;
				}
			}
		}
	}
	return bSuccess;
}


FGlobalShader::FGlobalShader(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
:	FShader(Initializer)
{}

#if WITH_EDITOR
void FGlobalShaderType::SetupCompileEnvironment(EShaderPlatform Platform, int32 PermutationId, EShaderPermutationFlags Flags, FShaderCompilerEnvironment& Environment) const
{
	FGlobalShaderPermutationParameters Parameters(GetFName(), Platform, PermutationId, Flags);

	// Modify the compilation environment based on platform.
	FGlobalShaderConfigDefines::ApplyConfigDefines(Parameters, Environment);

	// Allow the shader type to modify its compile environment.
	ModifyCompilationEnvironment(Parameters, Environment);
}
#endif // WITH_EDITOR

FGlobalShaderMap* GetGlobalShaderMap(EShaderPlatform Platform)
{
	// If the global shader map hasn't been created yet
	check(GGlobalShaderMap[Platform]);
	return GGlobalShaderMap[Platform];
}

void PrimeGlobalShaderSourceHashCache(EShaderPlatform Platform, const ITargetPlatform* TargetPlatform)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(PrimeGlobalShaderSourceHashCache);

#if WITH_EDITOR
	FPlatformTypeLayoutParameters LayoutParams;
	LayoutParams.InitializeForPlatform(TargetPlatform);
	const EShaderPermutationFlags PermutationFlags = GetShaderPermutationFlags(LayoutParams);

	TArray<FGlobalShaderType*> ShaderTypesToHash;
	for (TLinkedList<FShaderType*>::TIterator ShaderTypeIt(FShaderType::GetTypeList()); ShaderTypeIt; ShaderTypeIt.Next())
	{
		FGlobalShaderType* GlobalShaderType = ShaderTypeIt->GetGlobalShaderType();
		if (!GlobalShaderType)
		{
			continue;
		}

		for (int32 PermutationId = 0; PermutationId < GlobalShaderType->GetPermutationCount(); PermutationId++)
		{
			if (GlobalShaderType->ShouldCompilePermutation(Platform, PermutationId, PermutationFlags))
			{
				ShaderTypesToHash.Add(GlobalShaderType);
				break;
			}
		}
	}

	TArray<FString> ShaderFilenames;
	ShaderFilenames.Reserve(ShaderTypesToHash.Num());
	for (FGlobalShaderType* GlobalShaderType : ShaderTypesToHash)
	{
		ShaderFilenames.Add(GlobalShaderType->GetShaderFilename());
	}
	
	PrimeShaderFileHashCache(ShaderFilenames, Platform);
#endif
}

FGlobalShaderMapSection* FGlobalShaderMapSection::CreateFromArchive(FArchive& Ar)
{
	FGlobalShaderMapSection* Section = new FGlobalShaderMapSection();
	FShaderSerializeContext Ctx(Ar);
	if (Section->Serialize(Ctx))
	{
		return Section;
	}
	delete Section;
	return nullptr;
}

#if WITH_EDITOR
FGlobalShaderMapSection* FGlobalShaderMapSection::CreateFromCache(FShaderCacheLoadContext& Ctx)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FGlobalShaderMapSection::CreateFromCache);

	FGlobalShaderMapSection* Section = new FGlobalShaderMapSection();
	if (Section->Serialize(Ctx))
	{
		return Section;
	}
	delete Section;
	return nullptr;
}
#endif

bool FGlobalShaderMapSection::Serialize(FShaderSerializeContext& Ctx)
{
	return Super::Serialize(Ctx);
}

TShaderRef<FShader> FGlobalShaderMapSection::GetShader(EShaderPlatform ShaderPlatform, FShaderType* ShaderType, int32 PermutationId) const
{
	const bool bSupportsSpecConsts = FDataDrivenShaderPlatformInfo::GetSupportsSpecializationConstants(ShaderPlatform);
	FShader* Shader = GetContent()->GetShader(ShaderType, bSupportsSpecConsts ? ShaderType->GetUnspecializedId(PermutationId) : PermutationId);
	return Shader ? TShaderRef<FShader>(Shader, *this, bSupportsSpecConsts ? ShaderType->GetSpecializationValues(PermutationId) : FSpecializationContainerType()) : TShaderRef<FShader>();
}

FShaderPipelineRef FGlobalShaderMapSection::GetShaderPipeline(const FShaderPipelineType* PipelineType) const
{
	FShaderPipeline* Pipeline = GetContent()->GetShaderPipeline(PipelineType);
	return Pipeline ? FShaderPipelineRef(Pipeline, *this) : FShaderPipelineRef();
}

void FGlobalShaderMapSection::GetShaderList(TMap<FHashedName, TShaderRef<FShader>>& OutShaders) const
{
	GetContent()->GetShaderList(*this, OutShaders);
}

void FGlobalShaderMapSection::GetShaderPipelineList(TArray<FShaderPipelineRef>& OutShaderPipelines) const
{
	GetContent()->GetShaderPipelineList(*this, OutShaderPipelines, FShaderPipeline::EAll);
}

FGlobalShaderMap::FGlobalShaderMap(EShaderPlatform InPlatform)
	: Platform(InPlatform)
{
}

FGlobalShaderMap::~FGlobalShaderMap()
{
	ReleaseAllSections();
}

TShaderRef<FShader> FGlobalShaderMap::GetShader(FShaderType* ShaderType, int32 PermutationId) const
{
	FGlobalShaderMapSection* const* Section = SectionMap.Find(ShaderType->GetHashedShaderFilename());
	return Section ? (*Section)->GetShader(Platform, ShaderType, PermutationId) : TShaderRef<FShader>();
}

FShaderPipelineRef FGlobalShaderMap::GetShaderPipeline(const FShaderPipelineType* ShaderPipelineType) const
{
	FGlobalShaderMapSection* const* Section = SectionMap.Find(ShaderPipelineType->GetHashedPrimaryShaderFilename());
	return Section ? (*Section)->GetShaderPipeline(ShaderPipelineType) : FShaderPipelineRef();
}

void FGlobalShaderMap::BeginCreateAllShaders()
{
	for (const auto& It : SectionMap)
	{
		It.Value->GetResource()->BeginCreateAllShaders();
	}
}

#if WITH_EDITOR
void FGlobalShaderMap::GetOutdatedTypes(TArray<const FShaderType*>& OutdatedShaderTypes, TArray<const FShaderPipelineType*>& OutdatedShaderPipelineTypes, TArray<const FVertexFactoryType*>& OutdatedFactoryTypes) const
{
	for (const auto& It : SectionMap)
	{
		It.Value->GetOutdatedTypes(OutdatedShaderTypes, OutdatedShaderPipelineTypes, OutdatedFactoryTypes);
	}
}

void FGlobalShaderMap::SaveShaderStableKeys(EShaderPlatform TargetShaderPlatform)
{
	FStableShaderKeyAndValue SaveKeyVal;
	for (const auto& It : SectionMap)
	{
		It.Value->SaveShaderStableKeys(TargetShaderPlatform, SaveKeyVal);
	}
}

#endif // WITH_EDITOR

bool FGlobalShaderMap::IsEmpty() const
{
	for (const auto& It : SectionMap)
	{
		if (!It.Value->GetContent()->IsEmpty())
		{
			return false;
		}
	}
	return true;
}

bool FGlobalShaderMap::IsComplete(const ITargetPlatform* TargetPlatform) const
{
	FGlobalShaderMapId ShaderMapId(Platform, TargetPlatform);
	
	// IsODSCOnly shaders aren't in shader map so don't include when gathering for IsComplete().
	const EShaderPermutationFlags PermutationFlags = ShaderMapId.GetShaderPermutationFlags() & ~EShaderPermutationFlags::IsODSCOnly;

	// traverse all global shader types
	for (TLinkedList<FShaderType*>::TIterator ShaderTypeIt(FShaderType::GetTypeList()); ShaderTypeIt; ShaderTypeIt.Next())
	{
		FGlobalShaderType* GlobalShaderType = ShaderTypeIt->GetGlobalShaderType();
		if (!GlobalShaderType)
		{
			continue;
		}

		int32 PermutationCountToCompile = 0;
		const bool bSuccess = GlobalShaderType->ForEachCompiledPermutation(Platform, PermutationFlags, [&](int32 PermutationId)
		{
			return HasShader(GlobalShaderType, PermutationId);
		});
		if (!bSuccess)
		{
			return false;
		}
	}

	// traverse all pipelines. Note that there's no ShouldCompile call for them. Materials instead test individual stages, but it leads to another problems
	// like including the standalone types even if they are not going to be used. This code follows VerifyGlobalShaders() logic that includes all global pipelines unconditionally.
	for (TLinkedList<FShaderPipelineType*>::TIterator ShaderPipelineIt(FShaderPipelineType::GetTypeList()); ShaderPipelineIt; ShaderPipelineIt.Next())
	{
		const FShaderPipelineType* Pipeline = *ShaderPipelineIt;
		if (Pipeline->IsGlobalTypePipeline()
			&& FGlobalShaderType::ShouldCompilePipeline(Pipeline, Platform, PermutationFlags)
			&& !HasShaderPipeline(Pipeline))
		{
			return false;
		}
	}

	return true;
}

void FGlobalShaderMap::Empty()
{
	for (const auto& It : SectionMap)
	{
		It.Value->GetMutableContent()->Empty();
	}
}

void FGlobalShaderMap::ReleaseAllSections()
{
	for (auto& It : SectionMap)
	{
		delete It.Value;
	}
	SectionMap.Empty();
}

FShader* FGlobalShaderMap::FindOrAddShader(const FShaderType* ShaderType, int32 PermutationId, FShader* Shader)
{
	const FHashedName HashedFilename(ShaderType->GetHashedShaderFilename());
	FGlobalShaderMapSection*& Section = SectionMap.FindOrAdd(HashedFilename);
	if (!Section)
	{
		Section = new FGlobalShaderMapSection(Platform, HashedFilename);
	}
	return Section->GetMutableContent()->FindOrAddShader(ShaderType->GetHashedName(), PermutationId, Shader);
}

FShaderPipeline* FGlobalShaderMap::FindOrAddShaderPipeline(const FShaderPipelineType* ShaderPipelineType, FShaderPipeline* ShaderPipeline)
{
	FGlobalShaderMapSection*& Section = SectionMap.FindOrAdd(ShaderPipelineType->GetHashedPrimaryShaderFilename());
	if (!Section)
	{
		Section = new FGlobalShaderMapSection(Platform, ShaderPipelineType->GetHashedPrimaryShaderFilename());
	}
	return Section->GetMutableContent()->FindOrAddShaderPipeline(ShaderPipeline);
}

void FGlobalShaderMap::RemoveShaderTypePermutaion(const FShaderType* Type, int32 PermutationId)
{
	FGlobalShaderMapSection** Section = SectionMap.Find(Type->GetHashedShaderFilename());
	if (Section)
	{
		(*Section)->GetMutableContent()->RemoveShaderTypePermutaion(Type->GetHashedName(), PermutationId);
	}
}

void FGlobalShaderMap::RemoveShaderPipelineType(const FShaderPipelineType* ShaderPipelineType)
{
	FGlobalShaderMapSection** Section = SectionMap.Find(ShaderPipelineType->GetHashedPrimaryShaderFilename());
	if (Section)
	{
		(*Section)->GetMutableContent()->RemoveShaderPipelineType(ShaderPipelineType);
	}
}

void FGlobalShaderMap::AddSection(FGlobalShaderMapSection* InSection)
{
	check(InSection);
	const FGlobalShaderMapContent* Content = InSection->GetContent();
	const FHashedName& HashedFilename = Content->HashedSourceFilename;

	SectionMap.Add(HashedFilename, InSection);
}

FGlobalShaderMapSection* FGlobalShaderMap::FindSection(const FHashedName& HashedShaderFilename)
{
	FGlobalShaderMapSection* const* Section = SectionMap.Find(HashedShaderFilename);
	return Section ? *Section : nullptr;
}

FGlobalShaderMapSection* FGlobalShaderMap::FindOrAddSection(const FShaderType* ShaderType)
{
	const FHashedName HashedFilename(ShaderType->GetHashedShaderFilename());
	FGlobalShaderMapSection* Section = FindSection(HashedFilename);
	if(!Section)
	{
		Section = new FGlobalShaderMapSection(Platform, HashedFilename);
		AddSection(Section);
	}
	return Section;
}

void FGlobalShaderMap::SaveToGlobalArchive(FArchive& Ar)
{
	int32 NumSections = SectionMap.Num();
	Ar << NumSections;

	for (const auto& It : SectionMap)
	{
		FShaderSerializeContext Ctx(Ar);
		It.Value->Serialize(Ctx);
	}
}

void FGlobalShaderMap::LoadFromGlobalArchive(FArchive& Ar)
{
	int32 NumSections = 0;
	Ar << NumSections;

	for (int32 i = 0; i < NumSections; ++i)
	{
		FGlobalShaderMapSection* Section = FGlobalShaderMapSection::CreateFromArchive(Ar);
		if (Section)
		{
			AddSection(Section);
		}
		else
		{
			UE_LOGF(LogShaders, Fatal, "Could not load section %d (of %d) of the global shadermap.", i, NumSections);
		}
	}
}

RENDERCORE_API ERecursiveShader GRequiredRecursiveShaders = ERecursiveShader::None;

void ForceInitGlobalShaderType(FShaderType& ShaderType)
{
	FGlobalShaderMap* GlobalShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);
	for (int32 Permutation = 0; Permutation < ShaderType.GetPermutationCount(); ++Permutation)
	{
		TShaderRef<FShader> ShaderRef = GlobalShaderMap->GetShader(&ShaderType, Permutation);

		if (ShaderRef.IsValid())
		{
			FShaderMapResource& MapResource = ShaderRef.GetResourceChecked();
			for (int32 Index = 0; Index < MapResource.GetNumShaders(); ++Index)
			{
				MapResource.GetShader(Index);
			}
		}
	}
}

RENDERCORE_API void CreateRecursiveShaders()
{
	ensureMsgf(!GRHISupportsMultithreadedShaderCreation, TEXT("CreateRecursiveShaders() is called while GRHISupportsMultithreadedShaderCreation is true. This is an unnecessary call."));
	ensureMsgf(IsInRenderingThread(), TEXT("CreateRecursiveShaders() is expected to be called from the render thread."));

	if (EnumHasAnyFlags(GRequiredRecursiveShaders, ERecursiveShader::Resolve))
	{
		extern void CreateResolveShaders();
		CreateResolveShaders();
	}

	if (EnumHasAnyFlags(GRequiredRecursiveShaders, ERecursiveShader::Clear))
	{
		extern void CreateClearReplacementShaders();
		CreateClearReplacementShaders();
	}

	if (EnumHasAnyFlags(GRequiredRecursiveShaders, ERecursiveShader::Null))
	{
		ForceInitGlobalShaderType<FNULLPS>();
	}
}
