// Copyright Epic Games, Inc. All Rights Reserved.

#include "LongGPUTask.h"
#include "OneColorShader.h"
#include "PipelineStateCache.h"
#include "RHIStaticStates.h"
#include "RenderUtils.h"
#include "ClearQuad.h"

FLongGPUTaskPS::FLongGPUTaskPS() = default;
FLongGPUTaskPS::FLongGPUTaskPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
	: FGlobalShader(Initializer)
{
}

IMPLEMENT_SHADER_TYPE(, FLongGPUTaskPS, TEXT("/Engine/Private/OneColorShader.usf"), TEXT("MainLongGPUTask"), SF_Pixel);

int32 NumMeasuredIterationsToAchieve500ms = 0;

void IssueScalableLongGPUTask(FRHICommandListImmediate& RHICmdList, int32 NumIteration /* = -1 by default */)
{
	const FRHITextureCreateDesc Desc =
		FRHITextureCreateDesc::Create2D(TEXT("LongTaskRenderTarget"), 1920, 1080, PF_B8G8R8A8)
		.SetFlags(ETextureCreateFlags::RenderTargetable);

	FTextureRHIRef LongTaskRenderTarget = RHICmdList.CreateTexture(Desc);

	FRHIRenderPassInfo RPInfo(LongTaskRenderTarget, ERenderTargetActions::DontLoad_Store);
	RHICmdList.Transition(FRHITransitionInfo(LongTaskRenderTarget, ERHIAccess::Unknown, ERHIAccess::RTV));
	RHICmdList.BeginRenderPass(RPInfo, TEXT("LongGPUTask"));

	{
		FGraphicsPipelineStateInitializer GraphicsPSOInit;
		RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);

		GraphicsPSOInit.BlendState = TStaticBlendState<>::GetRHI();
		GraphicsPSOInit.RasterizerState = TStaticRasterizerState<>::GetRHI();
		GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();

		auto ShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);
		TShaderMapRef<TOneColorVS<true>> VertexShader(ShaderMap);
		TShaderMapRef<FLongGPUTaskPS> PixelShader(ShaderMap);

		GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GetVertexDeclarationFVector4();
		GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
		GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
		GraphicsPSOInit.PrimitiveType = PT_TriangleStrip;

		SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

		SetShaderParametersLegacyVS(RHICmdList, VertexShader, 0.0f);

		RHICmdList.SetStreamSource(0, GClearVertexBuffer.VertexBufferRHI, 0);

		if (NumIteration == -1)
		{
			NumIteration = NumMeasuredIterationsToAchieve500ms;
		}

		for (int32 Iteration = 0; Iteration < NumIteration; Iteration++)
		{
			RHICmdList.DrawPrimitive(0, 2, 1);
		}
	}

	RHICmdList.EndRenderPass();
}

void MeasureLongGPUTaskExecutionTime(FRHICommandListImmediate& RHICmdList)
{
	const int32 NumIterationsForMeasurement = 5;

	FRenderQueryPoolRHIRef TimerQueryPool = RHICreateRenderQueryPool(RQT_AbsoluteTime, 2);

	FRHIPooledRenderQuery TimeQueryStart = TimerQueryPool->AllocateQuery();
	FRHIPooledRenderQuery TimeQueryEnd = TimerQueryPool->AllocateQuery();

	if (!GSupportsTimestampRenderQueries)
	{
		// Not all platforms/drivers support RQT_AbsoluteTime queries
		// Use fixed number of iterations on those platforms
		NumMeasuredIterationsToAchieve500ms = 5;
		return;
	}

	uint64 StartTime = 0;
	uint64 EndTime = 0;

	RHICmdList.EndRenderQuery(TimeQueryStart.GetQuery());

	IssueScalableLongGPUTask(RHICmdList, NumIterationsForMeasurement);

	RHICmdList.EndRenderQuery(TimeQueryEnd.GetQuery());

	// Required by DX12 to resolve the query
	RHICmdList.ImmediateFlush(EImmediateFlushType::FlushRHIThread);

	if (RHIGetRenderQueryResult(TimeQueryStart.GetQuery(), StartTime, true) && RHIGetRenderQueryResult(TimeQueryEnd.GetQuery(), EndTime, true))
	{
		NumMeasuredIterationsToAchieve500ms = FMath::Clamp(FMath::FloorToInt(500.0f / ((EndTime - StartTime) / 1000.0f / NumIterationsForMeasurement)), 1, 2000);
	}
	else
	{
		// Sometimes it fails even the platform supports RQT_AbsoluteTime
		// Fallback and show a warning
		NumMeasuredIterationsToAchieve500ms = 5;
		UE_LOGF(LogTemp, Display, "Unable to get render query result on a platform supporting RQT_AbsoluteTime queries, defaulting to %d iterations for LongGPUTask", NumMeasuredIterationsToAchieve500ms);
	}
}
