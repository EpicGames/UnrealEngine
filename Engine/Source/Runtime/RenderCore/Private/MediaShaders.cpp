// Copyright Epic Games, Inc. All Rights Reserved.

#include "MediaShaders.h"
#include "RenderGraphBuilder.h"
#include "RHIStaticStates.h"
#include "ShaderParameterUtils.h"
#include "PipelineStateCache.h"
#include "ColorManagement/ColorManagementDefines.h"
#include "ColorManagement/ColorSpace.h"
#include "HAL/IConsoleManager.h"

namespace
{
	TAutoConsoleVariable<bool> CVarMediaChromaInterpolation(
		TEXT("r.Media.ChromaInterpolation"),
		true,
		TEXT("Global kill-switch for chroma interpolation in 4:2:2 YUV media conversion shaders.\n")
		TEXT(" false: force off (nearest chroma)\n")
		TEXT(" true: allow callers to request interpolation (default)"),
		ECVF_RenderThreadSafe);

	// Callers opt in via SetParameters; the CVar can force-disable globally.
	uint32 ResolveChromaInterpolation(MediaShaders::EChromaInterpolation ChromaInterpolation)
	{
		const bool bEnabled = (ChromaInterpolation == MediaShaders::EChromaInterpolation::Enabled)
			&& CVarMediaChromaInterpolation.GetValueOnRenderThread();
		return bEnabled ? 1u : 0u;
	}
}

FMediaVertexDeclaration::FMediaVertexDeclaration() = default;
FMediaVertexDeclaration::~FMediaVertexDeclaration() = default;

void FMediaVertexDeclaration::InitRHI(FRHICommandListBase& RHICmdList)
{
	FVertexDeclarationElementList Elements;
	uint16 Stride = sizeof(FMediaElementVertex);
	Elements.Add(FVertexElement(0, STRUCT_OFFSET(FMediaElementVertex, Position), VET_Float4, 0, Stride));
	Elements.Add(FVertexElement(0, STRUCT_OFFSET(FMediaElementVertex, TextureCoordinate), VET_Float2, 1, Stride));
	VertexDeclarationRHI = PipelineStateCache::GetOrCreateVertexDeclaration(Elements);
}

void FMediaVertexDeclaration::ReleaseRHI()
{
	VertexDeclarationRHI.SafeRelease();
}

TGlobalResource<FMediaVertexDeclaration> GMediaVertexDeclaration;


/* MediaShaders namespace
 *****************************************************************************/

namespace MediaShaders
{
	/**
	 * YUV to RGB matrices for Rec601, Rec709, and Rec2020. Each are provided with a Scaled and Unscaled version.
	 *
	 * The scaling is according to SMPTE EG36, resulting in (255/219.0, 255/224.0, 255/224.0) being multiplied by the Unscaled matrix.
	 */

	const FMatrix YuvToRgbRec601Unscaled = FMatrix(
		FPlane(1.000000f,  0.00000000000f,  1.4020000000f, 0.000000f),
		FPlane(1.000000f, -0.34414362862f, -0.7141362862f, 0.000000f),
		FPlane(1.000000f,  1.77200000000f,  0.0000000000f, 0.000000f),
		FPlane(0.000000f,  0.000000f,  0.000000f, 0.000000f)
	);

	const FMatrix YuvToRgbRec601Scaled = FMatrix(
		FPlane(1.16438356164f,  0.000000000000f,  1.596026785714f, 0.000000f),
		FPlane(1.16438356164f, -0.391762290095f, -0.812967647238f, 0.000000f),
		FPlane(1.16438356164f,  2.017232142857f,  0.000000000000f, 0.000000f),
		FPlane(0.000000f,  0.000000f,  0.000000f, 0.000000f)
	);

	const FMatrix YuvToRgbRec709Unscaled = FMatrix(
		FPlane(1.000000f,  0.000000000000f,  1.574721988260f, 0.000000f),
		FPlane(1.000000f, -0.187314089535f, -0.468207470556f, 0.000000f),
		FPlane(1.000000f,  1.855615369279f,  0.000000000000f, 0.000000f),
		FPlane(0.000000f, 0.000000f, 0.000000f, 0.000000f)
	);

	const FMatrix YuvToRgbRec709Scaled = FMatrix(
		FPlane(1.16438356164f,  0.000000000000f,  1.792652263418f, 0.000000f),
		FPlane(1.16438356164f, -0.213237021569f, -0.533004040142f, 0.000000f),
		FPlane(1.16438356164f,  2.112419281991f,  0.000000000000f, 0.000000f),
		FPlane(0.000000f, 0.000000f, 0.000000f, 0.000000f)
	);

	const FMatrix YuvToRgbRec2020Unscaled = FMatrix(
		FPlane(1.000000f,  0.000000000000f,  1.474599575977f, 0.000000f),
		FPlane(1.000000f, -0.164558057720f, -0.571355048803f, 0.000000f),
		FPlane(1.000000f,  1.881396567060f,  0.000000000000f, 0.000000f),
		FPlane(0.000000f, 0.000000f, 0.000000f, 0.000000f)
	);

	const FMatrix YuvToRgbRec2020Scaled = FMatrix(
		FPlane(1.16438356164f,  0.000000000000f,  1.678673624439f, 0.000000f),
		FPlane(1.16438356164f, -0.187331717494f, -0.650426506450f, 0.000000f),
		FPlane(1.16438356164f,  2.141768413395f,  0.000000000000f, 0.000000f),
		FPlane(0.000000f, 0.000000f, 0.000000f, 0.000000f)
	);

	// YuvToSrgbPs4 is legacy. It is apparently a YuvToRgbRec709Scaled that was rounded to four digits after radix.
	const FMatrix YuvToSrgbPs4 = FMatrix(
		FPlane(1.164400f,  0.000000f,  1.792700f, 0.000000f),
		FPlane(1.164400f, -0.213300f, -0.532900f, 0.000000f),
		FPlane(1.164400f,  2.112400f,  0.000000f, 0.000000f),
		FPlane(0.000000f,  0.000000f,  0.000000f, 0.000000f)
	);

	// Inverse of YuvToRgbRec709Scaled
	const FMatrix RgbToYuvRec709Scaled = FMatrix(
		FPlane( 0.18261938151317932966f,  0.61420368882407283539f,  0.062000459074512487279f, 0.000000f),
		FPlane(-0.10066136381366230790f, -0.33855432246084737891f,  0.439215686274509797830f, 0.000000f),
		FPlane( 0.43921568627450979783f, -0.39894445418228852152f, -0.040271232092221290189f, 0.000000f),
		FPlane(0.000000f, 0.000000f, 0.000000f, 0.000000f)
	);

	// Inverse of YuvToRgbRec2020Scaled
	const FMatrix RgbToYuvRec2020Scaled = FMatrix(
		FPlane( 0.225613123257483f,  0.582280696718123f,  0.0509297094389877f, 0.000000f),
		FPlane(-0.122655750438889f, -0.316559935835522f,  0.439215686274411f, 0.000000f),
		FPlane( 0.439215686274411f, -0.403889154894806f, -0.0353265313796045f, 0.000000f),
		FPlane(0.000000f, 0.000000f, 0.000000f, 0.000000f)
	);

	const FVector YUVOffset8bits = FVector(0.06274509803921568627f, 0.5019607843137254902f, 0.5019607843137254902f);
	const FVector YUVOffsetNoScale8bits = FVector(0.0f, 0.5019607843137254902f, 0.5019607843137254902f);

	const FVector YUVOffset10bits = FVector(0.06256109481915933529f, 0.50048875855327468231f, 0.50048875855327468231f);
	const FVector YUVOffsetNoScale10bits = FVector(0.0f, 0.50048875855327468231f, 0.50048875855327468231f);

	const FVector YUVOffset16bits = FVector(4096.0f / 65535.0f, 32768.0f / 65535.0f, 32768.0f / 65535.0f);
	const FVector YUVOffsetNoScale16bits = FVector(0.0f, 32768.0f / 65535.0f, 32768.0f / 65535.0f);

	const FVector YUVOffsetFloat = FVector(1.0f / 16.0f, 0.5f, 0.5f);
	const FVector YUVOffsetNoScaleFloat = FVector(0.0f, 0.5f, 0.5f);

	/** Setup YUV Offset in matrix */
	FMatrix CombineColorTransformAndOffset(const FMatrix& InMatrix, const FVector& InYUVOffset)
	{
		FMatrix Result = InMatrix;
		// Offset in last column:
		// 1) to allow for 4x4 matrix multiplication optimization when going from RGB to YUV (hence the 1.0 in the [3][3] matrix location)
		// 2) stored in empty space when going from YUV to RGB
		Result.M[0][3] = InYUVOffset.X;
		Result.M[1][3] = InYUVOffset.Y;
		Result.M[2][3] = InYUVOffset.Z;
		Result.M[3][3] = 1.0f;
		return Result;
	}
}

/* FMediaShadersVS shader
 *****************************************************************************/

IMPLEMENT_SHADER_TYPE(, FMediaShadersVS, TEXT("/Engine/Private/MediaShaders.usf"), TEXT("MainVertexShader"), SF_Vertex);


/* FAYUVConvertPS shader
 *****************************************************************************/

BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT(FAYUVConvertUB, )
SHADER_PARAMETER(FMatrix44f, ColorTransform)
SHADER_PARAMETER(uint32, SrgbToLinear)
SHADER_PARAMETER_TEXTURE(Texture2D, Texture)
SHADER_PARAMETER_SAMPLER(SamplerState, Sampler)
END_GLOBAL_SHADER_PARAMETER_STRUCT()

IMPLEMENT_GLOBAL_SHADER_PARAMETER_STRUCT(FAYUVConvertUB, "AYUVConvertUB");
IMPLEMENT_SHADER_TYPE(, FAYUVConvertPS, TEXT("/Engine/Private/MediaShaders.usf"), TEXT("AYUVConvertPS"), SF_Pixel);


void FAYUVConvertPS::SetParameters(FRHIBatchedShaderParameters& BatchedParameters, TRefCountPtr<FRHITexture> AYUVTexture, const FMatrix& ColorTransform, const FVector& YUVOffset, bool SrgbToLinear)
{
	FAYUVConvertUB UB;
	{
		UB.ColorTransform = (FMatrix44f)MediaShaders::CombineColorTransformAndOffset(ColorTransform, YUVOffset);
		UB.Sampler = TStaticSamplerState<SF_Bilinear>::GetRHI();
		UB.SrgbToLinear = SrgbToLinear;
		UB.Texture = AYUVTexture;
	}

	TUniformBufferRef<FAYUVConvertUB> Data = TUniformBufferRef<FAYUVConvertUB>::CreateUniformBufferImmediate(UB, UniformBuffer_SingleFrame);
	SetUniformBufferParameter(BatchedParameters, GetUniformBufferParameter<FAYUVConvertUB>(), Data);
}


/* FBMPConvertPS shader
 *****************************************************************************/

BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT(FBMPConvertUB, )
SHADER_PARAMETER(uint32, SrgbToLinear)
SHADER_PARAMETER(FVector2f, UVScale)
SHADER_PARAMETER_TEXTURE(Texture2D, Texture)
SHADER_PARAMETER_SAMPLER(SamplerState, Sampler)
END_GLOBAL_SHADER_PARAMETER_STRUCT()

IMPLEMENT_GLOBAL_SHADER_PARAMETER_STRUCT(FBMPConvertUB, "BMPConvertUB");
IMPLEMENT_SHADER_TYPE(, FBMPConvertPS, TEXT("/Engine/Private/MediaShaders.usf"), TEXT("BMPConvertPS"), SF_Pixel);


void FBMPConvertPS::SetParameters(FRHIBatchedShaderParameters& BatchedParameters, TRefCountPtr<FRHITexture> BMPTexture, const FIntPoint& OutputDimensions, bool SrgbToLinear)
{
	FBMPConvertUB UB;
	{
		UB.SrgbToLinear = SrgbToLinear;
		UB.UVScale = FVector2f((float)OutputDimensions.X / (float)BMPTexture->GetSizeX(), (float)OutputDimensions.Y / (float)BMPTexture->GetSizeY());
		UB.Texture = BMPTexture;
		UB.Sampler = TStaticSamplerState<SF_Bilinear>::GetRHI();
	}

	TUniformBufferRef<FBMPConvertUB> Data = TUniformBufferRef<FBMPConvertUB>::CreateUniformBufferImmediate(UB, UniformBuffer_SingleFrame);
	SetUniformBufferParameter(BatchedParameters, GetUniformBufferParameter<FBMPConvertUB>(), Data);
}


/* FNV12ConvertPS shader
 *****************************************************************************/

BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT(FNV12ConvertUB, )
SHADER_PARAMETER(FMatrix44f, ColorTransform)
SHADER_PARAMETER(FMatrix44f, CSTransform)
SHADER_PARAMETER(uint32, OutputWidth)
SHADER_PARAMETER(uint32, EOTF)
SHADER_PARAMETER(uint32, ToneMapMethod)
SHADER_PARAMETER(uint32, SwapChroma)
SHADER_PARAMETER(FVector2f, UVScale)
SHADER_PARAMETER_SRV(Texture2D, SRV_Y)
SHADER_PARAMETER_SRV(Texture2D, SRV_UV)
SHADER_PARAMETER_SAMPLER(SamplerState, SamplerB)
SHADER_PARAMETER_SAMPLER(SamplerState, SamplerP)
END_GLOBAL_SHADER_PARAMETER_STRUCT()

IMPLEMENT_GLOBAL_SHADER_PARAMETER_STRUCT(FNV12ConvertUB, "NV12ConvertUB");
IMPLEMENT_SHADER_TYPE(, FNV12ConvertPS, TEXT("/Engine/Private/MediaShaders.usf"), TEXT("NV12ConvertPS"), SF_Pixel);

void FNV12ConvertPS::SetParameters(FRHIBatchedShaderParameters& BatchedParameters, const FIntPoint & TexDim, FShaderResourceViewRHIRef SRV_Y, FShaderResourceViewRHIRef SRV_UV, const FIntPoint& OutputDimensions, const FMatrix44f& ColorTransform, UE::Color::EEncoding Encoding, const FMatrix44f& CSTransform, bool bSwapChroma, MediaShaders::EToneMapMethod ToneMapMethod)
{
	// Ensure shader code assumptions about the layout of this enum are correct
	static_assert(int32(UE::Color::EEncoding::Linear) == 1, "Enum mismatch with use in shader code!");
	static_assert(int32(UE::Color::EEncoding::sRGB) == 2, "Enum mismatch with use in shader code!");
	static_assert(int32(UE::Color::EEncoding::ST2084) == 3, "Enum mismatch with use in shader code!");
	static_assert(int32(UE::Color::EEncoding::Gamma22) == 4, "Enum mismatch with use in shader code!");
	static_assert(int32(UE::Color::EEncoding::BT1886) == 5, "Enum mismatch with use in shader code!");
	static_assert(int32(UE::Color::EEncoding::Gamma26) == 6, "Enum mismatch with use in shader code!");
	static_assert(int32(UE::Color::EEncoding::Cineon) == 7, "Enum mismatch with use in shader code!");
	static_assert(int32(UE::Color::EEncoding::REDLog) == 8, "Enum mismatch with use in shader code!");
	static_assert(int32(UE::Color::EEncoding::REDLog3G10) == 9, "Enum mismatch with use in shader code!");
	static_assert(int32(UE::Color::EEncoding::SLog1) == 10, "Enum mismatch with use in shader code!");
	static_assert(int32(UE::Color::EEncoding::SLog2) == 11, "Enum mismatch with use in shader code!");
	static_assert(int32(UE::Color::EEncoding::SLog3) == 12, "Enum mismatch with use in shader code!");
	static_assert(int32(UE::Color::EEncoding::AlexaV3LogC) == 13, "Enum mismatch with use in shader code!");
	static_assert(int32(UE::Color::EEncoding::CanonLog) == 14, "Enum mismatch with use in shader code!");
	static_assert(int32(UE::Color::EEncoding::ProTune) == 15, "Enum mismatch with use in shader code!");
	static_assert(int32(UE::Color::EEncoding::VLog) == 16, "Enum mismatch with use in shader code!");
	static_assert(int32(UE::Color::EEncoding::HLG) == 17, "Enum mismatch with use in shader code!");

	FNV12ConvertUB UB;
	{
		UB.ColorTransform = ColorTransform;
		UB.CSTransform = CSTransform;
		UB.OutputWidth = OutputDimensions.X;
		UB.SamplerB = TStaticSamplerState<SF_Bilinear>::GetRHI();
		UB.SamplerP = TStaticSamplerState<SF_Point>::GetRHI();
		UB.EOTF = static_cast<uint32>(Encoding);
		UB.ToneMapMethod = static_cast<uint32>(ToneMapMethod);
		UB.SRV_Y = SRV_Y;
		UB.SRV_UV = SRV_UV;
		UB.UVScale = FVector2f((float)OutputDimensions.X / (float)TexDim.X, (float)OutputDimensions.Y / (float)TexDim.Y);
		UB.SwapChroma = bSwapChroma;
	}

	TUniformBufferRef<FNV12ConvertUB> Data = TUniformBufferRef<FNV12ConvertUB>::CreateUniformBufferImmediate(UB, UniformBuffer_SingleFrame);
	SetUniformBufferParameter(BatchedParameters, GetUniformBufferParameter<FNV12ConvertUB>(), Data);
}


/* FNV12ConvertAsBytesPS shader
 *****************************************************************************/

BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT(FNV12ConvertAsBytesUB, )
SHADER_PARAMETER(FMatrix44f, ColorTransform)
SHADER_PARAMETER(FMatrix44f, CSTransform)
SHADER_PARAMETER(uint32, OutputWidth)
SHADER_PARAMETER(uint32, EOTF)
SHADER_PARAMETER(uint32, ToneMapMethod)
SHADER_PARAMETER(uint32, SwapChroma)
SHADER_PARAMETER(FVector2f, UVScale)
SHADER_PARAMETER_TEXTURE(Texture2D, Texture)
SHADER_PARAMETER_SAMPLER(SamplerState, SamplerB)
SHADER_PARAMETER_SAMPLER(SamplerState, SamplerP)
END_GLOBAL_SHADER_PARAMETER_STRUCT()

IMPLEMENT_GLOBAL_SHADER_PARAMETER_STRUCT(FNV12ConvertAsBytesUB, "NV12ConvertAsBytesUB");
IMPLEMENT_SHADER_TYPE(, FNV12ConvertAsBytesPS, TEXT("/Engine/Private/MediaShaders.usf"), TEXT("NV12ConvertAsBytesPS"), SF_Pixel);

void FNV12ConvertAsBytesPS::SetParameters(FRHIBatchedShaderParameters& BatchedParameters, TRefCountPtr<FRHITexture> NV12Texture, const FIntPoint& OutputDimensions, const FMatrix44f& ColorTransform, UE::Color::EEncoding Encoding, const FMatrix44f& CSTransform, bool bSwapChroma, MediaShaders::EToneMapMethod ToneMapMethod)
{
	FNV12ConvertAsBytesUB UB;
	{
		UB.ColorTransform = ColorTransform;
		UB.CSTransform = CSTransform;
		UB.OutputWidth = OutputDimensions.X;
		UB.SamplerB = TStaticSamplerState<SF_Bilinear>::GetRHI();
		UB.SamplerP = TStaticSamplerState<SF_Point>::GetRHI();
		UB.EOTF = static_cast<uint32>(Encoding);
		UB.ToneMapMethod = static_cast<uint32>(ToneMapMethod);
		UB.Texture = NV12Texture;
		UB.UVScale = FVector2f((float)OutputDimensions.X / (float)NV12Texture->GetSizeX(), (float)OutputDimensions.Y / (float)NV12Texture->GetSizeY());
		UB.SwapChroma = bSwapChroma;
	}

	TUniformBufferRef<FNV12ConvertAsBytesUB> Data = TUniformBufferRef<FNV12ConvertAsBytesUB>::CreateUniformBufferImmediate(UB, UniformBuffer_SingleFrame);
	SetUniformBufferParameter(BatchedParameters, GetUniformBufferParameter<FNV12ConvertAsBytesUB>(), Data);
}


/* FNV21ConvertPS shader
 *****************************************************************************/

BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT(FNV21ConvertUB, )
SHADER_PARAMETER(FMatrix44f, ColorTransform)
SHADER_PARAMETER(uint32, OutputWidth)
SHADER_PARAMETER(uint32, SrgbToLinear)
SHADER_PARAMETER(FVector2f, UVScale)
SHADER_PARAMETER_TEXTURE(Texture2D, Texture)
SHADER_PARAMETER_SAMPLER(SamplerState, SamplerB)
SHADER_PARAMETER_SAMPLER(SamplerState, SamplerP)
END_GLOBAL_SHADER_PARAMETER_STRUCT()

IMPLEMENT_GLOBAL_SHADER_PARAMETER_STRUCT(FNV21ConvertUB, "NV21ConvertUB");
IMPLEMENT_SHADER_TYPE(, FNV21ConvertPS, TEXT("/Engine/Private/MediaShaders.usf"), TEXT("NV21ConvertPS"), SF_Pixel);


void FNV21ConvertPS::SetParameters(FRHIBatchedShaderParameters& BatchedParameters, TRefCountPtr<FRHITexture> NV21Texture, const FIntPoint& OutputDimensions, const FMatrix& ColorTransform, const FVector& YUVOffset, bool SrgbToLinear)
{
	FNV21ConvertUB UB;
	{
		UB.ColorTransform = (FMatrix44f)MediaShaders::CombineColorTransformAndOffset(ColorTransform, YUVOffset);
		UB.OutputWidth = OutputDimensions.Y;
		UB.SamplerB = TStaticSamplerState<SF_Bilinear>::GetRHI();
		UB.SamplerP = TStaticSamplerState<SF_Point>::GetRHI();
		UB.SrgbToLinear = SrgbToLinear;
		UB.Texture = NV21Texture;
		UB.UVScale = FVector2f((float)OutputDimensions.X / (float)NV21Texture->GetSizeX(), (float)OutputDimensions.Y / (float)NV21Texture->GetSizeY());
	}

	TUniformBufferRef<FNV21ConvertUB> Data = TUniformBufferRef<FNV21ConvertUB>::CreateUniformBufferImmediate(UB, UniformBuffer_SingleFrame);
	SetUniformBufferParameter(BatchedParameters, GetUniformBufferParameter<FNV21ConvertUB>(), Data);
}


/* FP010ConvertPS shader
 *****************************************************************************/

BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT(FP010ConvertUB, )
SHADER_PARAMETER(FMatrix44f, ColorTransform)
SHADER_PARAMETER(FMatrix44f, CSTransform)
SHADER_PARAMETER(uint32, EOTF)
SHADER_PARAMETER(uint32, ToneMapMethod)
SHADER_PARAMETER(FVector2f, UVScale)
SHADER_PARAMETER_SRV(Texture2D, SRV_Y)
SHADER_PARAMETER_SRV(Texture2D, SRV_UV)
SHADER_PARAMETER_SAMPLER(SamplerState, SamplerB)
SHADER_PARAMETER_SAMPLER(SamplerState, SamplerP)
END_GLOBAL_SHADER_PARAMETER_STRUCT()

IMPLEMENT_GLOBAL_SHADER_PARAMETER_STRUCT(FP010ConvertUB, "P010ConvertUB");
IMPLEMENT_SHADER_TYPE(, FP010ConvertPS, TEXT("/Engine/Private/MediaShaders.usf"), TEXT("P010ConvertPS"), SF_Pixel);

void FP010ConvertPS::SetParameters(FRHIBatchedShaderParameters& BatchedParameters, const FIntPoint& TexDim, FShaderResourceViewRHIRef SRV_Y, FShaderResourceViewRHIRef SRV_UV, const FIntPoint& OutputDimensions, const FMatrix44f& ColorTransform, const FMatrix44f& CSTransform, UE::Color::EEncoding Encoding, MediaShaders::EToneMapMethod ToneMapMethod)
{
	FP010ConvertUB UB;
	{
		UB.ColorTransform = ColorTransform;
		UB.CSTransform = CSTransform;
		UB.SamplerB = TStaticSamplerState<SF_Bilinear>::GetRHI();
		UB.SamplerP = TStaticSamplerState<SF_Point>::GetRHI();
		UB.EOTF = static_cast<uint32>(Encoding);
		UB.ToneMapMethod = static_cast<uint32>(ToneMapMethod);
		UB.SRV_Y = SRV_Y;
		UB.SRV_UV = SRV_UV;
		UB.UVScale = FVector2f((float)OutputDimensions.X / (float)TexDim.X, (float)OutputDimensions.Y / (float)TexDim.Y);
	}

	TUniformBufferRef<FP010ConvertUB> Data = TUniformBufferRef<FP010ConvertUB>::CreateUniformBufferImmediate(UB, UniformBuffer_SingleFrame);
	SetUniformBufferParameter(BatchedParameters, GetUniformBufferParameter<FP010ConvertUB>(), Data);
}


/* FP010ConvertAsUINT16sPS shader (from G16 texture)
 *****************************************************************************/

BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT(FP010ConvertAsUINT16sUB, )
SHADER_PARAMETER(FMatrix44f, ColorTransform)
SHADER_PARAMETER(FMatrix44f, CSTransform)
SHADER_PARAMETER(uint32, OutputWidth)
SHADER_PARAMETER(uint32, EOTF)
SHADER_PARAMETER(uint32, ToneMapMethod)
SHADER_PARAMETER(FVector2f, UVScale)
SHADER_PARAMETER_TEXTURE(Texture2D, Texture)
SHADER_PARAMETER_SAMPLER(SamplerState, SamplerB)
SHADER_PARAMETER_SAMPLER(SamplerState, SamplerP)
END_GLOBAL_SHADER_PARAMETER_STRUCT()

IMPLEMENT_GLOBAL_SHADER_PARAMETER_STRUCT(FP010ConvertAsUINT16sUB, "P010ConvertAsUINT16sUB");
IMPLEMENT_SHADER_TYPE(, FP010ConvertAsUINT16sPS, TEXT("/Engine/Private/MediaShaders.usf"), TEXT("P010ConvertAsUINT16sPS"), SF_Pixel);

void FP010ConvertAsUINT16sPS::SetParameters(FRHIBatchedShaderParameters& BatchedParameters, const FIntPoint& TexDim, TRefCountPtr<FRHITexture> NV12Texture, const FIntPoint& OutputDimensions, const FMatrix44f& ColorTransform, const FMatrix44f& CSTransform, UE::Color::EEncoding Encoding, MediaShaders::EToneMapMethod ToneMapMethod)
{
	FP010ConvertAsUINT16sUB UB;
	{
		UB.ColorTransform = ColorTransform;
		UB.CSTransform = CSTransform;
		UB.OutputWidth = OutputDimensions.X;
		UB.SamplerB = TStaticSamplerState<SF_Bilinear>::GetRHI();
		UB.SamplerP = TStaticSamplerState<SF_Point>::GetRHI();
		UB.EOTF = static_cast<uint32>(Encoding);
		UB.ToneMapMethod = static_cast<uint32>(ToneMapMethod);
		UB.Texture = NV12Texture;
		UB.UVScale = FVector2f((float)OutputDimensions.X / (float)TexDim.X, (float)OutputDimensions.Y / (float)TexDim.Y);
	}

	TUniformBufferRef<FP010ConvertAsUINT16sUB> Data = TUniformBufferRef<FP010ConvertAsUINT16sUB>::CreateUniformBufferImmediate(UB, UniformBuffer_SingleFrame);
	SetUniformBufferParameter(BatchedParameters, GetUniformBufferParameter<FP010ConvertAsUINT16sUB>(), Data);
}


/* FP010_2101010ConvertPS shader
 *****************************************************************************/

BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT(FP010_2101010ConvertUB, )
SHADER_PARAMETER(FMatrix44f, ColorTransform)
SHADER_PARAMETER(FMatrix44f, CSTransform)
SHADER_PARAMETER(uint32, EOTF)
SHADER_PARAMETER(uint32, ToneMapMethod)
SHADER_PARAMETER(FVector2f, UVScaleY)
SHADER_PARAMETER(FVector2f, UVScaleUV)
SHADER_PARAMETER_SRV(Texture2D, SRV_Y)
SHADER_PARAMETER_SRV(Texture2D, SRV_U)
SHADER_PARAMETER_SRV(Texture2D, SRV_V)
SHADER_PARAMETER_SAMPLER(SamplerState, SamplerP)
SHADER_PARAMETER(uint32, OutputWidthY)
SHADER_PARAMETER(uint32, OutputWidthUV)
END_GLOBAL_SHADER_PARAMETER_STRUCT()

IMPLEMENT_GLOBAL_SHADER_PARAMETER_STRUCT(FP010_2101010ConvertUB, "P010_2101010ConvertUB");
IMPLEMENT_SHADER_TYPE(, FP010_2101010ConvertPS, TEXT("/Engine/Private/MediaShaders.usf"), TEXT("P010_2101010ConvertPS"), SF_Pixel);

void FP010_2101010ConvertPS::SetParameters(FRHIBatchedShaderParameters& BatchedParameters, const FIntPoint& TexDim, FShaderResourceViewRHIRef SRV_Y, FShaderResourceViewRHIRef SRV_U, FShaderResourceViewRHIRef SRV_V, const FIntPoint& OutputDimensions, const FMatrix44f& ColorTransform, const FMatrix44f& CSTransform, UE::Color::EEncoding Encoding, MediaShaders::EToneMapMethod ToneMapMethod)
{
	FP010_2101010ConvertUB UB;
	{
		UB.ColorTransform = ColorTransform;
		UB.CSTransform = CSTransform;
		UB.SamplerP = TStaticSamplerState<SF_Point>::GetRHI();
		UB.EOTF = static_cast<uint32>(Encoding);
		UB.ToneMapMethod = static_cast<uint32>(ToneMapMethod);
		UB.SRV_Y = SRV_Y;
		UB.SRV_U = SRV_U;
		UB.SRV_V = SRV_V;
		// Output width used in shader needs to be aligned to the RGB triplets we store the data in
		UB.OutputWidthY = ((OutputDimensions.X + 2) / 3) * 3;
		UB.OutputWidthUV = ((OutputDimensions.X / 2) / 3) * 3;
		// The scale contains both: a scale for any cropping (as in: get rid of lower, right) as well as a correction for non-3-texel aligned width of the 1010102 texture
		float SX = float(OutputDimensions.X) / float(UB.OutputWidthY);
		UB.UVScaleY = FVector2f(SX * ((float)OutputDimensions.X / (float)TexDim.X), (float)OutputDimensions.Y / (float)TexDim.Y);
		SX = float(OutputDimensions.X / 2) / float(UB.OutputWidthUV);
		UB.UVScaleUV = FVector2f(SX * ((float)OutputDimensions.X / (float)TexDim.X), (float)OutputDimensions.Y / (float)TexDim.Y);
	}

	TUniformBufferRef<FP010_2101010ConvertUB> Data = TUniformBufferRef<FP010_2101010ConvertUB>::CreateUniformBufferImmediate(UB, UniformBuffer_SingleFrame);
	SetUniformBufferParameter(BatchedParameters, GetUniformBufferParameter<FP010_2101010ConvertUB>(), Data);
}


/* FRGBConvertPS shader
 *****************************************************************************/

BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT(FRGBConvertUB, )
SHADER_PARAMETER(FVector2f, UVScale)
SHADER_PARAMETER_TEXTURE(Texture2D, Texture)
SHADER_PARAMETER_SAMPLER(SamplerState, Sampler)
SHADER_PARAMETER(FMatrix44f, CSTransform)
SHADER_PARAMETER(uint32, EOTF)				// UE::Color::EEncoding value (see ColorManagementDefines.h).
SHADER_PARAMETER(uint32, ToneMapMethod)
END_GLOBAL_SHADER_PARAMETER_STRUCT()

IMPLEMENT_GLOBAL_SHADER_PARAMETER_STRUCT(FRGBConvertUB, "RGBConvertUB");
IMPLEMENT_SHADER_TYPE(, FRGBConvertPS, TEXT("/Engine/Private/MediaShaders.usf"), TEXT("RGBConvertPS"), SF_Pixel);


void FRGBConvertPS::SetParameters(FRHIBatchedShaderParameters& BatchedParameters, TRefCountPtr<FRHITexture> RGBTexture, const FIntPoint& OutputDimensions, UE::Color::EEncoding Encoding, const FMatrix44f& CSTransform, MediaShaders::EToneMapMethod ToneMapMethod)
{
	FRGBConvertUB UB;
	{
		UB.Sampler = TStaticSamplerState<SF_Bilinear>::GetRHI();
		UB.Texture = RGBTexture;
		UB.UVScale = FVector2f((float)OutputDimensions.X / (float)RGBTexture->GetSizeX(), (float)OutputDimensions.Y / (float)RGBTexture->GetSizeY());
		UB.EOTF = static_cast<uint32>(Encoding);
		UB.ToneMapMethod = static_cast<uint32>(ToneMapMethod);
		UB.CSTransform = CSTransform;
	}

	TUniformBufferRef<FRGBConvertUB> Data = TUniformBufferRef<FRGBConvertUB>::CreateUniformBufferImmediate(UB, UniformBuffer_SingleFrame);
	SetUniformBufferParameter(BatchedParameters, GetUniformBufferParameter<FRGBConvertUB>(), Data);
}


/* FYCoCgConvertPS shader
 *****************************************************************************/

BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT(FYCoCgConvertUB, )
SHADER_PARAMETER(FVector2f, UVScale)
SHADER_PARAMETER_TEXTURE(Texture2D, Texture)
SHADER_PARAMETER_SAMPLER(SamplerState, Sampler)
SHADER_PARAMETER(FMatrix44f, CSTransform)
SHADER_PARAMETER(uint32, EOTF)				// UE::Color::EEncoding value (see ColorManagementDefines.h).
SHADER_PARAMETER(uint32, ToneMapMethod)
END_GLOBAL_SHADER_PARAMETER_STRUCT()

IMPLEMENT_GLOBAL_SHADER_PARAMETER_STRUCT(FYCoCgConvertUB, "YCoCgConvertUB");
IMPLEMENT_SHADER_TYPE(, FYCoCgConvertPS, TEXT("/Engine/Private/MediaShaders.usf"), TEXT("YCoCgConvertPS"), SF_Pixel);


void FYCoCgConvertPS::SetParameters(FRHIBatchedShaderParameters& BatchedParameters, TRefCountPtr<FRHITexture> RGBTexture, const FIntPoint& OutputDimensions, UE::Color::EEncoding Encoding, const FMatrix44f& CSTransform, MediaShaders::EToneMapMethod ToneMapMethod)
{
	FYCoCgConvertUB UB;
	{
		UB.Sampler = TStaticSamplerState<SF_Bilinear>::GetRHI();
		UB.Texture = RGBTexture;
		UB.UVScale = FVector2f((float)OutputDimensions.X / (float)RGBTexture->GetSizeX(), (float)OutputDimensions.Y / (float)RGBTexture->GetSizeY());
		UB.EOTF = static_cast<uint32>(Encoding);
		UB.ToneMapMethod = static_cast<uint32>(ToneMapMethod);
		UB.CSTransform = CSTransform;
	}

	TUniformBufferRef<FYCoCgConvertUB> Data = TUniformBufferRef<FYCoCgConvertUB>::CreateUniformBufferImmediate(UB, UniformBuffer_SingleFrame);
	SetUniformBufferParameter(BatchedParameters, GetUniformBufferParameter<FYCoCgConvertUB>(), Data);
}


/* FYCbCrConvertUB shader
 *****************************************************************************/

BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT(FYCbCrConvertUB, )
SHADER_PARAMETER(FMatrix44f, ColorTransform)
SHADER_PARAMETER(uint32, SrgbToLinear)
SHADER_PARAMETER_TEXTURE(Texture2D, LumaTexture)
SHADER_PARAMETER_TEXTURE(Texture2D, CbCrTexture)
SHADER_PARAMETER_SAMPLER(SamplerState, LumaSampler)
SHADER_PARAMETER_SAMPLER(SamplerState, CbCrSampler)
END_GLOBAL_SHADER_PARAMETER_STRUCT()

IMPLEMENT_GLOBAL_SHADER_PARAMETER_STRUCT(FYCbCrConvertUB, "YCbCrConvertUB");
IMPLEMENT_SHADER_TYPE(, FYCbCrConvertPS, TEXT("/Engine/Private/MediaShaders.usf"), TEXT("YCbCrConvertPS"), SF_Pixel);
IMPLEMENT_SHADER_TYPE(, FYCbCrConvertPS_4x4Matrix, TEXT("/Engine/Private/MediaShaders.usf"), TEXT("YCbCrConvertPS_4x4Matrix"), SF_Pixel);


void FYCbCrConvertPS::SetParameters(FRHIBatchedShaderParameters& BatchedParameters, TRefCountPtr<FRHITexture> LumaTexture, TRefCountPtr<FRHITexture> CbCrTexture, const FMatrix& ColorTransform, const FVector& YUVOffset, bool SrgbToLinear)
{
	FYCbCrConvertUB UB;
	{
		// Chroma is not usually 1:1 with the output textxure
		UB.CbCrSampler = TStaticSamplerState<SF_Bilinear>::GetRHI();
		UB.CbCrTexture = CbCrTexture;
		UB.ColorTransform = (FMatrix44f)MediaShaders::CombineColorTransformAndOffset(ColorTransform, YUVOffset);

		// Luma should be 1:1 with the output texture and needs to be point sampled
		UB.LumaSampler = TStaticSamplerState<SF_Point>::GetRHI();
		UB.LumaTexture = LumaTexture;
		UB.SrgbToLinear = SrgbToLinear;
	}

	TUniformBufferRef<FYCbCrConvertUB> Data = TUniformBufferRef<FYCbCrConvertUB>::CreateUniformBufferImmediate(UB, UniformBuffer_SingleFrame);
	SetUniformBufferParameter(BatchedParameters, GetUniformBufferParameter<FYCbCrConvertUB>(), Data);
}


/* FUYVYConvertPS shader
 *****************************************************************************/

BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT(FUYVYConvertUB, )
SHADER_PARAMETER(FMatrix44f, ColorTransform)
SHADER_PARAMETER(uint32, SrgbToLinear)
SHADER_PARAMETER(uint32, Width)
SHADER_PARAMETER_TEXTURE(Texture2D, Texture)
SHADER_PARAMETER_SAMPLER(SamplerState, SamplerB)
SHADER_PARAMETER_SAMPLER(SamplerState, SamplerP)
END_GLOBAL_SHADER_PARAMETER_STRUCT()

IMPLEMENT_GLOBAL_SHADER_PARAMETER_STRUCT(FUYVYConvertUB, "UYVYConvertUB");
IMPLEMENT_SHADER_TYPE(, FUYVYConvertPS, TEXT("/Engine/Private/MediaShaders.usf"), TEXT("UYVYConvertPS"), SF_Pixel);


void FUYVYConvertPS::SetParameters(FRHIBatchedShaderParameters& BatchedParameters, TRefCountPtr<FRHITexture> UYVYTexture, const FMatrix& ColorTransform, const FVector& YUVOffset, bool SrgbToLinear)
{
	FUYVYConvertUB UB;
	{
		UB.ColorTransform = (FMatrix44f)MediaShaders::CombineColorTransformAndOffset(ColorTransform, YUVOffset);
		UB.SamplerB = TStaticSamplerState<SF_Bilinear>::GetRHI();
		UB.SamplerP = TStaticSamplerState<SF_Point>::GetRHI();
		UB.SrgbToLinear = SrgbToLinear;
		UB.Texture = UYVYTexture;
		UB.Width = UYVYTexture->GetSizeX();
	}

	TUniformBufferRef<FUYVYConvertUB> Data = TUniformBufferRef<FUYVYConvertUB>::CreateUniformBufferImmediate(UB, UniformBuffer_SingleFrame);
	SetUniformBufferParameter(BatchedParameters, GetUniformBufferParameter<FUYVYConvertUB>(), Data);
}


/* FYUVConvertPS shader
 *****************************************************************************/

BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT(FYUVConvertUB, )
SHADER_PARAMETER(FMatrix44f, ColorTransform)
SHADER_PARAMETER(uint32, SrgbToLinear)
SHADER_PARAMETER(FVector2f, UVScale)
SHADER_PARAMETER_TEXTURE(Texture2D, YTexture)
SHADER_PARAMETER_TEXTURE(Texture2D, UTexture)
SHADER_PARAMETER_TEXTURE(Texture2D, VTexture)
SHADER_PARAMETER_SAMPLER(SamplerState, YSampler)
SHADER_PARAMETER_SAMPLER(SamplerState, USampler)
SHADER_PARAMETER_SAMPLER(SamplerState, VSampler)
END_GLOBAL_SHADER_PARAMETER_STRUCT()

IMPLEMENT_GLOBAL_SHADER_PARAMETER_STRUCT(FYUVConvertUB, "YUVConvertUB");
IMPLEMENT_SHADER_TYPE(, FYUVConvertPS, TEXT("/Engine/Private/MediaShaders.usf"), TEXT("YUVConvertPS"), SF_Pixel);


void FYUVConvertPS::SetParameters(FRHIBatchedShaderParameters& BatchedParameters, TRefCountPtr<FRHITexture> YTexture, TRefCountPtr<FRHITexture> UTexture, TRefCountPtr<FRHITexture> VTexture, const FIntPoint& OutputDimensions, const FMatrix& ColorTransform, const FVector& YUVOffset, bool SrgbToLinear)
{
	FYUVConvertUB UB;
	{
		UB.ColorTransform = (FMatrix44f)MediaShaders::CombineColorTransformAndOffset(ColorTransform, YUVOffset);
		UB.SrgbToLinear = SrgbToLinear;
		UB.YTexture = YTexture;
		UB.UTexture = UTexture;
		UB.VTexture = VTexture;
		UB.YSampler = TStaticSamplerState<SF_Bilinear>::GetRHI();
		UB.USampler = TStaticSamplerState<SF_Bilinear>::GetRHI();
		UB.VSampler = TStaticSamplerState<SF_Bilinear>::GetRHI();
		UB.UVScale = FVector2f((float) OutputDimensions.X / (float) YTexture->GetSizeX(), (float) OutputDimensions.Y / (float) YTexture->GetSizeY());
	}

	TUniformBufferRef<FYUVConvertUB> Data = TUniformBufferRef<FYUVConvertUB>::CreateUniformBufferImmediate(UB, UniformBuffer_SingleFrame);
	SetUniformBufferParameter(BatchedParameters, GetUniformBufferParameter<FYUVConvertUB>(), Data);
}


/* FYUVScaledConvertPS shader
 *****************************************************************************/

BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT(FYUVScaledConvertUB, )
SHADER_PARAMETER(FMatrix44f, ColorTransform)
SHADER_PARAMETER(uint32, SrgbToLinear)
SHADER_PARAMETER(FVector2f, UVScale)
SHADER_PARAMETER(float, YUVScale)
SHADER_PARAMETER_TEXTURE(Texture2D, YTexture)
SHADER_PARAMETER_TEXTURE(Texture2D, UTexture)
SHADER_PARAMETER_TEXTURE(Texture2D, VTexture)
SHADER_PARAMETER_SAMPLER(SamplerState, YSampler)
SHADER_PARAMETER_SAMPLER(SamplerState, USampler)
SHADER_PARAMETER_SAMPLER(SamplerState, VSampler)
END_GLOBAL_SHADER_PARAMETER_STRUCT()

IMPLEMENT_GLOBAL_SHADER_PARAMETER_STRUCT(FYUVScaledConvertUB, "YUVScaledConvertUB");
IMPLEMENT_SHADER_TYPE(, FYUVScaledConvertPS, TEXT("/Engine/Private/MediaShaders.usf"), TEXT("YUVScaledConvertPS"), SF_Pixel);


void FYUVScaledConvertPS::SetParameters(FRHIBatchedShaderParameters& BatchedParameters, TRefCountPtr<FRHITexture> YTexture, TRefCountPtr<FRHITexture> UTexture, TRefCountPtr<FRHITexture> VTexture, const FIntPoint& OutputDimensions, const FMatrix& ColorTransform, const FVector& YUVOffset, bool SrgbToLinear, float YUVScale)
{
	FYUVScaledConvertUB UB;
	{
		UB.ColorTransform = (FMatrix44f)MediaShaders::CombineColorTransformAndOffset(ColorTransform, YUVOffset);
		UB.SrgbToLinear = SrgbToLinear;
		UB.YTexture = YTexture;
		UB.UTexture = UTexture;
		UB.VTexture = VTexture;
		UB.YSampler = TStaticSamplerState<SF_Bilinear>::GetRHI();
		UB.USampler = TStaticSamplerState<SF_Bilinear>::GetRHI();
		UB.VSampler = TStaticSamplerState<SF_Bilinear>::GetRHI();
		UB.UVScale = FVector2f((float) OutputDimensions.X / (float) YTexture->GetSizeX(), (float) OutputDimensions.Y / (float) YTexture->GetSizeY());
		UB.YUVScale = YUVScale;
	}

	TUniformBufferRef<FYUVScaledConvertUB> Data = TUniformBufferRef<FYUVScaledConvertUB>::CreateUniformBufferImmediate(UB, UniformBuffer_SingleFrame);
	SetUniformBufferParameter(BatchedParameters, GetUniformBufferParameter<FYUVScaledConvertUB>(), Data);
}


/* FYUVv216ConvertPS shader
 *****************************************************************************/

BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT(FYUVv216ConvertUB, )
SHADER_PARAMETER(FMatrix44f, ColorTransform)
SHADER_PARAMETER(FMatrix44f, CSTransform)
SHADER_PARAMETER(uint32, EOTF)
SHADER_PARAMETER(uint32, ToneMapMethod)
SHADER_PARAMETER(uint32, IsCbY0CrY1)
SHADER_PARAMETER(uint32, IsARGBFmt)
SHADER_PARAMETER(uint32, SwapChroma)
SHADER_PARAMETER(uint32, UseBilinearSampler)
SHADER_PARAMETER_SAMPLER(SamplerState, SamplerB)
SHADER_PARAMETER(float, OutputDimX)
SHADER_PARAMETER(float, OutputDimY)
SHADER_PARAMETER_TEXTURE(Texture2D<float4>, YUVTexture)
END_GLOBAL_SHADER_PARAMETER_STRUCT()

IMPLEMENT_GLOBAL_SHADER_PARAMETER_STRUCT(FYUVv216ConvertUB, "YUVv216ConvertUB");
IMPLEMENT_SHADER_TYPE(, FYUVv216ConvertPS, TEXT("/Engine/Private/MediaShaders.usf"), TEXT("YUVv216ConvertPS"), SF_Pixel);


void FYUVv216ConvertPS::SetParameters(FRHIBatchedShaderParameters& BatchedParameters, TRefCountPtr<FRHITexture> YUVTexture, const FIntPoint& OutputDimensions, const FMatrix44f& ColorTransform, UE::Color::EEncoding Encoding, const FMatrix44f& CSTransform, bool bIsCbY0CrY1, bool bIsARGBFmt, bool bSwapChroma, MediaShaders::EToneMapMethod ToneMapMethod, bool bUseBilinearSamplingForChroma)
{
	FYUVv216ConvertUB UB;
	{
		UB.ColorTransform = ColorTransform;
		UB.CSTransform = CSTransform;
		UB.EOTF = static_cast<uint32>(Encoding);
		UB.ToneMapMethod = static_cast<uint32>(ToneMapMethod);
		UB.IsCbY0CrY1 = bIsCbY0CrY1;
		UB.IsARGBFmt = bIsARGBFmt;
		UB.SwapChroma = bSwapChroma;
		UB.UseBilinearSampler = bUseBilinearSamplingForChroma;
		UB.SamplerB = TStaticSamplerState<SF_Bilinear>::GetRHI();
		UB.OutputDimX = (float)OutputDimensions.X;
		UB.OutputDimY = (float)OutputDimensions.Y;
		UB.YUVTexture = YUVTexture;
	}

	TUniformBufferRef<FYUVv216ConvertUB> Data = TUniformBufferRef<FYUVv216ConvertUB>::CreateUniformBufferImmediate(UB, UniformBuffer_SingleFrame);
	SetUniformBufferParameter(BatchedParameters, GetUniformBufferParameter<FYUVv216ConvertUB>(), Data);
}


/* FYUVv210ConvertPS shader
 *****************************************************************************/

BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT(FYUVv210ConvertUB, )
SHADER_PARAMETER(FMatrix44f, ColorTransform)
SHADER_PARAMETER(FMatrix44f, CSTransform)
SHADER_PARAMETER(uint32, EOTF)
SHADER_PARAMETER(uint32, ToneMapMethod)
SHADER_PARAMETER(uint32, IsCbY0CrY1)
SHADER_PARAMETER(uint32, InterpolateChroma)
SHADER_PARAMETER(float, OutputDimX)
SHADER_PARAMETER(float, OutputDimY)
SHADER_PARAMETER_TEXTURE(Texture2D<float4>, YUVTexture)
END_GLOBAL_SHADER_PARAMETER_STRUCT()

IMPLEMENT_GLOBAL_SHADER_PARAMETER_STRUCT(FYUVv210ConvertUB, "YUVv210ConvertUB");
IMPLEMENT_SHADER_TYPE(, FYUVv210ConvertPS, TEXT("/Engine/Private/MediaShaders.usf"), TEXT("YUVv210ConvertPS"), SF_Pixel);


void FYUVv210ConvertPS::SetParameters(FRHIBatchedShaderParameters& BatchedParameters, TRefCountPtr<FRHITexture> YUVTexture, const FIntPoint& OutputDimensions, const FMatrix44f& ColorTransform, UE::Color::EEncoding Encoding, const FMatrix44f& CSTransform, bool bIsCbY0CrY1, MediaShaders::EToneMapMethod ToneMapMethod, MediaShaders::EChromaInterpolation ChromaInterpolation)
{
	FYUVv210ConvertUB UB;
	{
		UB.ColorTransform = ColorTransform;
		UB.CSTransform = CSTransform;
		UB.EOTF = static_cast<uint32>(Encoding);
		UB.ToneMapMethod = static_cast<uint32>(ToneMapMethod);
		UB.IsCbY0CrY1 = bIsCbY0CrY1;
		UB.InterpolateChroma = ResolveChromaInterpolation(ChromaInterpolation);
		UB.OutputDimX = (float)OutputDimensions.X;
		UB.OutputDimY = (float)OutputDimensions.Y;
		UB.YUVTexture = YUVTexture;
	}

	TUniformBufferRef<FYUVv210ConvertUB> Data = TUniformBufferRef<FYUVv210ConvertUB>::CreateUniformBufferImmediate(UB, UniformBuffer_SingleFrame);
	SetUniformBufferParameter(BatchedParameters, GetUniformBufferParameter<FYUVv210ConvertUB>(), Data);
}


/* FYUVY416ConvertPS shader
 *****************************************************************************/

BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT(FYUVY416ConvertUB, )
SHADER_PARAMETER(FMatrix44f, ColorTransform)
SHADER_PARAMETER(FMatrix44f, CSTransform)
SHADER_PARAMETER_SRV(Texture2D, SRV_Y)
SHADER_PARAMETER_SAMPLER(SamplerState, BilinearClampedSamplerUV)
SHADER_PARAMETER(uint32, EOTF)				// UE::Color::EEncoding value (see ColorManagementDefines.h).
SHADER_PARAMETER(uint32, ToneMapMethod)
SHADER_PARAMETER(uint32, bIsARGBFmt)
END_GLOBAL_SHADER_PARAMETER_STRUCT()

IMPLEMENT_GLOBAL_SHADER_PARAMETER_STRUCT(FYUVY416ConvertUB, "YUVY416ConvertUB");
IMPLEMENT_SHADER_TYPE(, FYUVY416ConvertPS, TEXT("/Engine/Private/MediaShaders.usf"), TEXT("YUVY416ConvertPS"), SF_Pixel);


void FYUVY416ConvertPS::SetParameters(FRHIBatchedShaderParameters& BatchedParameters, FShaderResourceViewRHIRef SRV_Y, const FMatrix44f& ColorTransform, UE::Color::EEncoding Encoding, const FMatrix44f& CSTransform, bool bIsARGBFmt, MediaShaders::EToneMapMethod ToneMapMethod)
{
	FYUVY416ConvertUB UB;
	{
		UB.SRV_Y = SRV_Y;
		UB.BilinearClampedSamplerUV = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
		UB.ColorTransform = ColorTransform;
		UB.CSTransform = CSTransform;
		UB.EOTF = static_cast<uint32>(Encoding);
		UB.ToneMapMethod = static_cast<uint32>(ToneMapMethod);
		UB.bIsARGBFmt = bIsARGBFmt;
	}

	TUniformBufferRef<FYUVY416ConvertUB> Data = TUniformBufferRef<FYUVY416ConvertUB>::CreateUniformBufferImmediate(UB, UniformBuffer_SingleFrame);
	SetUniformBufferParameter(BatchedParameters, GetUniformBufferParameter<FYUVY416ConvertUB>(), Data);
}

/* FARGB16BigConvertPS shader
 *****************************************************************************/

BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT(FARGB16BigConvertUB, )
SHADER_PARAMETER(FMatrix44f, CSTransform)
SHADER_PARAMETER(uint32, OutputWidth)
SHADER_PARAMETER(uint32, OutputHeight)
SHADER_PARAMETER_SRV(Texture2D<uint4>, Texture)
SHADER_PARAMETER(uint32, EOTF)				// UE::Color::EEncoding value (see ColorManagementDefines.h).
SHADER_PARAMETER(uint32, ToneMapMethod)
END_GLOBAL_SHADER_PARAMETER_STRUCT()

IMPLEMENT_GLOBAL_SHADER_PARAMETER_STRUCT(FARGB16BigConvertUB, "ARGB16BigConvertUB");
IMPLEMENT_SHADER_TYPE(, FARGB16BigConvertPS, TEXT("/Engine/Private/MediaShaders.usf"), TEXT("ARGB16BigConvertPS"), SF_Pixel);


void FARGB16BigConvertPS::SetParameters(FRHIBatchedShaderParameters& BatchedParameters, FShaderResourceViewRHIRef SRV, const FIntPoint& OutputDimensions, UE::Color::EEncoding Encoding, const FMatrix44f& CSTransform, MediaShaders::EToneMapMethod ToneMapMethod)
{
	FARGB16BigConvertUB UB;
	{
		UB.Texture = SRV;
		UB.OutputWidth = OutputDimensions.X;
		UB.OutputHeight = OutputDimensions.Y;
		UB.CSTransform = CSTransform;
		UB.EOTF = static_cast<uint32>(Encoding);
		UB.ToneMapMethod = static_cast<uint32>(ToneMapMethod);
	}

	TUniformBufferRef<FARGB16BigConvertUB> Data = TUniformBufferRef<FARGB16BigConvertUB>::CreateUniformBufferImmediate(UB, UniformBuffer_SingleFrame);
	SetUniformBufferParameter(BatchedParameters, GetUniformBufferParameter<FARGB16BigConvertUB>(), Data);
}

/* FYUY2ConvertPS shader
 *****************************************************************************/

BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT(FYUY2ConvertUB, )
SHADER_PARAMETER(FMatrix44f, ColorTransform)
SHADER_PARAMETER(uint32, OutputWidth)
SHADER_PARAMETER(uint32, SrgbToLinear)
SHADER_PARAMETER(FVector2f, UVScale)
SHADER_PARAMETER_TEXTURE(Texture2D, Texture)
SHADER_PARAMETER_SAMPLER(SamplerState, SamplerB)
SHADER_PARAMETER_SAMPLER(SamplerState, SamplerP)
END_GLOBAL_SHADER_PARAMETER_STRUCT()

IMPLEMENT_GLOBAL_SHADER_PARAMETER_STRUCT(FYUY2ConvertUB, "YUY2ConvertUB");
IMPLEMENT_SHADER_TYPE(, FYUY2ConvertPS, TEXT("/Engine/Private/MediaShaders.usf"), TEXT("YUY2ConvertPS"), SF_Pixel);


void FYUY2ConvertPS::SetParameters(FRHIBatchedShaderParameters& BatchedParameters, TRefCountPtr<FRHITexture> YUY2Texture, const FIntPoint& OutputDimensions, const FMatrix& ColorTransform, const FVector& YUVOffset, bool SrgbToLinear, MediaShaders::EToneMapMethod ToneMapMethod)
{
	FYUY2ConvertUB UB;
	{
		UB.ColorTransform = (FMatrix44f)MediaShaders::CombineColorTransformAndOffset(ColorTransform, YUVOffset);
		UB.OutputWidth = OutputDimensions.X;
		UB.SamplerB = TStaticSamplerState<SF_Bilinear>::GetRHI();
		UB.SamplerP = TStaticSamplerState<SF_Point>::GetRHI();
		UB.SrgbToLinear = SrgbToLinear;
		UB.Texture = YUY2Texture;
		UB.UVScale = FVector2f((float)OutputDimensions.X / (2.0f * YUY2Texture->GetSizeX()), (float)OutputDimensions.Y / (float)YUY2Texture->GetSizeY());
	}

	TUniformBufferRef<FYUY2ConvertUB> Data = TUniformBufferRef<FYUY2ConvertUB>::CreateUniformBufferImmediate(UB, UniformBuffer_SingleFrame);
	SetUniformBufferParameter(BatchedParameters, GetUniformBufferParameter<FYUY2ConvertUB>(), Data);
}

/* FYVYUConvertPS shader
 *****************************************************************************/

BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT(FYVYUConvertUB, )
SHADER_PARAMETER(FMatrix44f, ColorTransform)
SHADER_PARAMETER(uint32, SrgbToLinear)
SHADER_PARAMETER(uint32, Width)
SHADER_PARAMETER_TEXTURE(Texture2D, Texture)
SHADER_PARAMETER_SAMPLER(SamplerState, SamplerB)
SHADER_PARAMETER_SAMPLER(SamplerState, SamplerP)
END_GLOBAL_SHADER_PARAMETER_STRUCT()

IMPLEMENT_GLOBAL_SHADER_PARAMETER_STRUCT(FYVYUConvertUB, "YVYUConvertUB");
IMPLEMENT_SHADER_TYPE(, FYVYUConvertPS, TEXT("/Engine/Private/MediaShaders.usf"), TEXT("YVYUConvertPS"), SF_Pixel);


void FYVYUConvertPS::SetParameters(FRHIBatchedShaderParameters& BatchedParameters, TRefCountPtr<FRHITexture> YVYUTexture, const FMatrix& ColorTransform, const FVector& YUVOffset, bool SrgbToLinear)
{
	FYVYUConvertUB UB;
	{
		UB.ColorTransform = (FMatrix44f)MediaShaders::CombineColorTransformAndOffset(ColorTransform, YUVOffset);
		UB.SamplerB = TStaticSamplerState<SF_Bilinear>::GetRHI();
		UB.SamplerP = TStaticSamplerState<SF_Point>::GetRHI();
		UB.SrgbToLinear = SrgbToLinear;
		UB.Texture = YVYUTexture;
		UB.Width = YVYUTexture->GetSizeX();
	}

	TUniformBufferRef<FYVYUConvertUB> Data = TUniformBufferRef<FYVYUConvertUB>::CreateUniformBufferImmediate(UB, UniformBuffer_SingleFrame);
	SetUniformBufferParameter(BatchedParameters, GetUniformBufferParameter<FYVYUConvertUB>(), Data);
}


/* FRGB8toY8ConvertPS shader
 *****************************************************************************/

BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT(FRGB8toY8ConvertUB, )
SHADER_PARAMETER(FVector4f, ColorTransform)
SHADER_PARAMETER(uint32, LinearToSrgb)
SHADER_PARAMETER_TEXTURE(Texture2D, Texture)
SHADER_PARAMETER_SAMPLER(SamplerState, SamplerP)
END_GLOBAL_SHADER_PARAMETER_STRUCT()

IMPLEMENT_GLOBAL_SHADER_PARAMETER_STRUCT(FRGB8toY8ConvertUB, "RGB8toY8ConvertUB");
IMPLEMENT_SHADER_TYPE(, FRGB8toY8ConvertPS, TEXT("/Engine/Private/MediaShaders.usf"), TEXT("RGB8toY8ConvertPS"), SF_Pixel);


void FRGB8toY8ConvertPS::SetParameters(FRHIBatchedShaderParameters& BatchedParameters, TRefCountPtr<FRHITexture> RGBATexture, const FVector4f& ColorTransform, bool LinearToSrgb)
{
	FRGB8toY8ConvertUB UB;
	{
		UB.ColorTransform = ColorTransform;
		UB.SamplerP = TStaticSamplerState<SF_Point>::GetRHI();
		UB.LinearToSrgb = LinearToSrgb;
		UB.Texture = RGBATexture;
	}

	TUniformBufferRef<FRGB8toY8ConvertUB> Data = TUniformBufferRef<FRGB8toY8ConvertUB>::CreateUniformBufferImmediate(UB, UniformBuffer_SingleFrame);
	SetUniformBufferParameter(BatchedParameters, GetUniformBufferParameter<FRGB8toY8ConvertUB>(), Data);
}


/* FReadTextureExternalPS shader
 *****************************************************************************/

BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT(FReadTextureExternalUB, )
SHADER_PARAMETER_SAMPLER(SamplerState, SamplerP)
SHADER_PARAMETER(FLinearColor, ScaleRotation)
SHADER_PARAMETER(FVector2f, Offset)
END_GLOBAL_SHADER_PARAMETER_STRUCT()

IMPLEMENT_GLOBAL_SHADER_PARAMETER_STRUCT(FReadTextureExternalUB, "ReadTextureExternalUB");
IMPLEMENT_SHADER_TYPE(, FReadTextureExternalPS, TEXT("/Engine/Private/MediaShaders.usf"), TEXT("ReadTextureExternalPS"), SF_Pixel);


void FReadTextureExternalPS::SetParameters(FRHIBatchedShaderParameters& BatchedParameters, FTextureRHIRef TextureExt, FSamplerStateRHIRef SamplerState, const FLinearColor& ScaleRotation, const FLinearColor& Offset)
{
	FReadTextureExternalUB UB;
	{
		UB.SamplerP = SamplerState;
		UB.ScaleRotation = ScaleRotation;
		UB.Offset = FVector2f(Offset.R, Offset.G);
	}

	TUniformBufferRef<FReadTextureExternalUB> Data = TUniformBufferRef<FReadTextureExternalUB>::CreateUniformBufferImmediate(UB, UniformBuffer_SingleFrame);
	SetUniformBufferParameter(BatchedParameters, GetUniformBufferParameter<FReadTextureExternalUB>(), Data);
	SetTextureParameter(BatchedParameters, InTexture, TextureExt);
}

/* FRGB8toUYVY8ConvertPS shader
 *****************************************************************************/

IMPLEMENT_GLOBAL_SHADER(FRGB8toUYVY8ConvertPS, "/Engine/Private/MediaShaders.usf", "RGB8toUYVY8ConvertPS", SF_Pixel);

FRGB8toUYVY8ConvertPS::FParameters* FRGB8toUYVY8ConvertPS::AllocateAndSetParameters(FRDGBuilder& GraphBuilder, FRDGTextureRef RGBATexture, const FMatrix& ColorTransform, const FVector& YUVOffset, bool bDoLinearToSrgb, FRDGTextureRef OutputTexture)
{
	FRGB8toUYVY8ConvertPS::FParameters* Parameters = GraphBuilder.AllocParameters<FRGB8toUYVY8ConvertPS::FParameters>();

	Parameters->RGBToYUVConversion.InputTexture = RGBATexture;
	Parameters->RGBToYUVConversion.InputSampler = TStaticSamplerState<SF_Point>::GetRHI();
	Parameters->RGBToYUVConversion.ColorTransform = (FMatrix44f)MediaShaders::CombineColorTransformAndOffset(ColorTransform, YUVOffset);
	Parameters->RGBToYUVConversion.DoLinearToSrgb = bDoLinearToSrgb;
	Parameters->RGBToYUVConversion.OnePixelDeltaX = 1.0f / (float)RGBATexture->Desc.Extent.X;
	Parameters->RenderTargets[0] = FRenderTargetBinding{ OutputTexture, ERenderTargetLoadAction::ENoAction };

	return Parameters;
}


/* FRGB10toYUVv210ConvertPS shader
 *****************************************************************************/

IMPLEMENT_GLOBAL_SHADER(FRGB10toYUVv210ConvertPS, "/Engine/Private/MediaShaders.usf", "RGB10toYUVv210ConvertPS", SF_Pixel);

FRGB10toYUVv210ConvertPS::FParameters* FRGB10toYUVv210ConvertPS::AllocateAndSetParameters(FRDGBuilder& GraphBuilder, FRDGTextureRef RGBATexture, const FMatrix& ColorTransform, const FVector& YUVOffset, bool bDoLinearToSrgb, FRDGTextureRef OutputTexture)
{
	FRGB10toYUVv210ConvertPS::FParameters* Parameters = GraphBuilder.AllocParameters<FRGB10toYUVv210ConvertPS::FParameters>();

	Parameters->RGBToYUVConversion.InputTexture = RGBATexture;
	Parameters->RGBToYUVConversion.InputSampler = TStaticSamplerState<SF_Point>::GetRHI();
	Parameters->RGBToYUVConversion.ColorTransform = (FMatrix44f)MediaShaders::CombineColorTransformAndOffset(ColorTransform, YUVOffset);
	Parameters->RGBToYUVConversion.DoLinearToSrgb = bDoLinearToSrgb;
	Parameters->RGBToYUVConversion.OnePixelDeltaX = 1.0f / (float)RGBATexture->Desc.Extent.X;

	//Output texture will be based on a size dividable by 48 (i.e 1280 -> 1296) and divided by 6 (i.e 1296 / 6 = 216)
	//To map output texture UVs, we get a scale from the source texture original size to the mapped output size
	//And use the source texture size to get the pixel delta
	const float PaddedResolution = float(uint32((RGBATexture->Desc.Extent.X + 47) / 48) * 48);
	Parameters->PaddingScale = PaddedResolution / (float)RGBATexture->Desc.Extent.X;

	Parameters->RenderTargets[0] = FRenderTargetBinding{ OutputTexture, ERenderTargetLoadAction::ENoAction };

	return Parameters;
}


/* FModifyAlphaSwizzleRgbaPS shader
 *****************************************************************************/

IMPLEMENT_GLOBAL_SHADER(FModifyAlphaSwizzleRgbaPS, "/Engine/Private/MediaShaders.usf", "SwizzleRgbPS", SF_Pixel);

FModifyAlphaSwizzleRgbaPS::FParameters* FModifyAlphaSwizzleRgbaPS::AllocateAndSetParameters(FRDGBuilder& GraphBuilder, FRDGTextureRef RGBATexture, FRDGTextureRef OutputTexture)
{
	FModifyAlphaSwizzleRgbaPS::FParameters* Parameters = GraphBuilder.AllocParameters<FModifyAlphaSwizzleRgbaPS::FParameters>();

	Parameters->InputTexture = RGBATexture;
	Parameters->InputSampler = TStaticSamplerState<SF_Point>::GetRHI();
	Parameters->RenderTargets[0] = FRenderTargetBinding{ OutputTexture, ERenderTargetLoadAction::ENoAction };

	return Parameters;
}


/* FVYUConvertPS shader
 *****************************************************************************/

BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT(FVYUConvertUB, )
SHADER_PARAMETER(FMatrix44f, ColorTransform)
SHADER_PARAMETER(FMatrix44f, CSTransform)
SHADER_PARAMETER(uint32, EOTF)
SHADER_PARAMETER(uint32, ToneMapMethod)
SHADER_PARAMETER(FLinearColor, UVRotation)
SHADER_PARAMETER(FLinearColor, UVRotOffsetAndScale)
SHADER_PARAMETER_TEXTURE(Texture2D, Texture)
SHADER_PARAMETER_SAMPLER(SamplerState, Sampler)
END_GLOBAL_SHADER_PARAMETER_STRUCT()

IMPLEMENT_GLOBAL_SHADER_PARAMETER_STRUCT(FVYUConvertUB, "VYUConvertUB");
IMPLEMENT_SHADER_TYPE(, FVYUConvertPS, TEXT("/Engine/Private/MediaShaders.usf"), TEXT("VYUConvertPS"), SF_Pixel);


void FVYUConvertPS::SetParameters(FRHIBatchedShaderParameters& BatchedParameters, TRefCountPtr<FRHITexture> VYUTexture, const FIntPoint& OutputDimensions, const FMatrix44f& ColorTransform, UE::Color::EEncoding Encoding, const FMatrix44f& CSTransform,
	MediaShaders::EToneMapMethod ToneMapMethod, const FLinearColor& UVRotation, const FLinearColor& UVRotOffset)
{
	FVYUConvertUB UB;
	{
		UB.ColorTransform = ColorTransform;
		UB.CSTransform = CSTransform;
		UB.EOTF = static_cast<uint32>(Encoding);
		UB.ToneMapMethod = static_cast<uint32>(ToneMapMethod);
		UB.UVRotation = UVRotation;
		UB.UVRotOffsetAndScale = FLinearColor(UVRotOffset.R, UVRotOffset.G, (float)OutputDimensions.X / (float)VYUTexture->GetSizeX(), (float)OutputDimensions.Y / (float)VYUTexture->GetSizeY());
		UB.Texture = VYUTexture;
		UB.Sampler = TStaticSamplerState<SF_Bilinear>::GetRHI();
	}

	TUniformBufferRef<FVYUConvertUB> Data = TUniformBufferRef<FVYUConvertUB>::CreateUniformBufferImmediate(UB, UniformBuffer_SingleFrame);
	SetUniformBufferParameter(BatchedParameters, GetUniformBufferParameter<FVYUConvertUB>(), Data);
}
