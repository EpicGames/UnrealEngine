// Copyright Epic Games, Inc. All Rights Reserved.

#include "OculusShaders.h"

PRAGMA_DISABLE_DEPRECATION_WARNINGS
IMPLEMENT_SHADER_TYPE(, FOculusVertexShader, TEXT("/Engine/Private/OculusShaders.usf"), TEXT("MainVertexShader"), SF_Vertex);
IMPLEMENT_SHADER_TYPE(, FOculusWhiteShader, TEXT("/Engine/Private/OculusShaders.usf"), TEXT("MainWhiteShader"), SF_Pixel);
IMPLEMENT_SHADER_TYPE(, FOculusBlackShader, TEXT("/Engine/Private/OculusShaders.usf"), TEXT("MainBlackShader"), SF_Pixel);
IMPLEMENT_SHADER_TYPE(, FOculusAlphaInverseShader, TEXT("/Engine/Private/OculusShaders.usf"), TEXT("MainAlphaInverseShader"), SF_Pixel);
IMPLEMENT_SHADER_TYPE(, FOculusCubemapPS, TEXT("/Engine/Private/OculusShaders.usf"), TEXT("MainForCubemap"), SF_Pixel);
PRAGMA_ENABLE_DEPRECATION_WARNINGS