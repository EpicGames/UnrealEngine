// Copyright Epic Games, Inc. All Rights Reserved.

#include "OneColorShader.h"
#include "DataDrivenShaderPlatformInfo.h"

// #define avoids a lot of code duplication
#define IMPLEMENT_ONECOLORVS(A,B) typedef TOneColorVS<A,B> TOneColorVS##A##B; \
	IMPLEMENT_SHADER_TYPE4_WITH_TEMPLATE_PREFIX(template<>, RENDERCORE_API, TOneColorVS##A##B, SF_Vertex);

IMPLEMENT_ONECOLORVS(false,false)
IMPLEMENT_ONECOLORVS(false,true)
IMPLEMENT_ONECOLORVS(true,true)
IMPLEMENT_ONECOLORVS(true,false)
#undef IMPLEMENT_ONECOLORVS

void FOneColorPS::FillParameters(FParameters& Parameters, const FLinearColor* Colors, int32 NumColors)
{
	check(NumColors <= MaxSimultaneousRenderTargets);

	int32 Index = 0;

	for (; Index < NumColors; ++Index)
	{
		Parameters.DrawColorMRT[Index] = Colors[Index];
	}

	for (; Index < MaxSimultaneousRenderTargets; ++Index)
	{
		Parameters.DrawColorMRT[Index] = FLinearColor::Black;
	}
}

IMPLEMENT_GLOBAL_SHADER(FOneColorPS,"/Engine/Private/OneColorShader.usf","MainPixelShader",SF_Pixel);

TOneColorPixelShaderMRT::TOneColorPixelShaderMRT() = default;
TOneColorPixelShaderMRT::TOneColorPixelShaderMRT(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
	: FOneColorPS(Initializer)
{
}

bool TOneColorPixelShaderMRT::ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
{
	FPermutationDomain PermutationVector(Parameters.PermutationId);

	const int32 NumOutputs = PermutationVector.Get<TOneColorPixelShaderMRT::TOneColorPixelShaderNumOutputs>();
	const int32 NumUintOutputs = PermutationVector.Get<TOneColorPixelShaderMRT::TOneColorPixelNumUintOutputs>();
	if (NumOutputs < NumUintOutputs)
	{
		return false;
	}

	if (PermutationVector.Get<TOneColorPixelShaderMRT::TOneColorPixelShaderNumOutputs>())
	{
		return (PermutationVector.Get<TOneColorPixelShaderMRT::TOneColorPixelShader128bitRT>() ? FDataDrivenShaderPlatformInfo::GetRequiresExplicit128bitRT(Parameters.Platform) : true);
	}

	return true;
}

void TOneColorPixelShaderMRT::ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
{
	FOneColorPS::ModifyCompilationEnvironment(Parameters, OutEnvironment);

	FPermutationDomain PermutationVector(Parameters.PermutationId);
	if (PermutationVector.Get<TOneColorPixelShaderMRT::TOneColorPixelShader128bitRT>())
	{
		OutEnvironment.SetRenderTargetOutputFormat(0, PF_A32B32G32R32F);
	}

	const int32 NumOutputs = PermutationVector.Get<TOneColorPixelShaderMRT::TOneColorPixelShaderNumOutputs>();
	const int32 NumUintOutputs = PermutationVector.Get<TOneColorPixelShaderMRT::TOneColorPixelNumUintOutputs>();
	const int32 UintOutputStartIndex = NumOutputs - NumUintOutputs;
	for (int32 UintTargetIndex = 0; UintTargetIndex < NumUintOutputs; UintTargetIndex++)
	{
		OutEnvironment.SetRenderTargetOutputFormat(UintOutputStartIndex + UintTargetIndex, PF_R32_UINT);
	}
}

// Compiling a version for every number of MRT's
// On AMD PC hardware, outputting to a color index in the shader without a matching render target set has a significant performance hit
IMPLEMENT_GLOBAL_SHADER(TOneColorPixelShaderMRT,"/Engine/Private/OneColorShader.usf","MainPixelShaderMRT",SF_Pixel);


FFillTextureCS::FFillTextureCS() = default;
FFillTextureCS::FFillTextureCS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
	: FGlobalShader(Initializer)
{
	FillValue.Bind(Initializer.ParameterMap, TEXT("FillValue"), SPF_Mandatory);
	Params0.Bind(Initializer.ParameterMap, TEXT("Params0"), SPF_Mandatory);
	Params1.Bind(Initializer.ParameterMap, TEXT("Params1"), SPF_Mandatory);
	Params2.Bind(Initializer.ParameterMap, TEXT("Params2"), SPF_Optional);
	FillTexture.Bind(Initializer.ParameterMap, TEXT("FillTexture"), SPF_Mandatory);
}

bool FFillTextureCS::ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
{
	return true;
}

IMPLEMENT_SHADER_TYPE(,FFillTextureCS,TEXT("/Engine/Private/OneColorShader.usf"),TEXT("MainFillTextureCS"),SF_Compute);
