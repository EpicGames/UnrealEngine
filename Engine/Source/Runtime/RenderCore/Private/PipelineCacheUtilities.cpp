// Copyright Epic Games, Inc. All Rights Reserved.
#include "PipelineCacheUtilities.h"

#if UE_WITH_PIPELINE_CACHE_UTILITIES
#include "Algo/Accumulate.h"
#include "Async/ParallelFor.h"
#include "Async/TaskGraphInterfaces.h"
#include "Misc/Compression.h"
#include "Misc/SecureHash.h"
#include "Serialization/NameAsStringIndexProxyArchive.h"
#include "Serialization/VarInt.h"
#include "PipelineFileCache.h"
#include "Serialization/MemoryWriter.h"
#include "Serialization/MemoryReader.h"
#include "Policies/PrettyJsonPrintPolicy.h"
#include "Serialization/JsonSerializer.h"
#include "Interfaces/ITargetPlatform.h"
#include "Misc/FileHelper.h"
#include "ShaderCodeLibrary.h"

DEFINE_LOG_CATEGORY_STATIC(LogPipelineCacheUtilities, Log, All);

namespace UE
{
namespace PipelineCacheUtilities
{
namespace Private
{
#pragma pack(push)
#pragma pack(1)
	/** Header of the binary stable keys file. */
	struct FStableKeysSerializedHeader
	{
		enum class EMagic : uint64
		{
			Magic = 0x524448534C425453ULL	// STBLSHDR
		};

		enum class EVersion : int32
		{
			Current = 2
		};

		/** Magic to reject other files */
		EMagic		Magic = EMagic::Magic;
		/** Format version */
		EVersion	Version = EVersion::Current;
		/** Number of stable key entries. */
		int64		NumEntries;

		friend FArchive& operator<<(FArchive& Ar, FStableKeysSerializedHeader& Info)
		{
			return Ar << Info.Magic << Info.Version << Info.NumEntries;
		}
	};
#pragma pack(pop)

#if WITH_EDITOR

#pragma pack(push)
#pragma pack(1)
	/** Header of the binary stable pipeline cache file. */
	struct FStablePipelineCacheSerializedHeader
	{
		enum class EMagic : uint64
		{
			Magic = 0x484341434C425453ULL	// STBLCACH
		};

		enum class EVersion : int32
		{
			AddingPipelineCacheVersion = 5,
			AddingDepthBounds = 6,
			AddedWorkGraphFrequencies = 7,
			AddRTPSOShaderBindingLayout = 8,

			Current = AddRTPSOShaderBindingLayout
		};

		/** Magic to reject other files */
		EMagic		Magic = EMagic::Magic;
		/** Format version */
		EVersion	Version = EVersion::Current;

		/** So many things can change underneath, so serialize sizeof of the structure as an extra compatibility check. */
		int32		Sizeof_FPipelineCacheFileFormatPSO = sizeof(FPipelineCacheFileFormatPSO);

		/** Number of stable shader key entries. */
		int64		NumStableKeyEntries;
		/** Number of FPermPerPSO entries. */
		int64		NumPermutationGroups;

		/** Size of the rest of the file to read (this is normally compressed). */
		uint64		DataSize;

		/** UncompressedSize stores the uncompressed size of the rest of the file.
		    The rest of the file is compressed with Zlib (it's unlikely we need any other method). In an unlikely case it's 0, that means that the rest of the file is not compressed. */
		uint64		UncompressedSize;

		/** Target platform as string */
		FString		TargetPlatform;

		/** Compression method: note - as of version 1 at least it is NOT saved into the binary, and assumed to be Zlib when loading.*/
		static FName CompressionMethod;

		friend FArchive& operator<<(FArchive& Ar, FStablePipelineCacheSerializedHeader& Info)
		{
			return Ar << Info.Magic << Info.Version << Info.Sizeof_FPipelineCacheFileFormatPSO << Info.NumStableKeyEntries << Info.NumPermutationGroups << Info.DataSize << Info.UncompressedSize << Info.TargetPlatform;
		}
	};
#pragma pack(pop)

	FName FStablePipelineCacheSerializedHeader::CompressionMethod = NAME_Oodle;

	/**
	 * Implements a proxy archive that serializes FName and FShaderHash as verbatim data or an index (if the same value is repeated).
	 */
	struct FIndexedHashAndFNameProxyArchive : public FNameAsStringIndexProxyArchive
	{
		using Super = FNameAsStringIndexProxyArchive;

		/** When a hash is first encountered, it is added to the table and saved verbatim, otherwise, its index is written. Indices can be looked up from this TSet since it is not compacted. */
		TSet<FShaderHash> HashesSeenOnSave;

		/** Table of hashes that is populated as the archive is being loaded. */
		TArray<FShaderHash> HashesLoaded;

		/**
		 * Creates and initializes a new instance.
		 *
		 * @param InInnerArchive The inner archive to proxy.
		 */
		FIndexedHashAndFNameProxyArchive(FArchive& InInnerArchive)
			: FNameAsStringIndexProxyArchive(InInnerArchive)
		{ }

		/**
		 * FShaderHash values are serialized like a binary stream, so assume every serialization of this size is a FShaderHash.
		 */
		virtual void Serialize(void* V, int64 Length) override
		{
			if (Length == sizeof(FShaderHash))
			{
				FShaderHash Hash;
				if (IsLoading())
				{
					uint64 Index64 = ReadVarUIntFromArchive(InnerArchive);

					// if this is 0, then it was saved verbatim. If not zero, then it refers to the index in the array
					if (Index64 == 0)
					{
						InnerArchive.Serialize(&Hash.Hash, sizeof(Hash.Hash));

						HashesLoaded.Add(Hash);
					}
					else
					{
						int32 Index = static_cast<int32>(Index64 - 1);
						if (Index >= 0 && Index < HashesLoaded.Num())
						{
							Hash = HashesLoaded[Index];
						}
						else
						{
							SetError();
						}
					}

					FMemory::Memcpy(V, &Hash.Hash, sizeof(Hash));
				}
				else
				{
					FMemory::Memcpy(&Hash.Hash, V, sizeof(Hash));

					// We rely on elements' indices in TSet being in the insertion order, which they are now and should remain so in the future.
					FSetElementId Id = HashesSeenOnSave.FindId(Hash);
					if (Id.IsValidId())
					{
						int32 Index = Id.AsInteger();
						WriteVarUIntToArchive(InnerArchive, uint64(Index) + 1);
					}
					else
					{
						WriteVarUIntToArchive(InnerArchive, 0ULL);
						InnerArchive.Serialize(&Hash.Hash, sizeof(Hash.Hash));

						HashesSeenOnSave.Add(Hash);
					}
				}
			}
			else
			{
				return Super::Serialize(V, Length);
			}
		}
	};

#if DO_CHECK
	bool SanityCheckActiveSlots(const FPermsPerPSO& PermDescriptor)
	{
		check(PermDescriptor.PSO != nullptr);
		switch (PermDescriptor.PSO->Type)
		{
			case FPipelineCacheFileFormatPSO::DescriptorType::Compute:
				check(PermDescriptor.ActivePerSlot[SF_Compute]);
				// all the rest should be false
				for (int32 Idx = 0; Idx < UE_ARRAY_COUNT(PermDescriptor.ActivePerSlot); ++Idx)
				{
					check(Idx == SF_Compute || !PermDescriptor.ActivePerSlot[Idx]);
				}
				break;

			case FPipelineCacheFileFormatPSO::DescriptorType::Graphics:
				// all non-graphics should be false
				for (int32 Idx = 0; Idx < UE_ARRAY_COUNT(PermDescriptor.ActivePerSlot); ++Idx)
				{
					check(Idx == SF_Vertex || Idx == SF_Mesh || Idx == SF_Amplification || Idx == SF_Pixel || Idx == SF_Geometry || !PermDescriptor.ActivePerSlot[Idx]);
				}
				break;

			case FPipelineCacheFileFormatPSO::DescriptorType::RayTracing:
				// all non-RT should be false
				for (int32 Idx = 0; Idx < UE_ARRAY_COUNT(PermDescriptor.ActivePerSlot); ++Idx)
				{
					check(Idx == SF_RayGen || Idx == SF_RayMiss || Idx == SF_RayHitGroup || Idx == SF_RayCallable || !PermDescriptor.ActivePerSlot[Idx]);
				}
				break;

			default:
				checkNoEntry();
				break;
		}
		return true;
	}
#endif

	void SaveActiveSlots(FArchive& Ar, const FPermsPerPSO& PermDescriptor)
	{
		static_assert(SF_NumFrequencies <= 16, "Increase the bit width of the underlying format");
		checkf(UE_ARRAY_COUNT(PermDescriptor.ActivePerSlot) <= 16, TEXT("Increase the bit width of the underlying format"));

		uint16 ActiveMask = 0;
		for (int32 Idx = 0; Idx < UE_ARRAY_COUNT(PermDescriptor.ActivePerSlot); ++Idx)
		{
			ActiveMask <<= 1;
			ActiveMask |= (PermDescriptor.ActivePerSlot[Idx] ? 1 : 0);
		}

		Ar << ActiveMask;
	}

	void LoadActiveSlots(FArchive& Ar, FPermsPerPSO& PermDescriptor)
	{
		static_assert(SF_NumFrequencies <= 16, "Increase the bit width of the underlying format");
		checkf(UE_ARRAY_COUNT(PermDescriptor.ActivePerSlot) <= 16, TEXT("Increase the bit width of the underlying format"));

		uint16 ActiveMask = 0;
		Ar << ActiveMask;

		for (int32 Idx = UE_ARRAY_COUNT(PermDescriptor.ActivePerSlot) - 1; Idx >= 0; --Idx)
		{
			PermDescriptor.ActivePerSlot[Idx] = (ActiveMask & 1) != 0;
			ActiveMask >>= 1;
		}
	}

	/** Saves a permutation - total number of shader keys is passed for validation purposes. */
	void SavePermutation(FArchive& Ar, const FPermsPerPSO& PermDescriptor, const UE::PipelineCacheUtilities::FPermutation& Perm, int64 TotalNumberOfShaderKeys)
	{
		check(Ar.IsSaving());
		for (int32 Idx = 0; Idx < UE_ARRAY_COUNT(Perm.Slots); ++Idx)
		{
			if (PermDescriptor.ActivePerSlot[Idx])
			{
				checkf(Perm.Slots[Idx] < TotalNumberOfShaderKeys, TEXT("Slot %d contains impossible stable shader key index %d (more than %lld we have)"),
					Idx, Perm.Slots[Idx], TotalNumberOfShaderKeys);
				WriteVarIntToArchive(Ar, Perm.Slots[Idx]);
			}
		}
	}

	void LoadPermutation(FArchive& Ar, const FPermsPerPSO& PermDescriptor, UE::PipelineCacheUtilities::FPermutation& Perm, int64 TotalNumberOfShaderKeys)
	{
		check(Ar.IsLoading());
		for (int32 Idx = 0; Idx < UE_ARRAY_COUNT(Perm.Slots); ++Idx)
		{
			if (PermDescriptor.ActivePerSlot[Idx])
			{
				int64 StableShaderKeyIndex = ReadVarIntFromArchive(Ar);
				checkf(StableShaderKeyIndex < TotalNumberOfShaderKeys, TEXT("Slot %d would contain impossible stable shader key index %lld (more than %lld we have)"),
					Idx, StableShaderKeyIndex, TotalNumberOfShaderKeys);
				Perm.Slots[Idx] = static_cast<int32>(StableShaderKeyIndex);
			}
			else
			{
				Perm.Slots[Idx] = 0;
			}
		}
	}

	struct FPSOCacheChunkInfo
	{
		enum class EVersion : int32
		{
			Current = 1
		};
	};

#endif // WITH_EDITOR
}
}
}

bool UE::PipelineCacheUtilities::LoadStableKeysFile(const FStringView& Filename, TArray<FStableShaderKeyAndValue>& InOutArray)
{
	TUniquePtr<FArchive> FileArchiveInner(IFileManager::Get().CreateFileReader(*FString::ConstructFromPtrSize(Filename.GetData(), Filename.Len())));
	if (!FileArchiveInner)
	{
		return false;
	}

	TUniquePtr<FArchive> Archive(new FNameAsStringIndexProxyArchive(*FileArchiveInner.Get()));
	UE::PipelineCacheUtilities::Private::FStableKeysSerializedHeader Header;
	UE::PipelineCacheUtilities::Private::FStableKeysSerializedHeader SupportedHeader;

	*Archive << Header;

	if (Header.Magic != SupportedHeader.Magic)
	{
		return false;
	}

	// start restrictive, as the format isn't really forward compatible, nor needs to be
	if (Header.Version != SupportedHeader.Version)
	{
		return false;
	}

	TArray<FShaderHash> Hashes;
	int32 NumHashes;
	*Archive << NumHashes;
	Hashes.AddUninitialized(NumHashes);
	for (int32 IdxHash = 0; IdxHash < NumHashes; ++IdxHash)
	{
		*Archive << Hashes[IdxHash];
	}

	InOutArray.Reserve(Header.NumEntries);
	for (int64 Idx = 0; Idx < Header.NumEntries; ++Idx)
	{
		FStableShaderKeyAndValue Item;
		int8 CompactNamesNum = -1;
		*Archive << CompactNamesNum;
		if (CompactNamesNum > 0)
		{
			Item.ClassNameAndObjectPath.ObjectClassAndPath.AddDefaulted(CompactNamesNum);

			for (int IdxName = 0; IdxName < (int)CompactNamesNum; ++IdxName)
			{
				*Archive << Item.ClassNameAndObjectPath.ObjectClassAndPath[IdxName];
			}
		}

		*Archive << Item.ShaderType;
		*Archive << Item.ShaderClass;
		*Archive << Item.MaterialDomain;
		*Archive << Item.FeatureLevel;
		*Archive << Item.QualityLevel;
		*Archive << Item.TargetFrequency;
		*Archive << Item.TargetPlatform;
		*Archive << Item.VFType;
		*Archive << Item.PermutationId;

		auto ReadHash = [&Hashes, &Item](FArchive& Archive, FShaderHash& OutHash) -> bool
			{
				uint64 HashIdx = ReadVarUIntFromArchive(Archive);
				if (HashIdx >= Hashes.Num())
				{
					return false;
				}
				OutHash = Hashes[static_cast<int32>(HashIdx)];
				return true;
			};

		bool bOk = ReadHash(*Archive, Item.PipelineHash) && ReadHash(*Archive, Item.OutputHash);
		if (!bOk)
		{
			return false;
		}
		Item.ComputeKeyHash();
		InOutArray.Add(Item);
	}

	return true;
}

#if WITH_EDITOR

uint32 UE::PipelineCacheUtilities::LoadMultipleStableKeysFiles(TArrayView<const FStringView> FileNames, FStableShaderSet& StableMap)
{
	TArray<TArray<FStableShaderKeyAndValue>> StableArrays;
	StableArrays.AddDefaulted(FileNames.Num());
	ParallelFor(FileNames.Num(), [&StableArrays, &FileNames](int32 Index) { LoadStableKeysFile(FileNames[Index], StableArrays[Index]); });
	uint32 MergedCount = 0;

	if (StableArrays.Num() > 0)
	{
		// Since stable keys are saved from a TSet, we assume that a single array does not have non-unique members, so add the largest one first in the most optimal manner
		StableArrays.Sort([](const TArray<FStableShaderKeyAndValue>& A, const TArray<FStableShaderKeyAndValue>& B) { return (A.Num() > B.Num()); });

		const int32 StableArrayCount = Algo::TransformAccumulate(StableArrays, &TArray<FStableShaderKeyAndValue>::Num, 0);
		StableMap.Reserve(StableMap.Num() + StableArrayCount);

		bool bFirst = true;
		for (const TArray<FStableShaderKeyAndValue>& StableArray : StableArrays)
		{
			// If this file had any data, increase merged file counter
			MergedCount += StableArray.IsEmpty() ? 0 : 1;

			if (bFirst)
			{
				bFirst = false;
				for (const FStableShaderKeyAndValue& Shader : StableArray)
				{
					StableMap.Add(MakeUnique<FStableShaderKeyAndValue>(Shader));
				}
			}
			else
			{
				for (const FStableShaderKeyAndValue& Shader : StableArray)
				{
					// Double lookup to avoid MakeUnique for duplicates
					if (!StableMap.ContainsByHash(GetTypeHash(Shader), Shader))
					{
						StableMap.Add(MakeUnique<FStableShaderKeyAndValue>(Shader));
					}
				}
			}
		}
	}

	return MergedCount;
}

bool UE::PipelineCacheUtilities::SaveStableKeysFile(const FStringView& Filename, const FStableShaderSet& Values)
{
	TUniquePtr<FArchive> FileArchiveInner(IFileManager::Get().CreateFileWriter(*FString::ConstructFromPtrSize(Filename.GetData(), Filename.Len())));
	if (!FileArchiveInner)
	{
		return false;
	}
	TUniquePtr<FArchive> Archive(new FNameAsStringIndexProxyArchive(*FileArchiveInner.Get()));

	UE::PipelineCacheUtilities::Private::FStableKeysSerializedHeader Header;
	Header.NumEntries = Values.Num();

	*Archive << Header;

	// go through all the hashes and index them
	TArray<FShaderHash> Hashes;
	TMap<FShaderHash, int32> HashToIndex;

	auto IndexHash = [&Hashes, &HashToIndex](const FShaderHash& Hash)
	{
		if (HashToIndex.Find(Hash) == nullptr)
		{
			Hashes.Add(Hash);
			HashToIndex.Add(Hash, Hashes.Num() - 1);
		}
	};

	for (const TUniquePtr<FStableShaderKeyAndValue>& Item : Values)
	{
		IndexHash(Item->PipelineHash);
		IndexHash(Item->OutputHash);
	}

	int32 NumHashes = Hashes.Num();
	*Archive << NumHashes;
	for (int32 IdxHash = 0; IdxHash < NumHashes; ++IdxHash)
	{
		*Archive << Hashes[IdxHash];
	}

	// save the rest of the properties
	for (const TUniquePtr<FStableShaderKeyAndValue>& ConstItem : Values)
	{
		// serialization unfortunately needs non-const and this is easier than const-casting every field
		FStableShaderKeyAndValue& Item = const_cast<FStableShaderKeyAndValue&>(*ConstItem);

		int8 CompactNamesNum = static_cast<int8>(Item.ClassNameAndObjectPath.ObjectClassAndPath.Num());
		ensure(Item.ClassNameAndObjectPath.ObjectClassAndPath.Num() < 256);
		*Archive << CompactNamesNum;
		for (int Idx = 0; Idx < (int)CompactNamesNum; ++Idx)
		{
			*Archive << Item.ClassNameAndObjectPath.ObjectClassAndPath[Idx];
		}

		*Archive << Item.ShaderType;
		*Archive << Item.ShaderClass;
		*Archive << Item.MaterialDomain;
		*Archive << Item.FeatureLevel;
		*Archive << Item.QualityLevel;
		*Archive << Item.TargetFrequency;
		*Archive << Item.TargetPlatform;
		*Archive << Item.VFType;
		*Archive << Item.PermutationId;

		uint64 PipelineHashIdx = static_cast<uint64>(*HashToIndex.Find(Item.PipelineHash));
		WriteVarUIntToArchive(*Archive, PipelineHashIdx);
		uint64 OutputHashIdx = static_cast<uint64>(*HashToIndex.Find(Item.OutputHash));
		WriteVarUIntToArchive(*Archive, OutputHashIdx);
	}

	return true;
}

bool UE::PipelineCacheUtilities::SaveStablePipelineCacheFile(const FString& OutputFilename, const TArray<FPermsPerPSO>& StableResults, const TArray<FStableShaderKeyAndValue>& StableShaderKeyIndexTable)
{
	TUniquePtr<FArchive> Archive(IFileManager::Get().CreateFileWriter(*OutputFilename));
	if (!Archive)
	{
		return false;
	}

	UE::PipelineCacheUtilities::Private::FStablePipelineCacheSerializedHeader Header;
	Header.NumStableKeyEntries = StableShaderKeyIndexTable.Num();

	// calculate the number of total PSO permutations
	Header.NumPermutationGroups = StableResults.Num();

	// use the first key's target platform to determine the shader platform
	Header.TargetPlatform = StableShaderKeyIndexTable.Num() > 0 ? StableShaderKeyIndexTable[0].TargetPlatform.ToString() : TEXT("");

	TArray<uint8> CompressedMemory;
	// the rest of the file is compressed
	{
		TArray<uint8> UncompressedMemory;
		FMemoryWriter PlainMemWriter(UncompressedMemory);
		UE::PipelineCacheUtilities::Private::FIndexedHashAndFNameProxyArchive MemWriter(PlainMemWriter);

		MemWriter.SetGameNetVer(FPipelineCacheFileFormatCurrentVersion);
		uint32 CacheVersion = FPipelineCacheFileFormatCurrentVersion;
		MemWriter << CacheVersion;

		// serialize the stable shader index table in exact order, but without Output hashes
		// (for now serialize PipelineHash inline, in hopes it will be later changed to a more stable identifier)
		for (const FStableShaderKeyAndValue& ConstItem : StableShaderKeyIndexTable)
		{
			// serialization unfortunately needs non-const and this is easier than const-casting every field
			FStableShaderKeyAndValue& Item = const_cast<FStableShaderKeyAndValue&>(ConstItem);

			int8 CompactNamesNum = static_cast<int8>(Item.ClassNameAndObjectPath.ObjectClassAndPath.Num());
			ensure(Item.ClassNameAndObjectPath.ObjectClassAndPath.Num() < 256);
			MemWriter << CompactNamesNum;
			for (int Idx = 0; Idx < (int)CompactNamesNum; ++Idx)
			{
				MemWriter << Item.ClassNameAndObjectPath.ObjectClassAndPath[Idx];
			}

			MemWriter << Item.ShaderType;
			MemWriter << Item.ShaderClass;
			MemWriter << Item.MaterialDomain;
			MemWriter << Item.FeatureLevel;
			MemWriter << Item.QualityLevel;
			MemWriter << Item.TargetFrequency;
			MemWriter << Item.TargetPlatform;
			MemWriter << Item.VFType;
			MemWriter << Item.PermutationId;
			MemWriter << Item.PipelineHash;	// should be replaced by a FName
		}

		// serialize the PSOs with their permutations
		int64 TotalNumberOfStableShaderKeys = StableShaderKeyIndexTable.Num();
		for (const FPermsPerPSO& Item : StableResults)
		{
#if DO_CHECK
			check(UE::PipelineCacheUtilities::Private::SanityCheckActiveSlots(Item));
#endif
			UE::PipelineCacheUtilities::Private::SaveActiveSlots(MemWriter, Item);

			FPipelineCacheFileFormatPSO NewPSO = *Item.PSO;
			// clear out every single hash
			NewPSO.ComputeDesc.ComputeShader = FShaderHash();
			NewPSO.GraphicsDesc.VertexShader = FShaderHash();
			NewPSO.GraphicsDesc.FragmentShader = FShaderHash();
			NewPSO.GraphicsDesc.GeometryShader = FShaderHash();
			NewPSO.GraphicsDesc.MeshShader = FShaderHash();
			NewPSO.GraphicsDesc.AmplificationShader = FShaderHash();
			NewPSO.RayTracingDesc.ShaderHash = FShaderHash();

#if !PSO_COOKONLY_DATA
#error "This code should not be compiled without the editoronly data."
#endif
			// first the PSO is serialized
			MemWriter << NewPSO;
			// regular operator<< will omit saving UsageMask and BindCount, so save them separately
			WriteVarUIntToArchive(MemWriter, NewPSO.UsageMask);
			WriteVarIntToArchive(MemWriter, NewPSO.BindCount);

			int64 NumPermutations = Item.Permutations.Num();
			WriteVarIntToArchive(MemWriter, NumPermutations);

			for (const UE::PipelineCacheUtilities::FPermutation& Perm : Item.Permutations)
			{
				UE::PipelineCacheUtilities::Private::SavePermutation(MemWriter, Item, Perm, TotalNumberOfStableShaderKeys);
			}
		}

		int32 CompressedSizeEstimate = FCompression::CompressMemoryBound(UE::PipelineCacheUtilities::Private::FStablePipelineCacheSerializedHeader::CompressionMethod, UncompressedMemory.Num());
		CompressedMemory.AddDefaulted(CompressedSizeEstimate);

		int32 RealCompressedSize = CompressedSizeEstimate;
		const bool bCompressed = FCompression::CompressMemory(
			UE::PipelineCacheUtilities::Private::FStablePipelineCacheSerializedHeader::CompressionMethod,
			CompressedMemory.GetData(),
			RealCompressedSize,
			UncompressedMemory.GetData(),
			UncompressedMemory.Num());

		// if the compression results are worse, just store uncompressed data
		if (!bCompressed || RealCompressedSize >= UncompressedMemory.Num())
		{
			CompressedMemory = UncompressedMemory;
			Header.UncompressedSize = 0;
			Header.DataSize = UncompressedMemory.Num();
		}
		else
		{
			Header.UncompressedSize = UncompressedMemory.Num();
			Header.DataSize = RealCompressedSize;
		}
	}

	*Archive << Header;
	Archive->Serialize(CompressedMemory.GetData(), Header.DataSize);

	return true;
}

bool UE::PipelineCacheUtilities::LoadStablePipelineCacheFile(const FString& Filename, const FStableShaderSet& StableMap, TSet<FPipelineCacheFileFormatPSO>& OutPSOs, FName& OutTargetPlatform, int32& OutPSOsRejected, int32& OutPSOsMerged)
{
	TUniquePtr<FArchive> FileArchiveInner(IFileManager::Get().CreateFileReader(*Filename));
	if (!FileArchiveInner)
	{
		return false;
	}

	TUniquePtr<FArchive> Archive(new FNameAsStringIndexProxyArchive(*FileArchiveInner.Get()));
	UE::PipelineCacheUtilities::Private::FStablePipelineCacheSerializedHeader Header;
	UE::PipelineCacheUtilities::Private::FStablePipelineCacheSerializedHeader SupportedHeader;

	*Archive << Header;

	if (Header.Magic != SupportedHeader.Magic)
	{
		UE_LOGF(LogPipelineCacheUtilities, Warning, "Rejecting %ls, wrong magic (0x%x vs expected 0x%x).", *Filename, int(Header.Magic), int(SupportedHeader.Magic));
		return false;
	}

	// start restrictive, as the format is neither forward compatible, nor backward compatible
	if (Header.Version != SupportedHeader.Version)
	{
		UE_LOGF(LogPipelineCacheUtilities, Warning, "Rejecting %ls, version %d is not supported (current supported version is %d).", *Filename, int(Header.Version), int(SupportedHeader.Version));
		return false;
	}

	if (Header.Sizeof_FPipelineCacheFileFormatPSO != SupportedHeader.Sizeof_FPipelineCacheFileFormatPSO)
	{
		UE_LOGF(LogPipelineCacheUtilities, Warning, "Rejecting %ls, different size of FPipelineCacheFileFormatPSO, serialization issues possible (%u vs expected %u).", *Filename, Header.Sizeof_FPipelineCacheFileFormatPSO, SupportedHeader.Sizeof_FPipelineCacheFileFormatPSO);
		return false;
	}

	OutTargetPlatform = FName(Header.TargetPlatform);

	TArray<uint8> UncompressedMemory;
	if (Header.UncompressedSize != 0)
	{
		TArray<uint8> CompressedMemory;
		CompressedMemory.AddUninitialized(Header.DataSize);
		Archive->Serialize(CompressedMemory.GetData(), Header.DataSize);

		UncompressedMemory.AddUninitialized(Header.UncompressedSize);
		const bool bDecompressed = FCompression::UncompressMemory(
			UE::PipelineCacheUtilities::Private::FStablePipelineCacheSerializedHeader::CompressionMethod,
			UncompressedMemory.GetData(),
			UncompressedMemory.Num(),
			CompressedMemory.GetData(),
			CompressedMemory.Num()
			);

		if (!bDecompressed)
		{
			return false;
		}
	}
	else
	{
		// unlikely case of loading an uncompressed data
		UncompressedMemory.AddUninitialized(Header.DataSize);
		Archive->Serialize(UncompressedMemory.GetData(), Header.DataSize);
	}

	// now the core logic of loading
	FMemoryReader PlainMemReader(UncompressedMemory);
	UE::PipelineCacheUtilities::Private::FIndexedHashAndFNameProxyArchive MemReader(PlainMemReader);

	uint32 CacheVersion = FPipelineCacheFileFormatCurrentVersion;
	MemReader << CacheVersion;

	MemReader.SetGameNetVer(CacheVersion);

	// read the stable keys as saved
	TArray<FStableShaderKeyAndValue> SavedStableKeys;
	SavedStableKeys.Reserve(Header.NumStableKeyEntries);
	for (int32 StableKeyIdx = 0; StableKeyIdx < Header.NumStableKeyEntries; ++StableKeyIdx)
	{
		FStableShaderKeyAndValue Item;

		int8 CompactNamesNum;
		MemReader << CompactNamesNum;
		for (int32 Idx = 0; Idx < (int32)CompactNamesNum; ++Idx)
		{
			FName PathElement;
			MemReader << PathElement;
			Item.ClassNameAndObjectPath.ObjectClassAndPath.Add(PathElement);
		}

		MemReader << Item.ShaderType;
		MemReader << Item.ShaderClass;
		MemReader << Item.MaterialDomain;
		MemReader << Item.FeatureLevel;
		MemReader << Item.QualityLevel;
		MemReader << Item.TargetFrequency;
		MemReader << Item.TargetPlatform;
		MemReader << Item.VFType;
		MemReader << Item.PermutationId;
		MemReader << Item.PipelineHash;	// should be replaced by a FName

		SavedStableKeys.Add(Item);
	}

	// kick off tasks that are remapping the old stable keys to the new ones while we're loading the rest
	TArray<FShaderHash> HashesForStableKeys;
	HashesForStableKeys.AddUninitialized(SavedStableKeys.Num());

	FGraphEventArray RemappingStableKeysTasks;
	int32 NumRemappingTasks = FPlatformMisc::NumberOfWorkerThreadsToSpawn();	// match the number of working threads, but essentially can be any number
	int32 NumKeysPerTask = FMath::Max(SavedStableKeys.Num() / NumRemappingTasks, 1);

	for (int32 FirstKey = 0; FirstKey < SavedStableKeys.Num();)
	{
		int32 KeysToRemapThisTask = FMath::Min(NumKeysPerTask, SavedStableKeys.Num() - FirstKey);
		RemappingStableKeysTasks.Add(
			FFunctionGraphTask::CreateAndDispatchWhenReady([FirstKey, KeysToRemapThisTask, &SavedStableKeys, &StableMap, &HashesForStableKeys, &OutTargetPlatform]
			{
				for (int32 IdxKey = 0; IdxKey < KeysToRemapThisTask; ++IdxKey)
				{
					int32 AbsKeyIdx = FirstKey + IdxKey;
					// should be safe to do as there's no overlap between the key ranges
					SavedStableKeys[AbsKeyIdx].ComputeKeyHash();

					FShaderHash Match = FShaderHash();
					if (const TUniquePtr<FStableShaderKeyAndValue>* ShaderKeyPtr = StableMap.Find(SavedStableKeys[AbsKeyIdx]))
					{
						check((*ShaderKeyPtr)->OutputHash != FShaderHash());
						check(OutTargetPlatform == (*ShaderKeyPtr)->TargetPlatform);
						Match = (*ShaderKeyPtr)->OutputHash;
					}

					HashesForStableKeys[AbsKeyIdx] = Match;
				}
			}, TStatId())
		);

		FirstKey += KeysToRemapThisTask;
	}

	// Load the PSOs and their permutations (i.e. stable shaders that were found to be compatible when expanding the recorded cache).
	// Now, we may need to reject or collapse certain [permutations of] PSOs. Why reject? If a stable shader key they reference is no longer found in the current stable map.
	// Why collapse? Well, we may find that the different stable shaders end up having the same shader code hash, then no reason to include two PSO that aren't different in terms of shader code

	int64 TotalNumberOfShaderKeys = SavedStableKeys.Num();
	TArray<FPermsPerPSO> PermutationGroups;
	TArray<FPipelineCacheFileFormatPSO> OriginalPSOs;
	PermutationGroups.Reserve(Header.NumPermutationGroups);
	OriginalPSOs.AddDefaulted(Header.NumPermutationGroups);	// need to avoid resizes and invalidating the pointeers
	for (int64 PermutationGroupIdx = 0; PermutationGroupIdx < Header.NumPermutationGroups; ++PermutationGroupIdx)
	{
		FPermsPerPSO Item;
		UE::PipelineCacheUtilities::Private::LoadActiveSlots(MemReader, Item);

#if !PSO_COOKONLY_DATA
#error "This code should not be compiled without the editoronly data."
#endif
		// load the original PSO that was recorded, this is the basis for all the permutations
		MemReader << OriginalPSOs[PermutationGroupIdx];
		// regular operator<< will omit saving UsageMask and BindCount, so save them separately
		OriginalPSOs[PermutationGroupIdx].UsageMask = ReadVarUIntFromArchive(MemReader);
		OriginalPSOs[PermutationGroupIdx].BindCount = ReadVarIntFromArchive(MemReader);

		Item.PSO = &OriginalPSOs[PermutationGroupIdx];
#if DO_CHECK
		check(UE::PipelineCacheUtilities::Private::SanityCheckActiveSlots(Item));
#endif

		int64 NumPermutations = ReadVarIntFromArchive(MemReader);
		Item.Permutations.Reserve(NumPermutations);
		for (int32 IdxPerm = 0; IdxPerm < NumPermutations; ++IdxPerm)
		{
			FPermutation Perm;
			UE::PipelineCacheUtilities::Private::LoadPermutation(MemReader, Item, Perm, TotalNumberOfShaderKeys);

			Item.Permutations.Add(Perm);
		}

		PermutationGroups.Add(Item);
	}

	// wait for the remapping tasks to finish
	FTaskGraphInterface::Get().WaitUntilTasksComplete(RemappingStableKeysTasks);

	// translate all PSOs into their hashes
	auto AddNewPSO = [&OutPSOs, &OutPSOsMerged, &OutPSOsRejected](const FPipelineCacheFileFormatPSO& NewPSO)
	{
		if (UNLIKELY(!NewPSO.Verify()))
		{
			++OutPSOsRejected;
		}
		else if (FPipelineCacheFileFormatPSO* ExistingPSO = OutPSOs.Find(NewPSO))
		{
			check(*ExistingPSO == NewPSO);
			ExistingPSO->UsageMask |= NewPSO.UsageMask;
			ExistingPSO->BindCount = FMath::Max(ExistingPSO->BindCount, NewPSO.BindCount);

			++OutPSOsMerged;
		}
		else
		{
			OutPSOs.Add(NewPSO);
		}
	};

	for (const FPermsPerPSO& PermGroup : PermutationGroups)
	{
		FPipelineCacheFileFormatPSO NewPSO = *PermGroup.PSO;

		if (NewPSO.Type == FPipelineCacheFileFormatPSO::DescriptorType::Graphics)
		{
			for (const UE::PipelineCacheUtilities::FPermutation& Perm : PermGroup.Permutations)
			{
#define UE_PCU_GET_SHADER_HASH_FOR_SLOT(ShaderFreq) \
				(PermGroup.ActivePerSlot[ShaderFreq] ? HashesForStableKeys[Perm.Slots[ShaderFreq]] : FShaderHash())

				NewPSO.GraphicsDesc.VertexShader = UE_PCU_GET_SHADER_HASH_FOR_SLOT(SF_Vertex);
				NewPSO.GraphicsDesc.FragmentShader = UE_PCU_GET_SHADER_HASH_FOR_SLOT(SF_Pixel);
				NewPSO.GraphicsDesc.GeometryShader = UE_PCU_GET_SHADER_HASH_FOR_SLOT(SF_Geometry);
				NewPSO.GraphicsDesc.MeshShader = UE_PCU_GET_SHADER_HASH_FOR_SLOT(SF_Mesh);
				NewPSO.GraphicsDesc.AmplificationShader = UE_PCU_GET_SHADER_HASH_FOR_SLOT(SF_Amplification);

#undef UE_PCU_GET_SHADER_HASH_FOR_SLOT

				AddNewPSO(NewPSO);
			}
		}
		else if (NewPSO.Type == FPipelineCacheFileFormatPSO::DescriptorType::Compute)
		{
			for (const UE::PipelineCacheUtilities::FPermutation& Perm : PermGroup.Permutations)
			{
				NewPSO.ComputeDesc.ComputeShader = HashesForStableKeys[Perm.Slots[SF_Compute]];

				AddNewPSO(NewPSO);
			}
		}
		else if (NewPSO.Type == FPipelineCacheFileFormatPSO::DescriptorType::RayTracing)
		{
			for (const UE::PipelineCacheUtilities::FPermutation& Perm : PermGroup.Permutations)
			{
				const EShaderFrequency Frequency = NewPSO.RayTracingDesc.Frequency;
				NewPSO.RayTracingDesc.ShaderHash = HashesForStableKeys[Perm.Slots[Frequency]];

				AddNewPSO(NewPSO);
			}
		}
	}

	return true;
}

bool UE::PipelineCacheUtilities::SaveChunkInfo(const FString& ShaderLibraryName, const int32 InChunkId, const TSet<FName>& InPackagesInChunk, const ITargetPlatform* TargetPlatform, const FString& PathToSaveTo, TArray<FString>& OutChunkFilenames)
{
	const FString InfoFile = FString::Printf(TEXT("%s_%s_Chunk%d.cacheinfo"), *TargetPlatform->PlatformName(), *ShaderLibraryName, InChunkId);
	const FString FullPath = PathToSaveTo / InfoFile;

	// Add names for the files that will be bundled with the game - they should be similar to name given to the monolithic file in UCookOnTheFlyServer::CreatePipelineCache.
	// Note that at this point we don't know which formats will actually have the PSO cache, so add all targeted ones
	TArray<FName> ShaderFormats;
	TargetPlatform->GetAllTargetedShaderFormats(ShaderFormats);
	TArray<FString> BundledFiles;
	for (FName ShaderFormat : ShaderFormats)
	{
		FString BundledFile = FString::Printf(TEXT("%s_Chunk%d_%s.stable.upipelinecache"), *ShaderLibraryName, InChunkId, *ShaderFormat.ToString());
		OutChunkFilenames.Add(BundledFile);
		BundledFiles.Add(BundledFile);
	}

	TUniquePtr<FArchive> AssetInfoWriter(IFileManager::Get().CreateFileWriter(*FullPath, FILEWRITE_NoFail));
	if (AssetInfoWriter)
	{
		FString JsonTcharText;
		{
			TSharedRef<TJsonWriter<TCHAR, TPrettyJsonPrintPolicy<TCHAR>>> Writer = TJsonWriterFactory<TCHAR, TPrettyJsonPrintPolicy<TCHAR>>::Create(&JsonTcharText);
			Writer->WriteObjectStart();

			Writer->WriteValue(TEXT("CacheChunkInfoVersion"), static_cast<int32>(UE::PipelineCacheUtilities::Private::FPSOCacheChunkInfo::EVersion::Current));

			Writer->WriteValue(TEXT("ChunkId"), InChunkId);

			Writer->WriteArrayStart(TEXT("OutputFilesPerFormat"));
			for (int32 Idx = 0, Num = ShaderFormats.Num(); Idx < Num; ++Idx)
			{
				Writer->WriteObjectStart();
				Writer->WriteValue(TEXT("ShaderFormat"), ShaderFormats[Idx].ToString());
				Writer->WriteValue(TEXT("OutputFile"), BundledFiles[Idx]);
				Writer->WriteObjectEnd();
			}
			Writer->WriteArrayEnd();

			Writer->WriteArrayStart(TEXT("Packages"));
			for (TSet<FName>::TConstIterator Iter(InPackagesInChunk); Iter; ++Iter)
			{
				Writer->WriteValue((*Iter).ToString());
			}
			Writer->WriteArrayEnd();

			Writer->WriteObjectEnd();
			Writer->Close();
		}

		FTCHARToUTF8 JsonUtf8(*JsonTcharText);
		AssetInfoWriter->Serialize(const_cast<void*>(reinterpret_cast<const void*>(JsonUtf8.Get())), JsonUtf8.Length() * sizeof(UTF8CHAR));
	}

	return true;
}

void UE::PipelineCacheUtilities::FindAllChunkInfos(const FString& ShaderLibraryName, const FString& TargetPlatformName, const FString& PathToSearchIn, TArray<FString>& OutInfoFilenames)
{
	TArray<FString> AllCacheInfoFiles;
	IFileManager::Get().FindFiles(AllCacheInfoFiles, *PathToSearchIn, TEXT("cacheinfo"));

	const FString Pattern = FString::Printf(TEXT("%s_%s"), *TargetPlatformName, *ShaderLibraryName);
	for (const FString& FoundFile : AllCacheInfoFiles)
	{
		if (FoundFile.StartsWith(Pattern))
		{
			OutInfoFilenames.Add(PathToSearchIn / FoundFile);
		}
	}
}

bool UE::PipelineCacheUtilities::LoadChunkInfo(const FString& Filename, const FString& InShaderFormat, int32& OutChunkId, FString& OutChunkedCacheFilename, TSet<FName>& OutPackages)
{
	TArray<uint8> FileData;
	if (!FFileHelper::LoadFileToArray(FileData, *Filename))
	{
		return false;
	}

	FString JsonText;
	FFileHelper::BufferToString(JsonText, FileData.GetData(), FileData.Num());

	TSharedPtr<FJsonObject> JsonObject;
	TSharedRef<TJsonReader<TCHAR>> Reader = TJsonReaderFactory<TCHAR>::Create(JsonText);

	// Attempt to deserialize JSON
	if (!FJsonSerializer::Deserialize(Reader, JsonObject) || !JsonObject.IsValid())
	{
		return false;
	}

	// check version
	{
		TSharedPtr<FJsonValue> FileVersionValue = JsonObject->Values.FindRef(TEXT("CacheChunkInfoVersion"));
		if (!FileVersionValue.IsValid())
		{
			UE_LOGF(LogPipelineCacheUtilities, Warning, "Rejecting asset info file %ls: missing CacheChunkInfoVersion (damaged file?)",
				*Filename);
			return false;
		}

		const int32 FileVersion = static_cast<int32>(FileVersionValue->AsNumber());
		if (FileVersion != static_cast<int32>(UE::PipelineCacheUtilities::Private::FPSOCacheChunkInfo::EVersion::Current))
		{
			UE_LOGF(LogPipelineCacheUtilities, Warning, "Rejecting cache chunk info file %ls: expected version %d, got unsupported version %d.",
						 *Filename, int(UE::PipelineCacheUtilities::Private::FPSOCacheChunkInfo::EVersion::Current), FileVersion);
			return false;
		}
	}

	// get ChunkID
	{
		TSharedPtr<FJsonValue> ChunkIdValue = JsonObject->Values.FindRef(TEXT("ChunkId"));
		if (!ChunkIdValue.IsValid())
		{
			UE_LOGF(LogPipelineCacheUtilities, Warning, "Rejecting asset info file %ls: missing ChunkId (damaged file?)",
				*Filename);
			return false;
		}

		OutChunkId = static_cast<int32>(ChunkIdValue->AsNumber());
	}

	// find what SF-specific output files we committed to producing
	{
		OutChunkedCacheFilename.Empty();

		TSharedPtr<FJsonValue> OutputFilesPerFormatArrayValue = JsonObject->Values.FindRef(TEXT("OutputFilesPerFormat"));
		if (!OutputFilesPerFormatArrayValue.IsValid())
		{
			UE_LOGF(LogPipelineCacheUtilities, Warning, "Rejecting cache chunk info info file %ls: missing OutputFilesPerFormat array (damaged file?)",
				*Filename);
			return false;
		}

		TArray<TSharedPtr<FJsonValue>> OutputFilesPerFormatArray = OutputFilesPerFormatArrayValue->AsArray();
		for (int32 IdxPair = 0, NumPairs = OutputFilesPerFormatArray.Num(); IdxPair < NumPairs; ++IdxPair)
		{
			TSharedPtr<FJsonObject> Pair = OutputFilesPerFormatArray[IdxPair]->AsObject();
			if (UNLIKELY(!Pair.IsValid()))
			{
				UE_LOGF(LogPipelineCacheUtilities, Warning, "Rejecting cache chunk info file %ls: OutputFilesPerFormat array contains unreadable mapping format #%d (damaged file?)",
					*Filename,
					IdxPair
				);
				return false;
			}

			TSharedPtr<FJsonValue> ShaderFormatJson = Pair->Values.FindRef(TEXT("ShaderFormat"));
			if (UNLIKELY(!ShaderFormatJson.IsValid()))
			{
				UE_LOGF(LogPipelineCacheUtilities, Warning, "Rejecting cache chunk info file %ls: OutputFilesPerFormat array contains unreadable ShaderFormat %d (damaged file?)",
					*Filename,
					IdxPair
				);
				return false;
			}

			FString ShaderFormat = ShaderFormatJson->AsString();
			if (ShaderFormat == InShaderFormat)
			{
				TSharedPtr<FJsonValue> OutputFileValue = Pair->Values.FindRef(TEXT("OutputFile"));
				if (UNLIKELY(!OutputFileValue.IsValid()))
				{
					UE_LOGF(LogPipelineCacheUtilities, Warning, "Rejecting cache chunk info file %ls: OutputFilesPerFormat array contains unreadable OutputFile %d (damaged file?)",
						*Filename,
						IdxPair
					);
					return false;
				}

				OutChunkedCacheFilename = OutputFileValue->AsString();
				break;
			}
		}

		if (OutChunkedCacheFilename.IsEmpty())
		{
			UE_LOGF(LogPipelineCacheUtilities, Warning, "Rejecting cache chunk info file %ls: unable to determine output file format for shader format %ls",
				*Filename,
				*InShaderFormat
			);
			return false;
		}
	}

	// read package (asset) names
	{
		TSharedPtr<FJsonValue> PackagesArrayValue = JsonObject->Values.FindRef(TEXT("Packages"));
		if (!PackagesArrayValue.IsValid())
		{
			UE_LOGF(LogPipelineCacheUtilities, Warning, "Rejecting cache chunk info info file %ls: missing Packages array (damaged file?)",
				*Filename);
			return false;
		}

		TArray<TSharedPtr<FJsonValue>> PackagesArray = PackagesArrayValue->AsArray();
		for (int32 IdxPackage = 0, NumPackages = PackagesArray.Num(); IdxPackage < NumPackages; ++IdxPackage)
		{
			OutPackages.Add(FName(*PackagesArray[IdxPackage]->AsString()));
		}
	}

	return true;
}
#endif // WITH_EDITOR
#endif // UE_WITH_PIPELINE_CACHE_UTILITIES
