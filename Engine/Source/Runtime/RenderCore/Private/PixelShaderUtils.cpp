// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	PixelShaderUtils.cpp: Implementations of utilities for pixel shaders.
=============================================================================*/

#include "PixelShaderUtils.h"

#include "CommonRenderResources.h"
#include "DataDrivenShaderPlatformInfo.h"

IMPLEMENT_SHADER_TYPE(, FPixelShaderUtils::FRasterizeToRectsVS, TEXT("/Engine/Private/RenderGraphUtilities.usf"), TEXT("RasterizeToRectsVS"), SF_Vertex);

bool FPixelShaderUtils::FRasterizeToRectsVS::ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
{
	return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);
}

// static
void FPixelShaderUtils::DrawFullscreenTriangle(FRHICommandList& RHICmdList, uint32 InstanceCount)
{
	RHICmdList.SetStreamSource(0, GScreenRectangleVertexBuffer.VertexBufferRHI, 0);

	RHICmdList.DrawIndexedPrimitive(
		GScreenRectangleIndexBuffer.IndexBufferRHI,
		/*BaseVertexIndex=*/ 0,
		/*MinIndex=*/ 0,
		/*NumVertices=*/ 3,
		/*StartIndex=*/ 6,
		/*NumPrimitives=*/ 1,
		/*NumInstances=*/ InstanceCount);
}

// static
void FPixelShaderUtils::DrawFullscreenQuad(FRHICommandList& RHICmdList, uint32 InstanceCount)
{
	RHICmdList.SetStreamSource(0, GScreenRectangleVertexBuffer.VertexBufferRHI, 0);

	RHICmdList.DrawIndexedPrimitive(
		GScreenRectangleIndexBuffer.IndexBufferRHI,
		/*BaseVertexIndex=*/ 0,
		/*MinIndex=*/ 0,
		/*NumVertices=*/ 4,
		/*StartIndex=*/ 0,
		/*NumPrimitives=*/ 2,
		/*NumInstances=*/ InstanceCount);
}

// static
void FPixelShaderUtils::InitFullscreenPipelineState(
	FRHICommandList& RHICmdList,
	const FGlobalShaderMap* GlobalShaderMap,
	const TShaderRef<FShader>& PixelShader,
	FGraphicsPipelineStateInitializer& GraphicsPSOInit)
{
	TShaderMapRef<FScreenVertexShaderVS> VertexShader(GlobalShaderMap);
		
	RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);
	GraphicsPSOInit.BlendState = TStaticBlendState<>::GetRHI();
	GraphicsPSOInit.RasterizerState = TStaticRasterizerState<>::GetRHI();
	GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();

	GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GFilterVertexDeclaration.VertexDeclarationRHI;
	GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
	GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
	GraphicsPSOInit.PrimitiveType = PT_TriangleList;
}

void FPixelShaderUtils::InitFullscreenMultiviewportPipelineState(
	FRHICommandList& RHICmdList,
	const FGlobalShaderMap* GlobalShaderMap,
	const TShaderRef<FShader>& PixelShader,
	FGraphicsPipelineStateInitializer& GraphicsPSOInit)
{
	TShaderMapRef<FInstancedScreenVertexShaderVS> VertexShader(GlobalShaderMap);

	RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);
	GraphicsPSOInit.BlendState = TStaticBlendState<>::GetRHI();
	GraphicsPSOInit.RasterizerState = TStaticRasterizerState<>::GetRHI();
	GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();

	GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GFilterVertexDeclaration.VertexDeclarationRHI;
	GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
	GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
	GraphicsPSOInit.PrimitiveType = PT_TriangleList;
}
