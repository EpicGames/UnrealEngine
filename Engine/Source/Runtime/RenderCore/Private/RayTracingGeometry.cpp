// Copyright Epic Games, Inc. All Rights Reserved.

#include "RayTracingGeometry.h"
#include "RHICommandList.h"
#include "HAL/IConsoleManager.h"
#include "RayTracingGeometryManagerInterface.h"
#include "RenderUtils.h"
#include "RHIResourceReplace.h"
#include "RHITextureReference.h" // IWYU pragma: keep

#if RHI_RAYTRACING

IRayTracingGeometryManager* GRayTracingGeometryManager = nullptr;

#endif

static TAutoConsoleVariable<int32> CVarDebugForceRuntimeBLAS(
	TEXT("r.Raytracing.Debug.ForceRuntimeBLAS"),
	0,
	TEXT("Force building BLAS at runtime."),
	ECVF_ReadOnly);

bool RayTracing::ShouldForceRuntimeBLAS()
{
	static const bool bDebugForceRuntimeBLAS = CVarDebugForceRuntimeBLAS.GetValueOnAnyThread() != 0;
	return bDebugForceRuntimeBLAS;
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS // Remove this in UE 5.10 once Initializer is made private

#if RHI_RAYTRACING

bool FRayTracingGeometry::HasValidInitializer() const
{
	bool bAllSegmentsAreValid = Initializer.Segments.Num() > 0;
	for (const FRayTracingGeometrySegment& Segment : Initializer.Segments)
	{
		if (!Segment.VertexBuffer)
		{
			bAllSegmentsAreValid = false;
			checkf(Initializer.bTemplate || Initializer.OfflineData == nullptr, TEXT("RayTracingGeometry (%s) should not have OfflineData"), *Initializer.DebugName.ToString());
			break;
		}
	}

	return bAllSegmentsAreValid;
}

void FRayTracingGeometry::SetInitializer(FRayTracingGeometryInitializer InInitializer)
{
	// TODO: Check or warn if RHI is already initialized. 
	// High level code should release RHI and/or recreate geometry when changing the initializer to avoid ending up in stale state.

	Initializer = MoveTemp(InInitializer);

	if (!CVarDebugForceRuntimeBLAS.GetValueOnAnyThread())
	{
		Initializer.OfflineDataHeader = RawDataHeader;
	}
}

void FRayTracingGeometry::InitRHIForStreaming(FRHIRayTracingGeometry* IntermediateGeometry, FRHIResourceReplaceBatcher& Batcher)
{
	checkf(IntermediateGeometry,
		TEXT("IntermediateGeometry should be valid when streaming-in ray tracing geometry.\n")
		TEXT("This will result in FRayTracingGeometry (%s) not being correctly initialized.\n"),
		*Initializer.DebugName.ToString()
	);

	checkf(RayTracingGeometryRHI, TEXT("RayTracingGeometryRHI (%s) must be valid when InitRHIForStreaming is called.\n"), *Initializer.DebugName.ToString());

	Initializer.Type = ERayTracingGeometryInitializerType::Rendering;

	Batcher.EnqueueReplace(RayTracingGeometryRHI, IntermediateGeometry);

	EnumAddFlags(GeometryState, EGeometryStateFlags::Valid);
	EnumAddFlags(GeometryState, EGeometryStateFlags::StreamedIn);

	GRayTracingGeometryManager->RefreshRegisteredGeometry(RayTracingGeometryHandle);
}

void FRayTracingGeometry::ReleaseRHIForStreaming(FRHIResourceReplaceBatcher& Batcher)
{
	RemoveBuildRequest();

	checkf(RayTracingGeometryRHI, TEXT("RayTracingGeometryRHI (%s) must be valid when ReleaseRHIForStreaming is called.\n"), *Initializer.DebugName.ToString());

	EnumRemoveFlags(GeometryState, EGeometryStateFlags::StreamedIn);
	EnumRemoveFlags(GeometryState, EGeometryStateFlags::Valid);

	Batcher.EnqueueReplace(RayTracingGeometryRHI, nullptr);

	Initializer.Type = ERayTracingGeometryInitializerType::StreamingDestination;

	GRayTracingGeometryManager->RefreshRegisteredGeometry(RayTracingGeometryHandle);
}

void FRayTracingGeometry::RequestBuildIfNeeded(FRHICommandListBase& RHICmdList, ERTAccelerationStructureBuildPriority InBuildPriority)
{
	if (GetRequiresBuild())
	{
		GRayTracingGeometryManager->RequestBuildAccelerationStructure(this, InBuildPriority);
		SetRequiresBuild(false);
	}
}

void FRayTracingGeometry::MakeResident(FRHICommandList& RHICmdList)
{
	checkf(EnumHasAllFlags(GeometryState, EGeometryStateFlags::Evicted) && RayTracingGeometryRHI == nullptr, 
		TEXT("Evicted FRayTracingGeometry (%s) should have evicted flag set and no RHI object."), *Initializer.DebugName.ToString());
	checkf(!EnumHasAllFlags(GeometryState, EGeometryStateFlags::StreamedIn),
		TEXT("Evicted FRayTracingGeometry (%s) shouldn't have StreamedIn flag set."), *Initializer.DebugName.ToString());

	if (!ensureMsgf(DynamicGeometrySharedBufferGenerationID == NonSharedVertexBuffers,
		TEXT("Cannot call MakeResident(...) on FRayTracingGeometry using shared vertex buffers.\n")
		TEXT("Dynamic geometry (%s) should be rebuilt instead."), *Initializer.DebugName.ToString()))
	{
		// if geometry is using shared buffers those buffers might not be valid at this point
		// instead of being made resident here, dynamic geometries need to be manually updated as necessary
		return;
	}

	EnumRemoveFlags(GeometryState, EGeometryStateFlags::Evicted);

	InitRHI(RHICmdList);
}

void FRayTracingGeometry::Evict()
{
	checkf(!EnumHasAllFlags(GeometryState, EGeometryStateFlags::Evicted) && RayTracingGeometryRHI != nullptr,
		TEXT("RayTracingGeometry (%s) must not have been evicted already."), *Initializer.DebugName.ToString());
	checkf(!EnumHasAllFlags(GeometryState, EGeometryStateFlags::StreamedIn), 
		TEXT("RayTracingGeometry (%s) must be streamed out before it can be evicted."), *Initializer.DebugName.ToString());

	RemoveBuildRequest();
	RayTracingGeometryRHI.SafeRelease();
	EnumAddFlags(GeometryState, EGeometryStateFlags::Evicted);
	
	GRayTracingGeometryManager->RefreshRegisteredGeometry(RayTracingGeometryHandle);

	if (!GroupHandle.IsNull())
	{
		GRayTracingGeometryManager->RequestUpdateCachedRenderState(GroupHandle);
	}
}

void FRayTracingGeometry::CreateRayTracingGeometry(FRHICommandListBase& RHICmdList, ERTAccelerationStructureBuildPriority InBuildPriority)
{
	// Release previous RHI object if any
	ReleaseRHI();

	if (RawData.Num())
	{
		check(!RayTracing::ShouldForceRuntimeBLAS());
		check(Initializer.OfflineData == nullptr);
		Initializer.OfflineData = &RawData;
	}

	if (HasValidInitializer())
	{
		// Geometries with StreamingDestination type are initially created in invalid state until they are streamed in (see InitRHIForStreaming).
		const bool bWithNativeResource = Initializer.Type != ERayTracingGeometryInitializerType::StreamingDestination;
		if (bWithNativeResource)
		{
			EnumAddFlags(GeometryState, EGeometryStateFlags::Valid);
		}

		const bool bWithOfflineData = Initializer.OfflineData != nullptr;

		if (IsRayTracingEnabled())
		{
			RayTracingGeometryRHI = RHICmdList.CreateRayTracingGeometry(Initializer);

			// Offline data ownership is transferred to the RHI, which discards it after use.
			// It is no longer valid to use it after this point.
			Initializer.OfflineData = nullptr;
		}
		else
		{
			EnumAddFlags(GeometryState, EGeometryStateFlags::Evicted);
		}

		if (!bWithOfflineData)
		{
			// Request build if not skip
			if (InBuildPriority != ERTAccelerationStructureBuildPriority::Skip)
			{
				if (RayTracingGeometryRHI)
				{
					GRayTracingGeometryManager->RequestBuildAccelerationStructure(this, InBuildPriority);
				}
				SetRequiresBuild(false);
			}
			else if (bWithNativeResource)
			{
				SetRequiresBuild(true);
			}
		}
		else
		{
			if (RayTracingGeometryRHI && RayTracingGeometryRHI->IsCompressed() && !Initializer.bTemplate)
			{
				GRayTracingGeometryManager->RequestBuildAccelerationStructure(this, InBuildPriority);
			}

			SetRequiresBuild(false);
		}
	}

	GRayTracingGeometryManager->RefreshRegisteredGeometry(RayTracingGeometryHandle);
}

bool FRayTracingGeometry::IsValid() const
{
	// can't check IsInitialized() because current implementation of hair ray tracing support doesn't initialize resource
	//check(IsInitialized());

	const bool bIsValid = EnumHasAllFlags(GeometryState, EGeometryStateFlags::Valid);

	if (bIsValid)
	{
		checkf(Initializer.TotalPrimitiveCount > 0, TEXT("Valid RayTracingGeometry (%s) must have non-zero primitive count."), *Initializer.DebugName.ToString());
		checkf(RayTracingGeometryRHI != nullptr || EnumHasAllFlags(GeometryState, EGeometryStateFlags::Evicted), TEXT("Valid RayTracingGeometry (%s) must have valid RHI object or evicted flag set"), *Initializer.DebugName.ToString());
	}

	return bIsValid;
}

bool FRayTracingGeometry::IsEvicted() const
{
	// can't check IsInitialized() because current implementation of hair ray tracing support doesn't initialize resource
	//check(IsInitialized());

	const bool bIsEvicted = EnumHasAllFlags(GeometryState, EGeometryStateFlags::Evicted);

	if (bIsEvicted)
	{
		checkf(RayTracingGeometryRHI == nullptr, TEXT("Evicted RayTracingGeometry (%s) cannot have valid RHI object."), *Initializer.DebugName.ToString());
	}

	return bIsEvicted;
}

void FRayTracingGeometry::InitRHI(FRHICommandListBase& RHICmdList)
{
	if (!IsRayTracingAllowed())
		return;

	ERTAccelerationStructureBuildPriority BuildPriority = (Initializer.Type != ERayTracingGeometryInitializerType::Rendering || Initializer.bTemplate)
		? ERTAccelerationStructureBuildPriority::Skip
		: ERTAccelerationStructureBuildPriority::Normal;
	CreateRayTracingGeometry(RHICmdList, BuildPriority);
}

void FRayTracingGeometry::ReleaseRHI()
{
	RemoveBuildRequest();
	RayTracingGeometryRHI.SafeRelease();
	GeometryState = EGeometryStateFlags::Invalid;

	GRayTracingGeometryManager->RefreshRegisteredGeometry(RayTracingGeometryHandle);
}

void FRayTracingGeometry::RemoveBuildRequest()
{
	if (HasPendingBuildRequest())
	{
		GRayTracingGeometryManager->RemoveBuildRequest(RayTracingBuildRequestIndex);
	}
}

void FRayTracingGeometry::InitResource(FRHICommandListBase& RHICmdList)
{
	ensureMsgf(IsRayTracingAllowed(), TEXT("FRayTracingGeometry (%s) should only be initialized when Ray Tracing is allowed."), *Initializer.DebugName.ToString());

	FRenderResource::InitResource(RHICmdList);

	if (RayTracingGeometryHandle.IsNull())
	{
		RayTracingGeometryHandle = GRayTracingGeometryManager->RegisterRayTracingGeometry(this);
	}
}

void FRayTracingGeometry::ReleaseResource()
{
	ensureMsgf(IsRayTracingAllowed() || !IsInitialized(), TEXT("FRayTracingGeometry (%s) should only be initialized when Ray Tracing is allowed."), *Initializer.DebugName.ToString());

	if (!RayTracingGeometryHandle.IsNull())
	{
		GRayTracingGeometryManager->ReleaseRayTracingGeometryHandle(RayTracingGeometryHandle);
		RayTracingGeometryHandle = {};
	}

	FRenderResource::ReleaseResource();

	// Release any resource references held by the initializer.
	// This includes index and vertex buffers used for building the BLAS.
	Initializer = FRayTracingGeometryInitializer{};
}

bool FRayTracingGeometry::HasPendingBuildRequest() const
{
	const bool bHasPendingBuildRequest = !RayTracingBuildRequestIndex.IsNull();

	if (bHasPendingBuildRequest)
	{
		ensureMsgf(IsValid() && !IsEvicted(), TEXT("RayTracingGeometry (%s) with pending build request must be valid and not evicted."), *Initializer.DebugName.ToString());
	}

	return bHasPendingBuildRequest;
}

void FRayTracingGeometry::BoostBuildPriority(float InBoostValue) const
{
	checkf(HasPendingBuildRequest(), TEXT("RayTracingGeometry (%s) must have pending build request"), *Initializer.DebugName.ToString());
	GRayTracingGeometryManager->BoostPriority(RayTracingBuildRequestIndex, InBoostValue);
}

#endif // RHI_RAYTRACING

PRAGMA_ENABLE_DEPRECATION_WARNINGS
