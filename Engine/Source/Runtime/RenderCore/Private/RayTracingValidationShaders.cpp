// Copyright Epic Games, Inc. All Rights Reserved.

#include "RayTracingValidationShaders.h"
#include "ShaderParameterUtils.h"
#include "PipelineStateCache.h"

#if RHI_RAYTRACING

// FRayTracingValidateGeometryBuildParamsCS

IMPLEMENT_GLOBAL_SHADER(FRayTracingValidateGeometryBuildParamsCS, "/Engine/Private/RayTracing/RayTracingValidation.usf", "RayTracingValidateGeometryBuildParamsCS", SF_Compute);

void FRayTracingValidateGeometryBuildParamsCS::Dispatch(FRHICommandList& RHICmdList, const FRayTracingGeometryBuildParams& Params)
{
	const bool bSupportsWaveOps = GRHISupportsWaveOperations && RHISupportsWaveOperations(GMaxRHIShaderPlatform);
	if (!ensureMsgf(bSupportsWaveOps, TEXT("Wave operations are required to run ray tracing GPU validation shaders.")))
	{
		return;
	}

	const FRayTracingGeometryInitializer& Initializer = Params.Geometry->GetInitializer();

	TShaderMapRef<FRayTracingValidateGeometryBuildParamsCS> ComputeShader(GetGlobalShaderMap(GMaxRHIFeatureLevel));
	FRHIComputeShader* ShaderRHI = ComputeShader.GetComputeShader();
	SetComputePipelineState(RHICmdList, ShaderRHI);

	// TODO: handle non-indexed geometry
	if (Initializer.IndexBuffer == nullptr)
	{
		return;
	}

	const uint32 IndexStride = Initializer.IndexBuffer->GetStride();

	const FRawBufferShaderResourceViewInitializer IBViewInitializer(Initializer.IndexBuffer);
	FShaderResourceViewRHIRef IndexBufferSRV = RHICmdList.CreateShaderResourceView(IBViewInitializer.Buffer, FRHIViewDesc::CreateBufferSRV().SetTypeFromBuffer(IBViewInitializer.Buffer));

	for (const FRayTracingGeometrySegment& Segment : Initializer.Segments)
	{
		if (Segment.VertexBufferElementType != VET_Float3)
		{
			// Only Float3 vertex positions are currently supported
			continue;
		}

		const uint32 IndexBufferOffsetInBytes = Segment.FirstPrimitive * IndexStride * 3;

		FRHIBatchedShaderParameters& BatchedParameters = RHICmdList.GetScratchShaderParameters();

		SetShaderValue(BatchedParameters, ComputeShader->VertexBufferStrideParam, Segment.VertexBufferStride);
		SetShaderValue(BatchedParameters, ComputeShader->VertexBufferOffsetInBytesParam, Segment.VertexBufferOffset);
		SetShaderValue(BatchedParameters, ComputeShader->IndexBufferOffsetInBytesParam, IndexBufferOffsetInBytes);
		SetShaderValue(BatchedParameters, ComputeShader->IndexBufferStrideParam, IndexStride);
		SetShaderValue(BatchedParameters, ComputeShader->NumPrimitivesParam, Segment.NumPrimitives);
		SetShaderValue(BatchedParameters, ComputeShader->MaxVerticesParam, Segment.MaxVertices);

		const FRawBufferShaderResourceViewInitializer VBViewInitializer(Segment.VertexBuffer);
		FShaderResourceViewRHIRef VertexBufferSRV = RHICmdList.CreateShaderResourceView(VBViewInitializer.Buffer, FRHIViewDesc::CreateBufferSRV().SetTypeFromBuffer(VBViewInitializer.Buffer));

		SetSRVParameter(BatchedParameters, ComputeShader->VertexBufferParam, VertexBufferSRV);
		SetSRVParameter(BatchedParameters, ComputeShader->IndexBufferParam, IndexBufferSRV);

		RHICmdList.SetBatchedShaderParameters(ShaderRHI, BatchedParameters);

		// TODO: handle arbitrary large meshes that may overrun the 1D dispatch limit
		const uint32 MaxDispatchDimension = 65536; // D3D11_CS_DISPATCH_MAX_THREAD_GROUPS_PER_DIMENSION
		const uint32 NumGroupsX = FMath::Min((Segment.NumPrimitives + NumThreadsX - 1) / NumThreadsX, MaxDispatchDimension);

		RHICmdList.DispatchComputeShader(NumGroupsX, 1, 1);
	}

	if (RHICmdList.NeedsShaderUnbinds())
	{
		FRHIBatchedShaderUnbinds& BatchedUnbinds = RHICmdList.GetScratchShaderUnbinds();

		UnsetSRVParameter(BatchedUnbinds, ComputeShader->VertexBufferParam);
		UnsetSRVParameter(BatchedUnbinds, ComputeShader->IndexBufferParam);

		RHICmdList.SetBatchedShaderUnbinds(ShaderRHI, BatchedUnbinds);
	}
}

// FRayTracingValidateSceneBuildParamsCS

IMPLEMENT_GLOBAL_SHADER(FRayTracingValidateSceneBuildParamsCS, "/Engine/Private/RayTracing/RayTracingValidation.usf", "RayTracingValidateSceneBuildParamsCS", SF_Compute);

void FRayTracingValidateSceneBuildParamsCS::Dispatch(FRHICommandList& RHICmdList, 
	uint32 NumHitGroups, uint32 NumInstances, 
	FRHIBuffer* InstanceBuffer, uint32 InstanceBufferOffset, uint32 InstanceBufferStride)
{
	const bool bSupportsWaveOps = GRHISupportsWaveOperations && RHISupportsWaveOperations(GMaxRHIShaderPlatform);
	if (!ensureMsgf(bSupportsWaveOps, TEXT("Wave operations are required to run ray tracing GPU validation shaders.")))
	{
		return;
	}

	TShaderMapRef<FRayTracingValidateSceneBuildParamsCS> ComputeShader(GetGlobalShaderMap(GMaxRHIFeatureLevel));
	FRHIComputeShader* ShaderRHI = ComputeShader.GetComputeShader();
	SetComputePipelineState(RHICmdList, ShaderRHI);

	{
		FRHIBatchedShaderParameters& BatchedParameters = RHICmdList.GetScratchShaderParameters();

		const FRawBufferShaderResourceViewInitializer InstanceBufferViewInitializer(InstanceBuffer);
		FShaderResourceViewRHIRef InstanceBufferSRV = RHICmdList.CreateShaderResourceView(InstanceBufferViewInitializer.Buffer, FRHIViewDesc::CreateBufferSRV().SetTypeFromBuffer(InstanceBufferViewInitializer.Buffer));

		SetShaderValue(BatchedParameters, ComputeShader->NumInstancesParam, NumInstances);
		SetShaderValue(BatchedParameters, ComputeShader->NumHitGroupsParam, NumHitGroups);
		SetShaderValue(BatchedParameters, ComputeShader->InstanceBufferOffsetInBytesParam, InstanceBufferOffset);
		SetShaderValue(BatchedParameters, ComputeShader->InstanceBufferStrideInBytesParam, InstanceBufferStride);
		SetSRVParameter(BatchedParameters, ComputeShader->InstanceBufferParam, InstanceBufferSRV);

		const uint32 MaxDispatchDimension = 65536; // D3D11_CS_DISPATCH_MAX_THREAD_GROUPS_PER_DIMENSION
		const uint32 NumGroupsX = FMath::Min((NumInstances + NumThreadsX - 1) / NumThreadsX, MaxDispatchDimension);

		RHICmdList.SetBatchedShaderParameters(ShaderRHI, BatchedParameters);

		RHICmdList.DispatchComputeShader(NumGroupsX, 1, 1);
	}

	FRHIBatchedShaderUnbinds& BatchedUnbinds = RHICmdList.GetScratchShaderUnbinds();
	UnsetSRVParameter(BatchedUnbinds, ComputeShader->InstanceBufferParam);
	RHICmdList.SetBatchedShaderUnbinds(ShaderRHI, BatchedUnbinds);
}

#endif // RHI_RAYTRACING
