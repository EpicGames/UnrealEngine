// Copyright Epic Games, Inc. All Rights Reserved.

#include "ReadOnlyCVARCache.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "Interfaces/ITargetPlatform.h"
#include "ShaderPlatformCachedIniValue.h"

#if WITH_EDITOR
#include "Interfaces/ITargetPlatformManagerModule.h"
#endif

bool FReadOnlyCVARCache::bInitialized = false;

bool FReadOnlyCVARCache::bEnablePointLightShadows = true;
bool FReadOnlyCVARCache::bEnableStationarySkylight = true;
bool FReadOnlyCVARCache::bEnableLowQualityLightmaps = true;
bool FReadOnlyCVARCache::bAllowStaticLighting = true;
bool FReadOnlyCVARCache::bSupportSkyAtmosphere = true;

// Mobile specific
bool FReadOnlyCVARCache::bMobileHDR = true;
bool FReadOnlyCVARCache::bMobileAllowDistanceFieldShadows = true;
bool FReadOnlyCVARCache::bMobileEnableStaticAndCSMShadowReceivers = true;
bool FReadOnlyCVARCache::bMobileEnableMovableLightCSMShaderCulling = true;
bool FReadOnlyCVARCache::bMobileSupportsGPUScene = false;
int32 FReadOnlyCVARCache::MobileEarlyZPassValue = 0;
int32 FReadOnlyCVARCache::MobileForwardLocalLightsValue = 1;
bool FReadOnlyCVARCache::bMobileForwardParticleLights = false;
bool FReadOnlyCVARCache::bMobileDeferredShadingValue = false;
bool FReadOnlyCVARCache::bMobileEnableMovableSpotlightsShadowValue = false;
int32 FReadOnlyCVARCache::MobileForwardDecalLightingValue = 1;
bool FReadOnlyCVARCache::bMobileAllowDitheredLODTransitionValue = false;
bool FReadOnlyCVARCache::bMobileAllowFramebufferFetchValue = false;

int32 FReadOnlyCVARCache::MobileEarlyZPassIniValue(EShaderPlatform Platform)
{
	static FShaderPlatformCachedIniValue<int32> CVar(TEXT("r.Mobile.EarlyZPass"));
	return CVar.Get(Platform);
}

int32 FReadOnlyCVARCache::MobileForwardLocalLightsIniValue(EShaderPlatform Platform)
{
	static FShaderPlatformCachedIniValue<int32> CVar(TEXT("r.Mobile.Forward.EnableLocalLights"));
	return CVar.Get(Platform);
}

bool FReadOnlyCVARCache::MobileForwardParticleLightsIniValue(EShaderPlatform Platform)
{
	static FShaderPlatformCachedIniValue<bool> CVar(TEXT("r.Mobile.Forward.EnableParticleLights"));
	return CVar.Get(Platform);
}

bool FReadOnlyCVARCache::MobileDeferredShadingIniValue(EShaderPlatform Platform)
{
	static FShaderPlatformCachedIniValue<bool> MobileShadingPathIniValue(TEXT("r.Mobile.ShadingPath"));
	static TConsoleVariableData<int32>* MobileAllowDeferredShadingOpenGL = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.Mobile.AllowDeferredShadingOpenGL"));
	// a separate cvar so we can exclude deferred from OpenGL specificaly

	bool bIsOpenGLPlatform = IsOpenGLPlatform(Platform);
#if WITH_EDITOR
	if (FDataDrivenShaderPlatformInfo::GetIsPreviewPlatform(Platform))
	{
		EShaderPlatform ParentShaderPlatform = FDataDrivenShaderPlatformInfo::GetPreviewShaderPlatformParent(Platform);
		bIsOpenGLPlatform = IsOpenGLPlatform(ParentShaderPlatform);
	}
#endif
	const bool bSupportedPlatform = !bIsOpenGLPlatform || (MobileAllowDeferredShadingOpenGL && MobileAllowDeferredShadingOpenGL->GetValueOnAnyThread() != 0);
	return MobileShadingPathIniValue.Get(Platform) && bSupportedPlatform;
}

bool FReadOnlyCVARCache::MobileEnableMovableSpotlightsShadowIniValue(EShaderPlatform Platform)
{
	static FShaderPlatformCachedIniValue<bool> CVar(TEXT("r.Mobile.EnableMovableSpotlightsShadow"));
	return CVar.Get(Platform);
}

bool FReadOnlyCVARCache::MobileAllowDitheredLODTransitionValueIniValue(EShaderPlatform Platform)
{
	static FShaderPlatformCachedIniValue<bool> CVar(TEXT("r.Mobile.AllowDitheredLODTransition"));
	return CVar.Get(Platform);
}

bool FReadOnlyCVARCache::MobileAllowFramebufferFetchIniValue(EShaderPlatform Platform)
{
	if (IsVulkanMobilePlatform(Platform) ||
		IsMetalMobilePlatform(Platform) ||
		IsAndroidOpenGLESPlatform(Platform))
	{
		static FShaderPlatformCachedIniValue<bool> CVar(TEXT("r.Mobile.AllowFramebufferFetch"));
		// Always allow in Forward, optional in Deferred
		return !MobileDeferredShading(Platform) || CVar.Get(Platform);
	}

	// Only GLES, Vulkan and Metal have FBF
	return false;
}

void FReadOnlyCVARCache::Initialize()
{
	UE_LOGF(LogInit, Log, "Initializing FReadOnlyCVARCache");

	const auto CVarSupportStationarySkylight = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.SupportStationarySkylight"));
	const auto CVarSupportLowQualityLightmaps = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.SupportLowQualityLightmaps"));
	const auto CVarSupportPointLightWholeSceneShadows = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.SupportPointLightWholeSceneShadows"));
	const auto CVarSupportAllShaderPermutations = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.SupportAllShaderPermutations"));	
	const auto CVarVertexFoggingForOpaque = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.VertexFoggingForOpaque"));	
	const auto CVarAllowStaticLighting = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.AllowStaticLighting"));
	const auto CVarSupportSkyAtmosphere = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.SupportSkyAtmosphere"));
	
	const auto CVarMobileHDR = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.MobileHDR"));
	const auto CVarMobileEnableStaticAndCSMShadowReceivers = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.Mobile.EnableStaticAndCSMShadowReceivers"));
	const auto CVarMobileEnableMovableLightCSMShaderCulling = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.Mobile.EnableMovableLightCSMShaderCulling"));
	const auto CVarMobileAllowDistanceFieldShadows = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.Mobile.AllowDistanceFieldShadows"));
	const auto CVarMobileSupportGPUScene = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.Mobile.SupportGPUScene"));
	const auto CVarMobileForwardDecalLightingValue = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.Mobile.Forward.DecalLighting"));
	
	const bool bForceAllPermutations = CVarSupportAllShaderPermutations && CVarSupportAllShaderPermutations->GetValueOnAnyThread() != 0;

	bEnableStationarySkylight = !CVarSupportStationarySkylight || CVarSupportStationarySkylight->GetValueOnAnyThread() != 0 || bForceAllPermutations;
	bEnablePointLightShadows = !CVarSupportPointLightWholeSceneShadows || CVarSupportPointLightWholeSceneShadows->GetValueOnAnyThread() != 0 || bForceAllPermutations;
	bEnableLowQualityLightmaps = !CVarSupportLowQualityLightmaps || CVarSupportLowQualityLightmaps->GetValueOnAnyThread() != 0 || bForceAllPermutations;
	bAllowStaticLighting = CVarAllowStaticLighting->GetValueOnAnyThread() != 0;
	bSupportSkyAtmosphere = !CVarSupportSkyAtmosphere || CVarSupportSkyAtmosphere->GetValueOnAnyThread() != 0 || bForceAllPermutations;

	// mobile
	bMobileHDR = CVarMobileHDR->GetValueOnAnyThread() == 1;
	bMobileAllowDistanceFieldShadows = CVarMobileAllowDistanceFieldShadows->GetValueOnAnyThread() != 0;
	bMobileEnableStaticAndCSMShadowReceivers = CVarMobileEnableStaticAndCSMShadowReceivers->GetValueOnAnyThread() != 0;
	bMobileEnableMovableLightCSMShaderCulling = CVarMobileEnableMovableLightCSMShaderCulling->GetValueOnAnyThread() != 0;
	MobileEarlyZPassValue = MobileEarlyZPassIniValue(GMaxRHIShaderPlatform);
	MobileForwardLocalLightsValue = MobileForwardLocalLightsIniValue(GMaxRHIShaderPlatform);
	bMobileDeferredShadingValue = MobileDeferredShadingIniValue(GMaxRHIShaderPlatform);
	bMobileEnableMovableSpotlightsShadowValue = MobileEnableMovableSpotlightsShadowIniValue(GMaxRHIShaderPlatform);
	bMobileSupportsGPUScene = CVarMobileSupportGPUScene->GetValueOnAnyThread() != 0;
	bMobileForwardParticleLights = MobileForwardParticleLightsIniValue(GMaxRHIShaderPlatform);
	MobileForwardDecalLightingValue = CVarMobileForwardDecalLightingValue->GetValueOnAnyThread();
	bMobileAllowDitheredLODTransitionValue = MobileAllowDitheredLODTransitionValueIniValue(GMaxRHIShaderPlatform);
	bMobileAllowFramebufferFetchValue = MobileAllowFramebufferFetchIniValue(GMaxRHIShaderPlatform);

#ifdef PROJECT_CVAR_ALLOW_STATIC_LIGHTING
	check(!!PROJECT_CVAR_ALLOW_STATIC_LIGHTING == bAllowStaticLighting);
#endif

#ifdef PROJECT_CVAR_MOBILE_HDR
	check(!!PROJECT_CVAR_MOBILE_HDR == bMobileHDR);
#endif

#ifdef PROJECT_CVAR_MOBILE_SUPPORTS_GPUSCENE
	check(!!PROJECT_CVAR_MOBILE_SUPPORTS_GPUSCENE == bMobileSupportsGPUScene);
#endif

#ifdef PROJECT_CVAR_MOBILE_FORWARD_LOCALLIGHTS
	check(PROJECT_CVAR_MOBILE_FORWARD_LOCALLIGHTS == MobileForwardLocalLightsValue);
#endif

#ifdef PROJECT_CVAR_MOBILE_DEFERRED_SHADING
	check(!!PROJECT_CVAR_MOBILE_DEFERRED_SHADING == bMobileDeferredShadingValue);
#endif
	
	bInitialized = true;
}
