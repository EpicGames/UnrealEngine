// Copyright Epic Games, Inc. All Rights Reserved.

#include "RenderCaptureInterface.h"
#include "IRenderCaptureProvider.h"
#include "RenderGraphBuilder.h"
#include "RenderingThread.h"

namespace RenderCaptureInterface
{
	FScopedCapture::FScopedCapture(bool bEnable, TCHAR const* InEventName, TCHAR const* InFileName)
		: bCapture(bEnable && IRenderCaptureProvider::IsAvailable())
		, bEvent(InEventName != nullptr)
		, RHICommandList(nullptr)
		, GraphBuilder(nullptr)
	{
		check(!GIsThreadedRendering || !IsInRenderingThread());

		if (bCapture)
		{
#if WITH_RHI_BREADCRUMBS
			RHIBreadcrumb = MakeUnique<TOptional<FRHIBreadcrumbEventManual>>();
#endif

			ENQUEUE_RENDER_COMMAND(BeginCaptureCommand)([
				  FileName = FString(InFileName)
#if WITH_RHI_BREADCRUMBS
				, EventName = FString(InEventName)
				, Breadcrumb = RHIBreadcrumb.Get()
				, bPushEvent = bEvent
#endif
			](FRHICommandListImmediate& RHICommandListLocal)
			{
				IRenderCaptureProvider::Get().BeginCapture(&RHICommandListLocal, IRenderCaptureProvider::ECaptureFlags_Launch, FileName);

#if WITH_RHI_BREADCRUMBS
				if (bPushEvent)
				{
					Breadcrumb->Emplace(RHICommandListLocal, RHI_BREADCRUMB_DESC_FORWARD_VALUES(TEXT("FScopedCapture"), TEXT("%s"), RHI_GPU_STAT_ARGS_NONE)(EventName));
				}
#endif
			});
		}
	}

	FScopedCapture::FScopedCapture(bool bEnable, FRHICommandList* InRHICommandList, TCHAR const* InEventName, TCHAR const* InFileName)
		: bCapture(bEnable && IRenderCaptureProvider::IsAvailable() && InRHICommandList->IsImmediate())
		, bEvent(InEventName != nullptr)
		, RHICommandList(InRHICommandList)
		, GraphBuilder(nullptr)
	{
		if (bCapture)
		{
			check(!GIsThreadedRendering || IsInRenderingThread());

			IRenderCaptureProvider::Get().BeginCapture(&FRHICommandListImmediate::Get(*RHICommandList), IRenderCaptureProvider::ECaptureFlags_Launch, FString(InFileName));
		
#if WITH_RHI_BREADCRUMBS
			if (bEvent)
			{
				RHIBreadcrumb = MakeUnique<TOptional<FRHIBreadcrumbEventManual>>();
				RHIBreadcrumb->Emplace(*RHICommandList, RHI_BREADCRUMB_DESC_FORWARD_VALUES(TEXT("FScopedCapture"), TEXT("%s"), RHI_GPU_STAT_ARGS_NONE)(FString(InEventName)));
			}
#endif
		}
	}

	FScopedCapture::FScopedCapture(bool bEnable, FRDGBuilder& InGraphBuilder, TCHAR const* InEventName, TCHAR const* InFileName)
		: bCapture(bEnable&& IRenderCaptureProvider::IsAvailable())
		, bEvent(InEventName != nullptr)
		, RHICommandList(nullptr)
		, GraphBuilder(&InGraphBuilder) 
	{
		check(!GIsThreadedRendering || IsInRenderingThread());

		if (bCapture)
		{
			GraphBuilder->AddPass(
				RDG_EVENT_NAME("BeginCapture"),
				ERDGPassFlags::NeverCull, 
				[FileName = FString(InFileName)](FRHICommandListImmediate& RHICommandListLocal)
				{
					IRenderCaptureProvider::Get().BeginCapture(&RHICommandListLocal, IRenderCaptureProvider::ECaptureFlags_Launch, FString(FileName));
				});

#if RDG_EVENTS
			RDG_EVENT_SCOPE_CONSTRUCT(RDGEvent, *GraphBuilder, bEvent, ERDGScopeFlags::None, RHI_GPU_STAT_ARGS_NONE, TEXT("FScopedCapture"), TEXT("%s"), FString(InEventName));
#endif
		}
	}

	FScopedCapture::~FScopedCapture()
	{
		if (bCapture)
		{
			if (GraphBuilder != nullptr)
			{
				check(!GIsThreadedRendering || IsInRenderingThread());

#if RDG_EVENTS
				if (bEvent)
				{
					RDGEvent.Reset();
				}
#endif

				GraphBuilder->AddPass(
					RDG_EVENT_NAME("EndCapture"), 
					ERDGPassFlags::NeverCull, 
					[](FRHICommandListImmediate& RHICommandListLocal)
					{
						IRenderCaptureProvider::Get().EndCapture(&RHICommandListLocal);
					});
			}
			else if (RHICommandList != nullptr)
			{
				check(!GIsThreadedRendering || IsInRenderingThread());
				
#if WITH_RHI_BREADCRUMBS
				if (bEvent)
				{
					(*RHIBreadcrumb)->End(*RHICommandList);
				}
#endif

				IRenderCaptureProvider::Get().EndCapture(&FRHICommandListImmediate::Get(*RHICommandList));
			}
			else
			{
				check(!GIsThreadedRendering || !IsInRenderingThread());

				ENQUEUE_RENDER_COMMAND(EndCaptureCommand)([
#if WITH_RHI_BREADCRUMBS
					  bPopEvent = bEvent
					, Breadcrumb = MoveTemp(RHIBreadcrumb)
#endif
				](FRHICommandListImmediate& RHICommandListLocal)
				{
#if WITH_RHI_BREADCRUMBS
					if (bPopEvent)
					{
						(*Breadcrumb)->End(RHICommandListLocal);
					}
#endif

					IRenderCaptureProvider::Get().EndCapture(&RHICommandListLocal);
				});
			}
		}
 	}
}
