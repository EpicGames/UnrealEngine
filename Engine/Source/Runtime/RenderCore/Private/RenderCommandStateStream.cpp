// Copyright Epic Games, Inc. All Rights Reserved.

#include "RenderCommandStateStream.h"

#if WITH_STATE_STREAM

#include "RenderCommandFence.h"
#include "RenderingThread.h"
#include "StateStream.h"
#include "StateStreamCreator.h"
#include "StateStreamDebugRenderer.h"
#include "StateStreamManagerImpl.h"

#define UE_RENDER_COMMAND_DEBUG_DRAW 1

////////////////////////////////////////////////////////////////////////////////////////////////////

class FRenderCommandStateStream;
FRenderCommandStateStream* g_renderCommandStateStream;

constexpr uint32 RenderCommandCategoryCount = uint32(ERenderCommandCategory::Unknown) + 1;

class FRenderCommandStateStream : public IStateStream
{
public:
	FRenderCommandStateStream()
	{
		g_renderCommandStateStream = this;
		// TODO: Flush
		//FRenderCommandFence Fence;
		//Fence.BeginFence();
		//Fence.Wait(false);
	}
	~FRenderCommandStateStream()
	{
		g_renderCommandStateStream = nullptr;
	}
	virtual void Game_BeginTick(uint32 LaneId) override
	{
	}
	virtual void Game_EndTick(StateStreamTime AbsoluteTime, uint32 LaneId) override
	{
	}
	virtual void Game_SetDefaultLane(uint32 LaneId) override
	{
		ActiveDefaultLaneId = LaneId;
	}
	virtual void Game_Exit() override
	{
	}
	virtual void* Game_GetVoidPointer() override
	{
		return this;
	}
	virtual void Game_DebugRender(IStateStreamDebugRenderer& Renderer)
	{
		TStringBuilder<1024> DebugLine;
		DebugLine.Appendf(TEXT("State stream : [%s]"), GetDebugName());
		Renderer.DrawText(*DebugLine, FColor::Orange);

#if UE_RENDER_COMMAND_DEBUG_DRAW
		uint32 X = 500;

		static const TCHAR* Categories[] = 
		{
			TEXT("EnterTick"),
			TEXT("LoopTick"),
			TEXT("LeaveTick"),
			TEXT("Unknown"),
		};
		static_assert(UE_ARRAY_COUNT(Categories) == RenderCommandCategoryCount);

		for (uint32 I=0; I!=4; ++I)
		{
			uint32 Y = 100;
			FScopeLock Lock(&RenderCommandsLock[I]);
			uint32 Num = RenderCommands[I].Num();
			if (!Num)
			{
				continue;
			}
			DebugLine.Reset();
			DebugLine.Appendf(TEXT("%s RenderCommands = %d"), Categories[I], Num);
			Renderer.DrawText(X, Y, *DebugLine, FColor::Blue);
			Y += 20;

			for (auto& Pair : RenderCommands[I])
			{
				if (Y == 1000)
				{
					Y = 100;
					X += 300;
				}
				DebugLine.Reset();
				DebugLine.Appendf(TEXT("%s, %d"), *Pair.Key, Pair.Value);
				Renderer.DrawText(X, Y, *DebugLine, FColor::Orange);
				Y += 20;
			}
			RenderCommands[I].Empty();
			X += 300;
		}
#endif
	}

	virtual void Render_Update(StateStreamTime AbsoluteTime) override
	{
	}
	virtual void Render_PostUpdate() override
	{
	}
	virtual void Render_Exit() override
	{
	}
	virtual void Render_GarbageCollect() override
	{
	}
	virtual void Render_Enable(bool Enable) override
	{
	}
	virtual uint32 GetId() override
	{
		return RenderCommandStateStreamId;
	}
	virtual const TCHAR* GetDebugName() override { return TEXT("RenderCommand"); }

	bool AddCommand(TUniqueFunction<void(FRHICommandListImmediate&)>& Function, const FRenderCommandTag& Tag)
	{
		// If no lane is created, then we just flush things through... when lanes are created we wait and clear RC


		if (IsInGameThread() || IsInParallelGameThread())
		{
			//check(g_renderCommandStateStream->ActiveDefaultLaneId != DefaultLaneId);

#if UE_RENDER_COMMAND_DEBUG_DRAW
			FScopeLock Lock(&RenderCommandsLock[(uint32)Tag.GetCategory()]);
			++RenderCommands[(uint32)Tag.GetCategory()].FindOrAdd(Tag.GetName(), 0);
#endif

			// These should be added to current tick of RenderCommandStateStream
		}
		else // if (GRenderingThreadHeartbeat != FRunnableThread::GetRunnableThread())
		{
			// These can be called in between game ticks but still needs to be in sync with ticks.. will need to queue up separately
			// and inject to beginning of next tick when begin?

			// Things that could end up here
			//   GRenderingThreadHeartbeat (FDefaultGameMoviePlayer, etc)
			//   AsyncPackage2 loading (FMaterialShaderMap, FTextureReference, etc)
			//   FSlateRHIRenderer::ReleaseDrawBuffer
		}
		return false;
	}

	enum { Id = RenderCommandStateStreamId };

	uint32 ActiveDefaultLaneId = DefaultLaneId;

	FCriticalSection RenderCommandsLock[RenderCommandCategoryCount];
	TMap<FString,uint32> RenderCommands[RenderCommandCategoryCount];
};

////////////////////////////////////////////////////////////////////////////////////////////////////

bool RenderCommandStateStream_AddCommand(TUniqueFunction<void(FRHICommandListImmediate&)>& Function, const FRenderCommandTag& Tag)
{
	check(g_renderCommandStateStream);
	return g_renderCommandStateStream->AddCommand(Function, Tag);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

STATESTREAM_CREATOR_INSTANCE(FRenderCommandStateStream)

////////////////////////////////////////////////////////////////////////////////////////////////////

#endif
