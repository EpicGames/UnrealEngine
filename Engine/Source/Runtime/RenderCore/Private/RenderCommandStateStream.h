// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Build.h"

#if WITH_STATE_STREAM

#include "Templates/Function.h"

class FRHICommandListImmediate;
class FRenderCommandTag;

////////////////////////////////////////////////////////////////////////////////////////////////////

inline constexpr uint32 RenderCommandStateStreamId = 14;

// This will take Function if returning true
bool RenderCommandStateStream_AddCommand(TUniqueFunction<void(FRHICommandListImmediate&)>& Function, const FRenderCommandTag& Tag);

////////////////////////////////////////////////////////////////////////////////////////////////////

#endif