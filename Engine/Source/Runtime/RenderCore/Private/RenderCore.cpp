// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	RenderCore.h: Render core module implementation.
=============================================================================*/

#include "RenderCore.h"
#include "DynamicRHI.h"
#include "HAL/IConsoleManager.h"
#include "Misc/CommandLine.h"
#include "Modules/ModuleManager.h"
#include "HDRHelper.h"
#include "RHI.h"
#include "RenderTimer.h"
#include "RenderCounters.h"
#include "RenderingThread.h"
#include "RendererInterface.h"
#include "VirtualTexturing.h"

void UpdateShaderDevelopmentMode();

void InitRenderGraph();
void ShutdownRenderGraph();

static void InitPixelRenderCounters();

class FRenderCoreModule : public FDefaultModuleImpl
{
public:

	virtual void StartupModule() override
	{
		// TODO(RDG): Why is this not getting called?!
		IConsoleManager::Get().RegisterConsoleVariableSink_Handle(FConsoleCommandDelegate::CreateStatic(&UpdateShaderDevelopmentMode));

		InitRenderGraph();
		InitPixelRenderCounters();
	}

	virtual void ShutdownModule() override
	{
		ShutdownRenderGraph();
	}
};

IMPLEMENT_MODULE(FRenderCoreModule, RenderCore);

DEFINE_LOG_CATEGORY(LogRendererCore);

/*------------------------------------------------------------------------------
	Stat declarations.
-----------------------------------------------------------------------------*/
// Cycle stats are rendered in reverse order from what they are declared in.
// They are organized so that stats at the top of the screen are earlier in the frame, 
// And stats that are indented are lower in the call hierarchy.

// The purpose of the SceneRendering stat group is to show where rendering thread time is going from a high level.
// It should only contain stats that are likely to track a lot of time in a typical scene, not edge case stats that are rarely non-zero.


// Amount of time measured by 'RenderViewFamily' that is not accounted for in its children stats
// Use a more detailed profiler (like an instruction trace or sampling capture on Xbox 360) to track down where this time is going if needed
DEFINE_STAT(STAT_RenderVelocities);
DEFINE_STAT(STAT_FinishRenderViewTargetTime);
DEFINE_STAT(STAT_CacheUniformExpressions);
DEFINE_STAT(STAT_TranslucencyDrawTime);
DEFINE_STAT(STAT_BeginOcclusionTestsTime);
// Use 'stat shadowrendering' to get more detail
DEFINE_STAT(STAT_ProjectedShadowDrawTime);
DEFINE_STAT(STAT_LightingDrawTime);
DEFINE_STAT(STAT_DynamicPrimitiveDrawTime);
DEFINE_STAT(STAT_StaticDrawListDrawTime);
DEFINE_STAT(STAT_BasePassDrawTime);
DEFINE_STAT(STAT_AnisotropyPassDrawTime);
DEFINE_STAT(STAT_DepthDrawTime);
DEFINE_STAT(STAT_WaterPassDrawTime);
DEFINE_STAT(STAT_DynamicShadowSetupTime);
DEFINE_STAT(STAT_RenderQueryResultTime);
// Use 'stat initviews' to get more detail
DEFINE_STAT(STAT_InitViewsTime);
DEFINE_STAT(STAT_RayTracing_FinishGatherInstances);
DEFINE_STAT(STAT_InitViewsPossiblyAfterPrepass);
// Measures the time spent in RenderViewFamily_RenderThread
// Note that this is not the total rendering thread time, any other rendering commands will not be counted here
DEFINE_STAT(STAT_TotalSceneRenderingTime);
DEFINE_STAT(STAT_TotalGPUFrameTime);
DEFINE_STAT(STAT_PresentTime);

DEFINE_STAT(STAT_SceneLights);
DEFINE_STAT(STAT_MeshDrawCalls);

DEFINE_STAT(STAT_SceneDecals);
DEFINE_STAT(STAT_Decals);
DEFINE_STAT(STAT_DecalsDrawTime);

// Memory stats for tracking virtual allocations used by the renderer to represent the scene
// The purpose of these memory stats is to capture where most of the renderer allocated memory is going, 
// Not to track all of the allocations, and not to track resource memory (index buffers, vertex buffers, etc).

DEFINE_STAT(STAT_PrimitiveInfoMemory);
DEFINE_STAT(STAT_RenderingSceneMemory);
DEFINE_STAT(STAT_ViewStateMemory);
DEFINE_STAT(STAT_LightInteractionMemory);

// The InitViews stats group contains information on how long visibility culling took and how effective it was

DEFINE_STAT(STAT_GatherShadowPrimitivesTime);
DEFINE_STAT(STAT_BuildCSMVisibilityState);
DEFINE_STAT(STAT_UpdateIndirectLightingCache);
DEFINE_STAT(STAT_UpdateIndirectLightingCachePrims);
DEFINE_STAT(STAT_UpdateIndirectLightingCacheBlocks);
DEFINE_STAT(STAT_InterpolateVolumetricLightmapOnCPU);
DEFINE_STAT(STAT_UpdateIndirectLightingCacheTransitions);
DEFINE_STAT(STAT_UpdateIndirectLightingCacheFinalize);
DEFINE_STAT(STAT_SortStaticDrawLists);
DEFINE_STAT(STAT_InitDynamicShadowsTime);
DEFINE_STAT(STAT_InitProjectedShadowVisibility);
DEFINE_STAT(STAT_UpdatePreshadowCache);
DEFINE_STAT(STAT_CreateWholeSceneProjectedShadow);
DEFINE_STAT(STAT_AddViewDependentWholeSceneShadowsForView);
DEFINE_STAT(STAT_SetupInteractionShadows);
DEFINE_STAT(STAT_GetDynamicMeshElements);
DEFINE_STAT(STAT_SetupMeshPass);
DEFINE_STAT(STAT_UpdateStaticMeshesTime);
DEFINE_STAT(STAT_StaticRelevance);
DEFINE_STAT(STAT_ViewRelevance);
DEFINE_STAT(STAT_ComputeViewRelevance);
DEFINE_STAT(STAT_OcclusionCull);
DEFINE_STAT(STAT_UpdatePrimitiveFading);
DEFINE_STAT(STAT_UpdateAlwaysVisible);
DEFINE_STAT(STAT_DecompressPrecomputedOcclusion);
DEFINE_STAT(STAT_ViewVisibilityTime);

DEFINE_STAT(STAT_RayTracingTotalInstances);
DEFINE_STAT(STAT_RayTracingActiveInstances);
DEFINE_STAT(STAT_ProcessedPrimitives);
DEFINE_STAT(STAT_CulledPrimitives);
DEFINE_STAT(STAT_VisibleRayTracingPrimitives);
DEFINE_STAT(STAT_StaticallyOccludedPrimitives);
DEFINE_STAT(STAT_OccludedPrimitives);
DEFINE_STAT(STAT_OcclusionQueries);
DEFINE_STAT(STAT_VisibleStaticMeshElements);
DEFINE_STAT(STAT_VisibleDynamicPrimitives);
DEFINE_STAT(STAT_IndirectLightingCacheUpdates);
DEFINE_STAT(STAT_PrecomputedLightingBufferUpdates);
DEFINE_STAT(STAT_CSMSubjects);
DEFINE_STAT(STAT_CSMStaticMeshReceivers);
DEFINE_STAT(STAT_CSMStaticPrimitiveReceivers);

DEFINE_STAT(STAT_CreateRayTracingPipeline);
DEFINE_STAT(STAT_CreateLumenRayTracingPipeline);

// The ShadowRendering stats group shows what kind of shadows are taking a lot of rendering thread time to render
// Shadow setup is tracked in the InitViews group

DEFINE_STAT(STAT_RenderWholeSceneShadowProjectionsTime);
DEFINE_STAT(STAT_RenderWholeSceneShadowDepthsTime);
DEFINE_STAT(STAT_RenderPerObjectShadowProjectionsTime);
DEFINE_STAT(STAT_RenderPerObjectShadowDepthsTime);

DEFINE_STAT(STAT_WholeSceneShadows);
DEFINE_STAT(STAT_CachedWholeSceneShadows);
DEFINE_STAT(STAT_PerObjectShadows);
DEFINE_STAT(STAT_PreShadows);
DEFINE_STAT(STAT_CachedPreShadows);
DEFINE_STAT(STAT_ShadowDynamicPathDrawCalls);

DEFINE_STAT(STAT_TranslucentInjectTime);
DEFINE_STAT(STAT_DirectLightRenderingTime);
DEFINE_STAT(STAT_LightRendering);

DEFINE_STAT(STAT_NumShadowedLights);
DEFINE_STAT(STAT_NumLightFunctionOnlyLights);
DEFINE_STAT(STAT_NumBatchedLights);
DEFINE_STAT(STAT_NumLightsInjectedIntoTranslucency);
DEFINE_STAT(STAT_NumLightsInjectedIntoTranslucencyBatched);
DEFINE_STAT(STAT_NumLightsUsingStandardDeferred);

DEFINE_STAT(STAT_LightShaftsLights);

DEFINE_STAT(STAT_ParticleUpdateRTTime);
DEFINE_STAT(STAT_InfluenceWeightsUpdateRTTime);
DEFINE_STAT(STAT_GPUSkinUpdateRTTime);
DEFINE_STAT(STAT_CPUSkinUpdateRTTime);

DEFINE_STAT(STAT_UpdateGPUSceneTime);

DEFINE_STAT(STAT_RemoveSceneLightTime);
DEFINE_STAT(STAT_UpdateSceneLightTime);
DEFINE_STAT(STAT_AddSceneLightTime);

DEFINE_STAT(STAT_RemoveScenePrimitiveTime);
DEFINE_STAT(STAT_AddScenePrimitiveRenderThreadTime);
DEFINE_STAT(STAT_UpdateScenePrimitiveRenderThreadTime);
DEFINE_STAT(STAT_UpdatePrimitiveTransformRenderThreadTime);
DEFINE_STAT(STAT_UpdatePrimitiveInstanceRenderThreadTime);

DEFINE_STAT(STAT_RemoveScenePrimitiveGT);
DEFINE_STAT(STAT_AddScenePrimitiveGT);
DEFINE_STAT(STAT_UpdatePrimitiveTransformGT);
DEFINE_STAT(STAT_UpdatePrimitiveInstanceGT);
DEFINE_STAT(STAT_UpdatePrimitiveInstanceFromComputeGT);

DEFINE_STAT(STAT_Scene_SetShaderMapsOnMaterialResources_RT);
DEFINE_STAT(STAT_Scene_UpdateStaticDrawLists_RT);
DEFINE_STAT(STAT_Scene_UpdateStaticDrawListsForMaterials_RT);
DEFINE_STAT(STAT_GameToRendererMallocTotal);

DEFINE_STAT(STAT_NumReflectiveShadowMapLights);

DEFINE_STAT(STAT_ShadowmapAtlasMemory);
DEFINE_STAT(STAT_CachedShadowmapMemory);

DEFINE_STAT(STAT_RenderTargetPoolSize);
DEFINE_STAT(STAT_RenderTargetPoolUsed);
DEFINE_STAT(STAT_RenderTargetPoolCount);

#define EXPOSE_FORCE_LOD !(UE_BUILD_SHIPPING || UE_BUILD_TEST) || WITH_EDITOR

#if EXPOSE_FORCE_LOD

static TAutoConsoleVariable<int32> CVarForceLOD(
	TEXT("r.ForceLOD"),
	-1,
	TEXT("LOD level to force, -1 is off."),
	ECVF_Scalability | ECVF_Default | ECVF_RenderThreadSafe
	);

static TAutoConsoleVariable<int32> CVarForceLODShadow(
	TEXT("r.ForceLODShadow"),
	-1,
	TEXT("LOD level to force for the shadow map generation only, -1 is off."),
	ECVF_Scalability | ECVF_Default | ECVF_RenderThreadSafe
);

#endif // EXPOSE_FORCE_LOD

FPixelRenderCounters GPixelRenderCounters;

void TickPixelRenderCounters()
{
	check(IsInRenderingThread());
	GPixelRenderCounters.PrevPixelRenderCount = GPixelRenderCounters.CurrentPixelRenderCount;
	GPixelRenderCounters.PrevPixelDisplayCount = GPixelRenderCounters.CurrentPixelDisplayCount;
	GPixelRenderCounters.CurrentPixelRenderCount = 0;
	GPixelRenderCounters.CurrentPixelDisplayCount = 0;
}

void InitPixelRenderCounters()
{
	FCoreDelegates::OnBeginFrameRT.AddStatic(TickPixelRenderCounters);
}

// Can be optimized to avoid the virtual function call but it's compiled out for final release anyway
RENDERCORE_API int32 GetCVarForceLOD()
{
	int32 Ret = -1;

#if EXPOSE_FORCE_LOD
	{
		Ret = CVarForceLOD.GetValueOnRenderThread();
	}
#endif // EXPOSE_FORCE_LOD

	return Ret;
}

RENDERCORE_API int32 GetCVarForceLOD_AnyThread()
{
	int32 Ret = -1;

#if EXPOSE_FORCE_LOD
	{
		Ret = CVarForceLOD.GetValueOnAnyThread();
	}
#endif // EXPOSE_FORCE_LOD

	return Ret;
}

RENDERCORE_API int32 GetCVarForceLODShadow()
{
	int32 Ret = -1;

#if EXPOSE_FORCE_LOD
	{
		Ret = CVarForceLODShadow.GetValueOnRenderThread();
	}
#endif // EXPOSE_FORCE_LOD

	return Ret;
}

RENDERCORE_API int32 GetCVarForceLODShadow_AnyThread()
{
	int32 Ret = -1;

#if EXPOSE_FORCE_LOD
	{
		Ret = CVarForceLODShadow.GetValueOnAnyThread();
	}
#endif // EXPOSE_FORCE_LOD

	return Ret;
}

//Setter function to keep GNearClippingPlane and GNearClippingPlane_RenderThread in sync
RENDERCORE_API void SetNearClipPlaneGlobals(float NewNearClipPlane)
{
	GNearClippingPlane = NewNearClipPlane;

	//Set GNearClippingPlane_RenderThread in a render command to be RT safe
	ENQUEUE_RENDER_COMMAND(SetNearClipPlane_RenderThread)(
		[NewNearClipPlane](FRHICommandListImmediate& RHICmdList)
		{
			GNearClippingPlane_RenderThread = NewNearClipPlane;
		});
}

// Note: Enables or disables HDR support for a project. Typically this would be set on a per-project/per-platform basis in defaultengine.ini
TAutoConsoleVariable<int32> CVarAllowHDR(
	TEXT("r.AllowHDR"),
	0,
	TEXT("Creates an HDR compatible swap-chain and enables HDR display output.")
	TEXT("0: Disabled (default)\n")
	TEXT("1: Allow HDR, if supported by the platform and display \n"),
	ECVF_ReadOnly);

// Note: These values are directly referenced in code. They are set in code at runtime and therefore cannot be set via ini files
// Please update all paths if changing
TAutoConsoleVariable<int32> CVarDisplayColorGamut(
	TEXT("r.HDR.Display.ColorGamut"),
	0,
	TEXT("Color gamut of the output display:\n")
	TEXT("0: Rec709 / sRGB, D65 (default)\n")
	TEXT("1: DCI-P3, D65\n")
	TEXT("2: Rec2020 / BT2020, D65\n")
	TEXT("3: ACES, D60\n")
	TEXT("4: ACEScg, D60\n"),
	ECVF_Scalability | ECVF_RenderThreadSafe);

TAutoConsoleVariable<int32> CVarDisplayOutputDevice(
	TEXT("r.HDR.Display.OutputDevice"),
	0,
	TEXT("Device format of the output display:\n")
	TEXT("0: sRGB (LDR)\n")
	TEXT("1: Rec709 (LDR)\n")
	TEXT("2: Explicit gamma mapping (LDR)\n")
	TEXT("3: ACES 1000 nit ST-2084 (Dolby PQ) (HDR)\n")
	TEXT("4: ACES 2000 nit ST-2084 (Dolby PQ) (HDR)\n")
	TEXT("5: ACES 1000 nit ScRGB (HDR)\n")
	TEXT("6: ACES 2000 nit ScRGB (HDR)\n")
	TEXT("7: Linear EXR (HDR)\n")
	TEXT("8: Linear final color, no tone curve (HDR)\n")
	TEXT("9: Linear final color with tone curve\n"),
	ECVF_Scalability | ECVF_RenderThreadSafe);

static TAutoConsoleVariable<float> CVarHDRDisplayMinLuminanceLog10(
	TEXT("r.HDR.Display.MinLuminanceLog10"),
	-4.0f,
	TEXT("The configured minimum display output nit level (log10 value)"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<float> CVarHDRDisplayMidLuminance(
	TEXT("r.HDR.Display.MidLuminance"),
	15.0f,
	TEXT("The configured display output nit level for 18% gray"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarHDRDisplayMaxLuminance(
	TEXT("r.HDR.Display.MaxLuminance"),
	0,
	TEXT("The configured display output nit level, assuming HDR output is enabled."),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<bool> CVarHDRDisplayOverrideOSMaxLuminance(
	TEXT("r.HDR.Display.OverrideOSMaxLuminance"),
	false,
	TEXT("Whether to override the maximum display luminance provided by the OS."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<float> CVarHDRAcesColorMultiplier(
	TEXT("r.HDR.Aces.SceneColorMultiplier"),
	1.5f,
	TEXT("Multiplier applied to scene color. Helps to"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<float> CVarHDRAcesGamutCompression(
	TEXT("r.HDR.Aces.GamutCompression"),
	0.0f,
	TEXT("HDR equivalent of BlueCorrection: Bright blue desaturates instead of going to violet"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarHDRAcesVersion(
	TEXT("r.HDR.Aces.Version"),
	2,
	TEXT("Version of the ACES transform:\n")
	TEXT("1: ACES 1.3 (default)\n")
	TEXT("2: ACES 2.0\n"),
	ECVF_RenderThreadSafe
);

TAutoConsoleVariable<int32> CVarHDROutputEnabled(
	TEXT("r.HDR.EnableHDROutput"),
	0,
	TEXT("Creates an HDR compatible swap-chain and enables HDR display output.\n")
	TEXT("0: Disabled (default)\n")
	TEXT("1: Enable hardware-specific implementation\n"),
	ECVF_RenderThreadSafe
);

TAutoConsoleVariable<float> CVarHDRPaperWhite(
	TEXT("r.HDR.PaperWhite"),
	203.f,
	TEXT("Luminance in nits to redefine HDR Reference White as specified in BT.2408."),
	ECVF_RenderThreadSafe
);

TAutoConsoleVariable<int32> CVarHDRPaperWhiteMode(
	TEXT("r.HDR.PaperWhite.Mode"),
	0,
	TEXT("Specifies how scene content incorporates paper white values.\n")
	TEXT("0: Use the platform's paper white if available, otherwise r.HDR.PaperWhite (default)\n")
	TEXT("1: Use r.HDR.PaperWhite"),
	ECVF_RenderThreadSafe
);

RENDERCORE_API bool IsHDREnabled()
{
	return GRHISupportsHDROutput && CVarHDROutputEnabled.GetValueOnAnyThread() != 0;
}

RENDERCORE_API bool IsHDRAllowed()
{
	// HDR can be forced on or off on the commandline. Otherwise we check the cvar r.AllowHDR
	static uint32 LastCommandLineVersionParsed = FCommandLine::GetCommandLineVersion();
	static bool bYesHDR = FParse::Param(FCommandLine::Get(), TEXT("hdr"));
	static bool bNoHDR = FParse::Param(FCommandLine::Get(), TEXT("nohdr"));

	if (LastCommandLineVersionParsed != FCommandLine::GetCommandLineVersion())
	{
		LastCommandLineVersionParsed = FCommandLine::GetCommandLineVersion();
		bYesHDR = FParse::Param(FCommandLine::Get(), TEXT("hdr"));
		bNoHDR = FParse::Param(FCommandLine::Get(), TEXT("nohdr"));
	}

	if (bYesHDR)
	{
		return true;
	}
	else if (bNoHDR)
	{
		return false;
	}

	return (CVarAllowHDR.GetValueOnAnyThread() != 0);
}

RENDERCORE_API bool IsDefaultBackBufferLinearSDR()
{
	// Note: Renderer settings console variable requires project restart, so we can statically cache this check.
	static const bool bIsDefaultBackBufferLinear = []() -> bool
		{
			const TConsoleVariableData<int32>* CVarDefaultBackBufferPixelFormat = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.DefaultBackBufferPixelFormat"));
			return CVarDefaultBackBufferPixelFormat ? CVarDefaultBackBufferPixelFormat->GetValueOnAnyThread() == 3 /* EDefaultBackBufferPixelFormat::DBBPF_FloatRGBA */ : false;
		}();
	
	return bIsDefaultBackBufferLinear && !IsHDREnabled();
}

RENDERCORE_API EDisplayOutputFormat HDRGetDefaultDisplayOutputFormat()
{
	return static_cast<EDisplayOutputFormat>(FMath::Clamp(CVarDisplayOutputDevice.GetValueOnAnyThread(), 0, static_cast<int32>(EDisplayOutputFormat::MAX) - 1));
}

RENDERCORE_API EDisplayColorGamut HDRGetDefaultDisplayColorGamut()
{
	return static_cast<EDisplayColorGamut>(FMath::Clamp(CVarDisplayColorGamut.GetValueOnAnyThread(), 0, static_cast<int32>(EDisplayColorGamut::MAX) - 1));
}

inline FHDRMetaData HDRGetDefaultMetaData()
{
	const bool bHDRSupported = IsHDREnabled() && GRHISupportsHDROutput;
	return
	{
		.DisplayOutputFormat = HDRGetDefaultDisplayOutputFormat(),
		.DisplayColorGamut = HDRGetDefaultDisplayColorGamut(),
		.MinimumLuminanceInNits = bHDRSupported ?
			FMath::Pow(10.f, static_cast<float>(CVarHDRDisplayMinLuminanceLog10.GetValueOnAnyThread())) :
			0.f,
		.MaximumLuminanceInNits = bHDRSupported ?
			static_cast<float>(CVarHDRDisplayMaxLuminance.GetValueOnAnyThread()) :
			100.f,
		.MaximumFullFrameLuminanceInNits = bHDRSupported ?
			static_cast<float>(CVarHDRDisplayMaxLuminance.GetValueOnAnyThread()) :
			100.f,
		.HDRPaperWhiteInNits = 203.f,
		.LimitingColorSpace = bHDRSupported ? UE::Color::FColorSpace::GetP3D65() : UE::Color::FColorSpace::GetSRGB(),
		.bHDRSupported = bHDRSupported,
	};
}

inline int32 WindowDisplayIntersectionArea(FIntRect WindowRect, FIntRect DisplayRect)
{
	return FMath::Max<int32>(0, FMath::Min<int32>(WindowRect.Max.X, DisplayRect.Max.X) - FMath::Max<int32>(WindowRect.Min.X, DisplayRect.Min.X)) *
		FMath::Max<int32>(0, FMath::Min<int32>(WindowRect.Max.Y, DisplayRect.Max.Y) - FMath::Max<int32>(WindowRect.Min.Y, DisplayRect.Min.Y));
}

TMap<void*, FHDRMetaData> GWindowsWithDefaultParams;
FCriticalSection GWindowsWithDefaultParamsCS;

RENDERCORE_API void HDRAddCustomMetaData(void* OSWindow, EDisplayOutputFormat DisplayOutputFormat, EDisplayColorGamut DisplayColorGamut, bool bHDREnabled)
{
	ensure(OSWindow != nullptr);
	if (OSWindow == nullptr)
	{
		return;
	}
	FScopeLock Lock(&GWindowsWithDefaultParamsCS);
	FHDRMetaData& HDRMetaData = GWindowsWithDefaultParams.Emplace(OSWindow);
	HDRMetaData.DisplayOutputFormat = DisplayOutputFormat;
	HDRMetaData.DisplayColorGamut = DisplayColorGamut;
	// TODO: Plumb color space.
	HDRMetaData.LimitingColorSpace = bHDREnabled ?
		UE::Color::FColorSpace::GetP3D65() :
		UE::Color::FColorSpace::GetSRGB();
	HDRMetaData.MinimumLuminanceInNits = bHDREnabled ?
		FMath::Pow(10.f, static_cast<float>(CVarHDRDisplayMinLuminanceLog10.GetValueOnAnyThread())) :
		0.f,
	HDRMetaData.MaximumLuminanceInNits = bHDREnabled ?
		CVarHDRDisplayMaxLuminance.GetValueOnAnyThread() :
		100.f;
	HDRMetaData.MaximumFullFrameLuminanceInNits = bHDREnabled ?
		CVarHDRDisplayMaxLuminance.GetValueOnAnyThread() :
		100.f;
	HDRMetaData.HDRPaperWhiteInNits = 203.f;
	HDRMetaData.bHDRSupported = bHDREnabled;
}

RENDERCORE_API void HDRRemoveCustomMetaData(void* OSWindow)
{
	ensure(OSWindow != nullptr);
	if (OSWindow == nullptr)
	{
		return;
	}
	FScopeLock Lock(&GWindowsWithDefaultParamsCS);
	GWindowsWithDefaultParams.Remove(OSWindow);
}

bool HdrHasWindowParamsFromCVars(void* OSWindow, FHDRMetaData& HDRMetaData)
{
	if (GWindowsWithDefaultParams.Num() == 0)
	{
		return false;
	}
	FScopeLock Lock(&GWindowsWithDefaultParamsCS);
	FHDRMetaData* FoundHDRMetaData = GWindowsWithDefaultParams.Find(OSWindow);
	if (FoundHDRMetaData)
	{
		HDRMetaData = *FoundHDRMetaData;
		return true;
	}

	return false;
}

RENDERCORE_API void HDRGetMetaData(EDisplayOutputFormat& OutDisplayOutputFormat, EDisplayColorGamut& OutDisplayColorGamut, bool& OutbHDRSupported, 
								   const FVector2D& WindowTopLeft, const FVector2D& WindowBottomRight, void* OSWindow)
{
	FHDRMetaData OutHDRMetaData;
	HDRGetMetaDataForWindow(OutHDRMetaData, WindowTopLeft, WindowBottomRight, OSWindow);
	OutDisplayOutputFormat = OutHDRMetaData.DisplayOutputFormat;
	OutDisplayColorGamut = OutHDRMetaData.DisplayColorGamut;
	OutbHDRSupported = OutHDRMetaData.bHDRSupported;
}

RENDERCORE_API void HDRGetMetaDataForWindow(FHDRMetaData& OutHDRMetaData, const FVector2D& WindowTopLeft, const FVector2D& WindowBottomRight, void* OSWindow)
{
#if WITH_EDITOR
	// this has priority over IsHDREnabled because MovieSceneCapture might request custom parameters
	if (HdrHasWindowParamsFromCVars(OSWindow, OutHDRMetaData))
	{
		return;
	}
#endif
	
	OutHDRMetaData = HDRGetDefaultMetaData();

#if PLATFORM_USES_FIXED_HDR_SETTING
	// IsHDREnabled() is controlled by code, depending on system and display HDR availability
	if (!IsHDRAllowed())
#else
	if (!IsHDREnabled())
#endif
	{
		return;
	}

	FDisplayInformationArray DisplayList;
	RHIGetDisplaysInformation(DisplayList);
	// In case we have no display registered, use the CVars
	if (DisplayList.Num() == 0)
	{
		return;
	}

	FIntRect WindowRect((int32)WindowTopLeft.X, (int32)WindowTopLeft.Y, (int32)WindowBottomRight.X, (int32)WindowBottomRight.Y);
	int32 BestDisplayForWindow = 0;
	int32 BestArea = 0;
	for (int32 DisplayIndex = 0; DisplayIndex < DisplayList.Num(); ++DisplayIndex)
	{
		// Compute the intersection
		int32 CurrentArea = WindowDisplayIntersectionArea(WindowRect, DisplayList[DisplayIndex].DesktopCoordinates);
		if (CurrentArea > BestArea)
		{
			BestDisplayForWindow = DisplayIndex;
			BestArea = CurrentArea;
		}
	}

	OutHDRMetaData.bHDRSupported = DisplayList[BestDisplayForWindow].bHDRSupported;
	OutHDRMetaData.DisplayOutputFormat = EDisplayOutputFormat::SDR_sRGB;
	OutHDRMetaData.DisplayColorGamut = EDisplayColorGamut::sRGB_D65;
	OutHDRMetaData.MinimumLuminanceInNits = OutHDRMetaData.bHDRSupported ? DisplayList[BestDisplayForWindow].MinimumLuminanceInNits : 0.f;
	OutHDRMetaData.MaximumLuminanceInNits = OutHDRMetaData.bHDRSupported ? DisplayList[BestDisplayForWindow].MaximumLuminanceInNits : 100.f;
	OutHDRMetaData.MaximumFullFrameLuminanceInNits = OutHDRMetaData.bHDRSupported ? DisplayList[BestDisplayForWindow].MaximumFullFrameLuminanceInNits : 100.f;
#if PLATFORM_PROVIDES_HDR_PAPER_WHITE
	if (CVarHDRPaperWhiteMode.GetValueOnAnyThread() == 0)
	{
		OutHDRMetaData.HDRPaperWhiteInNits = DisplayList[BestDisplayForWindow].HDRPaperWhiteInNits;
	}
	else
#endif
	{
		OutHDRMetaData.HDRPaperWhiteInNits = CVarHDRPaperWhite.GetValueOnAnyThread();
	}
	OutHDRMetaData.LimitingColorSpace = OutHDRMetaData.bHDRSupported ? DisplayList[BestDisplayForWindow].LimitingColorSpace : UE::Color::FColorSpace::GetSRGB();

	if (OutHDRMetaData.bHDRSupported)
	{
		FPlatformMisc::ChooseHDRDeviceAndColorGamut(GRHIVendorId, CVarHDRDisplayMaxLuminance.GetValueOnAnyThread(), OutHDRMetaData.DisplayOutputFormat, OutHDRMetaData.DisplayColorGamut);
	}
}

RENDERCORE_API float HDRGetDisplayMaximumLuminance()
{
	return CVarHDRDisplayMaxLuminance.GetValueOnAnyThread();
}

static FACESTonemapParams GACESTonemapParams;

RENDERCORE_API FACESTonemapParams HDRGetACESTonemapParams()
{
	return GACESTonemapParams;
}

void ConfigureACESTonemapParams(FACESTonemapParams& OutTsParams, float MinLuminance, float MidLuminance, float MaxLuminance);

RENDERCORE_API void HDRConfigureCVars(bool bIsHDREnabled, uint32 DisplayNits, bool bFromGameSettings)
{
	if (bIsHDREnabled && !GRHISupportsHDROutput)
	{
		// In case we request HDR but no display supports it, we still need to setup Gamut / OETF properly for the tonemapper to work
		bIsHDREnabled = false;
	}

	EDisplayOutputFormat OutputDevice = EDisplayOutputFormat::SDR_sRGB;
	EDisplayColorGamut ColorGamut = EDisplayColorGamut::sRGB_D65;

	// If we are turning HDR on we must set the appropriate OutputDevice and ColorGamut.
	// If we are turning it off, we'll reset back to 0/0
	if (bIsHDREnabled)
	{
		FPlatformMisc::ChooseHDRDeviceAndColorGamut(GRHIVendorId, DisplayNits, OutputDevice, ColorGamut);
	}

	//CVarHDRDisplayMaxLuminance is ECVF_SetByCode as it's only a mean of communicating the information from UGameUserSettings to the rest of the engine
	if (bIsHDREnabled)
	{
		CVarHDRDisplayMaxLuminance->Set((int32)DisplayNits, ECVF_SetByCode);
	}
	else
	{
		CVarHDRDisplayMaxLuminance->Set(0, ECVF_SetByCode);
	}

	CVarDisplayOutputDevice->Set((int32)OutputDevice, ECVF_SetByDeviceProfile);
	CVarDisplayColorGamut->Set((int32)ColorGamut, ECVF_SetByDeviceProfile);

	float HDRDisplayMinLuminance = FMath::Pow(10, CVarHDRDisplayMinLuminanceLog10.GetValueOnAnyThread());
	ConfigureACESTonemapParams(GACESTonemapParams, HDRDisplayMinLuminance, CVarHDRDisplayMidLuminance.GetValueOnAnyThread(), CVarHDRDisplayMaxLuminance.GetValueOnAnyThread());
}

RENDERCORE_API FMatrix44f GamutToXYZMatrix(EDisplayColorGamut ColorGamut)
{
	static const FMatrix44f sRGB_2_XYZ_MAT(
		FVector3f(0.4124564f, 0.3575761f, 0.1804375f),
		FVector3f(0.2126729f, 0.7151522f, 0.0721750f),
		FVector3f(0.0193339f, 0.1191920f, 0.9503041f),
		FVector3f(0, 0, 0)
	);

	static const FMatrix44f Rec2020_2_XYZ_MAT(
		FVector3f(0.6369736f, 0.1446172f, 0.1688585f),
		FVector3f(0.2627066f, 0.6779996f, 0.0592938f),
		FVector3f(0.0000000f, 0.0280728f, 1.0608437f),
		FVector3f(0, 0, 0)
	);

	static const FMatrix44f P3D65_2_XYZ_MAT(
		FVector3f(0.4865906f, 0.2656683f, 0.1981905f),
		FVector3f(0.2289838f, 0.6917402f, 0.0792762f),
		FVector3f(0.0000000f, 0.0451135f, 1.0438031f),
		FVector3f(0, 0, 0)
	);
	switch (ColorGamut)
	{
	case EDisplayColorGamut::sRGB_D65: return sRGB_2_XYZ_MAT;
	case EDisplayColorGamut::Rec2020_D65: return Rec2020_2_XYZ_MAT;
	case EDisplayColorGamut::DCIP3_D65: return P3D65_2_XYZ_MAT;
	default:
		checkNoEntry();
		return FMatrix44f::Identity;
	}

}

RENDERCORE_API FMatrix44f XYZToGamutMatrix(EDisplayColorGamut ColorGamut)
{
	static const FMatrix44f XYZ_2_sRGB_MAT(
		FVector3f(3.2409699419f, -1.5373831776f, -0.4986107603f),
		FVector3f(-0.9692436363f, 1.8759675015f, 0.0415550574f),
		FVector3f(0.0556300797f, -0.2039769589f, 1.0569715142f),
		FVector3f(0, 0, 0)
	);

	static const FMatrix44f XYZ_2_Rec2020_MAT(
		FVector3f(1.7166084f, -0.3556621f, -0.2533601f),
		FVector3f(-0.6666829f, 1.6164776f, 0.0157685f),
		FVector3f(0.0176422f, -0.0427763f, 0.94222867f),
		FVector3f(0, 0, 0)
	);

	static const FMatrix44f XYZ_2_P3D65_MAT(
		FVector3f(2.4933963f, -0.9313459f, -0.4026945f),
		FVector3f(-0.8294868f, 1.7626597f, 0.0236246f),
		FVector3f(0.0358507f, -0.0761827f, 0.9570140f),
		FVector3f(0, 0, 0)
	);

	switch (ColorGamut)
	{
	case EDisplayColorGamut::sRGB_D65: return XYZ_2_sRGB_MAT;
	case EDisplayColorGamut::Rec2020_D65: return XYZ_2_Rec2020_MAT;
	case EDisplayColorGamut::DCIP3_D65: return XYZ_2_P3D65_MAT;
	default:
		checkNoEntry();
		return FMatrix44f::Identity;
	}

}

static const float MIN_STOP_SDR = -6.5f;
static const float MAX_STOP_SDR = 6.5f;

static const float MIN_STOP_RRT = -15.f;
static const float MAX_STOP_RRT = 18.f;

static const float MIN_LUM_SDR = 0.02f;
static const float MAX_LUM_SDR = 48.0f;

static const float MIN_LUM_RRT = 0.0001f;
static const float MAX_LUM_RRT = 10000.0f;

///+ TODO: check if valid
float interpolate1D(const FVector2f table[2], float value)
{
	float t = FMath::Clamp((value - table[0].X) / (table[1].X - table[0].X), 0.0f, 1.0f);
	return FMath::Lerp(table[0].Y, table[1].Y, t);
}
///-

float lookup_ACESmin(float minLum)
{
	const FVector2f minTable[2] = { FVector2f(FMath::LogX(10, MIN_LUM_RRT), MIN_STOP_RRT),
								 FVector2f(FMath::LogX(10, MIN_LUM_SDR), MIN_STOP_SDR) };

	return 0.18 * pow(2., interpolate1D(minTable, FMath::LogX(10, minLum)));
}

float lookup_ACESmax(float maxLum)
{
	const FVector2f maxTable[2] = { FVector2f(FMath::LogX(10, MAX_LUM_SDR), MAX_STOP_SDR),
								   FVector2f(FMath::LogX(10, MAX_LUM_RRT), MAX_STOP_RRT) };

	return 0.18 * pow(2., interpolate1D(maxTable, FMath::LogX(10, maxLum)));
}

struct TsPoint
{
	float x;        // ACES
	float y;        // luminance
	float slope;    // 
};

struct TsParams
{
	TsPoint Min;
	TsPoint Mid;
	TsPoint Max;
	float coefsLow[5];
	float coefsHigh[5];
};

void init_coefsLow(
	const TsPoint& TsPointLow,
	const TsPoint& TsPointMid,
	float coefsLow[5]
)
{

	float knotIncLow = (FMath::LogX(10, TsPointMid.x) - FMath::LogX(10, TsPointLow.x)) / 3.;
	// float halfKnotInc = (FMath::LogX(10, TsPointMid.x) - FMath::LogX(10, TsPointLow.x)) / 6.;

	// Determine two lowest coefficients (straddling minPt)
	coefsLow[0] = (TsPointLow.slope * (FMath::LogX(10, TsPointLow.x) - 0.5 * knotIncLow)) + (FMath::LogX(10, TsPointLow.y) - TsPointLow.slope * FMath::LogX(10, TsPointLow.x));
	coefsLow[1] = (TsPointLow.slope * (FMath::LogX(10, TsPointLow.x) + 0.5 * knotIncLow)) + (FMath::LogX(10, TsPointLow.y) - TsPointLow.slope * FMath::LogX(10, TsPointLow.x));
	// NOTE: if slope=0, then the above becomes just 
		// coefsLow[0] = FMath::LogX(10, TsPointLow.y);
		// coefsLow[1] = FMath::LogX(10, TsPointLow.y);
	// leaving it as a variable for now in case we decide we need non-zero slope extensions

	// Determine two highest coefficients (straddling midPt)
	coefsLow[3] = (TsPointMid.slope * (FMath::LogX(10, TsPointMid.x) - 0.5 * knotIncLow)) + (FMath::LogX(10, TsPointMid.y) - TsPointMid.slope * FMath::LogX(10, TsPointMid.x));
	coefsLow[4] = (TsPointMid.slope * (FMath::LogX(10, TsPointMid.x) + 0.5 * knotIncLow)) + (FMath::LogX(10, TsPointMid.y) - TsPointMid.slope * FMath::LogX(10, TsPointMid.x));

	// Middle coefficient (which defines the "sharpness of the bend") is linearly interpolated
	FVector2f bendsLow[2] = { FVector2f(MIN_STOP_RRT, 0.18f),
						   FVector2f(MIN_STOP_SDR, 0.35f) };
	float pctLow = interpolate1D(bendsLow, log2(TsPointLow.x / 0.18));
	coefsLow[2] = FMath::LogX(10, TsPointLow.y) + pctLow * (FMath::LogX(10, TsPointMid.y) - FMath::LogX(10, TsPointLow.y));
}

void init_coefsHigh(
	const TsPoint& TsPointMid,
	const TsPoint& TsPointMax,
	float coefsHigh[5]
)
{

	float knotIncHigh = (FMath::LogX(10, TsPointMax.x) - FMath::LogX(10, TsPointMid.x)) / 3.;
	// float halfKnotInc = (FMath::LogX(10, TsPointMax.x) - FMath::LogX(10, TsPointMid.x)) / 6.;

	// Determine two lowest coefficients (straddling midPt)
	coefsHigh[0] = (TsPointMid.slope * (FMath::LogX(10, TsPointMid.x) - 0.5 * knotIncHigh)) + (FMath::LogX(10, TsPointMid.y) - TsPointMid.slope * FMath::LogX(10, TsPointMid.x));
	coefsHigh[1] = (TsPointMid.slope * (FMath::LogX(10, TsPointMid.x) + 0.5 * knotIncHigh)) + (FMath::LogX(10, TsPointMid.y) - TsPointMid.slope * FMath::LogX(10, TsPointMid.x));

	// Determine two highest coefficients (straddling maxPt)
	coefsHigh[3] = (TsPointMax.slope * (FMath::LogX(10, TsPointMax.x) - 0.5 * knotIncHigh)) + (FMath::LogX(10, TsPointMax.y) - TsPointMax.slope * FMath::LogX(10, TsPointMax.x));
	coefsHigh[4] = (TsPointMax.slope * (FMath::LogX(10, TsPointMax.x) + 0.5 * knotIncHigh)) + (FMath::LogX(10, TsPointMax.y) - TsPointMax.slope * FMath::LogX(10, TsPointMax.x));
	// NOTE: if slope=0, then the above becomes just
		// coefsHigh[0] = FMath::LogX(10, TsPointHigh.y);
		// coefsHigh[1] = FMath::LogX(10, TsPointHigh.y);
	// leaving it as a variable for now in case we decide we need non-zero slope extensions

	// Middle coefficient (which defines the "sharpness of the bend") is linearly interpolated
	FVector2f bendsHigh[2] = { FVector2f(MAX_STOP_SDR, 0.89f) ,
							FVector2f(MAX_STOP_RRT, 0.90f) };
	float pctHigh = interpolate1D(bendsHigh, log2(TsPointMax.x / 0.18));
	coefsHigh[2] = FMath::LogX(10, TsPointMid.y) + pctHigh * (FMath::LogX(10, TsPointMax.y) - FMath::LogX(10, TsPointMid.y));
}

float shift(float inValue, float expShift)
{
	return pow(2., (log2(inValue) - expShift));
}

TsParams init_TsParams(
	float minLum,
	float maxLum,
	float expShift = 0
)
{
	TsPoint MIN_PT = { lookup_ACESmin(minLum), minLum, 0.0 };
	TsPoint MID_PT = { 0.18f, 4.8f, 1.55f };
	TsPoint MAX_PT = { lookup_ACESmax(maxLum), maxLum, 0.0 };
	float cLow[5];
	init_coefsLow(MIN_PT, MID_PT, cLow);
	float cHigh[5];
	init_coefsHigh(MID_PT, MAX_PT, cHigh);
	MIN_PT.x = shift(lookup_ACESmin(minLum), expShift);
	MID_PT.x = shift(0.18f, expShift);
	MAX_PT.x = shift(lookup_ACESmax(maxLum), expShift);

	ensure(FMath::Abs(MIN_PT.slope) <= 1e-6f);
	ensure(FMath::Abs(MAX_PT.slope) <= 1e-6f);

	TsParams P = {
		{MIN_PT.x, MIN_PT.y, MIN_PT.slope},
		{MID_PT.x, MID_PT.y, MID_PT.slope},
		{MAX_PT.x, MAX_PT.y, MAX_PT.slope},
		{cLow[0],  cLow[1],  cLow[2],  cLow[3],  cLow[4]},
		{cHigh[0], cHigh[1], cHigh[2], cHigh[3], cHigh[4]}
	};

	return P;
}

FVector3f ApplyM1Matrix(const FVector3f& cf)
{
	return FVector3f(
		0.5f * cf[0] - 1.0f * cf[1] + 0.5f * cf[2],
	   -1.0f * cf[0] + 1.0f * cf[1] + 0.0f * cf[2],
		0.5f * cf[0] + 0.5f * cf[1] + 0.0f * cf[2]
	);
}

float inv_ssts
(
	const float y,
	const TsParams C
)
{
	const int N_KNOTS_LOW = 4;
	const int N_KNOTS_HIGH = 4;

	const float KNOT_INC_LOW = (FMath::LogX(10, C.Mid.x) - FMath::LogX(10, C.Min.x)) / (N_KNOTS_LOW - 1.);
	const float KNOT_INC_HIGH = (FMath::LogX(10, C.Max.x) - FMath::LogX(10, C.Mid.x)) / (N_KNOTS_HIGH - 1.);

	// KNOT_Y is luminance of the spline at each knot
	float KNOT_Y_LOW[N_KNOTS_LOW];
	///+warning: redefinition of 'i'
	{
		for (int i = 0; i < N_KNOTS_LOW; i = i + 1) {
			KNOT_Y_LOW[i] = (C.coefsLow[i] + C.coefsLow[i + 1]) / 2.;
		};
	}
	///-

	float KNOT_Y_HIGH[N_KNOTS_HIGH];
	///+ warning: redefinition of 'i'
	{
		for (int i = 0; i < N_KNOTS_HIGH; i = i + 1) {
			KNOT_Y_HIGH[i] = (C.coefsHigh[i] + C.coefsHigh[i + 1]) / 2.;
		};
	}
	///-
	float logy = FMath::LogX(10, FMath::Max(y, 1e-10));

	float logx;
	if (logy <= FMath::LogX(10, C.Min.y)) {

		logx = FMath::LogX(10, C.Min.x);

	}
	else if ((logy > FMath::LogX(10, C.Min.y)) && (logy <= FMath::LogX(10, C.Mid.y))) {

		int j=0;
		FVector4f cf;
		if (logy > KNOT_Y_LOW[0] && logy <= KNOT_Y_LOW[1]) {
			cf[0] = C.coefsLow[0];  cf[1] = C.coefsLow[1];  cf[2] = C.coefsLow[2];  j = 0;
		}
		else if (logy > KNOT_Y_LOW[1] && logy <= KNOT_Y_LOW[2]) {
			cf[0] = C.coefsLow[1];  cf[1] = C.coefsLow[2];  cf[2] = C.coefsLow[3];  j = 1;
		}
		else if (logy > KNOT_Y_LOW[2] && logy <= KNOT_Y_LOW[3]) {
			cf[0] = C.coefsLow[2];  cf[1] = C.coefsLow[3];  cf[2] = C.coefsLow[4];  j = 2;
		}

		cf[3] = 0;

		const FVector4f tmp = ApplyM1Matrix(cf);

		float a = tmp[0];
		float b = tmp[1];
		float c = tmp[2];
		c = c - logy;

		const float d = sqrt(b * b - 4. * a * c);

		const float t = (2. * c) / (-d - b);

		logx = FMath::LogX(10, C.Min.x) + (t + j) * KNOT_INC_LOW;

	}
	else if ((logy > FMath::LogX(10, C.Mid.y)) && (logy < FMath::LogX(10, C.Max.y))) {

		int j=0;
		FVector4f cf;
		if (logy >= KNOT_Y_HIGH[0] && logy <= KNOT_Y_HIGH[1]) {
			cf[0] = C.coefsHigh[0];  cf[1] = C.coefsHigh[1];  cf[2] = C.coefsHigh[2];  j = 0;
		}
		else if (logy > KNOT_Y_HIGH[1] && logy <= KNOT_Y_HIGH[2]) {
			cf[0] = C.coefsHigh[1];  cf[1] = C.coefsHigh[2];  cf[2] = C.coefsHigh[3];  j = 1;
		}
		else if (logy > KNOT_Y_HIGH[2] && logy <= KNOT_Y_HIGH[3]) {
			cf[0] = C.coefsHigh[2];  cf[1] = C.coefsHigh[3];  cf[2] = C.coefsHigh[4];  j = 2;
		}
		cf[3] = 0;

		const FVector4f tmp = ApplyM1Matrix(cf);

		float a = tmp[0];
		float b = tmp[1];
		float c = tmp[2];
		c = c - logy;

		const float d = sqrt(b * b - 4. * a * c);

		const float t = (2. * c) / (-d - b);

		logx = FMath::LogX(10, C.Mid.x) + (t + j) * KNOT_INC_HIGH;

	}
	else { //if ( logy >= FMath::LogX(10, C.Max.y) ) {

		logx = FMath::LogX(10, C.Max.x);

	}

	return FMath::Pow(10, logx);

}

void ConfigureACESTonemapParams(FACESTonemapParams& OutACESTonemapParams, float MinLuminance, float MidLuminance, float MaxLuminance)
{
	TsParams PARAMS_DEFAULT = init_TsParams(MinLuminance, MaxLuminance);
	float tmp = inv_ssts(MidLuminance, PARAMS_DEFAULT);
	float expShift = log2(tmp) - log2(0.18);
	TsParams PARAMS = init_TsParams(MinLuminance, MaxLuminance, expShift);

	OutACESTonemapParams.ACESMinMaxData.X = PARAMS.Min.x;
	OutACESTonemapParams.ACESMinMaxData.Y = PARAMS.Min.y;
	OutACESTonemapParams.ACESMinMaxData.Z = PARAMS.Max.x;
	OutACESTonemapParams.ACESMinMaxData.W = PARAMS.Max.y;
	ensure(FMath::Abs(PARAMS.Min.slope) <= 1e-6f);
	ensure(FMath::Abs(PARAMS.Max.slope) <= 1e-6f);

	OutACESTonemapParams.ACESMidData.X = PARAMS.Mid.x;
	OutACESTonemapParams.ACESMidData.Y = PARAMS.Mid.y;
	OutACESTonemapParams.ACESMidData.Z = PARAMS.Mid.slope;

	OutACESTonemapParams.ACESCoefsLow_0.X = PARAMS.coefsLow[0];
	OutACESTonemapParams.ACESCoefsLow_0.Y = PARAMS.coefsLow[1];
	OutACESTonemapParams.ACESCoefsLow_0.Z = PARAMS.coefsLow[2];
	OutACESTonemapParams.ACESCoefsLow_0.W = PARAMS.coefsLow[3];
	OutACESTonemapParams.ACESCoefsLow_4 = PARAMS.coefsLow[4];
	OutACESTonemapParams.ACESCoefsHigh_0.X = PARAMS.coefsHigh[0];
	OutACESTonemapParams.ACESCoefsHigh_0.Y = PARAMS.coefsHigh[1];
	OutACESTonemapParams.ACESCoefsHigh_0.Z = PARAMS.coefsHigh[2];
	OutACESTonemapParams.ACESCoefsHigh_0.W = PARAMS.coefsHigh[3];
	OutACESTonemapParams.ACESCoefsHigh_4 = PARAMS.coefsHigh[4];

	OutACESTonemapParams.ACESSceneColorMultiplier = CVarHDRAcesColorMultiplier.GetValueOnAnyThread();
	OutACESTonemapParams.ACESGamutCompression = CVarHDRAcesGamutCompression.GetValueOnAnyThread();
}

// Converts PQ signal to linear values, see https://www.itu.int/rec/R-REC-BT.2124-0-201901-I/en, conversion 3 
static inline float ST2084ToLinear(float pq)
{
	const float m1 = 0.1593017578125; // = 2610. / 4096. * .25;
	const float m2 = 78.84375; // = 2523. / 4096. *  128;
	const float c1 = 0.8359375; // = 2392. / 4096. * 32 - 2413./4096.*32 + 1;
	const float c2 = 18.8515625; // = 2413. / 4096. * 32;
	const float c3 = 18.6875; // = 2392. / 4096. * 32;
	const float C = 10000.;

	float Np = FMath::Pow(pq, 1. / m2);
	float L = Np - c1;
	L = FMath::Max(0.0f, L);
	L = L / (c2 - c3 * Np);
	L = FMath::Pow(L, 1. / m1);
	float P = L * C;

	return P;
}

RENDERCORE_API void ConvertPixelDataToSCRGB(TArray<FLinearColor>& InOutRawPixels, EDisplayOutputFormat Pixelformat)
{
	int32 PixelCount = InOutRawPixels.Num();
	if (Pixelformat == EDisplayOutputFormat::HDR_ACES_1000nit_ST2084 || Pixelformat == EDisplayOutputFormat::HDR_ACES_2000nit_ST2084)
	{
		const FMatrix44f Rec2020_2_sRGB_MAT = (XYZToGamutMatrix(EDisplayColorGamut::sRGB_D65) * GamutToXYZMatrix(EDisplayColorGamut::Rec2020_D65)).GetTransposed();

		for (int32 PixelIndex = 0; PixelIndex < PixelCount; ++PixelIndex)
		{
			FLinearColor& CurrentPixel = InOutRawPixels[PixelIndex];
			FVector4f PixelFloat4(CurrentPixel.R, CurrentPixel.G, CurrentPixel.B, 0);
			PixelFloat4.X = ST2084ToLinear(PixelFloat4.X) / 80;
			PixelFloat4.Y = ST2084ToLinear(PixelFloat4.Y) / 80;
			PixelFloat4.Z = ST2084ToLinear(PixelFloat4.Z) / 80;
			PixelFloat4 = Rec2020_2_sRGB_MAT.TransformFVector4(PixelFloat4);
			CurrentPixel.R = PixelFloat4.X;
			CurrentPixel.G = PixelFloat4.Y;
			CurrentPixel.B = PixelFloat4.Z;
		}
	}
}

FVirtualTextureProducerHandle IRendererModule::RegisterVirtualTextureProducer(const FVTProducerDescription& Desc, IVirtualTexture* Producer)
{
	return RegisterVirtualTextureProducer(FRHICommandListImmediate::Get(), Desc, Producer);
}

IAllocatedVirtualTexture* IRendererModule::AllocateVirtualTexture(const FAllocatedVTDescription& Desc)
{
	return AllocateVirtualTexture(FRHICommandListImmediate::Get(), Desc);
}

IAdaptiveVirtualTexture* IRendererModule::AllocateAdaptiveVirtualTexture(const FAdaptiveVTDescription& AdaptiveVTDesc, const FAllocatedVTDescription& AllocatedVTDesc)
{
	return AllocateAdaptiveVirtualTexture(FRHICommandListImmediate::Get(), AdaptiveVTDesc, AllocatedVTDesc);
}