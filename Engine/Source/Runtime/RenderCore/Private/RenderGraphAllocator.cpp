// Copyright Epic Games, Inc. All Rights Reserved.

#include "RenderGraphAllocator.h"
#include "RenderCore.h"
#include "RenderGraphPrivate.h"

#if RDG_ALLOCATOR_DEBUG
thread_local int32 FRDGAllocator::NumAccessesTLS = 0;
#endif

thread_local FRDGAllocator* FRDGAllocator::AllocatorTLS = nullptr;

FRDGAllocator::FRDGAllocator()
#if !RDG_USE_MALLOC
	: MemStack(FMemStackBase::EPageSize::Large)
#endif
{}

FRDGAllocator::FRDGAllocator(FRDGAllocator&& Other)
{
	*this = MoveTemp(Other);
}

FRDGAllocator& FRDGAllocator::operator= (FRDGAllocator&& Other)
{
#if RDG_USE_MALLOC
	Mallocs = MoveTemp(Other.Mallocs);
	NumMallocBytes = Other.NumMallocBytes;
	Other.NumMallocBytes = 0;
	check(Other.Mallocs.IsEmpty());
#else
	MemStack = MoveTemp(Other.MemStack);
	check(Other.MemStack.IsEmpty());
#endif
	Objects = MoveTemp(Other.Objects);
	check(Other.Objects.IsEmpty());

#if RDG_ALLOCATOR_DEBUG
	NumAccesses = Other.NumAccesses.load(std::memory_order_relaxed);
	Other.NumAccesses.store(0, std::memory_order_relaxed);
#endif

	return *this;
}

FRDGAllocator::~FRDGAllocator()
{
	ReleaseAll();
}

FRDGAllocator& FRDGAllocator::GetTLS()
{
	checkf(AllocatorTLS, TEXT("Attempted to access RDG allocator outside of FRDGAllocatorScope"));
	return *AllocatorTLS;
}

#if RDG_ALLOCATOR_DEBUG

void FRDGAllocator::AcquireAccess()
{
	check(NumAccesses.fetch_add(1, std::memory_order_acquire) == NumAccessesTLS++);
}

void FRDGAllocator::ReleaseAccess()
{
	check(NumAccesses.fetch_sub(1, std::memory_order_release) == NumAccessesTLS--);
}

#endif

void FRDGAllocator::ReleaseAll()
{
	CSV_SCOPED_TIMING_STAT_EXCLUSIVE_CONDITIONAL(RDGAllocator_Clear, GRDGVerboseCSVStats != 0 && IsInRenderingThread());
	TRACE_CPUPROFILER_EVENT_SCOPE(FRDGAllocator::ReleaseAll);

	for (int32 Index = Objects.Num() - 1; Index >= 0; --Index)
	{
#if RDG_USE_MALLOC
		delete Objects[Index];
#else
		Objects[Index]->~FObject();
#endif
	}
	Objects.Reset();

#if RDG_USE_MALLOC
	for (void* Malloc : Mallocs)
	{
		FMemory::Free(Malloc);
	}
	Mallocs.Reset();
	NumMallocBytes = 0;
#else
	MemStack.Flush();
#endif
}

FRDGAllocatorScope::FRDGAllocatorScope(FRDGAllocator& Allocator)
	: AllocatorToRestore(FRDGAllocator::AllocatorTLS)
{
	FRDGAllocator::AllocatorTLS = &Allocator;
}

FRDGAllocatorScope::~FRDGAllocatorScope()
{
	FRDGAllocator::AllocatorTLS = AllocatorToRestore;
}

FORCENOINLINE void UE::RenderCore::Private::OnInvalidRDGAllocatorNum(int32 NewNum, SIZE_T NumBytesPerElement)
{
	UE_LOG(LogRendererCore, Fatal, TEXT("Trying to resize TRDGArrayAllocator to an invalid size of %d with element size %" SIZE_T_FMT), NewNum, NumBytesPerElement);
	for (;;);
}