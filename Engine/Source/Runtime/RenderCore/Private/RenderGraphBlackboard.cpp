// Copyright Epic Games, Inc. All Rights Reserved.

#include "RenderGraphBlackboard.h"
#include "Containers/Map.h"

FString FRDGBlackboard::GetTypeName(const TCHAR* ClassName, const TCHAR* FileName, uint32 LineNumber)
{
	return FString::Printf(TEXT("%s %s %d"), ClassName, FileName, LineNumber);
}

uint32 FRDGBlackboard::AllocateIndex(FString&& TypeName)
{
	check(IsInRenderingThread());
	static TMap<FString, uint32> StructMap;
	static uint32 NextIndex = 0;

	uint32 Result;
	if (const uint32* FoundIndex = StructMap.Find(TypeName))
	{
		Result = *FoundIndex;
	}
	else
	{
		StructMap.Emplace(MoveTemp(TypeName), NextIndex);
		Result = NextIndex;
		NextIndex++;
	}
	return Result;
}
