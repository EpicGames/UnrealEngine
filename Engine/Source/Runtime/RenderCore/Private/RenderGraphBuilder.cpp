// Copyright Epic Games, Inc. All Rights Reserved.

#include "RenderGraphBuilder.h"
#include "RenderGraphPrivate.h"
#include "RenderGraphTrace.h"
#include "RenderGraphUtils.h"
#include "RenderTargetPool.h"
#include "RenderGraphResourcePool.h"
#include "VisualizeTexture.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "Async/ParallelFor.h"
#include "RenderTracing.h"

struct FParallelPassSet : public FRHICommandListImmediate::FQueuedCommandList
{
	FParallelPassSet() = default;

	TArray<FRDGPass*, FRDGArrayAllocator> Passes;
	bool bDispatchAfterExecute = false;
	bool bTaskModeAsync = false;
};

inline void BeginUAVOverlap(const FRDGPass* Pass, FRHIComputeCommandList& RHICmdList)
{
#if ENABLE_RHI_VALIDATION
	if (GRHIValidationEnabled)
	{
		RHICmdList.BeginUAVOverlap();
	}
#endif
}

inline void EndUAVOverlap(const FRDGPass* Pass, FRHIComputeCommandList& RHICmdList)
{
#if ENABLE_RHI_VALIDATION
	if (GRHIValidationEnabled)
	{
		RHICmdList.EndUAVOverlap();
	}
#endif
}

inline ERHIAccess MakeValidAccess(ERHIAccess AccessOld, ERHIAccess AccessNew)
{
	const ERHIAccess AccessUnion = AccessOld | AccessNew;
	const ERHIAccess NonMergeableAccessMask = ~GRHIMergeableAccessMask;

	// Return the union of new and old if they are okay to merge.
	if (!EnumHasAnyFlags(AccessUnion, NonMergeableAccessMask))
	{
		return IsWritableAccess(AccessUnion) ? (AccessUnion & ~ERHIAccess::ReadOnlyExclusiveMask) : AccessUnion;
	}

	// Keep the old one if it can't be merged.
	if (EnumHasAnyFlags(AccessOld, NonMergeableAccessMask))
	{
		return AccessOld;
	}

	// Replace with the new one if it can't be merged.
	return AccessNew;
}

inline void GetPassAccess(ERDGPassFlags PassFlags, ERHIAccess& SRVAccess, ERHIAccess& UAVAccess)
{
	SRVAccess = ERHIAccess::Unknown;
	UAVAccess = ERHIAccess::Unknown;

	if (EnumHasAnyFlags(PassFlags, ERDGPassFlags::Raster))
	{
		SRVAccess |= ERHIAccess::SRVGraphics;
		UAVAccess |= ERHIAccess::UAVGraphics;
	}

	if (EnumHasAnyFlags(PassFlags, ERDGPassFlags::AsyncCompute | ERDGPassFlags::Compute))
	{
		SRVAccess |= ERHIAccess::SRVCompute;
		UAVAccess |= ERHIAccess::UAVCompute;
	}

	if (EnumHasAnyFlags(PassFlags, ERDGPassFlags::Copy))
	{
		SRVAccess |= ERHIAccess::CopySrc;
	}
}

enum class ERDGTextureAccessFlags
{
	None = 0,

	// Access is within the fixed-function render pass.
	RenderTarget = 1 << 0
};
ENUM_CLASS_FLAGS(ERDGTextureAccessFlags);

/** Enumerates all texture accesses and provides the access and subresource range info. This results in
 *  multiple invocations of the same resource, but with different access / subresource range.
 */
template <typename TAccessFunction>
void EnumerateTextureAccess(FRDGParameterStruct PassParameters, ERDGPassFlags PassFlags, TAccessFunction AccessFunction)
{
	const ERDGTextureAccessFlags NoneFlags = ERDGTextureAccessFlags::None;

	ERHIAccess SRVAccess, UAVAccess;
	GetPassAccess(PassFlags, SRVAccess, UAVAccess);

	PassParameters.EnumerateTextures([&](FRDGParameter Parameter)
	{
		switch (Parameter.GetType())
		{
		case UBMT_RDG_TEXTURE:
			if (FRDGTextureRef Texture = Parameter.GetAsTexture())
			{
				ERHIAccess CurrentSRVAccess = SRVAccess;
				if (GRHIGlobals.SupportsDedicatedSRVGraphicsNonPixelAccess)
				{
					EnumRemoveFlags(CurrentSRVAccess, ERHIAccess::SRVGraphicsNonPixel);
				}
				AccessFunction(nullptr, Texture, CurrentSRVAccess, NoneFlags, Texture->GetSubresourceRangeSRV());
			}
		break;
		case UBMT_RDG_TEXTURE_ACCESS:
		{
			if (FRDGTextureAccess TextureAccess = Parameter.GetAsTextureAccess())
			{
				AccessFunction(nullptr, TextureAccess.GetTexture(), TextureAccess.GetAccess(), NoneFlags, TextureAccess.GetSubresourceRange());
			}
		}
		break;
		case UBMT_RDG_TEXTURE_ACCESS_ARRAY:
		{
			const FRDGTextureAccessArray& TextureAccessArray = Parameter.GetAsTextureAccessArray();

			for (FRDGTextureAccess TextureAccess : TextureAccessArray)
			{
				AccessFunction(nullptr, TextureAccess.GetTexture(), TextureAccess.GetAccess(), NoneFlags, TextureAccess.GetSubresourceRange());
			}
		}
		break;
		case UBMT_RDG_TEXTURE_SRV:
			if (FRDGTextureSRVRef SRV = Parameter.GetAsTextureSRV())
			{
				ERHIAccess CurrentSRVAccess = SRVAccess;
				if (GRHIGlobals.SupportsDedicatedSRVGraphicsNonPixelAccess)
				{
					EnumRemoveFlags(CurrentSRVAccess, ERHIAccess::SRVGraphicsNonPixel);
				}
				AccessFunction(SRV, SRV->GetParent(), CurrentSRVAccess, NoneFlags, SRV->GetSubresourceRange());
			}
		break;
		case UBMT_RDG_TEXTURE_NON_PIXEL_SRV:
			if (FRDGTextureSRVRef SRV = Parameter.GetAsTextureSRV())
			{
				AccessFunction(SRV, SRV->GetParent(), SRVAccess, NoneFlags, SRV->GetSubresourceRange());
			}
		break;
		case UBMT_RDG_TEXTURE_UAV:
			if (FRDGTextureUAVRef UAV = Parameter.GetAsTextureUAV())
			{
				AccessFunction(UAV, UAV->GetParent(), UAVAccess, NoneFlags, UAV->GetSubresourceRange());
			}
		break;
		case UBMT_RENDER_TARGET_BINDING_SLOTS:
		{
			const ERDGTextureAccessFlags RenderTargetAccess = ERDGTextureAccessFlags::RenderTarget;

			const ERHIAccess RTVAccess = ERHIAccess::RTV;

			const FRenderTargetBindingSlots& RenderTargets = Parameter.GetAsRenderTargetBindingSlots();

			RenderTargets.Enumerate([&](FRenderTargetBinding RenderTarget)
			{
				FRDGTextureRef Texture = RenderTarget.GetTexture();
				FRDGTextureRef ResolveTexture = RenderTarget.GetResolveTexture();

				FRDGTextureSubresourceRange Range(Texture->GetSubresourceRange());
				Range.MipIndex = RenderTarget.GetMipIndex();
				Range.NumMips = 1;

				if (RenderTarget.GetArraySlice() != -1)
				{
					Range.ArraySlice = RenderTarget.GetArraySlice();
					Range.NumArraySlices = 1;
				}

				AccessFunction(nullptr, Texture, RTVAccess, RenderTargetAccess, Range);

				if (ResolveTexture && ResolveTexture != Texture)
				{
					// RTV|ResolveDst is not a valid state for the platform RHI, use directly ERHIAccess::ResolveDst
					AccessFunction(nullptr, ResolveTexture, ERHIAccess::ResolveDst, RenderTargetAccess, Range);
				}
			});

			const FDepthStencilBinding& DepthStencil = RenderTargets.DepthStencil;

			if (FRDGTextureRef Texture = DepthStencil.GetTexture())
			{
				FRDGTextureRef ResolveTexture = DepthStencil.GetResolveTexture();
				DepthStencil.GetDepthStencilAccess().EnumerateSubresources([&](ERHIAccess NewAccess, uint32 PlaneSlice)
				{
					FRDGTextureSubresourceRange Range = Texture->GetSubresourceRange();

					// Adjust the range to use a single plane slice if not using of them all.
					if (PlaneSlice != FRHITransitionInfo::kAllSubresources)
					{
						Range.PlaneSlice = PlaneSlice;
						Range.NumPlaneSlices = 1;
					}

					AccessFunction(nullptr, Texture, NewAccess, RenderTargetAccess, Range);

					if (ResolveTexture && ResolveTexture != Texture)
					{
						// If we're resolving depth stencil, it must be DSVWrite and ResolveDst
						AccessFunction(nullptr, ResolveTexture, ERHIAccess::DSVWrite | ERHIAccess::ResolveDst, RenderTargetAccess, Range);
					}
				});
			}

			if (FRDGTextureRef Texture = RenderTargets.ShadingRateTexture)
			{
				AccessFunction(nullptr, Texture, ERHIAccess::ShadingRateSource, RenderTargetAccess, Texture->GetSubresourceRangeSRV());
			}
		}
		break;
		}
	});
}

/** Enumerates all buffer accesses and provides the access info. */
template <typename TAccessFunction>
void EnumerateBufferAccess(FRDGParameterStruct PassParameters, ERDGPassFlags PassFlags, TAccessFunction AccessFunction)
{
	ERHIAccess SRVAccess, UAVAccess;
	GetPassAccess(PassFlags, SRVAccess, UAVAccess);

	PassParameters.EnumerateBuffers([&](FRDGParameter Parameter)
	{
		switch (Parameter.GetType())
		{
		case UBMT_RDG_BUFFER_ACCESS:
			if (FRDGBufferAccess BufferAccess = Parameter.GetAsBufferAccess())
			{
				AccessFunction(nullptr, BufferAccess.GetBuffer(), BufferAccess.GetAccess());
			}
		break;
		case UBMT_RDG_BUFFER_ACCESS_ARRAY:
		{
			const FRDGBufferAccessArray& BufferAccessArray = Parameter.GetAsBufferAccessArray();

			for (FRDGBufferAccess BufferAccess : BufferAccessArray)
			{
				AccessFunction(nullptr, BufferAccess.GetBuffer(), BufferAccess.GetAccess());
			}
		}
		break;
		case UBMT_RDG_BUFFER_SRV:
			if (FRDGBufferSRVRef SRV = Parameter.GetAsBufferSRV())
			{
				FRDGBufferRef Buffer = SRV->GetParent();
				ERHIAccess BufferAccess = SRVAccess;

				if (EnumHasAnyFlags(Buffer->Desc.Usage, BUF_AccelerationStructure))
				{
					BufferAccess = ERHIAccess::BVHRead | ERHIAccess::SRVMask;
				}

				AccessFunction(SRV, Buffer, BufferAccess);
			}
		break;
		case UBMT_RDG_BUFFER_UAV:
			if (FRDGBufferUAVRef UAV = Parameter.GetAsBufferUAV())
			{
				AccessFunction(UAV, UAV->GetParent(), UAVAccess);
			}
		break;
		}
	});
}

inline FRDGViewHandle GetHandleIfNoUAVBarrier(FRDGViewRef Resource)
{
	if (Resource && (Resource->Type == ERDGViewType::BufferUAV || Resource->Type == ERDGViewType::TextureUAV))
	{
		if (EnumHasAnyFlags(static_cast<FRDGUnorderedAccessViewRef>(Resource)->Flags, ERDGUnorderedAccessViewFlags::SkipBarrier))
		{
			return Resource->GetHandle();
		}
	}
	return FRDGViewHandle::Null;
}

inline EResourceTransitionFlags GetTextureViewTransitionFlags(FRDGViewRef Resource, FRDGTextureRef Texture)
{
	if (Resource)
	{
		switch (Resource->Type)
		{
		case ERDGViewType::TextureUAV:
		{
			FRDGTextureUAVRef UAV = static_cast<FRDGTextureUAVRef>(Resource);
			if (UAV->Desc.MetaData != ERDGTextureMetaDataAccess::None)
			{
				return EResourceTransitionFlags::MaintainCompression;
			}
		}
		break;
		case ERDGViewType::TextureSRV:
		{
			FRDGTextureSRVRef SRV = static_cast<FRDGTextureSRVRef>(Resource);
			if (SRV->Desc.MetaData != ERDGTextureMetaDataAccess::None)
			{
				return EResourceTransitionFlags::MaintainCompression;
			}
		}
		break;
		}
	}
	else
	{
		if (EnumHasAnyFlags(Texture->Flags, ERDGTextureFlags::MaintainCompression))
		{
			return EResourceTransitionFlags::MaintainCompression;
		}
	}
	return EResourceTransitionFlags::None;
}

void FRDGBuilder::SetFlushResourcesRHI()
{
	if (GRHINeedsExtraDeletionLatency || !GRHICommandList.Bypass())
	{
		checkf(!bFlushResourcesRHI, TEXT("SetFlushRHIResources has been already been called. It may only be called once."));
		bFlushResourcesRHI = true;

		if (IsImmediateMode())
		{
			BeginFlushResourcesRHI();
			EndFlushResourcesRHI();
		}
	}
}

void FRDGBuilder::BeginFlushResourcesRHI()
{
	if (!bFlushResourcesRHI)
	{
		return;
	}

	CSV_SCOPED_TIMING_STAT_EXCLUSIVE(STAT_RDG_FlushResourcesRHI);
	SCOPED_NAMED_EVENT(BeginFlushResourcesRHI, FColor::Emerald);

	static const auto CVarEnablePSOAsyncCacheConsolidation = IConsoleManager::Get().FindConsoleVariable(TEXT("r.pso.EnableAsyncCacheConsolidation"));
	if (CVarEnablePSOAsyncCacheConsolidation->GetBool())
	{
		// Cache prior tasks before enqueuing setup tasks, which can run while the pipeline state cache flushes.
		WaitOutstandingTasks = GRHICommandList.WaitOutstandingTasks;
	}
	else
	{
		// Dispatch to RHI thread if cache consolidation is not asynchronous, so it can get some work started before blocking in EndFlushResourcesRHI.
		RHICmdList.ImmediateFlush(EImmediateFlushType::DispatchToRHIThread);
	}
}

void FRDGBuilder::EndFlushResourcesRHI()
{
	if (!bFlushResourcesRHI)
	{
		return;
	}

	CSV_SCOPED_TIMING_STAT_EXCLUSIVE(STAT_RDG_FlushResourcesRHI);
	CSV_SCOPED_SET_WAIT_STAT(FlushResourcesRHI);
	SCOPED_NAMED_EVENT(EndFlushResourcesRHI, FColor::Emerald);

	static const auto CVarEnablePSOAsyncCacheConsolidation = IConsoleManager::Get().FindConsoleVariable(TEXT("r.pso.EnableAsyncCacheConsolidation"));
	if (CVarEnablePSOAsyncCacheConsolidation->GetBool())
	{
		// Dispatch to RHI thread and delete resources.
		RHICmdList.ImmediateFlush(EImmediateFlushType::DispatchToRHIThread, ERHISubmitFlags::DeleteResources);

		// Wait for tasks cached in BeginFlushResourcesRHI.
		GRHICommandList.WaitForTasks(WaitOutstandingTasks);
	}
	else
	{
		// Wait until all RHI work is complete.
		RHICmdList.ImmediateFlush(EImmediateFlushType::FlushRHIThreadFlushResources);
	}

	// Flush the pipeline state cache.
	PipelineStateCache::FlushResources();
}

void FRDGBuilder::TickPoolElements()
{
	GRenderGraphResourcePool.TickPoolElements();

#if RDG_ENABLE_DEBUG
	if (GRDGTransitionLog > 0)
	{
		--GRDGTransitionLog;
	}
#endif

#if RDG_STATS
	CSV_CUSTOM_STAT(RDGCount, Passes, GRDGStatPassCount, ECsvCustomStatOp::Set);
	CSV_CUSTOM_STAT(RDGCount, Buffers, GRDGStatBufferCount, ECsvCustomStatOp::Set);
	CSV_CUSTOM_STAT(RDGCount, Textures, GRDGStatTextureCount, ECsvCustomStatOp::Set);

	TRACE_COUNTER_SET(COUNTER_RDG_PassCount, GRDGStatPassCount);
	TRACE_COUNTER_SET(COUNTER_RDG_PassCullCount, GRDGStatPassCullCount);
	TRACE_COUNTER_SET(COUNTER_RDG_RenderPassMergeCount, GRDGStatRenderPassMergeCount);
	TRACE_COUNTER_SET(COUNTER_RDG_PassDependencyCount, GRDGStatPassDependencyCount);
	TRACE_COUNTER_SET(COUNTER_RDG_TextureCount, GRDGStatTextureCount);
	TRACE_COUNTER_SET(COUNTER_RDG_TextureReferenceCount, GRDGStatTextureReferenceCount);
	TRACE_COUNTER_SET(COUNTER_RDG_TextureReferenceAverage, (float)(GRDGStatTextureReferenceCount / FMath::Max((float)GRDGStatTextureCount, 1.0f)));
	TRACE_COUNTER_SET(COUNTER_RDG_BufferCount, GRDGStatBufferCount);
	TRACE_COUNTER_SET(COUNTER_RDG_BufferReferenceCount, GRDGStatBufferReferenceCount);
	TRACE_COUNTER_SET(COUNTER_RDG_BufferReferenceAverage, (float)(GRDGStatBufferReferenceCount / FMath::Max((float)GRDGStatBufferCount, 1.0f)));
	TRACE_COUNTER_SET(COUNTER_RDG_ViewCount, GRDGStatViewCount);
	TRACE_COUNTER_SET(COUNTER_RDG_TransientTextureCount, GRDGStatTransientTextureCount);
	TRACE_COUNTER_SET(COUNTER_RDG_TransientBufferCount, GRDGStatTransientBufferCount);
	TRACE_COUNTER_SET(COUNTER_RDG_TransitionCount, GRDGStatTransitionCount);
	TRACE_COUNTER_SET(COUNTER_RDG_AliasingCount, GRDGStatAliasingCount);
	TRACE_COUNTER_SET(COUNTER_RDG_TransitionBatchCount, GRDGStatTransitionBatchCount);
	TRACE_COUNTER_SET(COUNTER_RDG_MemoryWatermark, int64(GRDGStatMemoryWatermark));

	SET_DWORD_STAT(STAT_RDG_PassCount, GRDGStatPassCount);
	SET_DWORD_STAT(STAT_RDG_PassCullCount, GRDGStatPassCullCount);
	SET_DWORD_STAT(STAT_RDG_RenderPassMergeCount, GRDGStatRenderPassMergeCount);
	SET_DWORD_STAT(STAT_RDG_PassDependencyCount, GRDGStatPassDependencyCount);
	SET_DWORD_STAT(STAT_RDG_TextureCount, GRDGStatTextureCount);
	SET_DWORD_STAT(STAT_RDG_TextureReferenceCount, GRDGStatTextureReferenceCount);
	SET_FLOAT_STAT(STAT_RDG_TextureReferenceAverage, (float)(GRDGStatTextureReferenceCount / FMath::Max((float)GRDGStatTextureCount, 1.0f)));
	SET_DWORD_STAT(STAT_RDG_BufferCount, GRDGStatBufferCount);
	SET_DWORD_STAT(STAT_RDG_BufferReferenceCount, GRDGStatBufferReferenceCount);
	SET_FLOAT_STAT(STAT_RDG_BufferReferenceAverage, (float)(GRDGStatBufferReferenceCount / FMath::Max((float)GRDGStatBufferCount, 1.0f)));
	SET_DWORD_STAT(STAT_RDG_ViewCount, GRDGStatViewCount);
	SET_DWORD_STAT(STAT_RDG_TransientTextureCount, GRDGStatTransientTextureCount);
	SET_DWORD_STAT(STAT_RDG_TransientBufferCount, GRDGStatTransientBufferCount);
	SET_DWORD_STAT(STAT_RDG_TransitionCount, GRDGStatTransitionCount);
	SET_DWORD_STAT(STAT_RDG_AliasingCount, GRDGStatAliasingCount);
	SET_DWORD_STAT(STAT_RDG_TransitionBatchCount, GRDGStatTransitionBatchCount);
	SET_MEMORY_STAT(STAT_RDG_MemoryWatermark, int64(GRDGStatMemoryWatermark));

	GRDGStatPassCount = 0;
	GRDGStatPassCullCount = 0;
	GRDGStatRenderPassMergeCount = 0;
	GRDGStatPassDependencyCount = 0;
	GRDGStatTextureCount = 0;
	GRDGStatTextureReferenceCount = 0;
	GRDGStatBufferCount = 0;
	GRDGStatBufferReferenceCount = 0;
	GRDGStatViewCount = 0;
	GRDGStatTransientTextureCount = 0;
	GRDGStatTransientBufferCount = 0;
	GRDGStatTransitionCount = 0;
	GRDGStatAliasingCount = 0;
	GRDGStatTransitionBatchCount = 0;
	GRDGStatMemoryWatermark = 0;
#endif
}

bool FRDGBuilder::IsImmediateMode()
{
	return ::IsImmediateMode();
}

ERDGPassFlags FRDGBuilder::OverridePassFlags(const TCHAR* PassName, ERDGPassFlags PassFlags) const
{
	const bool bDebugAllowedForPass =
#if RDG_ENABLE_DEBUG
		IsDebugAllowedForPass(PassName);
#else
		true;
#endif

	if (bSupportsAsyncCompute)
	{
		if (EnumHasAnyFlags(PassFlags, ERDGPassFlags::Compute) && GRDGAsyncCompute == RDG_ASYNC_COMPUTE_FORCE_ENABLED)
		{
			PassFlags &= ~ERDGPassFlags::Compute;
			PassFlags |= ERDGPassFlags::AsyncCompute;
		}
	}
	else
	{
		if (EnumHasAnyFlags(PassFlags, ERDGPassFlags::AsyncCompute))
		{
			PassFlags &= ~ERDGPassFlags::AsyncCompute;
			PassFlags |= ERDGPassFlags::Compute;
		}
	}

	return PassFlags;
}

bool FRDGBuilder::IsTransient(FRDGBufferRef Buffer) const
{
	if (!bSupportsTransientBuffers || Buffer->bQueuedForUpload)
	{
		return false;
	}

	if (!IsTransientInternal(Buffer, EnumHasAnyFlags(Buffer->Desc.Usage, BUF_FastVRAM)))
	{
		return false;
	}

	if (!GRDGTransientIndirectArgBuffers && EnumHasAnyFlags(Buffer->Desc.Usage, BUF_DrawIndirect))
	{
		return false;
	}

	return EnumHasAnyFlags(Buffer->Desc.Usage, BUF_UnorderedAccess);
}

bool FRDGBuilder::IsTransient(FRDGTextureRef Texture) const
{
	if (!bSupportsTransientTextures)
	{
		return false;
	}

	if (EnumHasAnyFlags(Texture->Desc.Flags, ETextureCreateFlags::Shared))
	{
		return false;
	}

	return IsTransientInternal(Texture, EnumHasAnyFlags(Texture->Desc.Flags, ETextureCreateFlags::FastVRAM));
}

bool FRDGBuilder::IsTransientInternal(FRDGViewableResource* Resource, bool bFastVRAM) const
{
	// FastVRAM resources are always transient regardless of extraction or other hints, since they are performance critical.
	if (!bFastVRAM || !FPlatformMemory::SupportsFastVRAMMemory())
	{
		if (GRDGTransientAllocator == 2)
		{
			return false;
		}
	
		if (Resource->bForceNonTransient)
		{
			return false;
		}

		if (Resource->bExtracted)
		{
			if (GRDGTransientExtractedResources == 0)
			{
				return false;
			}

			if (GRDGTransientExtractedResources == 1 && Resource->TransientExtractionHint == FRDGViewableResource::ETransientExtractionHint::Disable)
			{
				return false;
			}
		}
	}

#if RDG_ENABLE_DEBUG
	if (GRDGDebugDisableTransientResources != 0)
	{
		const bool bDebugAllowed = IsDebugAllowedForResource(Resource->Name);

		if (GRDGDebugDisableTransientResources == 2 && Resource->Type == ERDGViewableResourceType::Buffer && bDebugAllowed)
		{
			return false;
		}

		if (GRDGDebugDisableTransientResources == 3 && Resource->Type == ERDGViewableResourceType::Texture && bDebugAllowed)
		{
			return false;
		}
	}
#endif

	return true;
}

FRDGBuilder::FRDGBuilder(FRHICommandListImmediate& InRHICmdList, FRDGEventName InName, ERDGBuilderFlags InFlags, EShaderPlatform ShaderPlatform)
	: FRDGScopeState(InRHICmdList, IsImmediateMode(), ::IsParallelExecuteEnabled(ShaderPlatform) && EnumHasAnyFlags(InFlags, ERDGBuilderFlags::ParallelExecute))
	, RootAllocatorScope(Allocators.Root)
	, Blackboard(Allocators.Root)
	, BuilderName(InName)
	, TransientResourceAllocator(GRDGTransientAllocator != 0 && !::IsImmediateMode() ? GRDGTransientResourceAllocator.Get() : nullptr)
	, ExtendResourceLifetimeScope(RHICmdList)
#if RDG_ENABLE_DEBUG
	, UserValidation(Allocators.Root)
	, BarrierValidation(&Passes, BuilderName)
#endif
{
	ProloguePass = SetupEmptyPass(Passes.Allocate<FRDGSentinelPass>(Allocators.Root, RDG_EVENT_NAME("Graph Prologue (Graphics)")));

	bSupportsAsyncCompute = IsAsyncComputeSupported(ShaderPlatform);
	bSupportsRenderPassMerge = IsRenderPassMergeEnabled(ShaderPlatform);

	const bool bParallelExecuteFlag = EnumHasAnyFlags(InFlags, ERDGBuilderFlags::ParallelExecute);
	const bool bParallelExecuteAllowedAwait = ::IsParallelExecuteEnabled(ShaderPlatform);
	const bool bParallelExecuteAllowedAsync = bParallelExecuteAllowedAwait && GRDGParallelExecute > 1;

	if (bParallelExecuteFlag)
	{
		if (bParallelExecuteAllowedAsync)
		{
			ParallelExecute.TaskMode = ERDGPassTaskMode::Async;
		}
		else if (bParallelExecuteAllowedAwait)
		{
			ParallelExecute.TaskMode = ERDGPassTaskMode::Await;
		}
	}

	const bool bParallelSetupEnabledForPlatform = ::IsParallelSetupEnabled(ShaderPlatform);
	ParallelSetup.bEnabled   = bParallelSetupEnabledForPlatform && EnumHasAnyFlags(InFlags, ERDGBuilderFlags::ParallelSetup);
#if RDG_ENABLE_PARALLEL_TASKS
	ParallelSetup.TaskPriorityBias = GRDGParallelSetupTaskPriorityBias;
#endif
	AsyncSetupQueue.bEnabled = ParallelSetup.bEnabled && GRDGAsyncSetupQueue != 0;

	bParallelCompileEnabled  = GRDGParallelCompile && ::IsParallelSetupEnabled(ShaderPlatform) && EnumHasAnyFlags(InFlags, ERDGBuilderFlags::ParallelCompile);

	if (TransientResourceAllocator)
	{
		bSupportsTransientTextures = TransientResourceAllocator->SupportsResourceType(ERHITransientResourceType::Texture);
		bSupportsTransientBuffers  = TransientResourceAllocator->SupportsResourceType(ERHITransientResourceType::Buffer);
	}

#if RDG_DUMP_RESOURCES
	DumpNewGraphBuilder();
#endif

#if RDG_ENABLE_DEBUG
	UserValidation.SetParallelExecuteEnabled(ParallelExecute.TaskMode != ERDGPassTaskMode::Inline);
	if (GRDGAllowRHIAccessAsync.load(std::memory_order_relaxed) != bParallelExecuteAllowedAsync)
	{
		WaitForAsyncExecuteTask();
		GRDGAllowRHIAccessAsync.store(bParallelExecuteAllowedAsync, std::memory_order_relaxed);
	}
#endif
}

UE::Tasks::FTask FRDGBuilder::FAsyncDeleter::LastTask;

FRDGBuilder::FAsyncDeleter::~FAsyncDeleter()
{
	if (Function)
	{
		// Launch the task with a prerequisite on any previously launched RDG async delete task.
		LastTask = UE::Tasks::Launch(UE_SOURCE_LOCATION, [Function = MoveTemp(Function)] () mutable
		{
			// Call and release the contents of the function inside the task lamda.
			Function();
			Function = {};

		}, MakeArrayView({ LastTask, Prerequisites }));
	}
}

void FRDGBuilder::WaitForAsyncDeleteTask()
{
	FAsyncDeleter::LastTask.Wait();
	FAsyncDeleter::LastTask = {};
}

const UE::Tasks::FTask& FRDGBuilder::GetAsyncDeleteTask()
{
	return FAsyncDeleter::LastTask;
}

UE::Tasks::FTask FRDGBuilder::FParallelExecute::LastAsyncExecuteTask;

void FRDGBuilder::WaitForAsyncExecuteTask()
{
	if (FParallelExecute::LastAsyncExecuteTask.IsValid())
	{
		FParallelExecute::LastAsyncExecuteTask.Wait();
		FParallelExecute::LastAsyncExecuteTask = {};
	}
}

const UE::Tasks::FTask& FRDGBuilder::GetAsyncExecuteTask()
{
	return FParallelExecute::LastAsyncExecuteTask;
}

FRDGBuilder::~FRDGBuilder()
{
	if (ParallelExecute.TaskMode != ERDGPassTaskMode::Inline && (ParallelExecute.TasksAsync || GRDGParallelDestruction > 0))
	{
		if (ParallelExecute.TasksAsync)
		{
			ParallelExecute.TasksAsync->Trigger();
			AsyncDeleter.Prerequisites  = MoveTemp(*ParallelExecute.TasksAsync);
			ParallelExecute.TasksAsync.Reset();
		}

		AsyncDeleter.Function = [
			Allocators				= MoveTemp(Allocators),
			Passes					= MoveTemp(Passes),
			Textures				= MoveTemp(Textures),
			Buffers					= MoveTemp(Buffers),
			Views					= MoveTemp(Views),
			UniformBuffers			= MoveTemp(UniformBuffers),
			Blackboard				= MoveTemp(Blackboard),
			ActivePooledTextures	= MoveTemp(ActivePooledTextures),
			ActivePooledBuffers		= MoveTemp(ActivePooledBuffers),
			UploadedBuffers			= MoveTemp(UploadedBuffers)
#if WITH_RHI_BREADCRUMBS
			, BreadcrumbAllocator	= GetBreadcrumbAllocator().AsShared()
#endif
		] () mutable {};
	}
}

///////////////////////////////////////////////////////////////////////////////////////////////////

const TRefCountPtr<FRDGPooledBuffer>& FRDGBuilder::ConvertToExternalBuffer(FRDGBufferRef Buffer)
{
	IF_RDG_ENABLE_DEBUG(UserValidation.ValidateConvertToExternalResource(Buffer));
	if (!Buffer->bExternal)
	{
		Buffer->bExternal = 1;
		if (!Buffer->ResourceRHI)
		{
			SetExternalPooledBufferRHI(Buffer, AllocatePooledBufferRHI(RHICmdList, Buffer));
		}
		ExternalBuffers.FindOrAdd(Buffer->GetRHIUnchecked(), Buffer);
		AsyncSetupQueue.Push(FAsyncSetupOp::CullRootBuffer(Buffer));
	}
	return GetPooledBuffer(Buffer);
}

const TRefCountPtr<IPooledRenderTarget>& FRDGBuilder::ConvertToExternalTexture(FRDGTextureRef Texture)
{
	IF_RDG_ENABLE_DEBUG(UserValidation.ValidateConvertToExternalResource(Texture));
	if (!Texture->bExternal)
	{
		Texture->bExternal = 1;
		if (!Texture->ResourceRHI)
		{
			SetExternalPooledRenderTargetRHI(Texture, AllocatePooledRenderTargetRHI(RHICmdList, Texture));
		}
		ExternalTextures.FindOrAdd(Texture->GetRHIUnchecked(), Texture);
		AsyncSetupQueue.Push(FAsyncSetupOp::CullRootTexture(Texture));
	}
	return GetPooledTexture(Texture);
}

FRHIUniformBuffer* FRDGBuilder::ConvertToExternalUniformBuffer(FRDGUniformBufferRef UniformBuffer)
{
	if (!UniformBuffer->bExternal)
	{
		UniformBuffer->GetParameters().Enumerate([this](const FRDGParameter& Param)
		{
			const auto ConvertTexture = [](FRDGBuilder* Builder, FRDGTextureRef Texture)
			{
				if (Texture && !Texture->IsExternal())
				{
					Builder->ConvertToExternalTexture(Texture);
				}
			};

			const auto ConvertBuffer = [](FRDGBuilder* Builder, FRDGBufferRef Buffer)
			{
				if (Buffer && !Buffer->IsExternal())
				{
					Builder->ConvertToExternalBuffer(Buffer);
				}
			};

			const auto ConvertView = [this] (FRDGView* View)
			{
				if (!View->ResourceRHI)
				{
					InitViewRHI(RHICmdList, View);
				}
			};

			switch (Param.GetType())
			{
			case UBMT_RDG_TEXTURE:
			{
				ConvertTexture(this, Param.GetAsTexture());
			}
			break;
			case UBMT_RDG_TEXTURE_ACCESS:
			{
				ConvertTexture(this, Param.GetAsTextureAccess().GetTexture());
			}
			break;
			case UBMT_RDG_TEXTURE_ACCESS_ARRAY:
			{
				const FRDGTextureAccessArray& Array = Param.GetAsTextureAccessArray();
				for (int Index = 0; Index < Array.Num(); ++Index)
				{
					ConvertTexture(this, Array[Index].GetTexture());
				}
			}
			break;
			case UBMT_RDG_TEXTURE_SRV:
			case UBMT_RDG_TEXTURE_NON_PIXEL_SRV:
			{
				ConvertTexture(this, Param.GetAsTextureSRV()->Desc.Texture);
				ConvertView(Param.GetAsView());
			}
			break;
			case UBMT_RDG_TEXTURE_UAV:
			{
				ConvertTexture(this, Param.GetAsTextureUAV()->Desc.Texture);
				ConvertView(Param.GetAsView());
			}
			break;
			case UBMT_RDG_BUFFER_ACCESS:
			{
				ConvertBuffer(this, Param.GetAsBufferAccess().GetBuffer());
			}
			break;
			case UBMT_RDG_BUFFER_ACCESS_ARRAY:
			{
				const FRDGBufferAccessArray& Array = Param.GetAsBufferAccessArray();
				for (int Index = 0; Index < Array.Num(); ++Index)
				{
					ConvertBuffer(this, Array[Index].GetBuffer());
				}
			}
			break;
			case UBMT_RDG_BUFFER_SRV:
			{
				ConvertBuffer(this, Param.GetAsBufferSRV()->Desc.Buffer);
				ConvertView(Param.GetAsView());
			}
			break;
			case UBMT_RDG_BUFFER_UAV:
			{
				ConvertBuffer(this, Param.GetAsBufferUAV()->Desc.Buffer);
				ConvertView(Param.GetAsView());
			}
			break;
			case UBMT_RDG_UNIFORM_BUFFER:
			{
				FRDGUniformBufferRef Buffer = Param.GetAsUniformBuffer().GetUniformBuffer();
				if (Buffer)
				{
					ConvertToExternalUniformBuffer(Buffer);
				}
			}
			break;

			// Non-RDG cases
			case UBMT_INT32:
			case UBMT_UINT32:
			case UBMT_FLOAT32:
			case UBMT_TEXTURE:
			case UBMT_SRV:
			case UBMT_UAV:
			case UBMT_SAMPLER:
			case UBMT_NESTED_STRUCT:
			case UBMT_INCLUDED_STRUCT:
			case UBMT_REFERENCED_STRUCT:
			case UBMT_RENDER_TARGET_BINDING_SLOTS:
			break;

			default:
				check(0);
			}
		});
	}
	IF_RDG_ENABLE_DEBUG(UserValidation.ValidateConvertToExternalUniformBuffer(UniformBuffer));
	if (!UniformBuffer->bExternal)
	{
		UniformBuffer->bExternal = true;

		// Immediate mode can end up creating the resource first.
		if (!UniformBuffer->GetRHIUnchecked())
		{
			// It's safe to reset the access to false because validation won't allow this call during execution.
			IF_RDG_ENABLE_DEBUG(GRDGAllowRHIAccess.store(true, std::memory_order_relaxed));
			UniformBuffer->InitRHI();
			IF_RDG_ENABLE_DEBUG(GRDGAllowRHIAccess.store(false, std::memory_order_relaxed));
		}
	}
	return UniformBuffer->GetRHIUnchecked();
}

///////////////////////////////////////////////////////////////////////////////////////////////////

BEGIN_SHADER_PARAMETER_STRUCT(FAccessModePassParameters, )
	RDG_TEXTURE_ACCESS_ARRAY(Textures)
	RDG_BUFFER_ACCESS_ARRAY(Buffers)
END_SHADER_PARAMETER_STRUCT()

void FRDGBuilder::UseExternalAccessMode(FRDGViewableResource* Resource, ERHIAccess ReadOnlyAccess, ERHIPipeline Pipelines)
{
	if (!bSupportsAsyncCompute)
	{
		Pipelines = ERHIPipeline::Graphics;
	}

	IF_RDG_ENABLE_DEBUG(UserValidation.ValidateUseExternalAccessMode(Resource, ReadOnlyAccess, Pipelines));

	auto& AccessModeState = Resource->AccessModeState;

	// We already validated that back-to-back calls to UseExternalAccessMode are valid only if the parameters match,
	// so we can safely no-op this call.
	if (AccessModeState.Mode == FRDGViewableResource::EAccessMode::External || AccessModeState.bLocked)
	{
		return;
	}

	// We have to flush the queue when going from QueuedInternal -> External. A queued internal state
	// implies that the resource was in an external access mode before, so it needs an 'end' pass to 
	// contain any passes which might have used the resource in its external state.
	if (AccessModeState.bQueued)
	{
		FlushAccessModeQueue();
	}

	check(!AccessModeState.bQueued);
	AccessModeQueue.Emplace(Resource);
	AccessModeState.bQueued = 1;

	Resource->SetExternalAccessMode(ReadOnlyAccess, Pipelines);
}

void FRDGBuilder::UseInternalAccessMode(FRDGViewableResource* Resource)
{
	IF_RDG_ENABLE_DEBUG(UserValidation.ValidateUseInternalAccessMode(Resource));

	auto& AccessModeState = Resource->AccessModeState;

	// Just no-op if the resource is already in (or queued for) the Internal state.
	if (AccessModeState.Mode == FRDGViewableResource::EAccessMode::Internal || AccessModeState.bLocked)
	{
		return;
	}

	// If the resource has a queued transition to the external access state, then we can safely back it out.
	if (AccessModeState.bQueued)
	{
		int32 Index = AccessModeQueue.IndexOfByKey(Resource);
		check(Index < AccessModeQueue.Num());
		AccessModeQueue.RemoveAtSwap(Index, EAllowShrinking::No);
		AccessModeState.bQueued = 0;
	}
	else
	{
		AccessModeQueue.Emplace(Resource);
		AccessModeState.bQueued = 1;
	}

	AccessModeState.Mode = FRDGViewableResource::EAccessMode::Internal;
}

void FRDGBuilder::FlushAccessModeQueue()
{
	if (AccessModeQueue.IsEmpty() || !AuxiliaryPasses.IsFlushAccessModeQueueAllowed())
	{
		return;
	}

	// Don't allow Dump GPU to dump access mode passes. We rely on FlushAccessQueue in dump GPU to transition things back to external access.
	RDG_RECURSION_COUNTER_SCOPE(AuxiliaryPasses.Dump);
	RDG_RECURSION_COUNTER_SCOPE(AuxiliaryPasses.FlushAccessModeQueue);

	FAccessModePassParameters* ParametersByPipeline[] =
	{
		AllocParameters<FAccessModePassParameters>(),
		AllocParameters<FAccessModePassParameters>()
	};

	const ERHIAccess AccessMaskByPipeline[] =
	{
		ERHIAccess::ReadOnlyExclusiveMask,
		ERHIAccess::ReadOnlyExclusiveComputeMask
	};

	ERHIPipeline ParameterPipelines = ERHIPipeline::None;

	TArray<FRDGPass::FExternalAccessOp, FRDGArrayAllocator> Ops;
	Ops.Reserve(AsyncSetupQueue.bEnabled ? AccessModeQueue.Num() : 0);

	for (FRDGViewableResource* Resource : AccessModeQueue)
	{
		const auto& AccessModeState = Resource->AccessModeState;
		Resource->AccessModeState.bQueued = false;

		if (AsyncSetupQueue.bEnabled)
		{
			Ops.Emplace(Resource, AccessModeState.Mode);
		}
		else
		{
			Resource->AccessModeState.ActiveMode = Resource->AccessModeState.Mode;
		}

		ParameterPipelines |= AccessModeState.Pipelines;

		if (AccessModeState.Mode == FRDGViewableResource::EAccessMode::External)
		{
			ExternalAccessResources.Emplace(Resource);
		}
		else
		{
			ExternalAccessResources.Remove(Resource);
		}

		for (uint32 PipelineIndex = 0; PipelineIndex < GetRHIPipelineCount(); ++PipelineIndex)
		{
			const ERHIPipeline Pipeline = static_cast<ERHIPipeline>(1 << PipelineIndex);

			if (EnumHasAnyFlags(AccessModeState.Pipelines, Pipeline))
			{
				const ERHIAccess Access = AccessModeState.Access & AccessMaskByPipeline[PipelineIndex];
				check(Access != ERHIAccess::None);

				switch (Resource->Type)
				{
				case ERDGViewableResourceType::Texture:
					ParametersByPipeline[PipelineIndex]->Textures.Emplace(GetAsTexture(Resource), Access);
					break;
				case ERDGViewableResourceType::Buffer:
					ParametersByPipeline[PipelineIndex]->Buffers.Emplace(GetAsBuffer(Resource), Access);
					break;
				}
			}
		}
	}

	if (EnumHasAnyFlags(ParameterPipelines, ERHIPipeline::Graphics))
	{
		auto ExecuteLambda = [](FRDGAsyncTask, FRHIComputeCommandList&) {};
		using LambdaPassType = TRDGLambdaPass<FAccessModePassParameters, decltype(ExecuteLambda)>;

		FAccessModePassParameters* Parameters = ParametersByPipeline[GetRHIPipelineIndex(ERHIPipeline::Graphics)];

		FRDGPass* Pass = Passes.Allocate<LambdaPassType>(
			Allocators.Root,
			RDG_EVENT_NAME("AccessModePass[Graphics] (Textures: %d, Buffers: %d)", Parameters->Textures.Num(), Parameters->Buffers.Num()),
			FAccessModePassParameters::FTypeInfo::GetStructMetadata(),
			Parameters,
			// Use all of the work flags so that any access is valid.
			ERDGPassFlags::Copy | ERDGPassFlags::Compute | ERDGPassFlags::Raster | ERDGPassFlags::SkipRenderPass | ERDGPassFlags::NeverCull,
			MoveTemp(ExecuteLambda));

		Pass->ExternalAccessOps = MoveTemp(Ops);
		Pass->bExternalAccessPass = 1;
		SetupParameterPass(Pass);
	}

	if (EnumHasAnyFlags(ParameterPipelines, ERHIPipeline::AsyncCompute))
	{
		auto ExecuteLambda = [](FRDGAsyncTask, FRHIComputeCommandList&) {};
		using LambdaPassType = TRDGLambdaPass<FAccessModePassParameters, decltype(ExecuteLambda)>;

		FAccessModePassParameters* Parameters = ParametersByPipeline[GetRHIPipelineIndex(ERHIPipeline::AsyncCompute)];

		FRDGPass* Pass = Passes.Allocate<LambdaPassType>(
			Allocators.Root,
			RDG_EVENT_NAME("AccessModePass[AsyncCompute] (Textures: %d, Buffers: %d)", Parameters->Textures.Num(), Parameters->Buffers.Num()),
			FAccessModePassParameters::FTypeInfo::GetStructMetadata(),
			Parameters,
			ERDGPassFlags::AsyncCompute | ERDGPassFlags::NeverCull,
			MoveTemp(ExecuteLambda));

		Pass->ExternalAccessOps = MoveTemp(Ops);
		Pass->bExternalAccessPass = 1;
		SetupParameterPass(Pass);
	}

	AccessModeQueue.Reset();
}

///////////////////////////////////////////////////////////////////////////////////////////////////

FRDGTextureRef FRDGBuilder::RegisterExternalTexture(
	const TRefCountPtr<IPooledRenderTarget>& ExternalPooledTexture,
	ERDGTextureFlags Flags)
{
#if RDG_ENABLE_DEBUG
	checkf(ExternalPooledTexture.IsValid(), TEXT("Attempted to register NULL external texture."));
#endif

	const TCHAR* Name = ExternalPooledTexture->GetDesc().DebugName;
	if (!Name)
	{
		Name = TEXT("External");
	}
	return RegisterExternalTexture(ExternalPooledTexture, Name, Flags);
}

FRDGTexture* FRDGBuilder::RegisterExternalTexture(
	const TRefCountPtr<IPooledRenderTarget>& ExternalPooledTexture,
	const TCHAR* Name,
	ERDGTextureFlags Flags)
{
	IF_RDG_ENABLE_DEBUG(UserValidation.ValidateRegisterExternalTexture(ExternalPooledTexture, Name, Flags));
	FRHITexture* ExternalTextureRHI = ExternalPooledTexture->GetRHI();
	IF_RDG_ENABLE_DEBUG(checkf(ExternalTextureRHI, TEXT("Attempted to register texture %s, but its RHI texture is null."), Name));

	if (FRDGTexture* FoundTexture = FindExternalTexture(ExternalTextureRHI))
	{
		return FoundTexture;
	}

	const FRDGTextureDesc Desc = Translate(ExternalPooledTexture->GetDesc());
	FRDGTexture* Texture = Textures.Allocate(Allocators.Root, Name, Desc, Flags);
	SetExternalPooledRenderTargetRHI(Texture, ExternalPooledTexture.GetReference());
	Texture->bExternal = true;
	ExternalTextures.FindOrAdd(Texture->GetRHIUnchecked(), Texture);

	IF_RDG_ENABLE_DEBUG(UserValidation.ValidateRegisterExternalTexture(Texture));
	IF_RDG_ENABLE_TRACE(Trace.AddResource(Texture));
	return Texture;
}

FRDGBufferRef FRDGBuilder::RegisterExternalBuffer(const TRefCountPtr<FRDGPooledBuffer>& ExternalPooledBuffer, ERDGBufferFlags Flags)
{
#if RDG_ENABLE_DEBUG
	checkf(ExternalPooledBuffer.IsValid(), TEXT("Attempted to register NULL external buffer."));
#endif

	const TCHAR* Name = ExternalPooledBuffer->Name;
	if (!Name)
	{
		Name = TEXT("External");
	}
	return RegisterExternalBuffer(ExternalPooledBuffer, Name, Flags);
}

FRDGBufferRef FRDGBuilder::RegisterExternalBuffer(
	const TRefCountPtr<FRDGPooledBuffer>& ExternalPooledBuffer,
	const TCHAR* Name,
	ERDGBufferFlags Flags)
{
	IF_RDG_ENABLE_DEBUG(UserValidation.ValidateRegisterExternalBuffer(ExternalPooledBuffer, Name, Flags));

	if (FRDGBuffer* FoundBuffer = FindExternalBuffer(ExternalPooledBuffer))
	{
		return FoundBuffer;
	}

	FRDGBuffer* Buffer = Buffers.Allocate(Allocators.Root, Name, ExternalPooledBuffer->Desc, Flags);
	SetExternalPooledBufferRHI(Buffer, ExternalPooledBuffer);
	Buffer->bExternal = true;
	ExternalBuffers.FindOrAdd(Buffer->GetRHIUnchecked(), Buffer);

	IF_RDG_ENABLE_DEBUG(UserValidation.ValidateRegisterExternalBuffer(Buffer));
	IF_RDG_ENABLE_TRACE(Trace.AddResource(Buffer));
	return Buffer;
}

///////////////////////////////////////////////////////////////////////////////////////////////////

void FRDGBuilder::AddPassDependency(FRDGPass* Producer, FRDGPass* Consumer)
{
	auto& Producers = Consumer->Producers;

	if (Producers.Find(Producer) == INDEX_NONE)
	{
#if RDG_STATS
		GRDGStatPassDependencyCount++;
#endif

		if (Producer->Pipeline != Consumer->Pipeline)
		{
			const auto BinarySearchOrAdd = [](auto& Range, FRDGPassHandle Handle)
			{
				const int32 LowerBoundIndex = Algo::LowerBound(Range, Handle);
				if (LowerBoundIndex < Range.Num())
				{
					if (Range[LowerBoundIndex] == Handle)
					{
						return;
					}
				}
				Range.Insert(Handle, LowerBoundIndex);
			};

			// Consumers could be culled, so we have to store all of them in a sorted list.
			BinarySearchOrAdd(Producer->CrossPipelineConsumers, Consumer->Handle);

			// Finds the latest producer on the other pipeline for the consumer.
			if (Consumer->CrossPipelineProducer.IsNull() || Producer->Handle > Consumer->CrossPipelineProducer)
			{
				Consumer->CrossPipelineProducer = Producer->Handle;
			}
		}

		Producers.Add(Producer);
	}
}

bool FRDGBuilder::AddCullingDependency(FRDGProducerStatesByPipeline& LastProducers, const FRDGProducerState& NextState, ERHIPipeline NextPipeline)
{
	for (ERHIPipeline LastPipeline : MakeFlagsRange(ERHIPipeline::All))
	{
		FRDGProducerState& LastProducer = LastProducers[LastPipeline];

		if (LastProducer.Access != ERHIAccess::Unknown)
		{
			FRDGPass* LastProducerPass = LastProducer.Pass;

			if (LastPipeline != NextPipeline)
			{
				// Only certain platforms allow multi-pipe UAV access.
				const ERHIAccess MultiPipelineUAVMask = ERHIAccess::UAVMask & GRHIMultiPipelineMergeableAccessMask;

				// If skipping a UAV barrier across pipelines, use the producer pass that will emit the correct async fence.
				if (EnumHasAnyFlags(NextState.Access, MultiPipelineUAVMask) && SkipUAVBarrier(LastProducer.NoUAVBarrierHandle, NextState.NoUAVBarrierHandle))
				{
					LastProducerPass = LastProducer.PassIfSkipUAVBarrier;
				}
			}

			if (LastProducerPass)
			{
				AddPassDependency(LastProducerPass, NextState.Pass);
			}
		}
	}

	FRDGProducerState& LastProducer = LastProducers[NextPipeline];

	if (IsWritableAccess(NextState.Access))
	{
		// Add a dependency between the last read of a resource on the other pipe and the new write (this is necessary for async compute fencing).
		if (FRDGPass* PassIfReadAccess = LastProducers[NextPipeline == ERHIPipeline::Graphics ? ERHIPipeline::AsyncCompute : ERHIPipeline::Graphics].PassIfReadAccess)
		{
			AddPassDependency(PassIfReadAccess, NextState.Pass);
		}

		// A separate producer pass is tracked for UAV -> UAV dependencies that are skipped. Consider the following scenario:
		//
		//     Graphics:       A   ->    B         ->         D      ->     E       ->        G         ->            I
		//                   (UAV)   (SkipUAV0)           (SkipUAV1)    (SkipUAV1)          (SRV)                   (UAV2)
		//
		// Async Compute:                           C                ->               F       ->         H
		//                                      (SkipUAV0)                        (SkipUAV1)           (SRV)
		//
		// Expected Cross Pipe Dependencies: [A -> C], C -> D, [B -> F], F -> G, E -> H, F -> I. The dependencies wrapped in
		// braces are only introduced properly by tracking a different producer for cross-pipeline skip UAV dependencies, which
		// is only updated if skip UAV is inactive, or if transitioning from one skip UAV set to another (or another writable resource).

		if (LastProducer.NoUAVBarrierHandle.IsNull())
		{
			if (NextState.NoUAVBarrierHandle.IsNull())
			{
				// Assigns the next producer when no skip UAV sets are active.
				LastProducer.PassIfSkipUAVBarrier = NextState.Pass;
			}
		}
		else if (LastProducer.NoUAVBarrierHandle != NextState.NoUAVBarrierHandle)
		{
			// Assigns the last producer in the prior skip UAV barrier set when moving out of a skip UAV barrier set.
			LastProducer.PassIfSkipUAVBarrier = LastProducer.Pass;
		}

		LastProducer.Access             = NextState.Access;
		LastProducer.Pass               = NextState.Pass;
		LastProducer.NoUAVBarrierHandle = NextState.NoUAVBarrierHandle;
		LastProducer.PassIfReadAccess = nullptr;
		return true;
	}
	else
	{
		LastProducer.PassIfReadAccess = NextState.Pass;
	}

	return false;
}

void FRDGBuilder::AddCullRootTexture(FRDGTexture* Texture)
{
	check(Texture->IsCullRoot());

	for (auto& LastProducer : Texture->LastProducers)
	{
		AddLastProducersToCullStack(LastProducer);
	}

	FlushCullStack();
}

void FRDGBuilder::AddCullRootBuffer(FRDGBuffer* Buffer)
{
	check(Buffer->IsCullRoot());

	AddLastProducersToCullStack(Buffer->LastProducer);

	FlushCullStack();
}

void FRDGBuilder::AddLastProducersToCullStack(const FRDGProducerStatesByPipeline& LastProducers)
{
	for (const FRDGProducerState& LastProducer : LastProducers)
	{
		if (LastProducer.Pass)
		{
			CullPassStack.Emplace(LastProducer.Pass);
		}
	}
}

void FRDGBuilder::FlushCullStack()
{
	while (CullPassStack.Num())
	{
		FRDGPass* Pass = CullPassStack.Pop(EAllowShrinking::No);

		if (Pass->bCulled)
		{
			Pass->bCulled = 0;

			CullPassStack.Append(Pass->Producers);
		}
	}
}

///////////////////////////////////////////////////////////////////////////////

void FRDGBuilder::Compile()
{
	SCOPE_CYCLE_COUNTER(STAT_RDG_CompileTime);
	CSV_SCOPED_TIMING_STAT_EXCLUSIVE_CONDITIONAL(RDG_Compile, GRDGVerboseCSVStats != 0);

	const FRDGPassHandle ProloguePassHandle = GetProloguePassHandle();
	const FRDGPassHandle EpiloguePassHandle = GetEpiloguePassHandle();

	const uint32 CompilePassCount = Passes.Num();

	TransitionCreateQueue.Reserve(CompilePassCount);

	const bool bCullPasses = GRDGCullPasses > 0;

	if (bCullPasses || AsyncComputePassCount > 0)
	{
		SCOPED_NAMED_EVENT(PassDependencies, FColor::Emerald);

		if (!AsyncSetupQueue.bEnabled)
		{
			for (FRDGPassHandle PassHandle = ProloguePassHandle + 1; PassHandle < EpiloguePassHandle; ++PassHandle)
			{
				SetupPassDependencies(Passes[PassHandle]);
			}
		}
	}
	else if (!AsyncSetupQueue.bEnabled)
	{
		for (FRDGPassHandle PassHandle = ProloguePassHandle + 1; PassHandle < EpiloguePassHandle; ++PassHandle)
		{
			FRDGPass* Pass = Passes[PassHandle];

			// Add reference counts for passes.

			for (auto& PassState : Pass->TextureStates)
			{
				PassState.Texture->ReferenceCount += PassState.ReferenceCount;
			}

			for (auto& PassState : Pass->BufferStates)
			{
				PassState.Buffer->ReferenceCount += PassState.ReferenceCount;
			}
		}
	}

	for (const FExtractedTexture& ExtractedTexture : ExtractedTextures)
	{
		ExtractedTexture.Texture->ReferenceCount++;
	}

	for (const FExtractedBuffer& ExtractedBuffer : ExtractedBuffers)
	{
		ExtractedBuffer.Buffer->ReferenceCount++;
	}

	// All dependencies in the raw graph have been specified; if enabled, all passes are marked as culled and a
	// depth first search is employed to find reachable regions of the graph. Roots of the search are those passes
	// with outputs leaving the graph or those marked to never cull.

	if (bCullPasses)
	{
		SCOPED_NAMED_EVENT(PassCulling, FColor::Emerald);

		// Manually mark the prologue / epilogue passes as not culled.
		EpiloguePass->bCulled = 0;
		ProloguePass->bCulled = 0;

		check(CullPassStack.IsEmpty());

		for (FRDGPassHandle PassHandle = ProloguePassHandle + 1; PassHandle < EpiloguePassHandle; ++PassHandle)
		{
			FRDGPass* Pass = Passes[PassHandle];

			if (Pass->bCulled)
			{
#if RDG_STATS
				GRDGStatPassCullCount++;
#endif

				// Subtract reference counts from culled passes that were added during pass setup.

				for (auto& PassState : Pass->TextureStates)
				{
					PassState.Texture->ReferenceCount -= PassState.ReferenceCount;
				}

				for (auto& PassState : Pass->BufferStates)
				{
					PassState.Buffer->ReferenceCount -= PassState.ReferenceCount;
				}
			}
			else
			{
				CompilePassOps(Pass);
			}
		}
	}

	// Traverses passes on the graphics pipe and merges raster passes with the same render targets into a single RHI render pass.
	if (bSupportsRenderPassMerge && RasterPassCount > 0)
	{
		SCOPED_NAMED_EVENT(MergeRenderPasses, FColor::Emerald);

		TArray<FRDGPassHandle, TInlineAllocator<32, FRDGArrayAllocator>> PassesToMerge;
		FRDGPass* PrevPass = nullptr;
		const FRenderTargetBindingSlots* PrevRenderTargets = nullptr;

		const auto CommitMerge = [&]
		{
			if (PassesToMerge.Num())
			{
				const auto SetEpilogueBarrierPass = [&](FRDGPass* Pass, FRDGPassHandle EpilogueBarrierPassHandle)
				{
					Pass->EpilogueBarrierPass = EpilogueBarrierPassHandle;
					Pass->ResourcesToEnd.Reset();
					Passes[EpilogueBarrierPassHandle]->ResourcesToEnd.Add(Pass);
				};

				const auto SetPrologueBarrierPass = [&](FRDGPass* Pass, FRDGPassHandle PrologueBarrierPassHandle)
				{
					Pass->PrologueBarrierPass = PrologueBarrierPassHandle;
					Pass->ResourcesToBegin.Reset();
					Passes[PrologueBarrierPassHandle]->ResourcesToBegin.Add(Pass);
				};

				const FRDGPassHandle FirstPassHandle = PassesToMerge[0];
				const FRDGPassHandle LastPassHandle = PassesToMerge.Last();
				Passes[FirstPassHandle]->ResourcesToBegin.Reserve(PassesToMerge.Num());
				Passes[LastPassHandle]->ResourcesToEnd.Reserve(PassesToMerge.Num());

				// Given an interval of passes to merge into a single render pass: [B, X, X, X, X, E]
				//
				// The begin pass (B) and end (E) passes will call {Begin, End}RenderPass, respectively. Also,
				// begin will handle all prologue barriers for the entire merged interval, and end will handle all
				// epilogue barriers. This avoids transitioning of resources within the render pass and batches the
				// transitions more efficiently. This assumes we have filtered out dependencies between passes from
				// the merge set, which is done during traversal.

				// (B) First pass in the merge sequence.
				{
					FRDGPass* Pass = Passes[FirstPassHandle];
					Pass->bSkipRenderPassEnd = 1;
					SetEpilogueBarrierPass(Pass, LastPassHandle);
				}

				// (X) Intermediate passes.
				for (int32 PassIndex = 1, PassCount = PassesToMerge.Num() - 1; PassIndex < PassCount; ++PassIndex)
				{
					const FRDGPassHandle PassHandle = PassesToMerge[PassIndex];
					FRDGPass* Pass = Passes[PassHandle];
					Pass->bSkipRenderPassBegin = 1;
					Pass->bSkipRenderPassEnd = 1;
					SetPrologueBarrierPass(Pass, FirstPassHandle);
					SetEpilogueBarrierPass(Pass, LastPassHandle);
				}

				// (E) Last pass in the merge sequence.
				{
					FRDGPass* Pass = Passes[LastPassHandle];
					Pass->bSkipRenderPassBegin = 1;
					SetPrologueBarrierPass(Pass, FirstPassHandle);
				}

#if RDG_STATS
				GRDGStatRenderPassMergeCount += PassesToMerge.Num();
#endif
			}
			PassesToMerge.Reset();
			PrevPass = nullptr;
			PrevRenderTargets = nullptr;
		};

		for (FRDGPassHandle PassHandle = ProloguePassHandle + 1; PassHandle < EpiloguePassHandle; ++PassHandle)
		{
			FRDGPass* NextPass = Passes[PassHandle];

			if (NextPass->bCulled || NextPass->bEmptyParameters)
			{
				continue;
			}

			if (EnumHasAnyFlags(NextPass->Flags, ERDGPassFlags::Raster))
			{
				// A pass where the user controls the render pass or it is forced to skip pass merging can't merge with other passes
				if (EnumHasAnyFlags(NextPass->Flags, ERDGPassFlags::SkipRenderPass | ERDGPassFlags::NeverMerge))
				{
					CommitMerge();
					continue;
				}

				// A pass which writes to resources outside of the render pass introduces new dependencies which break merging.
				if (!NextPass->bRenderPassOnlyWrites)
				{
					CommitMerge();
					continue;
				}

				const FRenderTargetBindingSlots& RenderTargets = NextPass->GetParameters().GetRenderTargets();

				if (PrevPass)
				{
					check(PrevRenderTargets);

					if (PrevRenderTargets->CanMergeBefore(RenderTargets)
#if WITH_MGPU
						&& PrevPass->GPUMask == NextPass->GPUMask
#endif
						)
					{
						if (!PassesToMerge.Num())
						{
							PassesToMerge.Add(PrevPass->GetHandle());
						}
						PassesToMerge.Add(PassHandle);
					}
					else
					{
						CommitMerge();
					}
				}

				PrevPass = NextPass;
				PrevRenderTargets = &RenderTargets;
			}
			else if (!EnumHasAnyFlags(NextPass->Flags, ERDGPassFlags::AsyncCompute))
			{
				// A non-raster pass on the graphics pipe will invalidate the render target merge.
				CommitMerge();
			}
		}

		CommitMerge();
	}

	if (AsyncComputePassCount > 0)
	{
		SCOPED_NAMED_EVENT(AsyncComputeFences, FColor::Emerald);

		const bool bAsyncComputeTransientAliasing = IsAsyncComputeTransientAliasingEnabled();

		// Establishes fork / join overlap regions for async compute. This is used for fencing as well as resource
		// allocation / deallocation. Async compute passes can't allocate / release their resource references until
		// the fork / join is complete, since the two pipes run in parallel. Therefore, all resource lifetimes on
		// async compute are extended to cover the full async region.

		FRDGPassHandle CurrentGraphicsForkPassHandle;
		FRDGPass* AsyncComputePassBeforeFork = nullptr;

		for (FRDGPassHandle PassHandle = ProloguePassHandle + 1; PassHandle < EpiloguePassHandle; ++PassHandle)
		{
			FRDGPass* AsyncComputePass = Passes[PassHandle];

			if (!AsyncComputePass->IsAsyncCompute() || AsyncComputePass->bCulled)
			{
				continue;
			}

			FRDGPassHandle GraphicsForkPassHandle = FRDGPassHandle::Max(AsyncComputePass->CrossPipelineProducer, FRDGPassHandle::Max(CurrentGraphicsForkPassHandle, ProloguePassHandle));
			FRDGPass* GraphicsForkPass = Passes[GraphicsForkPassHandle];

			AsyncComputePass->GraphicsForkPass = GraphicsForkPassHandle;

			if (!bAsyncComputeTransientAliasing)
			{
				AsyncComputePass->ResourcesToBegin.Reset();
				Passes[GraphicsForkPass->PrologueBarrierPass]->ResourcesToBegin.Add(AsyncComputePass);
			}

			if (CurrentGraphicsForkPassHandle != GraphicsForkPassHandle)
			{
				CurrentGraphicsForkPassHandle = GraphicsForkPassHandle;

				FRDGBarrierBatchBegin& EpilogueBarriersToBeginForAsyncCompute = GraphicsForkPass->GetEpilogueBarriersToBeginForAsyncCompute(Allocators.Transition, TransitionCreateQueue);

				// Workaround for RHI validation. The prologue pass issues its own separate transition for the prologue pass
				// so that external access resources left in the all pipes state can be transitioned back to graphics.
				const bool bSeparateTransitionNeeded = GraphicsForkPass == ProloguePass;

				GraphicsForkPass->bGraphicsFork = 1;
				EpilogueBarriersToBeginForAsyncCompute.SetUseCrossPipelineFence(bSeparateTransitionNeeded);

				AsyncComputePass->bAsyncComputeBegin = 1;
				AsyncComputePass->GetPrologueBarriersToEnd(Allocators.Transition).AddDependency(&EpilogueBarriersToBeginForAsyncCompute);
			}

			AsyncComputePassBeforeFork = AsyncComputePass;
		}

		FRDGPassHandle CurrentGraphicsJoinPassHandle;

		for (FRDGPassHandle PassHandle = EpiloguePassHandle - 1; PassHandle > ProloguePassHandle; --PassHandle)
		{
			FRDGPass* AsyncComputePass = Passes[PassHandle];

			if (!AsyncComputePass->IsAsyncCompute() || AsyncComputePass->bCulled)
			{
				continue;
			}

			FRDGPassHandle CrossPipelineConsumer;

			// Cross pipeline consumers are sorted. Find the earliest consumer that isn't culled.
			for (FRDGPassHandle ConsumerHandle : AsyncComputePass->CrossPipelineConsumers)
			{
				FRDGPass* Consumer = Passes[ConsumerHandle];

				if (!Consumer->bCulled)
				{
					CrossPipelineConsumer = ConsumerHandle;
					break;
				}
			}

			FRDGPassHandle GraphicsJoinPassHandle = FRDGPassHandle::Min(CrossPipelineConsumer, FRDGPassHandle::Min(CurrentGraphicsJoinPassHandle, EpiloguePassHandle));
			FRDGPass* GraphicsJoinPass = Passes[GraphicsJoinPassHandle];

			AsyncComputePass->GraphicsJoinPass = GraphicsJoinPassHandle;

			if (!bAsyncComputeTransientAliasing)
			{
				AsyncComputePass->ResourcesToEnd.Reset();
				Passes[GraphicsJoinPass->EpilogueBarrierPass]->ResourcesToEnd.Add(AsyncComputePass);
			}

			if (CurrentGraphicsJoinPassHandle != GraphicsJoinPassHandle)
			{
				CurrentGraphicsJoinPassHandle = GraphicsJoinPassHandle;

				FRDGBarrierBatchBegin& EpilogueBarriersToBeginForGraphics = AsyncComputePass->GetEpilogueBarriersToBeginForGraphics(Allocators.Transition, TransitionCreateQueue);

				const bool bSeparateTransitionNeeded = false;

				AsyncComputePass->bAsyncComputeEnd = 1;
				EpilogueBarriersToBeginForGraphics.SetUseCrossPipelineFence(bSeparateTransitionNeeded);

				GraphicsJoinPass->bGraphicsJoin = 1;
				GraphicsJoinPass->GetPrologueBarriersToEnd(Allocators.Transition).AddDependency(&EpilogueBarriersToBeginForGraphics);
			}
		}
	}

#if WITH_RHI_BREADCRUMBS
	// Attach the RDG breadcrumb nodes to the current top-of-stack RHI immediate breadcrumb,
	// Also unlink them from each other.
	RHICmdList.AttachBreadcrumbSubTree(GetBreadcrumbAllocator(), LocalBreadcrumbList);
#endif
}

///////////////////////////////////////////////////////////////////////////////////////////////////

void FRDGBuilder::LaunchAsyncSetupQueueTask()
{
	check(!bCompiling);

	if (AsyncSetupQueue.LastTask.IsCompleted())
	{
		AsyncSetupQueue.LastTask = UE::Tasks::Launch(UE_SOURCE_LOCATION, [this]() mutable
		{
			ProcessAsyncSetupQueue();

		}, AsyncSetupQueue.LastTask, UE::Tasks::ETaskPriority::High);
	}
}

void FRDGBuilder::ProcessAsyncSetupQueue()
{
	SCOPED_NAMED_EVENT_TCHAR("FRDGBuilder::ProcessAsyncSetupQueue", FColor::Magenta);
	FRDGAllocatorScope AllocatorScope(Allocators.Task);

	while (true)
	{
		AsyncSetupQueue.Mutex.Lock();
		TArray<FAsyncSetupOp, FRDGArrayAllocator> PoppedOps = MoveTemp(AsyncSetupQueue.Ops);
		AsyncSetupQueue.Mutex.Unlock();

		if (PoppedOps.IsEmpty())
		{
			break;
		}

		for (FAsyncSetupOp Op : PoppedOps)
		{
			switch (Op.GetType())
			{
			case FAsyncSetupOp::EType::SetupPassResources:
				SetupPassResources(Op.Pass);
				break;

			case FAsyncSetupOp::EType::CullRootBuffer:
				AddCullRootBuffer(Op.Buffer);
				break;

			case FAsyncSetupOp::EType::CullRootTexture:
				AddCullRootTexture(Op.Texture);
				break;

			case FAsyncSetupOp::EType::ReservedBufferCommit:
				ensureMsgf(!Op.Buffer->AccessModeState.IsExternalAccess(), TEXT("Buffer %s has a pending reserved commit of %" UINT64_FMT " bytes but is marked for external access! The commit will be ignored!"), Op.Buffer->Name, Op.Payload);
				Op.Buffer->PendingCommitSize = Op.Payload;
				break;
			}
		}
	}
}

void FRDGBuilder::FlushSetupQueue()
{
	if (AsyncSetupQueue.bEnabled)
	{
		LaunchAsyncSetupQueueTask();
	}
}

///////////////////////////////////////////////////////////////////////////////////////////////////

void FRDGBuilder::WaitForParallelSetupTasks(ERDGSetupTaskWaitPoint WaitPoint)
{
	const auto WaitForTasksLambda = [this] (ERDGSetupTaskWaitPoint WaitPoint)
	{
		if (auto& Tasks = ParallelSetup.Tasks[(int32)WaitPoint]; !Tasks.IsEmpty())
		{
			UE::Tasks::Wait(Tasks);
			Tasks.Reset();
		}
	};

	switch (WaitPoint)
	{
	case ERDGSetupTaskWaitPoint::Execute:
		WaitForTasksLambda(ERDGSetupTaskWaitPoint::Execute);
		[[fallthrough]]; // Also flush any compile tasks that might have been added after the compile wait point.
	case ERDGSetupTaskWaitPoint::Compile:
		WaitForTasksLambda(ERDGSetupTaskWaitPoint::Compile);
	}
}

///////////////////////////////////////////////////////////////////////////////////////////////////

void FRDGBuilder::Execute()
{
	CSV_SCOPED_TIMING_STAT_EXCLUSIVE(RDG);
	RHI_BREADCRUMB_EVENT_F(RHICmdList, "RenderGraphExecute", "RenderGraphExecute - %s", BuilderName);

#if WITH_RHI_BREADCRUMBS
	check(LocalCurrentBreadcrumb == FRHIBreadcrumbNode::Sentinel);
	LocalCurrentBreadcrumb = RHICmdList.GetCurrentBreadcrumbRef();

#if WITH_ADDITIONAL_CRASH_CONTEXTS
	FScopedAdditionalCrashContextProvider CrashContext([&RHICmdList = RHICmdList](FCrashContextExtendedWriter& Writer)
	{
		if (FRHIBreadcrumbNode* CurrentBreadcrumb = RHICmdList.GetCurrentBreadcrumbRef())
		{
			CurrentBreadcrumb->WriteCrashData(Writer, TEXT("RDGExecute_RenderThread"));
		}
	});
#endif // WITH_ADDITIONAL_CRASH_CONTEXTS

#endif // WITH_RHI_BREADCRUMBS


	GRDGTransientResourceAllocator.ReleasePendingDeallocations();

	FlushAccessModeQueue();

	// Create the epilogue pass at the end of the graph just prior to compilation.
	EpiloguePass = SetupEmptyPass(Passes.Allocate<FRDGSentinelPass>(Allocators.Root, RDG_EVENT_NAME("Graph Epilogue")));

	const FRDGPassHandle ProloguePassHandle = GetProloguePassHandle();
	const FRDGPassHandle EpiloguePassHandle = GetEpiloguePassHandle();

	UE::Tasks::FTask CollectPassBarriersTask;
	UE::Tasks::FTask CreateViewsTask;

	IF_RDG_ENABLE_DEBUG(UserValidation.ValidateExecuteBegin());
	IF_RDG_ENABLE_DEBUG(GRDGAllowRHIAccess.store(true, std::memory_order_relaxed));

	FCollectResourceContext CollectResourceContext;

	bCompiling = true;

	if (!IsImmediateMode())
	{
		BeginFlushResourcesRHI();
		WaitForParallelSetupTasks(ERDGSetupTaskWaitPoint::Compile);
		AsyncSetupQueue.LastTask.Wait();
		AsyncSetupQueue.LastTask = {};
		ProcessAsyncSetupQueue();

		const int32 ParallelCompileResourceThreshold = 32;
		const int32 NumBuffers   = Buffers.Num();
		const int32 NumTextures  = Textures.Num();
		const int32 NumExternalBuffers = ExternalBuffers.Num();
		const int32 NumExternalTextures = ExternalTextures.Num();
		const int32 NumTransientBuffers = bSupportsTransientBuffers ? (NumBuffers - NumExternalBuffers) : 0;
		const int32 NumTransientTextures = bSupportsTransientTextures ? (NumTextures - NumExternalTextures) : 0;
		const int32 NumPooledTextures = NumTextures - NumTransientTextures;
		const int32 NumPooledBuffers = NumBuffers - NumTransientBuffers;
		const int32 NumUniformBuffers = UniformBuffers.Num();

		// Pre-allocate containers.
		{
			CollectResourceContext.TransientResources.Reserve(NumTransientBuffers + NumTransientTextures);
			CollectResourceContext.PooledTextures.Reserve(bSupportsTransientTextures ? NumExternalTextures : NumTextures);
			CollectResourceContext.PooledBuffers.Reserve(bSupportsTransientBuffers ? NumExternalBuffers : NumBuffers);
			CollectResourceContext.UniformBuffers.Reserve(UniformBuffers.Num());
			CollectResourceContext.Views.Reserve(Views.Num());
			CollectResourceContext.UniformBufferMap.Init(true, UniformBuffers.Num());
			CollectResourceContext.ViewMap.Init(true, Views.Num());

			PooledBufferOwnershipMap.Reserve(NumPooledBuffers);
			PooledTextureOwnershipMap.Reserve(NumPooledTextures);
			ActivePooledTextures.Reserve(NumPooledTextures);
			ActivePooledBuffers.Reserve(NumPooledBuffers);
			EpilogueResourceAccesses.Reserve(NumTextures + NumBuffers);

			ProloguePass->NumTransitionsToReserve = NumPooledBuffers + NumPooledTextures;
		}

		const UE::Tasks::ETaskPriority TaskPriority = UE::Tasks::ETaskPriority::High;

		const bool bParallelCompileBuffers = NumBuffers > ParallelCompileResourceThreshold;
		const bool bParallelCompileTextures = NumTextures > ParallelCompileResourceThreshold;
		const bool bParallelCompileResources = bParallelCompileBuffers || bParallelCompileTextures;

		UE::Tasks::FTask BufferNumElementsCallbacksTask = AddSetupTask([this]
		{
			FinalizeDescs();

		}, TaskPriority, bParallelCompileBuffers && !NumElementsCallbackBuffers.IsEmpty());

		UE::Tasks::FTask PrepareCollectResourcesTask = AddSetupTask([this]
		{
			SCOPED_NAMED_EVENT_TEXT("FRDGBuilder::PrepareCollectResources", FColor::Magenta);

			Buffers.Enumerate([&] (FRDGBuffer* Buffer)
			{
				Buffer->LastPasses = {};

				if (Buffer->ResourceRHI || Buffer->bQueuedForUpload)
				{
					Buffer->bCollectForAllocate = false;
				}

				if (Buffer->TransientBuffer || (!Buffer->ResourceRHI && IsTransient(Buffer)))
				{
					Buffer->bTransient = true;
				}
			});

			Textures.Enumerate([&] (FRDGTexture* Texture)
			{
				Texture->LastPasses = {};

				if (Texture->ResourceRHI)
				{
					Texture->bCollectForAllocate = false;
				}

				if (Texture->TransientTexture || (!Texture->ResourceRHI && IsTransient(Texture)))
				{
					Texture->bTransient = true;
				}
			});

		}, TaskPriority, bParallelCompileResources);

		UE::Tasks::FTaskEvent AllocateUploadBuffersTask{ UE_SOURCE_LOCATION };

		UE::Tasks::FTask SubmitBufferUploadsTask = AddCommandListSetupTask([this, AllocateUploadBuffersTask] (FRHICommandList& RHICmdListTask) mutable
		{
			SubmitBufferUploads(RHICmdListTask, &AllocateUploadBuffersTask);

		}, BufferNumElementsCallbacksTask, TaskPriority, !UploadedBuffers.IsEmpty());

		Compile();

		CollectPassBarriersTask = AddSetupTask([this]
		{
			CompilePassBarriers();
			CollectPassBarriers();

		}, TaskPriority, bParallelCompileResources);

		if (ParallelExecute.IsEnabled())
		{
			AddSetupTask([this, QueryBatchData = RHICmdList.GetQueryBatchData(RQT_AbsoluteTime)]
			{
				SetupParallelExecute(QueryBatchData);

			}, TaskPriority);
		}

		UE::Tasks::FTask AllocatePooledBuffersTask;
		UE::Tasks::FTask AllocatePooledTexturesTask;

		{
			SCOPE_CYCLE_COUNTER(STAT_RDG_CollectResourcesTime);
			CSV_SCOPED_TIMING_STAT_EXCLUSIVE(RDG_CollectResources);
			SCOPED_NAMED_EVENT_TEXT("FRDGBuilder::CollectResources", FColor::Magenta);

			PrepareCollectResourcesTask.Wait();

			EnumerateExtendedLifetimeResources(Textures, [](FRDGTexture* Texture)
			{
				++Texture->ReferenceCount;
			});

			EnumerateExtendedLifetimeResources(Buffers, [](FRDGBuffer* Buffer)
			{
				++Buffer->ReferenceCount;
			});

			// Null out any culled external resources so that the reference is freed up.

			for (const auto& Pair : ExternalTextures)
			{
				FRDGTexture* Texture = Pair.Value;

				if (Texture->IsCulled())
				{
					CollectDeallocateTexture(CollectResourceContext, ERHIPipeline::Graphics, ProloguePassHandle, Texture, 0);
				}
			}

			for (const auto& Pair : ExternalBuffers)
			{
				FRDGBuffer* Buffer = Pair.Value;

				if (Buffer->IsCulled())
				{
					CollectDeallocateBuffer(CollectResourceContext, ERHIPipeline::Graphics, ProloguePassHandle, Buffer, 0);
				}
			}

			for (FRDGPassHandle PassHandle = ProloguePassHandle; PassHandle <= EpiloguePassHandle; ++PassHandle)
			{
				FRDGPass* Pass = Passes[PassHandle];

				if (!Pass->bCulled)
				{
					CollectAllocations(CollectResourceContext, Pass);
					CollectDeallocations(CollectResourceContext, Pass);
				}
			}

			EnumerateExtendedLifetimeResources(Textures, [&](FRDGTextureRef Texture)
			{
				CollectDeallocateTexture(CollectResourceContext, ERHIPipeline::Graphics, EpiloguePassHandle, Texture, 1);
			});

			EnumerateExtendedLifetimeResources(Buffers, [&](FRDGBufferRef Buffer)
			{
				CollectDeallocateBuffer(CollectResourceContext, ERHIPipeline::Graphics, EpiloguePassHandle, Buffer, 1);
			});

			BufferNumElementsCallbacksTask.Wait();

			AllocatePooledBuffersTask = AddCommandListSetupTask([this, PooledBuffers = MoveTemp(CollectResourceContext.PooledBuffers)] (FRHICommandListBase& RHICmdListTask)
			{
				AllocatePooledBuffers(RHICmdListTask, PooledBuffers);

			}, AllocateUploadBuffersTask, TaskPriority, bParallelCompileBuffers);

			AllocatePooledTexturesTask = AddCommandListSetupTask([this, PooledTextures = MoveTemp(CollectResourceContext.PooledTextures)] (FRHICommandListBase& RHICmdListTask)
			{
				AllocatePooledTextures(RHICmdListTask, PooledTextures);

			}, TaskPriority, bParallelCompileTextures);

			AllocateTransientResources(MoveTemp(CollectResourceContext.TransientResources));

			AddSetupTask([this]
			{
				FinalizeResources();

			}, MakeArrayView<UE::Tasks::FTask>({ CollectPassBarriersTask, AllocatePooledBuffersTask, AllocatePooledTexturesTask }), TaskPriority, bParallelCompileResources);

			CreateViewsTask = AddCommandListSetupTask([this, InViews = MoveTemp(CollectResourceContext.Views)] (FRHICommandListBase& RHICmdListTask)
			{
				CreateViews(RHICmdListTask, InViews);

			}, MakeArrayView<UE::Tasks::FTask>({ AllocatePooledBuffersTask, AllocatePooledTexturesTask, SubmitBufferUploadsTask}), TaskPriority, bParallelCompileResources);

			if (TransientResourceAllocator)
			{
#if RDG_ENABLE_TRACE
				TransientResourceAllocator->Flush(RHICmdList, Trace.IsEnabled() ? &Trace.TransientAllocationStats : nullptr);
#else
				TransientResourceAllocator->Flush(RHICmdList);
#endif
			}
		}

		AddSetupTask([this, InUniformBuffers = MoveTemp(CollectResourceContext.UniformBuffers)]
		{
			CreateUniformBuffers(InUniformBuffers);

		}, CreateViewsTask, TaskPriority, NumUniformBuffers >= ParallelCompileResourceThreshold); // Uniform buffer creation require views to be valid.

		AllocatePooledBuffersTask.Wait();
		AllocatePooledTexturesTask.Wait();
	}
	else
	{
		SubmitBufferUploads(RHICmdList);
		FinalizeResources();
	}

	EndFlushResourcesRHI();
	WaitForParallelSetupTasks(ERDGSetupTaskWaitPoint::Execute);

	IF_RDG_ENABLE_DEBUG(GRDGAllowRHIAccess.store(ParallelExecute.IsEnabled(), std::memory_order_relaxed));
	IF_RDG_ENABLE_TRACE(Trace.OutputGraphBegin());

	bCompiling = false;

	ERHIPipeline OriginalPipeline = RHICmdList.GetPipeline();
	if (!IsImmediateMode())
	{
		SCOPED_NAMED_EVENT_TEXT("FRDGBuilder::ExecutePasses", FColor::Magenta);
		SCOPE_CYCLE_COUNTER(STAT_RDG_ExecuteTime);
		CSV_SCOPED_TIMING_STAT_EXCLUSIVE(RDG_Execute);

		if (ParallelExecute.IsEnabled())
		{
			// Launch a task to gather and launch dispatch pass tasks.
			if (!DispatchPasses.IsEmpty())
			{
				ParallelExecute.TasksAwait->AddPrerequisites(UE::Tasks::Launch(UE_SOURCE_LOCATION, [this]
				{
					FTaskTagScope TagScope(ETaskTag::EParallelRenderingThread);
					SetupDispatchPassExecute();

				}, UE::Tasks::ETaskPriority::High));
			}

			// Launch a task to absorb the cost of waking up threads and avoid stalling the render thread.
			ParallelExecute.TasksAwait->AddPrerequisites(UE::Tasks::Launch(UE_SOURCE_LOCATION, [this]
			{
				ParallelExecute.DispatchTaskEventAwait->Trigger();

				if (ParallelExecute.DispatchTaskEventAsync)
				{
					ParallelExecute.DispatchTaskEventAsync->Trigger();
	
					UE::Tasks::FTaskEvent Event(UE_SOURCE_LOCATION);
					Event.AddPrerequisites(MakeArrayView<UE::Tasks::FTask>({ *ParallelExecute.TasksAsync, FParallelExecute::LastAsyncExecuteTask }));
					Event.Trigger();
	
					FParallelExecute::LastAsyncExecuteTask = Event;
				}
			}));
		}
		else
		{
			SetupDispatchPassExecute();
		}

		FRDGPass* PrevSerialPass = nullptr;
		TArray<FRHICommandListImmediate::FQueuedCommandList, FRDGArrayAllocator> QueuedCmdLists;

		auto FlushParallel = [&]()
		{
			if (QueuedCmdLists.Num())
			{
				RHICmdList.QueueAsyncCommandListSubmit(QueuedCmdLists);
				QueuedCmdLists.Reset();
			}
		};

		if (bInitialAsyncComputeFence)
		{
			// Insert a manual fence from async compute to graphics to synchronize any all pipeline external access resources from the last run.
			RHICmdList.Transition({}, ERHIPipeline::AsyncCompute, ERHIPipeline::Graphics);
		}

		for (FRDGPassHandle PassHandle = ProloguePassHandle; PassHandle <= EpiloguePassHandle; ++PassHandle)
		{
			FRDGPass* Pass = Passes[PassHandle];

			if (Pass->bCulled)
			{
				continue;
			}

			if (Pass->bParallelExecute)
			{
				if (PrevSerialPass)
				{
					PopPreScopes(RHICmdList, PrevSerialPass);
					PrevSerialPass = nullptr;
				}

				bool bDispatchAfterExecute = false;

				if (Pass->bDispatchPass)
				{
					FRDGDispatchPass* DispatchPass = static_cast<FRDGDispatchPass*>(Pass);
					DispatchPass->CommandListsEvent.Wait();
					QueuedCmdLists.Append(MoveTemp(DispatchPass->CommandLists));

					bDispatchAfterExecute = Pass->bDispatchAfterExecute;
				}
				else if (Pass->bParallelExecuteBegin)
				{
					FParallelPassSet& ParallelPassSet = ParallelExecute.ParallelPassSets[Pass->ParallelPassSetIndex];
					check(ParallelPassSet.CmdList != nullptr);
					QueuedCmdLists.Add(ParallelPassSet);

					bDispatchAfterExecute = ParallelPassSet.bDispatchAfterExecute;
				}

				if (bDispatchAfterExecute)
				{
					FlushParallel();
					RHICmdList.ImmediateFlush(EImmediateFlushType::DispatchToRHIThread);
				};
			}
			else
			{
				if (!PrevSerialPass)
				{
					FlushParallel();
					PushPreScopes(RHICmdList, Pass);
				}

				PrevSerialPass = Pass;
				ExecuteSerialPass(RHICmdList, Pass);

				if (Pass->bDispatchAfterExecute)
				{
					RHICmdList.ImmediateFlush(EImmediateFlushType::DispatchToRHIThread);
				}

				if (GRDGDebugFlushGPU && !bSupportsAsyncCompute)
				{
					RHICmdList.SubmitAndBlockUntilGPUIdle();
				}
			}
		}

		if (PrevSerialPass)
		{
			PopPreScopes(RHICmdList, PrevSerialPass);
			PrevSerialPass = nullptr;
		}

		FlushParallel();
	}
	else
	{
		ExecuteSerialPass(RHICmdList, EpiloguePass);
	}

	RHICmdList.SwitchPipeline(OriginalPipeline);
	RHICmdList.SetStaticUniformBuffers({});

#if WITH_MGPU
	if (bForceCopyCrossGPU)
	{
		ForceCopyCrossGPU();
	}
#endif

	RHICmdList.SetTrackedAccess(EpilogueResourceAccesses);

	// Wait on the actual parallel execute tasks in the Execute call. This needs to be done before extraction of external resources to be consistent with non-parallel rendering.
	if (ParallelExecute.TasksAwait)
	{
		ParallelExecute.TasksAwait->Trigger();
		ParallelExecute.TasksAwait->Wait();
		ParallelExecute.TasksAwait.Reset();
	}

	for (const FExtractedTexture& ExtractedTexture : ExtractedTextures)
	{
		check(ExtractedTexture.Texture->RenderTarget);
		*ExtractedTexture.PooledTexture = ExtractedTexture.Texture->RenderTarget;
	}

	for (const FExtractedBuffer& ExtractedBuffer : ExtractedBuffers)
	{
		check(ExtractedBuffer.Buffer->PooledBuffer);
		*ExtractedBuffer.PooledBuffer = ExtractedBuffer.Buffer->PooledBuffer;
	}

	for (TUniqueFunction<void()>& Callback : PostExecuteCallbacks)
	{
		Callback();
	}
	PostExecuteCallbacks.Empty();

	IF_RDG_ENABLE_TRACE(Trace.OutputGraphEnd(*this));

	IF_RDG_ENABLE_DEBUG(UserValidation.ValidateExecuteEnd());
	IF_RDG_ENABLE_DEBUG(GRDGAllowRHIAccess.store(false, std::memory_order_relaxed));

#if RDG_STATS
	GRDGStatBufferCount += Buffers.Num();
	GRDGStatTextureCount += Textures.Num();
	GRDGStatViewCount += Views.Num();
	GRDGStatMemoryWatermark = FMath::Max(GRDGStatMemoryWatermark, Allocators.GetByteCount());
#endif

	RasterPassCount = 0;
	AsyncComputePassCount = 0;
}

///////////////////////////////////////////////////////////////////////////////////////////////////

void FRDGBuilder::MarkResourcesAsProduced(FRDGPass* Pass)
{
	const auto MarkAsProduced = [&](FRDGViewableResource* Resource)
	{
		Resource->bProduced = true;
	};

	const auto MarkAsProducedIfWritable = [&](FRDGViewableResource* Resource, ERHIAccess Access)
	{
		if (IsWritableAccess(Access))
		{
			Resource->bProduced = true;
		}
	};

	Pass->GetParameters().Enumerate([&](FRDGParameter Parameter)
	{
		switch (Parameter.GetType())
		{
		case UBMT_RDG_TEXTURE_UAV:
			if (FRDGTextureUAV* UAV = Parameter.GetAsTextureUAV())
			{
				MarkAsProduced(UAV->GetParent());
			}
		break;
		case UBMT_RDG_BUFFER_UAV:
			if (FRDGBufferUAV* UAV = Parameter.GetAsBufferUAV())
			{
				MarkAsProduced(UAV->GetParent());
			}
		break;
		case UBMT_RDG_TEXTURE_ACCESS:
		{
			if (FRDGTextureAccess TextureAccess = Parameter.GetAsTextureAccess())
			{
				MarkAsProducedIfWritable(TextureAccess.GetTexture(), TextureAccess.GetAccess());
			}
		}
		break;
		case UBMT_RDG_TEXTURE_ACCESS_ARRAY:
		{
			const FRDGTextureAccessArray& TextureAccessArray = Parameter.GetAsTextureAccessArray();

			for (FRDGTextureAccess TextureAccess : TextureAccessArray)
			{
				MarkAsProducedIfWritable(TextureAccess.GetTexture(), TextureAccess.GetAccess());
			}
		}
		break;
		case UBMT_RDG_BUFFER_ACCESS:
			if (FRDGBufferAccess BufferAccess = Parameter.GetAsBufferAccess())
			{
				MarkAsProducedIfWritable(BufferAccess.GetBuffer(), BufferAccess.GetAccess());
			}
		break;
		case UBMT_RDG_BUFFER_ACCESS_ARRAY:
		{
			const FRDGBufferAccessArray& BufferAccessArray = Parameter.GetAsBufferAccessArray();

			for (FRDGBufferAccess BufferAccess : BufferAccessArray)
			{
				MarkAsProducedIfWritable(BufferAccess.GetBuffer(), BufferAccess.GetAccess());
			}
		}
		break;
		case UBMT_RENDER_TARGET_BINDING_SLOTS:
		{
			const FRenderTargetBindingSlots& RenderTargets = Parameter.GetAsRenderTargetBindingSlots();

			RenderTargets.Enumerate([&](FRenderTargetBinding RenderTarget)
			{
				MarkAsProduced(RenderTarget.GetTexture());

				if (FRDGTexture* ResolveTexture = RenderTarget.GetResolveTexture())
				{
					MarkAsProduced(ResolveTexture);
				}
			});

			const FDepthStencilBinding& DepthStencil = RenderTargets.DepthStencil;

			if (DepthStencil.GetDepthStencilAccess().IsAnyWrite())
			{
				MarkAsProduced(DepthStencil.GetTexture());
			}
		}
		break;
		}
	});
}

void FRDGBuilder::SetupPassDependencies(FRDGPass* Pass)
{
	bool bIsCullRootProducer = false;

	for (auto& PassState : Pass->TextureStates)
	{
		FRDGTextureRef Texture = PassState.Texture;
		auto& LastProducers = Texture->LastProducers;

		Texture->ReferenceCount += PassState.ReferenceCount;

		for (uint32 Index = 0, Count = LastProducers.Num(); Index < Count; ++Index)
		{
			const FRDGSubresourceState* SubresourceState = PassState.State[Index];

			if (!SubresourceState)
			{
				continue;
			}

			FRDGProducerState ProducerState;
			ProducerState.Pass = Pass;
			ProducerState.Access = SubresourceState->Access;
			ProducerState.NoUAVBarrierHandle = SubresourceState->NoUAVBarrierFilter.GetUniqueHandle();

			bIsCullRootProducer |= AddCullingDependency(LastProducers[Index], ProducerState, Pass->Pipeline) && Texture->IsCullRoot();
		}
	}

	for (auto& PassState : Pass->BufferStates)
	{
		FRDGBufferRef Buffer = PassState.Buffer;
		const FRDGSubresourceState& SubresourceState = PassState.State;

		Buffer->ReferenceCount += PassState.ReferenceCount;

		FRDGProducerState ProducerState;
		ProducerState.Pass = Pass;
		ProducerState.Access = SubresourceState.Access;
		ProducerState.NoUAVBarrierHandle = SubresourceState.NoUAVBarrierFilter.GetUniqueHandle();

		bIsCullRootProducer |= AddCullingDependency(Buffer->LastProducer, ProducerState, Pass->Pipeline) && Buffer->IsCullRoot();
	}

	const bool bCullPasses = GRDGCullPasses > 0;
	Pass->bCulled = bCullPasses;

	if (bCullPasses && (bIsCullRootProducer || Pass->bHasExternalOutputs || EnumHasAnyFlags(Pass->Flags, ERDGPassFlags::NeverCull)))
	{
		CullPassStack.Emplace(Pass);

		FlushCullStack();
	}
}

void FRDGBuilder::SetupPassResources(FRDGPass* Pass)
{
	const FRDGParameterStruct PassParameters = Pass->GetParameters();
	const FRDGPassHandle PassHandle = Pass->Handle;
	const ERDGPassFlags PassFlags = Pass->Flags;
	const ERHIPipeline PassPipeline = Pass->Pipeline;

	bool bRenderPassOnlyWrites = true;

	const auto TryAddView = [&](FRDGViewRef View)
	{
		if (View && View->LastPass != PassHandle)
		{
			View->LastPass = PassHandle;
			Pass->Views.Add(View->Handle);
		}
	};

	Pass->Views.Reserve(PassParameters.GetBufferParameterCount() + PassParameters.GetTextureParameterCount());
	Pass->TextureStates.Reserve(PassParameters.GetTextureParameterCount() + (PassParameters.HasRenderTargets() ? (MaxSimultaneousRenderTargets + 1) : 0));
	EnumerateTextureAccess(PassParameters, PassFlags, [&](FRDGViewRef TextureView, FRDGTextureRef Texture, ERHIAccess Access, ERDGTextureAccessFlags AccessFlags, FRDGTextureSubresourceRange Range)
	{
		TryAddView(TextureView);

		if (Texture->AccessModeState.IsExternalAccess() && !Pass->bExternalAccessPass)
		{
			// Resources in external access mode are expected to remain in the same state and are ignored by the graph.
			// As only External | Extracted resources can be set as external by the user, the graph doesn't need to track
			// them any more for culling / transition purposes. Validation checks that these invariants are true.
			IF_RDG_ENABLE_DEBUG(UserValidation.ValidateExternalAccess(Texture, Access, Pass));
			return;
		}

		const FRDGViewHandle NoUAVBarrierHandle = GetHandleIfNoUAVBarrier(TextureView);
		const EResourceTransitionFlags TransitionFlags = GetTextureViewTransitionFlags(TextureView, Texture);

		FRDGPass::FTextureState* PassState;

		if (Texture->LastPasses[PassPipeline] != PassHandle)
		{
			Texture->LastPasses[PassPipeline] = PassHandle;
			Texture->PassStateIndex = Pass->TextureStates.Num();

			PassState = &Pass->TextureStates.Emplace_GetRef(Texture);
		}
		else
		{
			PassState = &Pass->TextureStates[Texture->PassStateIndex];
		}

		PassState->ReferenceCount++;

		EnumerateSubresourceRange(PassState->State, Texture->Layout, Range, [&](FRDGSubresourceState*& State)
		{
			if (!State)
			{
				State = AllocSubresource();
			}

			IF_RDG_ENABLE_DEBUG(UserValidation.ValidateAddSubresourceAccess(Texture, *State, Access));

			State->Access = MakeValidAccess(State->Access, Access);
			State->Flags |= TransitionFlags;
			State->NoUAVBarrierFilter.AddHandle(NoUAVBarrierHandle);
			State->SetPass(PassPipeline, PassHandle);
		});

		if (IsWritableAccess(Access))
		{
			bRenderPassOnlyWrites &= EnumHasAnyFlags(AccessFlags, ERDGTextureAccessFlags::RenderTarget);

			// When running in parallel this is set via MarkResourcesAsProduced. We also can't touch this as its a bitfield and not atomic.
			if (!AsyncSetupQueue.bEnabled)
			{
				Texture->bProduced = true;
			}
		}
	});

	Pass->BufferStates.Reserve(PassParameters.GetBufferParameterCount());
	EnumerateBufferAccess(PassParameters, PassFlags, [&](FRDGViewRef BufferView, FRDGBufferRef Buffer, ERHIAccess Access)
	{
		TryAddView(BufferView);

		if (Buffer->AccessModeState.IsExternalAccess() && !Pass->bExternalAccessPass)
		{
			// Resources in external access mode are expected to remain in the same state and are ignored by the graph.
			// As only External | Extracted resources can be set as external by the user, the graph doesn't need to track
			// them any more for culling / transition purposes. Validation checks that these invariants are true.
			IF_RDG_ENABLE_DEBUG(UserValidation.ValidateExternalAccess(Buffer, Access, Pass));
			return;
		}

		const FRDGViewHandle NoUAVBarrierHandle = GetHandleIfNoUAVBarrier(BufferView);

		FRDGPass::FBufferState* PassState;

		if (Buffer->LastPasses[PassPipeline] != PassHandle)
		{
			Buffer->LastPasses[PassPipeline] = PassHandle;
			Buffer->PassStateIndex = Pass->BufferStates.Num();

			PassState = &Pass->BufferStates.Emplace_GetRef(Buffer);
			PassState->State.ReservedCommitHandle = AcquireReservedCommitHandle(Buffer);
			PassState->State.SetPass(PassPipeline, PassHandle);
		}
		else
		{
			PassState = &Pass->BufferStates[Buffer->PassStateIndex];
		}

		IF_RDG_ENABLE_DEBUG(UserValidation.ValidateAddSubresourceAccess(Buffer, PassState->State, Access));

		PassState->ReferenceCount++;
		PassState->State.Access = MakeValidAccess(PassState->State.Access, Access);
		PassState->State.NoUAVBarrierFilter.AddHandle(NoUAVBarrierHandle);

		if (IsWritableAccess(Access))
		{
			bRenderPassOnlyWrites = false;

			// When running in parallel this is set via MarkResourcesAsProduced. We also can't touch this as its a bitfield and not atomic.
			if (!AsyncSetupQueue.bEnabled)
			{
				Buffer->bProduced = true;
			}
		}
	});

	Pass->bEmptyParameters = !Pass->TextureStates.Num() && !Pass->BufferStates.Num();
	Pass->bRenderPassOnlyWrites = bRenderPassOnlyWrites;
	Pass->bHasExternalOutputs = PassParameters.HasExternalOutputs();

	Pass->UniformBuffers.Reserve(PassParameters.GetUniformBufferParameterCount());
	PassParameters.EnumerateUniformBuffers([&](FRDGUniformBufferBinding UniformBuffer)
	{
		Pass->UniformBuffers.Emplace(UniformBuffer.GetUniformBuffer()->Handle);
	});

	if (AsyncSetupQueue.bEnabled)
	{
		SetupPassDependencies(Pass);

		for (FRDGPass::FExternalAccessOp Op : Pass->ExternalAccessOps)
		{
			Op.Resource->AccessModeState.ActiveMode = Op.Mode;
		}
	}
}

void FRDGBuilder::SetupPassInternals(FRDGPass* Pass)
{
	const FRDGPassHandle PassHandle = Pass->Handle;
	const ERDGPassFlags PassFlags = Pass->Flags;
	const ERHIPipeline PassPipeline = Pass->Pipeline;

	Pass->PrologueBarrierPass = PassHandle;
	Pass->EpilogueBarrierPass = PassHandle;
	Pass->ResourcesToBegin.Add(Pass);
	Pass->ResourcesToEnd.Add(Pass);

	AsyncComputePassCount += EnumHasAnyFlags(PassFlags, ERDGPassFlags::AsyncCompute) ? 1 : 0;
	RasterPassCount += EnumHasAnyFlags(PassFlags, ERDGPassFlags::Raster) ? 1 : 0;

#if WITH_MGPU
	Pass->GPUMask = RHICmdList.GetGPUMask();
#endif

#if RDG_STATS
	GRDGStatPassCount++;
#endif

	Pass->Scope = ScopeState.Current;

#if RDG_ENABLE_DEBUG
	if (GRDGValidation != 0 && Pass->Scope)
	{
		Pass->FullPathIfDebug = Pass->Scope->GetFullPath(Pass->Name);
	}
#endif
}

void FRDGBuilder::SetupAuxiliaryPasses(FRDGPass* Pass)
{
	if (IsImmediateMode() && !Pass->bSentinel)
	{
		SCOPED_NAMED_EVENT(FRDGBuilder_ExecutePass, FColor::Emerald);
		RDG_ALLOW_RHI_ACCESS_SCOPE();

		for (auto& PassState : Pass->TextureStates)
		{
			FRDGTexture* Texture = PassState.Texture;

			if (Texture->ResourceRHI)
			{
				Texture->bCollectForAllocate = false;
			}

			for (FRDGSubresourceState*& SubresourceState : Texture->State)
			{
				if (!SubresourceState)
				{
					SubresourceState = &PrologueSubresourceState;
				}
			}

			PassState.MergeState = PassState.State;
		}

		for (auto& PassState : Pass->BufferStates)
		{
			FRDGBuffer* Buffer = PassState.Buffer;

			if (Buffer->ResourceRHI || Buffer->bQueuedForUpload)
			{
				Buffer->bCollectForAllocate = false;
			}

			if (!Buffer->State)
			{
				Buffer->State = &PrologueSubresourceState;
			}

			PassState.MergeState = &PassState.State;
		}

		check(!EnumHasAnyFlags(Pass->Pipeline, ERHIPipeline::AsyncCompute));

		FCollectResourceContext Context;
		SubmitBufferUploads(RHICmdList);
		CompilePassOps(Pass);
		FinalizeDescs();
		CollectAllocations(Context, Pass);
		AllocatePooledTextures(RHICmdList, Context.PooledTextures);
		AllocatePooledBuffers(RHICmdList, Context.PooledBuffers);
		CreateViews(RHICmdList, Context.Views);
		CreateUniformBuffers(Context.UniformBuffers);
		CollectPassBarriers(Pass->Handle);
		CreatePassBarriers();
		SetupDispatchPassExecute();
		ExecuteSerialPass(RHICmdList, Pass);
	}

	IF_RDG_ENABLE_DEBUG(VisualizePassOutputs(Pass));

#if RDG_DUMP_RESOURCES
	DumpResourcePassOutputs(Pass);
#endif
}

FRDGPass* FRDGBuilder::SetupParameterPass(FRDGPass* Pass)
{
	IF_RDG_ENABLE_DEBUG(UserValidation.ValidateAddPass(Pass));
	CSV_SCOPED_TIMING_STAT_EXCLUSIVE_CONDITIONAL(RDGBuilder_SetupPass, GRDGVerboseCSVStats != 0);

	RDG_EVENT_SCOPE_CONDITIONAL_NAME(*this, ScopeState.ScopeMode == ERDGScopeMode::AllEventsAndPassNames, Pass->GetEventName());

	SetupPassInternals(Pass);

	if (AsyncSetupQueue.bEnabled)
	{
		MarkResourcesAsProduced(Pass);
		AsyncSetupQueue.Push(FAsyncSetupOp::SetupPassResources(Pass));
	}
	else
	{
		SetupPassResources(Pass);
	}

	SetupAuxiliaryPasses(Pass);
	return Pass;
}

FRDGPass* FRDGBuilder::SetupEmptyPass(FRDGPass* Pass)
{
	IF_RDG_ENABLE_DEBUG(UserValidation.ValidateAddPass(Pass));
	CSV_SCOPED_TIMING_STAT_EXCLUSIVE_CONDITIONAL(RDGBuilder_SetupPass, GRDGVerboseCSVStats != 0);

	Pass->bEmptyParameters = true;
	SetupPassInternals(Pass);
	SetupAuxiliaryPasses(Pass);
	return Pass;
}

void FRDGBuilder::CompilePassOps(FRDGPass* Pass)
{
	if (!IsImmediateMode() && Pass->Scope)
	{
		for (FRDGScope* Current = Pass->Scope; Current; Current = Current->Parent)
		{
			if (!Current->CPUFirstPass)
			{
				Current->CPUFirstPass = Pass;
			}
			if (!Current->GPUFirstPass[Pass->Pipeline])
			{
				Current->GPUFirstPass[Pass->Pipeline] = Pass;
			}

			Current->CPULastPass = Pass;
			Current->GPULastPass[Pass->Pipeline] = Pass;
		}
	}
}

///////////////////////////////////////////////////////////////////////////////////////////////////

void FRDGBuilder::SubmitBufferUploads(FRHICommandList& RHICmdListUpload, UE::Tasks::FTaskEvent* AllocateUploadBuffersTask)
{
	SCOPED_NAMED_EVENT_TEXT("FRDGBuilder::SubmitBufferUploads", FColor::Magenta);
	RHI_BREADCRUMB_EVENT(RHICmdListUpload, "FRDGBuilder::SubmitBufferUploads");

	FRHICommandListScopedFence ScopedFence(RHICmdListUpload);

	{
		SCOPED_NAMED_EVENT_TEXT("Allocate", FColor::Magenta);
		UE::TScopeLock Lock(GRenderGraphResourcePool.Mutex);

		for (FUploadedBuffer& UploadedBuffer : UploadedBuffers)
		{
			FRDGBuffer* Buffer = UploadedBuffer.Buffer;
			if (!Buffer->ResourceRHI)
			{
				SetExternalPooledBufferRHI(Buffer, AllocatePooledBufferRHI(RHICmdListUpload, Buffer));
			}
		}
	}

	if (AllocateUploadBuffersTask)
	{
		AllocateUploadBuffersTask->Trigger();
	}

	if ( (RHICmdListUpload.NeedsExtraTransitions()) && UploadedBuffers.Num() > 1)
	{
		SCOPED_NAMED_EVENT_TEXT("Upload_Multiple", FColor::Magenta);
		
		// This is here because we are explicitly batching a series of transition for all the buffers and we don't the individual extra transitions in Lock/Unlock
		FRHICommandListScopedAllowExtraTransitions ScopedExtraTransitions(RHICmdListUpload, false);

		TSet<FRHIBuffer*, DefaultKeyFuncs<void*>, FConcurrentLinearSetAllocator> BuffersSet;
		TArray<FRHITransitionInfo, FConcurrentLinearArrayAllocator> CopyDestTransitionInfo;
		TArray<FRHITransitionInfo, FConcurrentLinearArrayAllocator> RevertTransitionInfo;
		
		BuffersSet.Reserve(UploadedBuffers.Num());
		CopyDestTransitionInfo.Reserve(UploadedBuffers.Num());
		RevertTransitionInfo.Reserve(UploadedBuffers.Num());
		const EResourceLockMode LockMode = RLM_WriteOnly;
		
		// Lock all buffers, copy the data and create the transitions info
		for (FUploadedBuffer& UploadedBuffer : UploadedBuffers)
		{
			FRDGBuffer* Buffer = UploadedBuffer.Buffer;

			if (UploadedBuffer.DataFillCallback)
			{
				FRHIBuffer* RHIBuffer = Buffer->GetRHIUnchecked();
				const uint32 DataSize = Buffer->Desc.GetSize();
				void* DestPtr = RHICmdListUpload.LockBuffer(RHIBuffer, 0, DataSize, LockMode);
				static_assert((LockMode == RLM_WriteOnly), "Transitions optimized only for RLM_WriteOnly");

				UploadedBuffer.DataFillCallback(DestPtr, DataSize);

				if (BuffersSet.Contains(RHIBuffer) == false)
				{
					BuffersSet.Add(RHIBuffer);
					CopyDestTransitionInfo.Push(FRHITransitionInfo(RHIBuffer, ERHIAccess::Unknown, ERHIAccess::CopyDest, EResourceTransitionFlags::IgnoreAfterState));
					RevertTransitionInfo.Push(FRHITransitionInfo(RHIBuffer, ERHIAccess::CopyDest, ERHIAccess::Unknown, EResourceTransitionFlags::IgnoreAfterState));
				}
			}
			else
			{
				if (UploadedBuffer.bUseDataCallbacks)
				{
					UploadedBuffer.Data = UploadedBuffer.DataCallback();
					UploadedBuffer.DataSize = UploadedBuffer.DataSizeCallback();
				}

				if (UploadedBuffer.Data && UploadedBuffer.DataSize)
				{
					FRHIBuffer* RHIBuffer = Buffer->GetRHIUnchecked();
					check(UploadedBuffer.DataSize <= Buffer->Desc.GetSize());
					void* DestPtr = RHICmdListUpload.LockBuffer(RHIBuffer, 0, UploadedBuffer.DataSize, LockMode);
					static_assert((LockMode == RLM_WriteOnly), "Transitions optimized only for RLM_WriteOnly");

					FMemory::Memcpy(DestPtr, UploadedBuffer.Data, UploadedBuffer.DataSize);

					if (BuffersSet.Contains(RHIBuffer) == false)
					{
						BuffersSet.Add(RHIBuffer);
						CopyDestTransitionInfo.Push(FRHITransitionInfo(RHIBuffer, ERHIAccess::Unknown, ERHIAccess::CopyDest, EResourceTransitionFlags::IgnoreAfterState));
						RevertTransitionInfo.Push(FRHITransitionInfo(RHIBuffer, ERHIAccess::CopyDest, ERHIAccess::Unknown, EResourceTransitionFlags::IgnoreAfterState));
					}
				}
			}
		}

		// Issue all COPY_DEST buffer transitions together
		RHICmdListUpload.TransitionInternal(CopyDestTransitionInfo, ERHITransitionCreateFlags::AllowDuringRenderPass);

		// Unlock all buffers
		for (FUploadedBuffer& UploadedBuffer : UploadedBuffers)
		{
			FRDGBuffer* Buffer = UploadedBuffer.Buffer;

			if (UploadedBuffer.DataFillCallback || (UploadedBuffer.Data && UploadedBuffer.DataSize))
			{
				RHICmdListUpload.UnlockBuffer(Buffer->GetRHIUnchecked());
			}
		}
		
		// Issue all Revert buffer transitions together
		RHICmdListUpload.TransitionInternal(RevertTransitionInfo, ERHITransitionCreateFlags::AllowDuringRenderPass);

		BuffersSet.Reset();
		CopyDestTransitionInfo.Reset();
		RevertTransitionInfo.Reset();
	}
	else
	{
		SCOPED_NAMED_EVENT_TEXT("Upload", FColor::Magenta);

		for (FUploadedBuffer& UploadedBuffer : UploadedBuffers)
		{
			FRDGBuffer* Buffer = UploadedBuffer.Buffer;

			if (UploadedBuffer.DataFillCallback)
			{
				const uint32 DataSize = Buffer->Desc.GetSize();
				void* DestPtr = RHICmdListUpload.LockBuffer(Buffer->GetRHIUnchecked(), 0, DataSize, RLM_WriteOnly);
				UploadedBuffer.DataFillCallback(DestPtr, DataSize);
				RHICmdListUpload.UnlockBuffer(Buffer->GetRHIUnchecked());
			}
			else
			{
				if (UploadedBuffer.bUseDataCallbacks)
				{
					UploadedBuffer.Data = UploadedBuffer.DataCallback();
					UploadedBuffer.DataSize = UploadedBuffer.DataSizeCallback();
				}

				if (UploadedBuffer.Data && UploadedBuffer.DataSize)
				{
					check(UploadedBuffer.DataSize <= Buffer->Desc.GetSize());
					void* DestPtr = RHICmdListUpload.LockBuffer(Buffer->GetRHIUnchecked(), 0, UploadedBuffer.DataSize, RLM_WriteOnly);
					FMemory::Memcpy(DestPtr, UploadedBuffer.Data, UploadedBuffer.DataSize);
					RHICmdListUpload.UnlockBuffer(Buffer->GetRHIUnchecked());
				}
			}
		}
	}

	UploadedBuffers.Reset();
}

///////////////////////////////////////////////////////////////////////////////////////////////////

void FRDGBuilder::SetupParallelExecute(TStaticArray<void*, MAX_NUM_GPUS> const& QueryBatchData)
{
	SCOPED_NAMED_EVENT(SetupParallelExecute, FColor::Emerald);
	FRDGAllocatorScope AllocatorScope(Allocators.Task);

	const bool bTaskModeAsyncAllowed = ParallelExecute.TaskMode == ERDGPassTaskMode::Async;

	TArray<FRDGPass*, TInlineAllocator<64, FRDGArrayAllocator>> ParallelPassCandidates;
	uint32 ParallelPassCandidatesWorkload = 0;
	bool bDispatchAfterExecute = false;
	bool bTaskModeAsync = bTaskModeAsyncAllowed;

	const auto FlushParallelPassCandidates = [&]()
	{
		if (ParallelPassCandidates.IsEmpty())
		{
			return;
		}

		int32 PassBeginIndex = 0;
		int32 PassEndIndex = ParallelPassCandidates.Num();

		// It's possible that the first pass is inside a merged RHI render pass region. If so, we must push it forward until after the render pass ends.
		if (const FRDGPass* FirstPass = ParallelPassCandidates[PassBeginIndex]; FirstPass->PrologueBarrierPass < FirstPass->Handle)
		{
			const FRDGPass* EpilogueBarrierPass = Passes[FirstPass->EpilogueBarrierPass];

			for (; PassBeginIndex < ParallelPassCandidates.Num(); ++PassBeginIndex)
			{
				if (ParallelPassCandidates[PassBeginIndex] == EpilogueBarrierPass)
				{
					++PassBeginIndex;
					break;
				}
			}
		}

		if (PassBeginIndex < PassEndIndex)
		{
			// It's possible that the last pass is inside a merged RHI render pass region. If so, we must push it backwards until after the render pass begins.
			if (FRDGPass* LastPass = ParallelPassCandidates.Last(); LastPass->EpilogueBarrierPass > LastPass->Handle)
			{
				FRDGPass* PrologueBarrierPass = Passes[LastPass->PrologueBarrierPass];

				while (PassEndIndex > PassBeginIndex)
				{
					if (ParallelPassCandidates[--PassEndIndex] == PrologueBarrierPass)
					{
						break;
					}
				}
			}
		}

		const int32 ParallelPassCandidateCount = PassEndIndex - PassBeginIndex;

		if (ParallelPassCandidateCount >= GRDGParallelExecutePassMin)
		{
			FRDGPass* PassBegin = ParallelPassCandidates[PassBeginIndex];
			PassBegin->bParallelExecuteBegin = 1;
			PassBegin->ParallelPassSetIndex = ParallelExecute.ParallelPassSets.Num();

			FRDGPass* PassEnd = ParallelPassCandidates[PassEndIndex - 1];
			PassEnd->bParallelExecuteEnd = 1;
			PassEnd->ParallelPassSetIndex = ParallelExecute.ParallelPassSets.Num();

			for (int32 PassIndex = PassBeginIndex; PassIndex < PassEndIndex; ++PassIndex)
			{
				ParallelPassCandidates[PassIndex]->bParallelExecute = 1;
			}

			FParallelPassSet& ParallelPassSet = ParallelExecute.ParallelPassSets.Emplace_GetRef();
			ParallelPassSet.Passes.Append(ParallelPassCandidates.GetData() + PassBeginIndex, ParallelPassCandidateCount);
			ParallelPassSet.bDispatchAfterExecute = bDispatchAfterExecute;
			ParallelPassSet.bTaskModeAsync = bTaskModeAsync;
		}

		ParallelPassCandidates.Reset();
		ParallelPassCandidatesWorkload = 0;
		bDispatchAfterExecute = false;
		bTaskModeAsync = bTaskModeAsyncAllowed;
	};

	ParallelExecute.TasksAwait.Emplace(UE_SOURCE_LOCATION);
	ParallelExecute.DispatchTaskEventAwait.Emplace(UE_SOURCE_LOCATION);

	if (bTaskModeAsyncAllowed)
	{
		ParallelExecute.TasksAsync.Emplace(UE_SOURCE_LOCATION);
		ParallelExecute.DispatchTaskEventAsync.Emplace(UE_SOURCE_LOCATION);
	}

	ParallelExecute.ParallelPassSets.Reserve(32);
	ParallelPassCandidates.Emplace(ProloguePass);

	for (FRDGPassHandle PassHandle = GetProloguePassHandle() + 1; PassHandle < GetEpiloguePassHandle(); ++PassHandle)
	{
		FRDGPass* Pass = Passes[PassHandle];

		if (Pass->bCulled)
		{
			continue;
		}

		if (Pass->TaskMode == ERDGPassTaskMode::Inline)
		{
			FlushParallelPassCandidates();
			continue;
		}

		if (Pass->bDispatchPass)
		{
			FlushParallelPassCandidates();

			Pass->bParallelExecuteBegin = 1;
			Pass->bParallelExecute      = 1;
			Pass->bParallelExecuteEnd   = 1;
			continue;
		}

		const bool bPassTaskModeAsync = Pass->TaskMode == ERDGPassTaskMode::Async;
		const bool bPassTaskModeThresholdReached = ParallelPassCandidatesWorkload >= (uint32)GRDGParallelExecutePassTaskModeThreshold && GRDGParallelExecutePassTaskModeThreshold != 0;

		if (bTaskModeAsyncAllowed && bTaskModeAsync != bPassTaskModeAsync && bPassTaskModeThresholdReached)
		{
			FlushParallelPassCandidates();
		}

		bTaskModeAsync &= bPassTaskModeAsync;

		ParallelPassCandidates.Emplace(Pass);

		if (!Pass->bSkipRenderPassBegin && !Pass->bSkipRenderPassEnd)
		{
			ParallelPassCandidatesWorkload += Pass->Workload;
		}

		if (Pass->bDispatchAfterExecute)
		{
			bDispatchAfterExecute = true;
			FlushParallelPassCandidates();
		}

		if (ParallelPassCandidatesWorkload >= (uint32)GRDGParallelExecutePassMax)
		{
			FlushParallelPassCandidates();
		}
	}

	ParallelPassCandidates.Emplace(EpiloguePass);
	FlushParallelPassCandidates();

	for (FParallelPassSet& ParallelPassSet : ParallelExecute.ParallelPassSets)
	{
		FRHICommandList* RHICmdListPass = new FRHICommandList(FRHIGPUMask::All());

		// Propagate the immediate command list's timestamp query batch.
		// This is a workaround for poor fence batching on some platforms due to the realtime GPU profiler / timestamp query API design.
		RHICmdListPass->GetQueryBatchData(RQT_AbsoluteTime) = QueryBatchData;

		ParallelPassSet.CmdList = RHICmdListPass;

		const UE::Tasks::FTask& PrerequisiteTask = ParallelPassSet.bTaskModeAsync
			? *ParallelExecute.DispatchTaskEventAsync
			: *ParallelExecute.DispatchTaskEventAwait;
		
		const UE::Tasks::ETaskPriority TaskPriority = ParallelPassSet.bTaskModeAsync
			? UE::Tasks::ETaskPriority::Normal
			: UE::Tasks::ETaskPriority::High;

		UE::Tasks::FTask Task = UE::Tasks::Launch(TEXT("ParallelExecute"),
			[ParallelPasses = MakeArrayView(ParallelPassSet.Passes), bTaskModeAsync = ParallelPassSet.bTaskModeAsync, RHICmdListPass
#if WITH_RHI_BREADCRUMBS
				, LocalCurrentBreadcrumb = LocalCurrentBreadcrumb
#endif
			]
		{
			SCOPED_NAMED_EVENT_TCHAR_CONDITIONAL(TEXT("ParallelExecute (Await)"), FColor::Emerald, !bTaskModeAsync);
			SCOPED_NAMED_EVENT_TCHAR_CONDITIONAL(TEXT("ParallelExecute (Async)"), FColor::Emerald, bTaskModeAsync);
			FTaskTagScope TagScope(ETaskTag::EParallelRenderingThread);

#if WITH_RHI_BREADCRUMBS
			// Push all the CPU breadcrumbs this RDG builder is executing under
			// (i.e. push to the top breadcrumb on the render thread stack when Execute() was called).
			FRHIBreadcrumbNode::WalkIn(LocalCurrentBreadcrumb);
#endif

			PushPreScopes(*RHICmdListPass, ParallelPasses[0]);
			{
			#if WITH_RHI_BREADCRUMBS && WITH_ADDITIONAL_CRASH_CONTEXTS
				FScopedAdditionalCrashContextProvider CrashContext([RHICmdListPass](FCrashContextExtendedWriter& Writer)
				{
					if (FRHIBreadcrumbNode* CurrentBreadcrumb = RHICmdListPass->GetCurrentBreadcrumbRef())
					{
						FString ThreadIDStr = FString::Printf(TEXT("RDGTask_%d"), FPlatformTLS::GetCurrentThreadId());
						CurrentBreadcrumb->WriteCrashData(Writer, *ThreadIDStr);
					}
				});
			#endif

				for (FRDGPass* Pass : ParallelPasses)
				{
					ExecutePass(*RHICmdListPass, Pass);
				}
			}
			PopPreScopes(*RHICmdListPass, ParallelPasses.Last());

#if WITH_RHI_BREADCRUMBS
			// Restore breadcrumbs we pushed above.
			FRHIBreadcrumbNode::WalkOut(LocalCurrentBreadcrumb);
#endif

			RHICmdListPass->FinishRecording();

		}, PrerequisiteTask, TaskPriority);

		if (ParallelPassSet.bTaskModeAsync)
		{
			ParallelExecute.TasksAsync->AddPrerequisites(Task);
		}
		else
		{
			ParallelExecute.TasksAwait->AddPrerequisites(Task);
		}
	}
}

void FRDGBuilder::SetupDispatchPassExecute()
{
	if (!DispatchPasses.IsEmpty())
	{
		SCOPED_NAMED_EVENT_TEXT("FRDGBuilder::SetupDispatchPassExecute", FColor::Magenta);
		FRDGAllocatorScope AllocatorScope(Allocators.Task);

#if WITH_RHI_BREADCRUMBS
		// Push all the CPU breadcrumbs this RDG builder is executing under
		// (i.e. push to the top breadcrumb on the render thread stack when Execute() was called).
		FRHIBreadcrumbNode::WalkIn(LocalCurrentBreadcrumb);
#endif

		for (FRDGDispatchPass* DispatchPass : DispatchPasses)
		{
			if (DispatchPass->bCulled)
			{
				DispatchPass->CommandListsEvent.Trigger();
				continue;
			}

			IF_RDG_ENABLE_DEBUG(UserValidation.ValidateExecutePassBegin(DispatchPass));

			FRDGDispatchPassBuilder DispatchPassBuilder(DispatchPass);

			IF_RENDER_TRACING_ENABLED(RenderTracing::BeginExecuteDispatchPass(*DispatchPass));

			DispatchPass->LaunchDispatchPassTasks(DispatchPassBuilder);
			DispatchPassBuilder.Finish();

			for (const UE::Tasks::FTask& Task : DispatchPassBuilder.Prerequisites)
			{
				if (ParallelExecute.TasksAsync)
				{
					ParallelExecute.TasksAsync->AddPrerequisites(Task);
				}
				else if (ParallelExecute.TasksAwait)
				{
					ParallelExecute.TasksAwait->AddPrerequisites(Task);
				}
				else
				{
					Task.Wait();
				}
			}

			IF_RDG_ENABLE_DEBUG(UserValidation.ValidateExecutePassEnd(DispatchPass));
		}

#if WITH_RHI_BREADCRUMBS
		// Restore breadcrumbs we pushed above.
		FRHIBreadcrumbNode::WalkOut(LocalCurrentBreadcrumb);
#endif

		DispatchPasses.Empty();
	}
}

///////////////////////////////////////////////////////////////////////////////////////////////////

void FRDGBuilder::AllocatePooledTextures(FRHICommandListBase& InRHICmdList, TConstArrayView<FCollectResourceOp> Ops)
{
	SCOPED_NAMED_EVENT_TEXT("FRDGBuilder::AllocatePooledTextures", FColor::Magenta);
	UE::TScopeLock Lock(GRenderTargetPool.Mutex);

	for (FCollectResourceOp Op : Ops)
	{
		FRDGTexture* Texture = Textures[Op.GetTextureHandle()];

		// External render targets will have the allocation assigned. Scheduled render targets don't yet.
		check(Texture->Allocation.IsValid() == Texture->bExternal);

		switch (Op.GetOp())
		{
		case FCollectResourceOp::EOp::Allocate:
		{
			FPooledRenderTarget* RenderTarget = GRenderTargetPool.ScheduleAllocation(InRHICmdList, Texture->Desc, Texture->Name, GetAllocateFences(Texture));
			Texture->RenderTarget = RenderTarget;
			SetPooledTextureRHI(Texture, &RenderTarget->PooledTexture);
		}
		break;
		case FCollectResourceOp::EOp::Deallocate:
		{
			FPooledRenderTarget* RenderTarget = static_cast<FPooledRenderTarget*>(Texture->RenderTarget);
			GRenderTargetPool.ScheduleDeallocation(RenderTarget, GetDeallocateFences(Texture));

			if (Texture->Allocation && RenderTarget->IsTracked())
			{
				// This releases the reference without invoking a virtual function call.
				TRefCountPtr<FPooledRenderTarget>(MoveTemp(Texture->Allocation));
			}
		}
		break;
		}
	}

	for (FCollectResourceOp Op : Ops)
	{
		FRDGTexture* Texture = Textures[Op.GetTextureHandle()];

		if (!Texture->bSkipLastTransition && !Texture->Allocation)
		{
			FPooledRenderTarget* RenderTarget = static_cast<FPooledRenderTarget*>(Texture->RenderTarget);
			GRenderTargetPool.FinishSchedule(InRHICmdList, RenderTarget, Texture->Name);

			// Hold the last reference in a chain of pooled allocations.
			Texture->Allocation = RenderTarget;
		}
	}
}

void FRDGBuilder::AllocatePooledBuffers(FRHICommandListBase& InRHICmdList, TConstArrayView<FCollectResourceOp> Ops)
{
	SCOPED_NAMED_EVENT_TEXT("FRDGBuilder::AllocatePooledBuffers", FColor::Magenta);
	UE::TScopeLock Lock(GRenderGraphResourcePool.Mutex);

	for (FCollectResourceOp Op : Ops)
	{
		FRDGBuffer* Buffer = Buffers[Op.GetBufferHandle()];

		switch (Op.GetOp())
		{
		case FCollectResourceOp::EOp::Allocate:
		{
			FRDGPooledBuffer* PooledBuffer = GRenderGraphResourcePool.ScheduleAllocation(InRHICmdList, Buffer->Desc, Buffer->Name, ERDGPooledBufferAlignment::Page, GetAllocateFences(Buffer));
			SetPooledBufferRHI(Buffer, PooledBuffer);
		}
		break;
		case FCollectResourceOp::EOp::Deallocate:
			GRenderGraphResourcePool.ScheduleDeallocation(Buffer->PooledBuffer, GetDeallocateFences(Buffer));
			Buffer->Allocation = nullptr;
			break;
		}
	}

	for (FCollectResourceOp Op : Ops)
	{
		FRDGBuffer* Buffer = Buffers[Op.GetBufferHandle()];

		if (!Buffer->bSkipLastTransition && !Buffer->Allocation)
		{
			GRenderGraphResourcePool.FinishSchedule(InRHICmdList, Buffer->PooledBuffer);

			// Hold the last reference in a chain of pooled allocations.
			Buffer->Allocation = Buffer->PooledBuffer;
		}
	}
}

void FRDGBuilder::AllocateTransientResources(TConstArrayView<FCollectResourceOp> Ops)
{
	if (!TransientResourceAllocator)
	{
		return;
	}
	
	SCOPED_NAMED_EVENT_TEXT("FRDGBuilder::AllocateTransientResources", FColor::Magenta);
	TransientResourceAllocator->SetCreateMode(bParallelCompileEnabled ? ERHITransientResourceCreateMode::Task : ERHITransientResourceCreateMode::Inline);

	TArray<TPair<FRDGViewableResource*, FRHITransientResource*>, FRDGArrayAllocator> AllocatedResources;
	AllocatedResources.Reserve(Ops.Num() / 2);

	for (FCollectResourceOp Op : Ops)
	{
		switch (Op.GetOp())
		{
		default: checkNoEntry();
		case FCollectResourceOp::EOp::Allocate:
		{
			if (Op.GetResourceType() == ERDGViewableResourceType::Buffer)
			{
				FRDGBuffer* Buffer = Buffers[Op.GetBufferHandle()];
				FRHITransientBuffer* TransientBuffer = TransientResourceAllocator->CreateBuffer(Translate(Buffer->Desc), Buffer->Name, GetAllocateFences(Buffer));
				AllocatedResources.Emplace(Buffer, TransientBuffer);
				Buffer->TransientBuffer = TransientBuffer;
				Buffer->AcquirePass = FRDGPassHandle(TransientBuffer->GetAcquirePass());
			}
			else
			{
				FRDGTexture* Texture = Textures[Op.GetTextureHandle()];
				FRHITransientTexture* TransientTexture = TransientResourceAllocator->CreateTexture(Texture->Desc, Texture->Name, GetAllocateFences(Texture));
				AllocatedResources.Emplace(Texture, TransientTexture);
				Texture->TransientTexture = TransientTexture;
				Texture->AcquirePass = FRDGPassHandle(TransientTexture->GetAcquirePass());
			}
		}
		break;
		case FCollectResourceOp::EOp::Deallocate:
		{
			if (Op.GetResourceType() == ERDGViewableResourceType::Buffer)
			{
				FRDGBuffer* Buffer = Buffers[Op.GetBufferHandle()];
				FRHITransientBuffer* TransientBuffer = Buffer->TransientBuffer;
				TransientResourceAllocator->DeallocateMemory(TransientBuffer, GetDeallocateFences(Buffer));
			}
			else
			{
				FRDGTexture* Texture = Textures[FRDGTextureHandle(Op.ResourceIndex)];
				FRHITransientTexture* TransientTexture = Texture->TransientTexture;

				// Texture is using a transient external render target.
				if (Texture->RenderTarget)
				{
					if (!Texture->bExtracted)
					{
						// This releases the reference without invoking a virtual function call.
						GRDGTransientResourceAllocator.Release(TRefCountPtr<FRDGTransientRenderTarget>(MoveTemp(Texture->Allocation)), GetDeallocateFences(Texture));
						SetDiscardPass(Texture, TransientTexture);
					}
				}
				// Texture is using an internal transient texture.
				else
				{
					TransientResourceAllocator->DeallocateMemory(TransientTexture, GetDeallocateFences(Texture));
				}
			}
		}
		break;
		}
	}

	for (auto [Resource, TransientResource] : AllocatedResources)
	{
		TransientResource->Finish(RHICmdList);

		if (Resource->Type == ERDGViewableResourceType::Buffer)
		{
			SetTransientBufferRHI(static_cast<FRDGBuffer*>(Resource), static_cast<FRHITransientBuffer*>(TransientResource));
		}
		else
		{
			check(Resource->Type == ERDGViewableResourceType::Texture);
			FRDGTexture* Texture = static_cast<FRDGTexture*>(Resource);
			FRHITransientTexture* TransientTexture = static_cast<FRHITransientTexture*>(TransientResource);

			if (Texture->bExtracted)
			{
				SetExternalPooledRenderTargetRHI(Texture, GRDGTransientResourceAllocator.AllocateRenderTarget(TransientTexture));
			}
			else
			{
				SetTransientTextureRHI(Texture, TransientTexture);
			}
		}
	}
}

void FRDGBuilder::CreateViews(FRHICommandListBase& InRHICmdList, TConstArrayView<FRDGViewHandle> ViewsToCreate)
{
	SCOPED_NAMED_EVENT_TEXT("FRDGBuilder::CreateViews", FColor::Magenta);
	for (FRDGViewHandle ViewHandle : ViewsToCreate)
	{
		FRDGView* View = Views[ViewHandle];

		if (!View->ResourceRHI)
		{
			InitViewRHI(InRHICmdList, View);
		}
	}
}

void FRDGBuilder::CreateUniformBuffers(TConstArrayView<FRDGUniformBufferHandle> UniformBuffersToCreate)
{
	SCOPED_NAMED_EVENT_TEXT("FRDGBuilder::CreateUniformBuffers", FColor::Magenta);
	for (FRDGUniformBufferHandle UniformBufferHandle : UniformBuffersToCreate)
	{
		FRDGUniformBuffer* UniformBuffer = UniformBuffers[UniformBufferHandle];

		if (!UniformBuffer->ResourceRHI)
		{
			UniformBuffer->InitRHI();
		}
	}
}

///////////////////////////////////////////////////////////////////////////////////////////////////

// Pushes all the CPU scopes above the given pass.
void FRDGBuilder::PushPreScopes(FRHIComputeCommandList& RHICmdListPass, FRDGPass* FirstPass)
{
	// Execution of a pass set may start on a mid-frame pass which is nested several levels deep in the
	// scope tree. The executing thread needs to traverse into the scope tree before recording commands.

	// Skip past CPU scopes that will be pushed by the pass itself
	FRDGScope* Scope = FirstPass->Scope;
	while (Scope && Scope->CPUFirstPass == FirstPass)
	{
		Scope = Scope->Parent;
	}

	auto Recurse = [&RHICmdListPass](FRDGScope* Current, auto& Recurse)
	{
		if (!Current)
			return;

		Recurse(Current->Parent, Recurse);

		Current->BeginCPU(RHICmdListPass, true);
	};

	Recurse(Scope, Recurse);
}

void FRDGBuilder::PushPassScopes(FRHIComputeCommandList& RHICmdListPass, FRDGPass* Pass)
{
	auto Recurse = [Pass, &RHICmdListPass](FRDGScope* Current, auto& Recurse)
	{
		if (!Current)
			return;

		bool bBeginCPU = Pass == Current->CPUFirstPass;
		bool bBeginGPU = Pass == Current->GPUFirstPass[Pass->Pipeline];

		if (!(bBeginCPU || bBeginGPU))
			return;

		Recurse(Current->Parent, Recurse);

		if (bBeginCPU) { Current->BeginCPU(RHICmdListPass, false); }
		if (bBeginGPU) { Current->BeginGPU(RHICmdListPass); }
	};
	Recurse(Pass->Scope, Recurse);
}

void FRDGBuilder::PopPassScopes(FRHIComputeCommandList& RHICmdListPass, FRDGPass* Pass)
{
	for (FRDGScope* Current = Pass->Scope; Current; Current = Current->Parent)
	{
		bool bEndCPU = Pass == Current->CPULastPass;
		bool bEndGPU = Pass == Current->GPULastPass[Pass->Pipeline];

		if (!(bEndCPU || bEndGPU))
			break;

		if (bEndGPU) { Current->EndGPU(RHICmdListPass); }
		if (bEndCPU) { Current->EndCPU(RHICmdListPass, false); }
	}
}

// Reverses the CPU scope pushes that PushPreScopes() did.
void FRDGBuilder::PopPreScopes(FRHIComputeCommandList& RHICmdListPass, FRDGPass* LastPass)
{
	// Skip past scopes that were popped by the pass itself
	FRDGScope* Scope = LastPass->Scope;
	while (Scope && Scope->CPULastPass == LastPass)
	{
		Scope = Scope->Parent;
	}

	while (Scope)
	{
		Scope->EndCPU(RHICmdListPass, true);
		Scope = Scope->Parent;
	}
}

void FRDGBuilder::ExecutePassPrologue(FRHIComputeCommandList& RHICmdListPass, FRDGPass* Pass)
{
	CSV_SCOPED_TIMING_STAT_EXCLUSIVE_CONDITIONAL(RDGBuilder_ExecutePassPrologue, GRDGVerboseCSVStats != 0);

	if (!IsImmediateMode())
	{
		PushPassScopes(RHICmdListPass, Pass);
	}

	const ERDGPassFlags PassFlags = Pass->Flags;
	const ERHIPipeline PassPipeline = Pass->Pipeline;

	if (Pass->PrologueBarriersToBegin)
	{
		Pass->PrologueBarriersToBegin->Submit(RHICmdListPass, PassPipeline);
	}

	if (Pass->PrologueBarriersToEnd)
	{
		Pass->PrologueBarriersToEnd->Submit(RHICmdListPass, PassPipeline);
	}

	if (EnumHasAnyFlags(PassFlags, ERDGPassFlags::Raster))
	{
		if (!EnumHasAnyFlags(PassFlags, ERDGPassFlags::SkipRenderPass) && !Pass->SkipRenderPassBegin())
		{
			static_cast<FRHICommandList&>(RHICmdListPass).BeginRenderPass(Pass->GetParameters().GetRenderPassInfo(), Pass->GetName());
		}
	}

	BeginUAVOverlap(Pass, RHICmdListPass);
}

void FRDGBuilder::ExecutePassEpilogue(FRHIComputeCommandList& RHICmdListPass, FRDGPass* Pass)
{
	CSV_SCOPED_TIMING_STAT_EXCLUSIVE_CONDITIONAL(RDGBuilder_ExecutePassEpilogue, GRDGVerboseCSVStats != 0);

	EndUAVOverlap(Pass, RHICmdListPass);

	const ERDGPassFlags PassFlags = Pass->Flags;
	const ERHIPipeline PassPipeline = Pass->Pipeline;
	const FRDGParameterStruct PassParameters = Pass->GetParameters();

	if (EnumHasAnyFlags(PassFlags, ERDGPassFlags::Raster) && !EnumHasAnyFlags(PassFlags, ERDGPassFlags::SkipRenderPass) && !Pass->SkipRenderPassEnd())
	{
		static_cast<FRHICommandList&>(RHICmdListPass).EndRenderPass();
	}

	FRDGTransitionQueue Transitions;

	if (Pass->EpilogueBarriersToBeginForGraphics)
	{
		Pass->EpilogueBarriersToBeginForGraphics->Submit(RHICmdListPass, PassPipeline, Transitions);
	}

	if (Pass->EpilogueBarriersToBeginForAsyncCompute)
	{
		Pass->EpilogueBarriersToBeginForAsyncCompute->Submit(RHICmdListPass, PassPipeline, Transitions);
	}

	if (Pass->EpilogueBarriersToBeginForAll)
	{
		Pass->EpilogueBarriersToBeginForAll->Submit(RHICmdListPass, PassPipeline, Transitions);
	}

	for (FRDGBarrierBatchBegin* BarriersToBegin : Pass->SharedEpilogueBarriersToBegin)
	{
		BarriersToBegin->Submit(RHICmdListPass, PassPipeline, Transitions);
	}

	if (!Transitions.IsEmpty())
	{
		RHICmdListPass.BeginTransitions(Transitions);
	}

	if (Pass->EpilogueBarriersToEnd)
	{
		Pass->EpilogueBarriersToEnd->Submit(RHICmdListPass, PassPipeline);
	}

	// Pop scopes
	if (!IsImmediateMode())
	{
		PopPassScopes(RHICmdListPass, Pass);
	}
}

void FRDGBuilder::ExecutePass(FRHIComputeCommandList& RHICmdListPass, FRDGPass* Pass)
{
	IF_RENDER_TRACING_ENABLED(RenderTracing::BeginExecuteRDGPass(*Pass, RHICmdListPass));

	// Note that we must do this before doing anything with RHICmdListPass.
	// For example, if this pass only executes on GPU 1 we want to avoid adding a
	// 0-duration event for this pass on GPU 0's time line.
	SCOPED_GPU_MASK(RHICmdListPass, Pass->GPUMask);
	RHICmdListPass.SwitchPipeline(Pass->Pipeline);

	ExecutePassPrologue(RHICmdListPass, Pass);

	Pass->Execute(RHICmdListPass);

	ExecutePassEpilogue(RHICmdListPass, Pass);

	IF_RENDER_TRACING_ENABLED(RenderTracing::EndExecuteRDGPass(*Pass));
}

void FRDGBuilder::ExecuteSerialPass(FRHIComputeCommandList& RHICmdListPass, FRDGPass* Pass)
{
#if RDG_ENABLE_DEBUG
	UserValidation.ValidateExecutePassBegin(Pass);

	if (Pass->PrologueBarriersToBegin)
	{
		BarrierValidation.ValidateBarrierBatchBegin(Pass, *Pass->PrologueBarriersToBegin);
	}

	if (Pass->PrologueBarriersToEnd)
	{
		BarrierValidation.ValidateBarrierBatchEnd(Pass, *Pass->PrologueBarriersToEnd);
	}
#endif

	ExecutePass(RHICmdListPass, Pass);

#if RDG_ENABLE_DEBUG
	if (Pass->EpilogueBarriersToBeginForGraphics)
	{
		BarrierValidation.ValidateBarrierBatchBegin(Pass, *Pass->EpilogueBarriersToBeginForGraphics);
	}

	if (Pass->EpilogueBarriersToBeginForAsyncCompute)
	{
		BarrierValidation.ValidateBarrierBatchBegin(Pass, *Pass->EpilogueBarriersToBeginForAsyncCompute);
	}

	if (Pass->EpilogueBarriersToBeginForAll)
	{
		BarrierValidation.ValidateBarrierBatchBegin(Pass, *Pass->EpilogueBarriersToBeginForAll);
	}

	for (FRDGBarrierBatchBegin* BarriersToBegin : Pass->SharedEpilogueBarriersToBegin)
	{
		BarrierValidation.ValidateBarrierBatchBegin(Pass, *BarriersToBegin);
	}

	if (Pass->EpilogueBarriersToEnd)
	{
		BarrierValidation.ValidateBarrierBatchEnd(Pass, *Pass->EpilogueBarriersToEnd);
	}

	UserValidation.ValidateExecutePassEnd(Pass);
#endif
}

///////////////////////////////////////////////////////////////////////////////////////////////////

void FRDGBuilder::FinalizeDescs()
{
	SCOPED_NAMED_EVENT_TEXT("FRDGBuilder::FinalizeDescs", FColor::Magenta);
	for (FRDGBuffer* Buffer : NumElementsCallbackBuffers)
	{
		Buffer->FinalizeDesc();
	}
	NumElementsCallbackBuffers.Empty();
}

void FRDGBuilder::CollectAllocations(FCollectResourceContext& Context, FRDGPass* Pass)
{
	for (FRDGPass* PassToBegin : Pass->ResourcesToBegin)
	{
		for (FRDGPass::FTextureState& PassState : PassToBegin->TextureStates)
		{
			CollectAllocateTexture(Context, Pass->Pipeline, Pass->Handle, PassState.Texture);
		}

		for (FRDGPass::FBufferState& PassState : PassToBegin->BufferStates)
		{
			CollectAllocateBuffer(Context, Pass->Pipeline, Pass->Handle, PassState.Buffer);
		}

		if (!IsImmediateMode())
		{
			for (FRDGUniformBufferHandle UniformBufferHandle : PassToBegin->UniformBuffers)
			{
				if (auto BitRef = Context.UniformBufferMap[UniformBufferHandle]; BitRef)
				{
					Context.UniformBuffers.Add(UniformBufferHandle);
					BitRef = false;
				}
			}

			for (FRDGViewHandle ViewHandle : PassToBegin->Views)
			{
				if (auto BitRef = Context.ViewMap[ViewHandle]; BitRef)
				{
					Context.Views.Add(ViewHandle);
					BitRef = false;
				}
			}
		}
		else
		{
			Context.UniformBuffers = PassToBegin->UniformBuffers;
			Context.Views = PassToBegin->Views;
		}
	}
}

void FRDGBuilder::CollectDeallocations(FCollectResourceContext& Context, FRDGPass* Pass)
{
	for (FRDGPass* PassToEnd : Pass->ResourcesToEnd)
	{
		for (FRDGPass::FTextureState& PassState : PassToEnd->TextureStates)
		{
			CollectDeallocateTexture(Context, Pass->Pipeline, Pass->Handle, PassState.Texture, PassState.ReferenceCount);
		}

		for (FRDGPass::FBufferState& PassState : PassToEnd->BufferStates)
		{
			CollectDeallocateBuffer(Context, Pass->Pipeline, Pass->Handle, PassState.Buffer, PassState.ReferenceCount);
		}
	}
}

void FRDGBuilder::CollectAllocateTexture(FCollectResourceContext& Context, ERHIPipeline PassPipeline, FRDGPassHandle PassHandle, FRDGTextureRef Texture)
{
	check(Texture->ReferenceCount > 0 || Texture->bExternal || IsImmediateMode());

#if RDG_ENABLE_DEBUG
	{
		FRDGPass* Pass = Passes[PassHandle];

		// Cannot begin a resource within a merged render pass region.
		checkf(GetPrologueBarrierPassHandle(PassHandle) == PassHandle,
			TEXT("Cannot begin a resource within a merged render pass. Pass (Handle: %d, Name: %s), Resource %s"), PassHandle.GetIndex(), Pass->GetName(), Texture->Name);
	}
#endif

	if (Texture->FirstPass.IsNull())
	{
		Texture->FirstPass = PassHandle;
	}

	if (Texture->bCollectForAllocate)
	{
		Texture->bCollectForAllocate = false;
		check(!Texture->ResourceRHI);

		const FCollectResourceOp AllocateOp = FCollectResourceOp::Allocate(Texture->Handle);

		if (Texture->bTransient)
		{
			Context.TransientResources.Emplace(AllocateOp);

#if RDG_STATS
			GRDGStatTransientTextureCount++;
#endif
		}
		else
		{
			Context.PooledTextures.Emplace(AllocateOp);
		}
	}
}

void FRDGBuilder::CollectDeallocateTexture(FCollectResourceContext& Context, ERHIPipeline PassPipeline, FRDGPassHandle PassHandle, FRDGTexture* Texture, uint32 ReferenceCount)
{
	check(!IsImmediateMode());
	check(Texture->ReferenceCount != FRDGViewableResource::DeallocatedReferenceCount);
	check(Texture->ReferenceCount >= ReferenceCount);
	Texture->ReferenceCount -= ReferenceCount;
	Texture->LastPasses[PassPipeline] = PassHandle;

	if (Texture->ReferenceCount == 0)
	{
		check(!Texture->bCollectForAllocate);
		const FCollectResourceOp DeallocateOp = FCollectResourceOp::Deallocate(Texture->Handle);

		if (Texture->bTransient)
		{
			Context.TransientResources.Emplace(DeallocateOp);
		}
		else
		{
			Context.PooledTextures.Emplace(DeallocateOp);
		}

		Texture->ReferenceCount = FRDGViewableResource::DeallocatedReferenceCount;
	}
}

void FRDGBuilder::CollectAllocateBuffer(FCollectResourceContext& Context, ERHIPipeline PassPipeline, FRDGPassHandle PassHandle, FRDGBuffer* Buffer)
{
	check(Buffer->ReferenceCount > 0 || IsImmediateMode());

#if RDG_ENABLE_DEBUG
	{
		const FRDGPass* Pass = Passes[PassHandle];

		// Cannot begin a resource within a merged render pass region.
		checkf(GetPrologueBarrierPassHandle(PassHandle) == PassHandle,
			TEXT("Cannot begin a resource within a merged render pass. Pass (Handle: %d, Name: %s), Resource %s"), PassHandle.GetIndex(), Pass->GetName(), Buffer->Name);
	}
#endif

	if (Buffer->FirstPass.IsNull())
	{
		Buffer->FirstPass = PassHandle;
	}

	if (Buffer->bCollectForAllocate)
	{
		Buffer->bCollectForAllocate = false;
		check(!Buffer->ResourceRHI);

		const FCollectResourceOp AllocateOp = FCollectResourceOp::Allocate(Buffer->Handle);

		if (Buffer->bTransient)
		{
			Context.TransientResources.Emplace(AllocateOp);

#if RDG_STATS
			GRDGStatTransientBufferCount++;
#endif
		}
		else
		{
			Context.PooledBuffers.Emplace(AllocateOp);
		}
	}
}

void FRDGBuilder::CollectDeallocateBuffer(FCollectResourceContext& Context, ERHIPipeline PassPipeline, FRDGPassHandle PassHandle, FRDGBuffer* Buffer, uint32 ReferenceCount)
{
	check(!IsImmediateMode());
	check(Buffer->ReferenceCount != FRDGViewableResource::DeallocatedReferenceCount);
	check(Buffer->ReferenceCount >= ReferenceCount);
	Buffer->ReferenceCount -= ReferenceCount;
	Buffer->LastPasses[PassPipeline] = PassHandle;

	if (Buffer->ReferenceCount == 0)
	{
		const FCollectResourceOp DeallocateOp = FCollectResourceOp::Deallocate(Buffer->Handle);

		if (Buffer->bTransient)
		{
			Context.TransientResources.Emplace(DeallocateOp);
		}
		else
		{
			Context.PooledBuffers.Emplace(DeallocateOp);
		}

		Buffer->ReferenceCount = FRDGViewableResource::DeallocatedReferenceCount;
	}
}

///////////////////////////////////////////////////////////////////////////////////////////////////

void FRDGBuilder::CompilePassBarriers()
{
	// Walk the culled graph and compile barriers for each subresource. Certain transitions are redundant; read-to-read, for example.
	// We can avoid them by traversing and merging compatible states together. The merging states removes a transition, but the merging
	// heuristic is conservative and choosing not to merge doesn't necessarily mean a transition is performed. They are two distinct steps.
	// Merged states track the first and last pass used for all pipelines.

	SCOPED_NAMED_EVENT(CompileBarriers, FColor::Emerald);
	FRDGAllocatorScope AllocatorScope(Allocators.Transition);

	for (FRDGPassHandle PassHandle = GetProloguePassHandle() + 1; PassHandle < GetEpiloguePassHandle(); ++PassHandle)
	{
		FRDGPass* Pass = Passes[PassHandle];

		if (Pass->bCulled)
		{
			continue;
		}

		if (!Pass->NumTransitionsToReserve)
		{
			Pass->NumTransitionsToReserve = Pass->TextureStates.Num() + Pass->BufferStates.Num();
		}

		const ERHIPipeline PassPipeline = Pass->Pipeline;

		const auto MergeSubresourceStates = [&](ERDGViewableResourceType ResourceType, FRDGSubresourceState*& PassMergeState, FRDGSubresourceState*& ResourceMergeState, FRDGSubresourceState* PassState)
		{
			if (!ResourceMergeState || !FRDGSubresourceState::IsMergeAllowed(ResourceType, *ResourceMergeState, *PassState))
			{
				// Use the new pass state as the merge state for future passes.
				ResourceMergeState = PassState;
			}
			else
			{
				// Merge the pass state into the merged state.
				ResourceMergeState->Access |= PassState->Access;

				// If multiple reserved commits were requested, take the latest.
				if (PassState->ReservedCommitHandle.IsValid())
				{
					ResourceMergeState->ReservedCommitHandle = PassState->ReservedCommitHandle;
				}

				FRDGPassHandle& FirstPassHandle = ResourceMergeState->FirstPass[PassPipeline];

				if (FirstPassHandle.IsNull())
				{
					FirstPassHandle = PassHandle;
				}

				ResourceMergeState->LastPass[PassPipeline] = PassHandle;
			}

			PassMergeState = ResourceMergeState;
		};

		for (auto& PassState : Pass->TextureStates)
		{
			FRDGTexture* Texture = PassState.Texture;

		#if RDG_STATS
			GRDGStatTextureReferenceCount += PassState.ReferenceCount;
		#endif

			for (int32 Index = 0; Index < PassState.State.Num(); ++Index)
			{
				if (!PassState.State[Index])
				{
					continue;
				}

				MergeSubresourceStates(ERDGViewableResourceType::Texture, PassState.MergeState[Index], Texture->MergeState[Index], PassState.State[Index]);
			}
		}

		for (auto& PassState : Pass->BufferStates)
		{
			FRDGBuffer* Buffer = PassState.Buffer;

		#if RDG_STATS
			GRDGStatBufferReferenceCount += PassState.ReferenceCount;
		#endif

			MergeSubresourceStates(ERDGViewableResourceType::Buffer, PassState.MergeState, Buffer->MergeState, &PassState.State);
		}
	}
}

void FRDGBuilder::CollectPassBarriers()
{
	SCOPED_NAMED_EVENT_TEXT("FRDGBuilder::CollectBarriers", FColor::Magenta);
	SCOPE_CYCLE_COUNTER(STAT_RDG_CollectBarriersTime);
	CSV_SCOPED_TIMING_STAT_EXCLUSIVE_CONDITIONAL(RDG_CollectBarriers, GRDGVerboseCSVStats != 0);
	FRDGAllocatorScope AllocatorScope(Allocators.Transition);

	for (FRDGPassHandle PassHandle = GetProloguePassHandle() + 1; PassHandle < GetEpiloguePassHandle(); ++PassHandle)
	{
		CollectPassBarriers(PassHandle);
	}
}

void FRDGBuilder::CollectPassBarriers(FRDGPassHandle PassHandle)
{
	FRDGPass* Pass = Passes[PassHandle];

	if (Pass->bCulled || Pass->bEmptyParameters)
	{
		return;
	}


	for (auto& PassState : Pass->TextureStates)
	{
		FRDGTexture* Texture = PassState.Texture;

		AddTextureTransition(PassState.Texture, Texture->State, PassState.MergeState, [Texture] (FRDGSubresourceState* StateAfter, int32 SubresourceIndex)
		{
			if (!Texture->FirstState[SubresourceIndex])
			{
				Texture->FirstState[SubresourceIndex] = StateAfter;
				return IsImmediateMode();
			}
			return true;
		});

		IF_RDG_ENABLE_TRACE(Trace.AddTexturePassDependency(Texture, Pass));
	}

	for (auto& PassState : Pass->BufferStates)
	{
		FRDGBuffer* Buffer = PassState.Buffer;

		AddBufferTransition(PassState.Buffer, Buffer->State, PassState.MergeState, [Buffer] (FRDGSubresourceState* StateAfter)
		{
			if (!Buffer->FirstState)
			{
				Buffer->FirstState = StateAfter;
				return IsImmediateMode();
			}
			return true;
		});

		IF_RDG_ENABLE_TRACE(Trace.AddBufferPassDependency(Buffer, Pass));
	}
}

void FRDGBuilder::CreatePassBarriers()
{
	struct FTaskContext
	{
		TArray<FRHITransitionInfo, FConcurrentLinearArrayAllocator> Transitions;
	};

	const auto CreateTransition = [this] (FTaskContext& Context, FRDGBarrierBatchBegin* BeginBatch)
	{
		Context.Transitions.Reset(BeginBatch->Transitions.Num());

		for (FRDGTransitionInfo InfoRDG : BeginBatch->Transitions)
		{
			FRHITransitionInfo& InfoRHI = Context.Transitions.Emplace_GetRef();
			InfoRHI.AccessBefore = (ERHIAccess)InfoRDG.AccessBefore;
			InfoRHI.AccessAfter  = (ERHIAccess)InfoRDG.AccessAfter;
			InfoRHI.Flags        = (EResourceTransitionFlags)InfoRDG.ResourceTransitionFlags;

			if ((ERDGViewableResourceType)InfoRDG.ResourceType == ERDGViewableResourceType::Texture)
			{
				InfoRHI.Resource   = Textures[FRDGTextureHandle(InfoRDG.ResourceHandle)]->ResourceRHI;
				InfoRHI.Type       = FRHITransitionInfo::EType::Texture;
				InfoRHI.ArraySlice = InfoRDG.Texture.ArraySlice;
				InfoRHI.MipIndex   = InfoRDG.Texture.MipIndex;
				InfoRHI.PlaneSlice = InfoRDG.Texture.PlaneSlice;
			}
			else
			{
				FRDGBuffer* Buffer = Buffers[FRDGBufferHandle(InfoRDG.ResourceHandle)];

				InfoRHI.Resource = Buffer->ResourceRHI;
				InfoRHI.Type = FRHITransitionInfo::EType::Buffer;

				if (InfoRDG.Buffer.CommitSize > 0)
				{
					InfoRHI.CommitInfo.Emplace(InfoRDG.Buffer.CommitSize);
				}
			}
		}

		BeginBatch->CreateTransition(Context.Transitions);
	};

	TArray<FTaskContext, TInlineAllocator<1, FRDGArrayAllocator>> TaskContexts;
	ParallelForWithTaskContext(TEXT("FRDGBuilder::CreatePassBarriers"), TaskContexts, TransitionCreateQueue.Num(), 1, [&](FTaskContext& TaskContext, int32 Index)
	{
		CreateTransition(TaskContext, TransitionCreateQueue[Index]);

	}, bParallelCompileEnabled ? EParallelForFlags::None : EParallelForFlags::ForceSingleThread);

	TransitionCreateQueue.Reset();
}

///////////////////////////////////////////////////////////////////////////////////////////////////

void FRDGBuilder::FinalizeResources()
{
	SCOPED_NAMED_EVENT_TEXT("FRDGBuilder::FinalizeResources", FColor::Magenta);
	FRDGAllocatorScope AllocatorScope(Allocators.Transition);

	{
		SCOPED_NAMED_EVENT_TEXT("Textures", FColor::Magenta);
		Textures.Enumerate([&](FRDGTextureRef Texture)
		{
			if (Texture->FirstPass.IsValid())
			{
				if (!IsImmediateMode())
				{
					AddFirstTextureTransition(Texture);
				}

				if (!Texture->bSkipLastTransition)
				{
					AddLastTextureTransition(Texture);
				}
			}

			if (Texture->Allocation)
			{
				ActivePooledTextures.Emplace(MoveTemp(Texture->Allocation));
			}
		});
	}

	{
		SCOPED_NAMED_EVENT_TEXT("Buffers", FColor::Magenta);
		Buffers.Enumerate([&](FRDGBufferRef Buffer)
		{
			if (Buffer->FirstPass.IsValid())
			{
				if (!IsImmediateMode())
				{
					AddFirstBufferTransition(Buffer);
				}

				if (!Buffer->bSkipLastTransition)
				{
					AddLastBufferTransition(Buffer);
				}
			}
			else if (Buffer->PendingCommitSize != 0)
			{
				AddCulledReservedCommitTransition(Buffer);
			}

			if (Buffer->Allocation)
			{
				ActivePooledBuffers.Emplace(MoveTemp(Buffer->Allocation));
			}
		});
	}

	CreatePassBarriers();
}

void FRDGBuilder::AddFirstTextureTransition(FRDGTexture* Texture)
{
	check(!IsImmediateMode());
	check(Texture->HasRHI());

	FRDGTextureSubresourceState* StateBefore = &ScratchTextureState;
	FRDGSubresourceState& SubresourceStateBefore = *AllocSubresource(FRDGSubresourceState(ERHIPipeline::Graphics, GetProloguePassHandle()));

	if (Texture->PreviousOwner.IsValid())
	{
		// Previous state is the last used state of RDG texture that previously aliased the underlying pooled texture.
		StateBefore = &Textures[Texture->PreviousOwner]->State;

		for (int32 Index = 0; Index < Texture->FirstState.Num(); ++Index)
		{
			// If the new owner doesn't touch the subresource but the previous owner did, pull the previous owner subresource in so that the last transition is respected.
			if (!Texture->FirstState[Index])
			{
				Texture->State[Index] = (*StateBefore)[Index];
			}
			// If the previous owner didn't touch the subresource but the new owner does, assign the prologue subresource state so the first transition is respected.
			else if (!(*StateBefore)[Index])
			{
				(*StateBefore)[Index] = &SubresourceStateBefore;
			}
		}
	}
	else
	{
		if (Texture->AcquirePass.IsValid())
		{
			AddAliasingTransition(Texture->AcquirePass, Texture->FirstPass, Texture, FRHITransientAliasingInfo::Acquire(Texture->GetRHI(), Texture->AliasingOverlaps));

			SubresourceStateBefore.SetPass(GetPassPipeline(Texture->AcquirePass), Texture->AcquirePass);
			SubresourceStateBefore.Access = ERHIAccess::Discard;
		}
		else if (!Texture->bSplitFirstTransition)
		{
			SubresourceStateBefore.SetPass(GetPassPipeline(Texture->FirstPass), Texture->FirstPass);
		}

		InitTextureSubresources(*StateBefore, Texture->Layout, &SubresourceStateBefore);
	}

	AddTextureTransition(Texture, *StateBefore, Texture->FirstState);

	ScratchTextureState.Reset();
}

void FRDGBuilder::AddLastTextureTransition(FRDGTexture* Texture)
{
	check(IsImmediateMode() || Texture->bExtracted || Texture->ReferenceCount == FRDGViewableResource::DeallocatedReferenceCount);
	check(Texture->HasRHI());

	if (Texture->AccessModeState.ActiveMode == FRDGViewableResource::EAccessMode::External)
	{
		// Find the first subresource that is valid. There must be at least one.
		int32  FirstValidIndex = 0;
		for (; FirstValidIndex < Texture->State.Num() && !Texture->State[FirstValidIndex]; ++FirstValidIndex) {}
		check( FirstValidIndex < Texture->State.Num() &&  Texture->State[FirstValidIndex]);

		// Assign the final state that was enqueued by the external access pass, which may include merged states.
		EpilogueResourceAccesses.Emplace(Texture->GetRHI(), Texture->State[FirstValidIndex]->Access, Texture->State[FirstValidIndex]->GetPipelines());
		return;
	}

	const FRDGPassHandle EpiloguePassHandle = GetEpiloguePassHandle();

	FRDGSubresourceState* SubresourceStateBefore = nullptr;
	FRDGSubresourceState& SubresourceStateAfter = *AllocSubresource();
	SubresourceStateAfter.SetPass(ERHIPipeline::Graphics, EpiloguePassHandle);

	// Texture is using the RHI transient allocator. Transition it back to Discard in the final pass it is used.
	if (Texture->DiscardPass.IsValid())
	{
		const ERHIPipeline DiscardPassPipeline = GetPassPipeline(Texture->DiscardPass);

		SubresourceStateAfter.SetPass(DiscardPassPipeline, Texture->DiscardPass);
		SubresourceStateAfter.BarrierLocation = ERDGBarrierLocation::Epilogue;
		SubresourceStateAfter.Access = ERHIAccess::Discard;

		if (GRHIGlobals.NeedsTransientDiscardStateTracking)
		{
			ERHIAccess EpilogueAccess = ERHIAccess::Unknown;

			// Edge Case: Discarding Texture with RTV | DSV and multiple differing subresource states on async compute. Since we can't put multiple states
			// inside of the TrackedAccess we have to do an intermediate transition instead on discard by ORing the intermediate state with Discard. This
			// is going to be an incredibly rare case but needs to be handled correctly nonetheless.
			if (DiscardPassPipeline == ERHIPipeline::AsyncCompute && EnumHasAnyFlags(Texture->Desc.Flags, ETextureCreateFlags::RenderTargetable | ETextureCreateFlags::DepthStencilTargetable))
			{
				if (Texture->State[0])
				{
					EpilogueAccess = Texture->State[0]->Access;

					for (uint32 SubresourceIndex = 1; SubresourceIndex < Texture->GetSubresourceCount(); ++SubresourceIndex)
					{
						FRDGSubresourceState* SubresourceState = Texture->State[SubresourceIndex];

						if (!SubresourceState || SubresourceState->Access != EpilogueAccess)
						{
							EpilogueAccess = GRHIMultiSubresourceDiscardIntermediateAccess;
							SubresourceStateAfter.Access |= GRHIMultiSubresourceDiscardIntermediateAccess;
							break;
						}
					}
				}
				else
				{
					EpilogueAccess = GRHIMultiSubresourceDiscardIntermediateAccess;
					SubresourceStateAfter.Access |= GRHIMultiSubresourceDiscardIntermediateAccess;
				}
			}

			EpilogueResourceAccesses.Emplace(Texture->GetRHI(), ERHIAccess::Discard | EpilogueAccess, DiscardPassPipeline);
		}
	}
	else
	{
		SubresourceStateAfter.Access = Texture->EpilogueAccess;

		EpilogueResourceAccesses.Emplace(Texture->GetRHI(), SubresourceStateAfter.Access, ERHIPipeline::Graphics);
	}

	// Transition any unused (null) sub-resources to the epilogue state since we are assigning a monolithic state across all subresources.
	for (FRDGSubresourceState*& State : Texture->State)
	{
		if (!State)
		{
			if (!SubresourceStateBefore)
			{
				SubresourceStateBefore = AllocSubresource();

				FRDGPassHandle AcquirePass = GetProloguePassHandle();

				if (Texture->AcquirePass.IsValid())
				{
					AcquirePass = Texture->FirstPass;
				}

				SubresourceStateBefore->SetPass(GetPassPipeline(AcquirePass), AcquirePass);
			}

			State = SubresourceStateBefore;
		}
	}

	InitTextureSubresources(ScratchTextureState, Texture->Layout, &SubresourceStateAfter);
	AddTextureTransition(Texture, Texture->State, ScratchTextureState);
	ScratchTextureState.Reset();
}

void FRDGBuilder::AddFirstBufferTransition(FRDGBuffer* Buffer)
{
	check(!IsImmediateMode());
	check(Buffer->HasRHI());

	FRDGSubresourceState* StateBefore = nullptr;

	if (Buffer->PreviousOwner.IsValid())
	{
		// Previous state is the last used state of RDG buffer that previously aliased the underlying pooled buffer.
		StateBefore = Buffers[Buffer->PreviousOwner]->State;
	}

	if (!StateBefore)
	{
		StateBefore = AllocSubresource();

		if (Buffer->AcquirePass.IsValid())
		{
			AddAliasingTransition(Buffer->AcquirePass, Buffer->FirstPass, Buffer, FRHITransientAliasingInfo::Acquire(Buffer->GetRHI(), Buffer->AliasingOverlaps));

			StateBefore->SetPass(GetPassPipeline(Buffer->AcquirePass), Buffer->AcquirePass);
			StateBefore->Access = ERHIAccess::Discard;
		}
		else if (!Buffer->bSplitFirstTransition)
		{
			StateBefore->SetPass(GetPassPipeline(Buffer->FirstPass), Buffer->FirstPass);
		}
		else
		{
			StateBefore->SetPass(ERHIPipeline::Graphics, GetProloguePassHandle());
		}
	}

	AddBufferTransition(Buffer, StateBefore, Buffer->FirstState);
}

void FRDGBuilder::AddLastBufferTransition(FRDGBuffer* Buffer)
{
	check(IsImmediateMode() || Buffer->bExtracted || Buffer->ReferenceCount == FRDGViewableResource::DeallocatedReferenceCount);
	check(Buffer->HasRHI());

	if (Buffer->AccessModeState.IsExternalAccess())
	{
		// Assign the final state that was enqueued by the external access pass, which may include merged states.
		EpilogueResourceAccesses.Emplace(Buffer->GetRHI(), Buffer->State->Access, Buffer->State->GetPipelines());
		return;
	}

	const FRDGPassHandle EpiloguePassHandle = GetEpiloguePassHandle();

	FRDGSubresourceState* StateAfter = AllocSubresource();

	// Texture is using the RHI transient allocator. Transition it back to Discard in the final pass it is used.
	if (Buffer->DiscardPass.IsValid())
	{
		const ERHIPipeline DiscardPassPipeline = GetPassPipeline(Buffer->DiscardPass);

		StateAfter->SetPass(DiscardPassPipeline, Buffer->DiscardPass);
		StateAfter->BarrierLocation = ERDGBarrierLocation::Epilogue;
		StateAfter->Access = ERHIAccess::Discard;

		if (GRHIGlobals.NeedsTransientDiscardStateTracking)
		{
			EpilogueResourceAccesses.Emplace(Buffer->GetRHI(), ERHIAccess::Discard, DiscardPassPipeline);
		}
	}
	else
	{
		StateAfter->SetPass(ERHIPipeline::Graphics, EpiloguePassHandle);
		StateAfter->Access = Buffer->EpilogueAccess;
		StateAfter->ReservedCommitHandle = AcquireReservedCommitHandle(Buffer);

		EpilogueResourceAccesses.Emplace(Buffer->GetRHI(), StateAfter->Access, StateAfter->GetPipelines());
	}

	AddBufferTransition(Buffer, Buffer->State, StateAfter);
}

void FRDGBuilder::AddCulledReservedCommitTransition(FRDGBufferRef Buffer)
{
	check(Buffer->HasRHI() && Buffer->bExternal && Buffer->PendingCommitSize > 0);
	check(Buffer->ReferenceCount == 0 || IsImmediateMode());

	const FRDGPassHandle ProloguePassHandle = GetProloguePassHandle();
	const FRDGPassHandle EpiloguePassHandle = GetEpiloguePassHandle();

	FRDGSubresourceState* StateBefore = AllocSubresource();
	StateBefore->SetPass(ERHIPipeline::Graphics, IsImmediateMode() ? EpiloguePassHandle : ProloguePassHandle);

	FRDGSubresourceState* StateAfter = AllocSubresource();
	StateAfter->SetPass(ERHIPipeline::Graphics, EpiloguePassHandle);
	StateAfter->Access = Buffer->EpilogueAccess;
	StateAfter->ReservedCommitHandle = AcquireReservedCommitHandle(Buffer);

	EpilogueResourceAccesses.Emplace(Buffer->GetRHI(), StateAfter->Access, StateAfter->GetPipelines());
	Buffer->Allocation = nullptr;

	AddBufferTransition(Buffer, StateBefore, StateAfter);
}

template <typename FilterSubresourceLambdaType>
void FRDGBuilder::AddTextureTransition(FRDGTexture* Texture, FRDGTextureSubresourceState& StateBefore, FRDGTextureSubresourceState& StateAfter, FilterSubresourceLambdaType&& FilterSubresourceLambda)
{
	const FRDGTextureSubresourceLayout Layout = Texture->Layout;
	const uint32 SubresourceCount = Texture->SubresourceCount;

	check(SubresourceCount == Layout.GetSubresourceCount() && StateBefore.Num() == StateAfter.Num());

	if (!GRHISupportsSeparateDepthStencilCopyAccess && Texture->Desc.Format == PF_DepthStencil)
	{
		// Certain RHIs require a fused depth / stencil copy state. For any mip / slice transition involving a copy state,
		// adjust the split transitions so both subresources are transitioned using the same barrier batch (i.e. the RHI transition).
		// Note that this is only possible when async compute is disabled, as it's not possible to merge transitions from different pipes.
		// There are two cases to correct (D for depth, S for stencil, horizontal axis is time):
		//
		// Case 1: both states transitioning from previous states on passes A and B to a copy state at pass C.
		//
		// [Pass] A     B     C                         A     B     C
		// [D]          X --> X      Corrected To:            X --> X
		// [S]    X --------> X                               X --> X (S is pushed forward to transition with D on pass B)
		//
		// Case 2a|b: one plane transitioning out of a copy state on pass A to pass B (this pass), but the other is not transitioning yet.
		//
		// [Pass] A     B     ?                         A     B
		// [D]    X --> X            Corrected To:      X --> X
		// [S]    X --------> X                         X --> X (S's state is unknown, so it transitions with D and matches D's state).

		const ERHIPipeline GraphicsPipe = ERHIPipeline::Graphics;
		const uint32 NumSlicesAndMips = Layout.NumMips * Layout.NumArraySlices;

		for (uint32 DepthIndex = 0, StencilIndex = NumSlicesAndMips; DepthIndex < NumSlicesAndMips; ++DepthIndex, ++StencilIndex)
		{
			FRDGSubresourceState*& DepthStateAfter   = StateAfter[DepthIndex];
			FRDGSubresourceState*& StencilStateAfter = StateAfter[StencilIndex];

			// Skip if neither depth nor stencil are being transitioned.
			if (!DepthStateAfter && !StencilStateAfter)
			{
				continue;
			}

			FRDGSubresourceState*& DepthStateBefore   = StateBefore[DepthIndex];
			FRDGSubresourceState*& StencilStateBefore = StateBefore[StencilIndex];

			// Case 1: transitioning into a fused copy state.
			if (DepthStateAfter && EnumHasAnyFlags(DepthStateAfter->Access, ERHIAccess::CopySrc | ERHIAccess::CopyDest))
			{
				check(StencilStateAfter && StencilStateAfter->Access == DepthStateAfter->Access);

				const FRDGPassHandle MaxPassHandle = FRDGPassHandle::Max(DepthStateBefore->LastPass[GraphicsPipe], StencilStateBefore->LastPass[GraphicsPipe]);

				DepthStateBefore = AllocSubresource(*DepthStateBefore);
				DepthStateAfter  = AllocSubresource(*DepthStateAfter);

				DepthStateBefore->LastPass[GraphicsPipe]   = MaxPassHandle;
				StencilStateBefore->LastPass[GraphicsPipe] = MaxPassHandle;
			}
			// Case 2: transitioning out of a fused copy state.
			else if (DepthStateBefore && EnumHasAnyFlags(DepthStateBefore->Access, ERHIAccess::CopySrc | ERHIAccess::CopyDest))
			{
				check(StencilStateBefore->Access        == DepthStateBefore->Access);
				check(StencilStateBefore->GetLastPass() == DepthStateBefore->GetLastPass());

				// Case 2a: depth unknown, so transition to match stencil.
				if (!DepthStateAfter)
				{
					DepthStateAfter = AllocSubresource(*StencilStateAfter);
				}
				// Case 2b: stencil unknown, so transition to match depth.
				else if (!StencilStateAfter)
				{
					StencilStateAfter = AllocSubresource(*DepthStateAfter);
				}
			}
		}
	}

	for (uint32 SubresourceIndex = 0; SubresourceIndex < SubresourceCount; ++SubresourceIndex)
	{
		FRDGSubresourceState*& SubresourceStateBefore = StateBefore[SubresourceIndex];
		FRDGSubresourceState* SubresourceStateAfter = StateAfter[SubresourceIndex];

		if (!SubresourceStateAfter)
		{
			continue;
		}

		if (FilterSubresourceLambda(SubresourceStateAfter, SubresourceIndex))
		{
			check(SubresourceStateAfter->Access != ERHIAccess::Unknown);

			if (SubresourceStateBefore && FRDGSubresourceState::IsTransitionRequired(*SubresourceStateBefore, *SubresourceStateAfter))
			{
				const FRDGTextureSubresource Subresource = Layout.GetSubresource(SubresourceIndex);

				EResourceTransitionFlags Flags = SubresourceStateAfter->Flags;

				FRDGTransitionInfo Info;
				Info.AccessBefore            = (uint64)SubresourceStateBefore->Access;
				Info.AccessAfter             = (uint64)SubresourceStateAfter->Access;
				Info.ResourceHandle          = (uint64)Texture->Handle.GetIndex();
				Info.ResourceType            = (uint64)ERDGViewableResourceType::Texture;
				Info.ResourceTransitionFlags = (uint64)Flags;
				Info.Texture.ArraySlice      = Subresource.ArraySlice;
				Info.Texture.MipIndex        = Subresource.MipIndex;
				Info.Texture.PlaneSlice      = Subresource.PlaneSlice;

				AddTransition(Texture, *SubresourceStateBefore, *SubresourceStateAfter, Info);
			}
		}

		SubresourceStateBefore = SubresourceStateAfter;
	}
}

template <typename FilterSubresourceLambdaType>
void FRDGBuilder::AddBufferTransition(FRDGBufferRef Buffer, FRDGSubresourceState*& StateBefore, FRDGSubresourceState* StateAfter, FilterSubresourceLambdaType&& FilterSubresourceLambda)
{
	check(StateAfter);
	check(StateAfter->Access != ERHIAccess::Unknown);

	if (FilterSubresourceLambda(StateAfter))
	{
		check(StateBefore);

		if (FRDGSubresourceState::IsTransitionRequired(*StateBefore, *StateAfter))
		{
			FRDGTransitionInfo Info;
			Info.AccessBefore            = (uint64)StateBefore->Access;
			Info.AccessAfter             = (uint64)StateAfter->Access;
			Info.ResourceHandle          = (uint64)Buffer->Handle.GetIndex();
			Info.ResourceType            = (uint64)ERDGViewableResourceType::Buffer;
			Info.ResourceTransitionFlags = (uint64)StateAfter->Flags;
			Info.Buffer.CommitSize       = GetReservedCommitSize(StateAfter->ReservedCommitHandle);

			AddTransition(Buffer, *StateBefore, *StateAfter, Info);
		}
	}

	StateBefore = StateAfter;
}

void FRDGBuilder::AddTransition(
	FRDGViewableResource* Resource,
	FRDGSubresourceState StateBefore,
	FRDGSubresourceState StateAfter,
	FRDGTransitionInfo TransitionInfo)
{
	const ERHIPipeline Graphics = ERHIPipeline::Graphics;
	const ERHIPipeline AsyncCompute = ERHIPipeline::AsyncCompute;

#if RDG_ENABLE_DEBUG
	StateBefore.Validate();
	StateAfter.Validate();
#endif

	if (IsImmediateMode())
	{
		// Immediate mode simply enqueues the barrier into the 'after' pass. Everything is on the graphics pipe.
		AddToPrologueBarriers(StateAfter.FirstPass[Graphics], [&](FRDGBarrierBatchBegin& Barriers)
		{
			Barriers.AddTransition(Resource, TransitionInfo);
		});
		return;
	}

	const ERHIPipeline PipelinesBefore = StateBefore.GetPipelines();
	const ERHIPipeline PipelinesAfter = StateAfter.GetPipelines();

	check(PipelinesBefore != ERHIPipeline::None && PipelinesAfter != ERHIPipeline::None);
	checkf(StateBefore.GetLastPass() <= StateAfter.GetFirstPass(), TEXT("Submitted a state for '%s' that begins before our previous state has ended."), Resource->Name);

	const FRDGPassHandlesByPipeline& PassesBefore = StateBefore.LastPass;
	const FRDGPassHandlesByPipeline& PassesAfter = StateAfter.FirstPass;

	// 1-to-1 same-pipe transition
	if (PipelinesBefore == PipelinesAfter && PipelinesAfter != ERHIPipeline::All)
	{
		const FRDGPassHandle BeginPassHandle = StateBefore.LastPass[PipelinesAfter];
		const FRDGPassHandle EndPassHandle   = StateAfter.FirstPass[PipelinesAfter];

		// Split the transition from the epilogue of the begin pass to the prologue of the end pass.
		if (BeginPassHandle < EndPassHandle)
		{
			FRDGPass* BeginPass = GetEpilogueBarrierPass(BeginPassHandle);
			FRDGBarrierBatchBegin* BarriersToBegin = &BeginPass->GetEpilogueBarriersToBeginFor(Allocators.Transition, TransitionCreateQueue, PipelinesAfter);
			BarriersToBegin->AddTransition(Resource, TransitionInfo);
			AddToPrologueBarriersToEnd(EndPassHandle, *BarriersToBegin);
		}
		// This is an immediate transition in the same pass on the same pipe done in the epilogue of the pass.
		else if (StateAfter.BarrierLocation == ERDGBarrierLocation::Epilogue)
		{
			FRDGPass* BeginPass = GetEpilogueBarrierPass(BeginPassHandle);
			FRDGBarrierBatchBegin* BarriersToBegin = &BeginPass->GetEpilogueBarriersToBeginFor(Allocators.Transition, TransitionCreateQueue, PipelinesAfter);
			BarriersToBegin->AddTransition(Resource, TransitionInfo);
			AddToEpilogueBarriersToEnd(EndPassHandle, *BarriersToBegin);
		}
		// This is an immediate transition in the same pass on the same pipe done in the prologue of the pass.
		else
		{
			FRDGPass* BeginPass = GetPrologueBarrierPass(BeginPassHandle);
			FRDGBarrierBatchBegin* BarriersToBegin = &BeginPass->GetPrologueBarriersToBegin(Allocators.Transition, TransitionCreateQueue);
			BarriersToBegin->AddTransition(Resource, TransitionInfo);
			AddToPrologueBarriersToEnd(EndPassHandle, *BarriersToBegin);
		}
	}
	// 1-to-1 or 1-to-N cross-pipe transition.
	else if (PipelinesBefore != ERHIPipeline::All)
	{
		const FRDGPassHandle BeginPassHandle = StateBefore.LastPass[PipelinesBefore];
		FRDGPass* BeginPass = GetEpilogueBarrierPass(BeginPassHandle);
		FRDGBarrierBatchBegin* BarriersToBegin = &BeginPass->GetEpilogueBarriersToBeginFor(Allocators.Transition, TransitionCreateQueue, PipelinesAfter);
		BarriersToBegin->AddTransition(Resource, TransitionInfo);

		for (ERHIPipeline Pipeline : MakeFlagsRange(ERHIPipeline::All))
		{
			/** If doing a 1-to-N transition and this is the same pipe as the begin, we end it immediately afterwards in the epilogue
			 *  of the begin pass. This is because we can't guarantee that the other pipeline won't join back before the end. This can
			 *  happen if the forking async compute pass joins back to graphics (via another independent transition) before the current
			 *  graphics transition is ended.
			 *
			 *  Async Compute Pipe:               EndA  BeginB
			 *                                   /            \
			 *  Graphics Pipe:            BeginA               EndB   EndA
			 *
			 *  A is our 1-to-N transition and B is a future transition of the same resource that we haven't evaluated yet. Instead, the
			 *  same pipe End is performed in the epilogue of the begin pass, which removes the spit barrier but simplifies the tracking:
			 *
			 *  Async Compute Pipe:               EndA  BeginB
			 *                                   /            \
			 *  Graphics Pipe:            BeginA  EndA         EndB
			 */
			if ((PipelinesBefore == Pipeline && PipelinesAfter == ERHIPipeline::All))
			{
				AddToEpilogueBarriersToEnd(BeginPassHandle, *BarriersToBegin);
			}
			else if (EnumHasAnyFlags(PipelinesAfter, Pipeline))
			{
				AddToPrologueBarriersToEnd(PassesAfter[Pipeline], *BarriersToBegin);
			}
		}
	}
	// N-to-1 or N-to-N
	else
	{
		FRDGBarrierBatchBeginId Id;
		Id.PipelinesAfter = PipelinesAfter;
		for (ERHIPipeline Pipeline : MakeFlagsRange(ERHIPipeline::All))
		{
			Id.Passes[Pipeline] = GetEpilogueBarrierPassHandle(PassesBefore[Pipeline]);
		}

		FRDGBarrierBatchBegin*& BarriersToBegin = BarrierBatchMap.FindOrAdd(Id);

		if (!BarriersToBegin)
		{
			FRDGPassesByPipeline BarrierBatchPasses;
			BarrierBatchPasses[Graphics]     = Passes[Id.Passes[Graphics]];
			BarrierBatchPasses[AsyncCompute] = Passes[Id.Passes[AsyncCompute]];

			BarriersToBegin = Allocators.Transition.AllocNoDestruct<FRDGBarrierBatchBegin>(PipelinesBefore, PipelinesAfter, GetEpilogueBarriersToBeginDebugName(PipelinesAfter), BarrierBatchPasses);
			TransitionCreateQueue.Emplace(BarriersToBegin);

			for (FRDGPass* Pass : BarrierBatchPasses)
			{
				Pass->SharedEpilogueBarriersToBegin.Add(BarriersToBegin);
			}
		}

		BarriersToBegin->AddTransition(Resource, TransitionInfo);

		for (ERHIPipeline Pipeline : MakeFlagsRange(PipelinesAfter))
		{
			FRDGPassHandle PassAfter = PassesAfter[Pipeline];

			// If the end pass is the same as the begin pass on one pipe, end it in the epilogue instead.
			if (PassesBefore[Pipeline] == PassesAfter[Pipeline])
			{
				check(StateAfter.BarrierLocation == ERDGBarrierLocation::Epilogue);
				AddToEpilogueBarriersToEnd(PassAfter, *BarriersToBegin);
			}
			else
			{
				AddToPrologueBarriersToEnd(PassAfter, *BarriersToBegin);
			}
		}
	}
}

void FRDGBuilder::AddAliasingTransition(FRDGPassHandle BeginPassHandle, FRDGPassHandle EndPassHandle, FRDGViewableResource* Resource, const FRHITransientAliasingInfo& Info)
{
	check(BeginPassHandle <= EndPassHandle);

	FRDGBarrierBatchBegin* BarriersToBegin{};
	FRDGPass* EndPass{};

	if (BeginPassHandle == EndPassHandle)
	{
		FRDGPass* BeginPass = Passes[BeginPassHandle];
		EndPass = BeginPass;

		check(GetPrologueBarrierPassHandle(BeginPassHandle) == BeginPassHandle);

		BarriersToBegin = &BeginPass->GetPrologueBarriersToBegin(Allocators.Transition, TransitionCreateQueue);
	}
	else
	{
		FRDGPass* BeginPass = GetEpilogueBarrierPass(BeginPassHandle);
		EndPass = Passes[EndPassHandle];

		check(GetPrologueBarrierPassHandle(EndPassHandle) == EndPassHandle);

		BarriersToBegin = &BeginPass->GetEpilogueBarriersToBeginFor(Allocators.Transition, TransitionCreateQueue, EndPass->GetPipeline());
	}

	BarriersToBegin->AddAlias(Resource, Info);
	EndPass->GetPrologueBarriersToEnd(Allocators.Transition).AddDependency(BarriersToBegin);
}

FRHITransientAllocationFences FRDGBuilder::GetAllocateFences(FRDGViewableResource* Resource) const
{
	FRDGPassHandle FirstPassHandle = Resource->FirstPass;

	ERHIPipeline Pipeline = GetPassPipeline(FirstPassHandle);

	FRHITransientAllocationFences Fences(Pipeline);

	if (Pipeline == ERHIPipeline::Graphics)
	{
		Fences.SetGraphics(FirstPassHandle.GetIndex());
	}
	else
	{
		const FRDGPass* FirstPass = Passes[FirstPassHandle];

		Fences.SetAsyncCompute(
			FirstPassHandle.GetIndex(),
			TInterval<uint32>(FirstPass->GraphicsForkPass.GetIndex(), FirstPass->GraphicsJoinPass.GetIndex()));
	}

	return Fences;
}

FRHITransientAllocationFences FRDGBuilder::GetDeallocateFences(FRDGViewableResource* Resource) const
{
	FRDGPassHandle GraphicsPassHandle     = Resource->LastPasses[ERHIPipeline::Graphics];
	FRDGPassHandle AsyncComputePassHandle = Resource->LastPasses[ERHIPipeline::AsyncCompute];

	FRDGPassHandle GraphicsForkPass;
	FRDGPassHandle GraphicsJoinPass;

	ERHIPipeline Pipelines = GraphicsPassHandle.IsValid() ? ERHIPipeline::Graphics : ERHIPipeline::None;

	if (AsyncComputePassHandle.IsValid())
	{
		Pipelines |= ERHIPipeline::AsyncCompute;

		const FRDGPass* Pass = Passes[AsyncComputePassHandle];
		GraphicsForkPass = Pass->GraphicsForkPass;
		GraphicsJoinPass = Pass->GraphicsJoinPass;

		if (GraphicsPassHandle.IsValid())
		{
			// Ignore graphics pass if earlier than the fork to async compute.
			if (GraphicsPassHandle <= GraphicsForkPass)
			{
				GraphicsPassHandle = {};
			}
			// Ignore async compute pass if earlier than the join back to graphics.
			else if (GraphicsPassHandle >= GraphicsJoinPass)
			{
				AsyncComputePassHandle = {};
			}
		}
	}

	FRHITransientAllocationFences Fences(Pipelines);

	if (GraphicsPassHandle.IsValid())
	{
		Fences.SetGraphics(GraphicsPassHandle.GetIndex());
	}

	if (AsyncComputePassHandle.IsValid())
	{
		Fences.SetAsyncCompute(AsyncComputePassHandle.GetIndex(), TInterval<uint32>(GraphicsForkPass.GetIndex(), GraphicsJoinPass.GetIndex()));
	}

	return Fences;
}

///////////////////////////////////////////////////////////////////////////////////////////////////

TRefCountPtr<IPooledRenderTarget> FRDGBuilder::AllocatePooledRenderTargetRHI(FRHICommandListBase& InRHICmdList, FRDGTextureRef Texture)
{
	return GRenderTargetPool.FindFreeElement(InRHICmdList, Texture->Desc, Texture->Name);
}

TRefCountPtr<FRDGPooledBuffer> FRDGBuilder::AllocatePooledBufferRHI(FRHICommandListBase& InRHICmdList, FRDGBufferRef Buffer)
{
	Buffer->FinalizeDesc();
	return GRenderGraphResourcePool.FindFreeBuffer(InRHICmdList, Buffer->Desc, Buffer->Name);
}

///////////////////////////////////////////////////////////////////////////////////////////////////

void FRDGBuilder::SetExternalPooledRenderTargetRHI(FRDGTexture* Texture, IPooledRenderTarget* RenderTarget)
{
	Texture->RenderTarget = RenderTarget;

	if (FRHITransientTexture* TransientTexture = RenderTarget->GetTransientTexture())
	{
		FRDGTransientRenderTarget* TransientRenderTarget = static_cast<FRDGTransientRenderTarget*>(RenderTarget);
		Texture->Allocation = TRefCountPtr<FRDGTransientRenderTarget>(TransientRenderTarget);

		SetTransientTextureRHI(Texture, TransientTexture);
	}
	else
	{
		FPooledRenderTarget* PooledRenderTarget = static_cast<FPooledRenderTarget*>(RenderTarget);
		Texture->Allocation = TRefCountPtr<FPooledRenderTarget>(PooledRenderTarget);

		SetPooledTextureRHI(Texture, &PooledRenderTarget->PooledTexture);
	}
}

void FRDGBuilder::SetPooledTextureRHI(FRDGTexture* Texture, FRDGPooledTexture* PooledTexture)
{
	check(!Texture->ResourceRHI);

	Texture->SetRHI(PooledTexture->GetRHI());
	Texture->ViewCache = &PooledTexture->ViewCache;

	FRDGTexture*& Owner = *PooledTextureOwnershipMap.FindOrAdd(PooledTexture, nullptr);

	// Link the previous alias to this one.
	if (Owner)
	{
		Texture->PreviousOwner = Owner->Handle;
		Owner->NextOwner = Texture->Handle;
		Owner->bSkipLastTransition = true;
	}
	else
	{
		Texture->bSkipLastTransition = EnumHasAnyFlags(Texture->Desc.Flags, TexCreate_Memoryless);
	}

	Owner = Texture;
}

void FRDGBuilder::SetDiscardPass(FRDGTexture* Texture, FRHITransientTexture* TransientTexture)
{
	if (TransientTexture->IsDiscarded())
	{
		Texture->DiscardPass = FRDGPassHandle(FMath::Min<uint32>(TransientTexture->GetDiscardPass(), GetEpiloguePassHandle().GetIndex()));
	}
}

void FRDGBuilder::SetTransientTextureRHI(FRDGTexture* Texture, FRHITransientTexture* TransientTexture)
{
	Texture->SetRHI(TransientTexture->GetRHI());
	Texture->TransientTexture = TransientTexture;
	Texture->ViewCache = &TransientTexture->ViewCache;
	Texture->AliasingOverlaps = TransientTexture->GetAliasingOverlaps();

	SetDiscardPass(Texture, TransientTexture);
}

void FRDGBuilder::SetExternalPooledBufferRHI(FRDGBuffer* Buffer, const TRefCountPtr<FRDGPooledBuffer>& PooledBuffer)
{
	SetPooledBufferRHI(Buffer, PooledBuffer);
	Buffer->Allocation = PooledBuffer;
}

void FRDGBuilder::SetPooledBufferRHI(FRDGBuffer* Buffer, FRDGPooledBuffer* PooledBuffer)
{
	Buffer->SetRHI(PooledBuffer->GetRHI());
	Buffer->PooledBuffer = PooledBuffer;
	Buffer->ViewCache = &PooledBuffer->ViewCache;

	FRDGBuffer*& Owner = *PooledBufferOwnershipMap.FindOrAdd(PooledBuffer, nullptr);

	// Link the previous owner to this one.
	if (Owner)
	{
		Buffer->PreviousOwner = Owner->Handle;
		Owner->NextOwner = Buffer->Handle;
		Owner->bSkipLastTransition = true;
	}

	Owner = Buffer;
}

void FRDGBuilder::SetTransientBufferRHI(FRDGBuffer* Buffer, FRHITransientBuffer* TransientBuffer)
{
	Buffer->SetRHI(TransientBuffer->GetRHI());
	Buffer->TransientBuffer = TransientBuffer;
	Buffer->ViewCache = &TransientBuffer->ViewCache;
	Buffer->AliasingOverlaps = TransientBuffer->GetAliasingOverlaps();

	if (TransientBuffer->IsDiscarded())
	{
		Buffer->DiscardPass = FRDGPassHandle(FMath::Min<uint32>(TransientBuffer->GetDiscardPass(), GetEpiloguePassHandle().GetIndex()));
	}
}

///////////////////////////////////////////////////////////////////////////////////////////////////

void FRDGBuilder::InitTextureViewRHI(FRHICommandListBase& InRHICmdList, FRDGTextureSRVRef SRV)
{
	check(SRV && !SRV->ResourceRHI);

	FRDGTextureRef Texture = SRV->Desc.Texture;
	FRHITexture* TextureRHI = Texture->GetRHIUnchecked();
	check(TextureRHI);

	SRV->ResourceRHI = Texture->ViewCache->GetOrCreateSRV(InRHICmdList, TextureRHI, SRV->Desc);
}

void FRDGBuilder::InitTextureViewRHI(FRHICommandListBase& InRHICmdList, FRDGTextureUAVRef UAV)
{
	check(UAV && !UAV->ResourceRHI);

	FRDGTextureRef Texture = UAV->Desc.Texture;
	FRHITexture* TextureRHI = Texture->GetRHIUnchecked();
	check(TextureRHI);

	UAV->ResourceRHI = Texture->ViewCache->GetOrCreateUAV(InRHICmdList, TextureRHI, UAV->Desc);
}

void FRDGBuilder::InitBufferViewRHI(FRHICommandListBase& InRHICmdList, FRDGBufferSRVRef SRV)
{
	check(SRV);

	if (SRV->HasRHI())
	{
		return;
	}

	FRDGBufferRef Buffer = SRV->Desc.Buffer;
	FRHIBuffer* BufferRHI = Buffer->GetRHIUnchecked();
	check(BufferRHI);

	FRHIBufferSRVCreateInfo SRVCreateInfo = SRV->Desc;

	if (EnumHasAnyFlags(Buffer->Desc.Usage, EBufferUsageFlags::StructuredBuffer))
	{
		// RDG allows structured buffer views to be typed, but the view creation logic requires that it
		// be unknown (as do platform APIs -- structured buffers are not typed). This could be validated
		// at the high level but the current API makes it confusing. For now, it's considered a no-op.
		SRVCreateInfo.Format = PF_Unknown;
	}

	SRV->ResourceRHI = Buffer->ViewCache->GetOrCreateSRV(InRHICmdList, BufferRHI, SRVCreateInfo);
}

void FRDGBuilder::InitBufferViewRHI(FRHICommandListBase& InRHICmdList, FRDGBufferUAV* UAV)
{
	check(UAV);

	if (UAV->HasRHI())
	{
		return;
	}

	FRDGBufferRef Buffer = UAV->Desc.Buffer;
	check(Buffer);

	FRHIBufferUAVCreateInfo UAVCreateInfo = UAV->Desc;

	if (EnumHasAnyFlags(Buffer->Desc.Usage, EBufferUsageFlags::StructuredBuffer))
	{
		// RDG allows structured buffer views to be typed, but the view creation logic requires that it
		// be unknown (as do platform APIs -- structured buffers are not typed). This could be validated
		// at the high level but the current API makes it confusing. For now, it's considered a no-op.
		UAVCreateInfo.Format = PF_Unknown;
	}

	UAV->ResourceRHI = Buffer->ViewCache->GetOrCreateUAV(InRHICmdList, Buffer->GetRHIUnchecked(), UAVCreateInfo);
}

void FRDGBuilder::InitViewRHI(FRHICommandListBase& InRHICmdList, FRDGView* View)
{
	check(!View->ResourceRHI);

	switch (View->Type)
	{
	case ERDGViewType::TextureUAV:
		InitTextureViewRHI(InRHICmdList, static_cast<FRDGTextureUAV*>(View));
		break;
	case ERDGViewType::TextureSRV:
		InitTextureViewRHI(InRHICmdList, static_cast<FRDGTextureSRV*>(View));
		break;
	case ERDGViewType::BufferUAV:
		InitBufferViewRHI(InRHICmdList, static_cast<FRDGBufferUAV*>(View));
		break;
	case ERDGViewType::BufferSRV:
		InitBufferViewRHI(InRHICmdList, static_cast<FRDGBufferSRV*>(View));
		break;
	}
}

///////////////////////////////////////////////////////////////////////////////////////////////////

#if RDG_ENABLE_DEBUG

void FRDGBuilder::VisualizePassOutputs(const FRDGPass* Pass)
{
#if SUPPORTS_VISUALIZE_TEXTURE
	if (!GVisualizeTexture.IsRequestedView() || !AuxiliaryPasses.IsVisualizeAllowed())
	{
		return;
	}

	RDG_RECURSION_COUNTER_SCOPE(AuxiliaryPasses.Visualize);

	Pass->GetParameters().EnumerateTextures([&](FRDGParameter Parameter)
	{
		switch (Parameter.GetType())
		{
		case UBMT_RDG_TEXTURE_ACCESS:
		{
			if (FRDGTextureAccess TextureAccess = Parameter.GetAsTextureAccess())
			{
				if (IsWritableAccess(TextureAccess.GetAccess()))
				{
					if (TOptional<uint32> CaptureId = GVisualizeTexture.ShouldCapture(TextureAccess->Name, TextureAccess.GetSubresourceRange().MipIndex))
					{
						GVisualizeTexture.CreateContentCapturePass(*this, TextureAccess.GetTexture(), *CaptureId);
					}
				}
			}
		}
		break;
		case UBMT_RDG_TEXTURE_ACCESS_ARRAY:
		{
			const FRDGTextureAccessArray& TextureAccessArray = Parameter.GetAsTextureAccessArray();

			for (FRDGTextureAccess TextureAccess : TextureAccessArray)
			{
				if (IsWritableAccess(TextureAccess.GetAccess()))
				{
					if (TOptional<uint32> CaptureId = GVisualizeTexture.ShouldCapture(TextureAccess->Name, TextureAccess.GetSubresourceRange().MipIndex))
					{
						GVisualizeTexture.CreateContentCapturePass(*this, TextureAccess.GetTexture(), *CaptureId);
					}
				}
			}
		}
		break;
		case UBMT_RDG_TEXTURE_UAV:
		{
			if (FRDGTextureUAVRef UAV = Parameter.GetAsTextureUAV())
			{
				FRDGTextureRef Texture = UAV->Desc.Texture;
				if (TOptional<uint32> CaptureId = GVisualizeTexture.ShouldCapture(Texture->Name, UAV->Desc.MipLevel))
				{
					GVisualizeTexture.CreateContentCapturePass(*this, Texture, *CaptureId);
				}
			}
		}
		break;
		case UBMT_RENDER_TARGET_BINDING_SLOTS:
		{
			const FRenderTargetBindingSlots& RenderTargets = Parameter.GetAsRenderTargetBindingSlots();

			RenderTargets.Enumerate([&](FRenderTargetBinding RenderTarget)
			{
				FRDGTextureRef Texture = RenderTarget.GetTexture();
				if (TOptional<uint32> CaptureId = GVisualizeTexture.ShouldCapture(Texture->Name, RenderTarget.GetMipIndex()))
				{
					GVisualizeTexture.CreateContentCapturePass(*this, Texture, *CaptureId);
				}
			});

			const FDepthStencilBinding& DepthStencil = RenderTargets.DepthStencil;

			if (FRDGTextureRef Texture = DepthStencil.GetTexture())
			{
				const bool bHasStoreAction = DepthStencil.GetDepthStencilAccess().IsAnyWrite();

				if (bHasStoreAction)
				{
					const uint32 MipIndex = 0;
					if (TOptional<uint32> CaptureId = GVisualizeTexture.ShouldCapture(Texture->Name, MipIndex))
					{
						GVisualizeTexture.CreateContentCapturePass(*this, Texture, *CaptureId);
					}
				}
			}
		}
		break;
		}
	});
#endif
}

void FRDGBuilder::ClobberPassOutputs(const FRDGPass* Pass)
{
	if (!GRDGValidation || !GRDGClobberResources || !AuxiliaryPasses.IsClobberAllowed())
	{
		return;
	}

	RDG_RECURSION_COUNTER_SCOPE(AuxiliaryPasses.Clobber);
	RDG_EVENT_SCOPE(*this, "RDG ClobberResources");

	const FLinearColor ClobberColor = GetClobberColor();

	const auto ClobberTextureUAV = [&](FRDGTextureUAV* TextureUAV)
	{
		if (IsInteger(TextureUAV->GetParent()->Desc.Format))
		{
			AddClearUAVPass(*this, TextureUAV, GetClobberBufferValue());
		}
		else if (IsBlockCompressedFormat(TextureUAV->GetParent()->Desc.Format))
		{
			// We shouldn't see BCn UAVs if SupportsUAVFormatAliasing is false in the first place, but it can't hurt to check.
			if (GRHIGlobals.SupportsUAVFormatAliasing)
			{
				AddClearUAVPass(*this, TextureUAV, GetClobberBufferValue());
			}
		}
		else
		{
			AddClearUAVPass(*this, TextureUAV, ClobberColor);
		}
	};

	const auto ClobberTextureAccess = [&](FRDGTextureAccess TextureAccess)
	{
		if (IsWritableAccess(TextureAccess.GetAccess()))
		{
			FRDGTextureRef Texture = TextureAccess.GetTexture();
	
			if (Texture && UserValidation.TryMarkForClobber(Texture))
			{
				if (EnumHasAnyFlags(TextureAccess.GetAccess(), ERHIAccess::UAVMask))
				{
					for (int32 MipLevel = 0; MipLevel < Texture->Desc.NumMips; MipLevel++)
					{
						ClobberTextureUAV(CreateUAV(FRDGTextureUAVDesc(Texture, MipLevel)));
					}
				}
				else if (EnumHasAnyFlags(TextureAccess.GetAccess(), ERHIAccess::RTV))
				{
					AddClearRenderTargetPass(*this, Texture, ClobberColor);
				}
			}
		}
	};

	const auto ClobberBufferAccess = [&](FRDGBufferAccess BufferAccess)
	{
		if (IsWritableAccess(BufferAccess.GetAccess()))
		{
			FRDGBufferRef Buffer = BufferAccess.GetBuffer();
	
			if (Buffer && UserValidation.TryMarkForClobber(Buffer))
			{
				AddClearUAVPass(*this, CreateUAV(Buffer), GetClobberBufferValue());
			}
		}
	};

	Pass->GetParameters().Enumerate([&](FRDGParameter Parameter)
	{
		switch (Parameter.GetType())
		{
		case UBMT_RDG_BUFFER_UAV:
		{
			if (FRDGBufferUAVRef UAV = Parameter.GetAsBufferUAV())
			{
				FRDGBufferRef Buffer = UAV->GetParent();

				if (UserValidation.TryMarkForClobber(Buffer))
				{
					AddClearUAVPass(*this, UAV, GetClobberBufferValue());
				}
			}
		}
		break;
		case UBMT_RDG_TEXTURE_ACCESS:
		{
			ClobberTextureAccess(Parameter.GetAsTextureAccess());
		}
		break;
		case UBMT_RDG_TEXTURE_ACCESS_ARRAY:
		{
			const FRDGTextureAccessArray& TextureAccessArray = Parameter.GetAsTextureAccessArray();

			for (FRDGTextureAccess TextureAccess : TextureAccessArray)
			{
				ClobberTextureAccess(TextureAccess);
			}
		}
		break;
		case UBMT_RDG_BUFFER_ACCESS:
		{
			ClobberBufferAccess(Parameter.GetAsBufferAccess());
		}
		break;
		case UBMT_RDG_BUFFER_ACCESS_ARRAY:
		{
			const FRDGBufferAccessArray& BufferAccessArray = Parameter.GetAsBufferAccessArray();

			for (FRDGBufferAccess BufferAccess : BufferAccessArray)
			{
				ClobberBufferAccess(BufferAccess);
			}
		}
		break;
		case UBMT_RDG_TEXTURE_UAV:
		{
			if (FRDGTextureUAVRef UAV = Parameter.GetAsTextureUAV())
			{
				FRDGTextureRef Texture = UAV->GetParent();

				if (UserValidation.TryMarkForClobber(Texture))
				{
					if (Texture->Desc.NumMips == 1)
					{
						ClobberTextureUAV(UAV);
					}
					else
					{
						for (int32 MipLevel = 0; MipLevel < Texture->Desc.NumMips; MipLevel++)
						{
							ClobberTextureUAV(CreateUAV(FRDGTextureUAVDesc(Texture, MipLevel)));
						}
					}
				}
			}
		}
		break;
		case UBMT_RENDER_TARGET_BINDING_SLOTS:
		{
			const FRenderTargetBindingSlots& RenderTargets = Parameter.GetAsRenderTargetBindingSlots();

			RenderTargets.Enumerate([&](FRenderTargetBinding RenderTarget)
			{
				FRDGTextureRef Texture = RenderTarget.GetTexture();

				if (UserValidation.TryMarkForClobber(Texture))
				{
					AddClearRenderTargetPass(*this, Texture, ClobberColor);
				}
			});

			if (FRDGTextureRef Texture = RenderTargets.DepthStencil.GetTexture())
			{
				if (UserValidation.TryMarkForClobber(Texture))
				{
					AddClearDepthStencilPass(*this, Texture, true, GetClobberDepth(), true, GetClobberStencil());
				}
			}
		}
		break;
		}
	});
}

#endif //! RDG_ENABLE_DEBUG

#if WITH_MGPU
void FRDGBuilder::ForceCopyCrossGPU()
{
	const auto GetLastProducerGPUMask = [](FRDGProducerStatesByPipeline& LastProducers) -> TOptional<FRHIGPUMask>
	{
		for (const FRDGProducerState& LastProducer : LastProducers)
		{
			if (LastProducer.Pass && !LastProducer.Pass->bCulled)
			{
				return LastProducer.Pass->GPUMask;
			}
		}
		return {};
	};

	Experimental::TRobinHoodHashMap<FRHIBuffer*, FRHIGPUMask, DefaultKeyFuncs<FRHIBuffer*>, FRDGArrayAllocator> BuffersToTransfer;
	BuffersToTransfer.Reserve(ExternalBuffers.Num());

	for (auto& ExternalBuffer : ExternalBuffers)
	{
		FRHIBuffer* BufferRHI = ExternalBuffer.Key;
		FRDGBuffer* BufferRDG = ExternalBuffer.Value;

		if (!EnumHasAnyFlags(BufferRDG->Desc.Usage, BUF_MultiGPUAllocate | BUF_MultiGPUGraphIgnore))
		{
			TOptional<FRHIGPUMask> GPUMask = GetLastProducerGPUMask(BufferRDG->LastProducer);

			if (GPUMask)
			{
				BuffersToTransfer.FindOrAdd(BufferRHI, *GPUMask);
			}
		}
	}

	Experimental::TRobinHoodHashMap<FRHITexture*, FRHIGPUMask, DefaultKeyFuncs<FRHITexture*>, FRDGArrayAllocator> TexturesToTransfer;
	TexturesToTransfer.Reserve(ExternalTextures.Num());

	for (auto& ExternalTexture : ExternalTextures)
	{
		FRHITexture* TextureRHI = ExternalTexture.Key;
		FRDGTexture* TextureRDG = ExternalTexture.Value;

		if (!EnumHasAnyFlags(TextureRDG->Desc.Flags, TexCreate_MultiGPUGraphIgnore))
		{
			for (auto& LastProducer : TextureRDG->LastProducers)
			{
				TOptional<FRHIGPUMask> GPUMask = GetLastProducerGPUMask(LastProducer);

				if (GPUMask)
				{
					TexturesToTransfer.FindOrAdd(TextureRHI, *GPUMask);
					break;
				}
			}
		}
	}

	// Now that we've got the list of external resources, and the GPU they were last written to, make a list of what needs to
	// be propagated to other GPUs.
	TArray<FTransferResourceParams, FRDGArrayAllocator> Transfers;
	Transfers.Reserve(BuffersToTransfer.Num() + TexturesToTransfer.Num());
	const FRHIGPUMask AllGPUMask = FRHIGPUMask::All();
	const bool bPullData = false;
	const bool bLockstepGPUs = true;

	for (auto& KeyValue : BuffersToTransfer)
	{
		FRHIBuffer* Buffer  = KeyValue.Key;
		FRHIGPUMask GPUMask = KeyValue.Value;

		for (uint32 GPUIndex : AllGPUMask)
		{
			if (!GPUMask.Contains(GPUIndex))
			{
				Transfers.Add(FTransferResourceParams(Buffer, GPUMask.GetFirstIndex(), GPUIndex, bPullData, bLockstepGPUs));
			}
		}
	}

	for (auto& KeyValue : TexturesToTransfer)
	{
		FRHITexture* Texture = KeyValue.Key;
		FRHIGPUMask GPUMask  = KeyValue.Value;

		for (uint32 GPUIndex : AllGPUMask)
		{
			if (!GPUMask.Contains(GPUIndex))
			{
				Transfers.Add(FTransferResourceParams(Texture, GPUMask.GetFirstIndex(), GPUIndex, bPullData, bLockstepGPUs));
			}
		}
	}

	if (Transfers.Num())
	{
		RHICmdList.TransferResources(Transfers);
	}
}
#endif  // WITH_MGPU
