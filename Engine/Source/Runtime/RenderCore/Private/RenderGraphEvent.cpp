// Copyright Epic Games, Inc. All Rights Reserved.

#include "RenderGraphEvent.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphPrivate.h"
#include "RenderGraphPass.h"
#include "RenderResource.h"
#include "Containers/List.h"
#include "DynamicRenderScaling.h"

/* Full frame of timestamp queries in flight. */
class FRDGTimingFrame
{
public:
	static const int32 kTimingScopesPreallocation = 64;
	static const int32 kTimestampQueriesPreallocation = kTimingScopesPreallocation * 2;

	FRHIRenderQueryPool* QueryPool;

	/* Scopes of a timing budget. */
	struct FInFlightTimingScope
	{
		DynamicRenderScaling::FBudget const& Budget;
		FRHIPooledRenderQuery Begin, End;
		bool bUsed = false;

		FInFlightTimingScope(DynamicRenderScaling::FBudget const& Budget, FRHIRenderQueryPool* Pool)
			: Budget(Budget)
			, Begin(Pool->AllocateQuery())
			, End(Pool->AllocateQuery())
		{}
	};

	// Arrays of all scopes issued in this frame.
	TArray<FInFlightTimingScope> TimingScopes;
	int32 NextScope = 0;

	// Fence for the RHI command to be completed before polling RHI queries.
	FGraphEventRef RHIEndFence;

	FRDGTimingFrame* Next = nullptr;

	DynamicRenderScaling::TMap<uint64> Timings;

	FRDGTimingFrame(FRHIRenderQueryPool* QueryPool)
		: QueryPool(QueryPool)
	{
		Timings.SetAll(uint64(0));
	}

	~FRDGTimingFrame()
	{}

	int32 AllocateScope(DynamicRenderScaling::FBudget const& Budget)
	{
		if (TimingScopes.Num() == 0)
		{
			TimingScopes.Reserve(kTimingScopesPreallocation);
		}
		else if (TimingScopes.Num() == TimingScopes.Max())
		{
			TimingScopes.Reserve(TimingScopes.Max() * 2);
		}

		return TimingScopes.Emplace(Budget, QueryPool);
	}

	void BeginScope(int32 ScopeIndex, FRHICommandList& RHICmdList)
	{
		RHICmdList.EndRenderQuery(TimingScopes[ScopeIndex].Begin.GetQuery());
		TimingScopes[ScopeIndex].bUsed = true;
	}

	void EndScope(int32 ScopeIndex, FRHICommandList& RHICmdList)
	{
		RHICmdList.EndRenderQuery(TimingScopes[ScopeIndex].End.GetQuery());
		TimingScopes[ScopeIndex].bUsed = true;
	}

	// Returns true when the results are available
	bool GatherResults(bool bWait)
	{
		check(IsInRenderingThread());

		// Ensure the RHI thread fence has passed, meaning all the queries have been begun/ended by RDG.
		if (RHIEndFence && !RHIEndFence->IsComplete())
		{
			if (!bWait)
				return false;

			FRHICommandListExecutor::WaitOnRHIThreadFence(RHIEndFence);
		}
		RHIEndFence = nullptr;

		// Read back the results from the GPU (resuming from the same position if we've tried before)
		for (; NextScope < TimingScopes.Num(); ++NextScope)
		{
			FInFlightTimingScope& Scope = TimingScopes[NextScope];

			if (!Scope.bUsed)
				continue;

			uint64 Begin;
			if (!RHIGetRenderQueryResult(Scope.Begin.GetQuery(), Begin, bWait))
				return false;

			uint64 End;
			if (!RHIGetRenderQueryResult(Scope.End.GetQuery(), End, bWait))
				return false;

			Timings[Scope.Budget] += End - Begin;
		}

		return true;
	}
};

class FRDGTimingPool : public FRenderResource
{
public:
	FRenderQueryPoolRHIRef QueryPool;

	// Destructor
	virtual ~FRDGTimingPool() = default;

	virtual void InitRHI(FRHICommandListBase& RHICmdList) override
	{
		check(IsInRenderingThread());
		LatestTimings.SetAll(uint64(0));
	}

	virtual void ReleaseRHI() override
	{
		check(IsInRenderingThread());

		if (QueryPool)
		{
			// Release all in-flight queries
			GatherResults(/* bWait = */ true);
			check(!Pending && !Recording && !Previous);

			// Release the pool
			QueryPool.SafeRelease();
		}
	}

	void BeginFrame(const DynamicRenderScaling::TMap<bool>& bInIsBudgetEnabled)
	{
		check(IsInRenderingThread());
		check(!Recording);

		// Land frames
		{
			GatherResults(/* bWait = */ false);
		}

		for (DynamicRenderScaling::FBudget* Budget : *DynamicRenderScaling::FBudget::GetGlobalList())
		{
			if (!bInIsBudgetEnabled[*Budget])
				continue;

			check(DynamicRenderScaling::IsSupported());

			if (!QueryPool.IsValid())
			{
				QueryPool = RHICreateRenderQueryPool(RQT_AbsoluteTime);
			}

			Recording = new FRDGTimingFrame(QueryPool);
			bIsBudgetRecordingEnabled = bInIsBudgetEnabled;
			break;
		}
	}

	void EndFrame(FRHICommandListImmediate& RHICmdList)
	{
		if (Recording)
		{
			Recording->RHIEndFence = RHICmdList.RHIThreadFence();
			RHICmdList.ImmediateFlush(EImmediateFlushType::DispatchToRHIThread);

			if (!Pending)
			{
				Pending = Recording;
			}

			if (Previous)
			{
				Previous->Next = Recording;
			}
			Previous = Recording;

			Recording = nullptr;
		}
	}

	void GatherResults(bool bWait)
	{
		while (Pending && Pending->GatherResults(bWait))
		{
			FRDGTimingFrame* Current = Pending;
			LatestTimings = MoveTemp(Current->Timings);

			if (Previous == Current)
				Previous = nullptr;

			Pending = Current->Next;
			delete Current;
		}
	}

	bool ShouldRecord(DynamicRenderScaling::FBudget const& Budget) const
	{
		check(IsInRenderingThread());
		return Recording && bIsBudgetRecordingEnabled[Budget];
	}
	
	// Current frame being built
	FRDGTimingFrame* Recording = nullptr;

	// Linked list of frames awaiting results from the GPU.
	FRDGTimingFrame* Previous = nullptr;
	FRDGTimingFrame* Pending = nullptr;

	// Latest available data from the GPU (or filled with zeros if no frames have been produced yet).
	DynamicRenderScaling::TMap<uint64> LatestTimings;

	DynamicRenderScaling::TMap<bool> bIsBudgetRecordingEnabled;
};

TGlobalResource<FRDGTimingPool> GRDGTimingPool;

namespace DynamicRenderScaling
{
	bool IsSupported()
	{
		return GRHISupportsGPUTimestampBubblesRemoval;
	}

	void BeginFrame(const DynamicRenderScaling::TMap<bool>& bIsBudgetEnabled)
	{
		check(IsInRenderingThread());
		GRDGTimingPool.BeginFrame(bIsBudgetEnabled);
	}

	void EndFrame()
	{
		GRDGTimingPool.EndFrame(FRHICommandListImmediate::Get());
	}

	const TMap<uint64>& GetLatestTimings()
	{
		check(IsInRenderingThread());
		return GRDGTimingPool.LatestTimings;
	}

} // namespace DynamicRenderScaling

// Lower overhead non-variadic version of constructor with arbitrary integer first argument to avoid overload resolution ambiguity.
// Avoids dynamic allocation of the formatted string and other overhead.
FRDGEventName::FRDGEventName(int32 NonVariadic, const TCHAR* InEventName)
#if RDG_EVENTS == RDG_EVENTS_STRING_REF || RDG_EVENTS == RDG_EVENTS_STRING_COPY
	: EventFormat(InEventName)
#endif
{
	check(InEventName != nullptr);
}

FRDGEventName::FRDGEventName(const TCHAR* EventFormat, ...)
#if RDG_EVENTS == RDG_EVENTS_STRING_REF || RDG_EVENTS == RDG_EVENTS_STRING_COPY
	: EventFormat(EventFormat)
#endif
{
#if RDG_EVENTS == RDG_EVENTS_STRING_COPY
	if (GRDGValidation != 0)
	{
		va_list VAList;
		va_start(VAList, EventFormat);
		TCHAR TempStr[256];
		// Build the string in the temp buffer
		FCString::GetVarArgs(TempStr, UE_ARRAY_COUNT(TempStr), EventFormat, VAList);
		va_end(VAList);

		FormattedEventName = TempStr;
	}
#endif
}

const TCHAR* FRDGEventName::GetTCHAR() const
{
#if RDG_EVENTS == RDG_EVENTS_STRING_COPY

	// Formatted name will be empty in cases where there are no variadic arguments -- EventFormat should be used in that case
	if (!FormattedEventName.IsEmpty())
	{
		return *FormattedEventName;
	}
	return EventFormat;

#elif RDG_EVENTS == RDG_EVENTS_STRING_REF

	// The event has not been formated, at least return the event format to have
	// error messages that give some clue when ShouldEmitEvents() == false.
	return EventFormat;

#else

	// Render graph draw events have been completely compiled for CPU performance reasons.
	return TEXT("[Compiled Out]");

#endif
}

FString FRDGScope::GetFullPath(FRDGEventName const& PassName)
{
	FString Path = PassName.GetTCHAR();
#if RDG_EVENTS
	FRHIBreadcrumb::FBuffer Buffer;
	for (FRDGScope* Current = Parent; Current; Current = Current->Parent)
	{
		if (FRDGScope_RHI* RHIScope = Current->Get<FRDGScope_RHI>())
		{
			Path = RHIScope->GetTCHAR(Buffer) / Path;
		}
	}
#endif // RDG_EVENTS

	return Path;
}

FRDGScope_Budget::FRDGScope_Budget(FRDGScopeState& State, DynamicRenderScaling::FBudget const& Budget)
	: bPop(!State.ScopeState.ActiveBudget)
{
	checkf(bPop || State.ScopeState.ActiveBudget == &Budget, TEXT("Cannot nest dynamic render scaling budgets."));
	State.ScopeState.ActiveBudget = &Budget;

	if (bPop && GRDGTimingPool.ShouldRecord(Budget))
	{
		Frame = GRDGTimingPool.Recording;
		ScopeId = Frame->AllocateScope(Budget);
	}
}

void FRDGScope_Budget::ImmediateEnd(FRDGScopeState& State)
{
	if (bPop)
	{
		State.ScopeState.ActiveBudget = nullptr;
	}
}

void FRDGScope_Budget::BeginGPU(FRHIComputeCommandList& RHICmdList)
{
	if (Frame && RHICmdList.GetPipeline() == ERHIPipeline::Graphics) // @todo async compute support (requires RHIGetRenderQueryResult on the compute cmdlist)
	{
		Frame->BeginScope(ScopeId, static_cast<FRHICommandList&>(RHICmdList));
	}
}

void FRDGScope_Budget::EndGPU(FRHIComputeCommandList& RHICmdList)
{
	if (Frame && RHICmdList.GetPipeline() == ERHIPipeline::Graphics) // @todo async compute support (requires RHIGetRenderQueryResult on the compute cmdlist)
	{
		Frame->EndScope(ScopeId, static_cast<FRHICommandList&>(RHICmdList));
	}
}
