// Copyright Epic Games, Inc. All Rights Reserved.

#include "RenderGraphPass.h"
#include "RenderGraphPrivate.h"
#include "RenderGraphBuilder.h"
#include "RenderTracing.h"

FEmptyShaderParameters FRDGSentinelPass::EmptyShaderParameters;

FUniformBufferStaticBindings FRDGParameterStruct::GetStaticUniformBuffers() const
{
	FUniformBufferStaticBindings GlobalUniformBuffers;

	for (uint32 Index = 0, Count = Layout->UniformBuffers.Num(); Index < Count; ++Index)
	{
		const uint32 MemberOffset = Layout->UniformBuffers[Index].MemberOffset;
		const FUniformBufferBinding& UniformBuffer = *reinterpret_cast<const FUniformBufferBinding*>(const_cast<uint8*>(Contents + MemberOffset));

		if (UniformBuffer && UniformBuffer.IsStatic())
		{
			GlobalUniformBuffers.AddUniformBuffer(UniformBuffer.GetUniformBuffer());
		}
	}

	EnumerateUniformBuffers([&](FRDGUniformBufferBinding UniformBuffer)
	{
		if (UniformBuffer.IsStatic())
		{
			GlobalUniformBuffers.AddUniformBuffer(UniformBuffer->GetRHI());
		}
	});

	return GlobalUniformBuffers;
}

FRHIRenderPassInfo FRDGParameterStruct::GetRenderPassInfo() const
{
	const FRenderTargetBindingSlots& RenderTargets = GetRenderTargets();

	FRHIRenderPassInfo RenderPassInfo;
	uint32 SampleCount = 0;
	uint32 RenderTargetIndex = 0;

	RenderTargets.Enumerate([&](FRenderTargetBinding RenderTarget)
	{
		FRDGTextureRef Texture = RenderTarget.GetTexture();
		FRDGTextureRef ResolveTexture = RenderTarget.GetResolveTexture();
		ERenderTargetStoreAction StoreAction = EnumHasAnyFlags(Texture->Desc.Flags, TexCreate_Memoryless) ? ERenderTargetStoreAction::ENoAction : ERenderTargetStoreAction::EStore;

		if (ResolveTexture)
		{
			// Silently skip the resolve if the resolve texture is the same as the render target texture.
			if (ResolveTexture != Texture)
			{
				StoreAction = ERenderTargetStoreAction::EMultisampleResolve;
			}
			else
			{
				ResolveTexture = nullptr;
			}
		}

		auto& ColorRenderTarget = RenderPassInfo.ColorRenderTargets[RenderTargetIndex];
		ColorRenderTarget.RenderTarget = Texture->GetRHI();
		ColorRenderTarget.ResolveTarget = ResolveTexture ? ResolveTexture->GetRHI() : nullptr;
		ColorRenderTarget.ArraySlice = RenderTarget.GetArraySlice();
		ColorRenderTarget.MipIndex = RenderTarget.GetMipIndex();
		ColorRenderTarget.Action = MakeRenderTargetActions(RenderTarget.GetLoadAction(), StoreAction);

		SampleCount |= ColorRenderTarget.RenderTarget->GetNumSamples();
		++RenderTargetIndex;
	});

	const FDepthStencilBinding& DepthStencil = RenderTargets.DepthStencil;

	if (FRDGTextureRef Texture = DepthStencil.GetTexture())
	{
		const FExclusiveDepthStencil ExclusiveDepthStencil = DepthStencil.GetDepthStencilAccess();
		ERenderTargetStoreAction StoreAction = EnumHasAnyFlags(Texture->Desc.Flags, TexCreate_Memoryless) ? ERenderTargetStoreAction::ENoAction : ERenderTargetStoreAction::EStore;
		FRDGTextureRef ResolveTexture = DepthStencil.GetResolveTexture();
		if (ResolveTexture)
		{
			// Silently skip the resolve if the resolve texture is the same as the render target texture.
			if (ResolveTexture != Texture)
			{
				StoreAction = ERenderTargetStoreAction::EMultisampleResolve;
			}
			else
			{
				ResolveTexture = nullptr;
			}
		}

		const ERenderTargetStoreAction DepthStoreAction = ExclusiveDepthStencil.IsUsingDepth() ? StoreAction : ERenderTargetStoreAction::ENoAction;
		const ERenderTargetStoreAction StencilStoreAction = ExclusiveDepthStencil.IsUsingStencil() ? StoreAction : ERenderTargetStoreAction::ENoAction;

		auto& DepthStencilTarget = RenderPassInfo.DepthStencilRenderTarget;
		DepthStencilTarget.DepthStencilTarget = Texture->GetRHI();
		DepthStencilTarget.ResolveTarget = ResolveTexture ? ResolveTexture->GetRHI() : nullptr;
		DepthStencilTarget.Action = MakeDepthStencilTargetActions(
			MakeRenderTargetActions(DepthStencil.GetDepthLoadAction(), DepthStoreAction),
			MakeRenderTargetActions(DepthStencil.GetStencilLoadAction(), StencilStoreAction));
		DepthStencilTarget.ExclusiveDepthStencil = ExclusiveDepthStencil;

		SampleCount |= DepthStencilTarget.DepthStencilTarget->GetNumSamples();
	}

	RenderPassInfo.ResolveRect = RenderTargets.ResolveRect;
	RenderPassInfo.NumOcclusionQueries = RenderTargets.NumOcclusionQueries;
	RenderPassInfo.SubpassHint = RenderTargets.SubpassHint;
	RenderPassInfo.MultiViewCount = RenderTargets.MultiViewCount;
	RenderPassInfo.ShadingRateTexture = RenderTargets.ShadingRateTexture ? RenderTargets.ShadingRateTexture->GetRHI() : nullptr;
	// @todo: should define this as a state that gets passed through? Max seems appropriate for now.
	RenderPassInfo.ShadingRateTextureCombiner = RenderPassInfo.ShadingRateTexture.IsValid() ? VRSRB_Max : VRSRB_Passthrough;

	return RenderPassInfo;
}

FRHICommandList* FRDGDispatchPassBuilder::CreateCommandList()
{
	FRHICommandList* RHICmdList = nullptr;
	
	if (RenderPassInfo && Pass->bParallelExecute && GRHIGlobals.ParallelRenderPasses.Supported)
	{
		FRHISubCommandList* SubCmdList = new FRHISubCommandList(Pass->GetGPUMask(), RenderPassInfo);
		SubCmdList->SwitchPipeline(Pass->GetPipeline());

		SubCommandLists.Emplace(SubCmdList);
		RHICmdList = SubCmdList;
	}
	else
	{
		RHICmdList = new FRHICommandList(Pass->GetGPUMask());
		RHICmdList->SwitchPipeline(Pass->GetPipeline());

		// When parallel executing, the pass commands are embedded directly into the first command list.
		if (Pass->bParallelExecute && Pass->CommandLists.IsEmpty())
		{
			FRDGBuilder::PushPreScopes(*RHICmdList, Pass);
			FRDGBuilder::ExecutePassPrologue(*RHICmdList, Pass);
		}
		
		if (RenderPassInfo)
		{
			RHICmdList->BeginRenderPass(*RenderPassInfo, TEXT("DispatchPass"));
		}
		
		Pass->CommandLists.Emplace(RHICmdList);
	}
	
	RHICmdList->SetStaticUniformBuffers(StaticUniformBuffers);

	IF_RENDER_TRACING_ENABLED(RenderTracing::AddDispatchPassCommandList(*Pass, *RHICmdList));

	return RHICmdList;
}

void FRDGDispatchPassBuilder::AddPrerequisite(const UE::Tasks::FTask& Task)
{
	if (Task.IsValid())
	{
		Prerequisites.Add(Task);
	}
}

void FRDGDispatchPassBuilder::Finish()
{
	// With serial execution the pass commands are embedded in the immediate command list instead.
	if (!Pass->bParallelExecute)
	{
		IF_RENDER_TRACING_ENABLED(RenderTracing::EndExecuteDispatchPass(*Pass, nullptr));
		Pass->CommandListsEvent.Trigger();
		return;
	}
	
	FRHICommandList* RHICmdList = nullptr;
	
	if(SubCommandLists.Num())
	{
		RHICmdList = new FRHICommandList(Pass->GetGPUMask());
		RHICmdList->SwitchPipeline(Pass->GetPipeline());
		
		FRDGBuilder::PushPreScopes(*RHICmdList, Pass);
		FRDGBuilder::ExecutePassPrologue(*RHICmdList, Pass);
		
		RHICmdList->InsertParallelRenderPass(RenderPassInfo, MoveTemp(SubCommandLists));
		
		Pass->CommandLists.Emplace(RHICmdList);
		Pass->CommandListsEvent.Trigger();
	}
	else
	{
		const bool bEmptyCommandLists = Pass->CommandLists.IsEmpty();
		
		// Create a command list to embed the epilogue (and prologue as well if no user command lists were requested).
		RHICmdList = new FRHICommandList(Pass->GetGPUMask());
		Pass->CommandLists.Emplace(RHICmdList);
		Pass->CommandListsEvent.Trigger();

		RHICmdList->SwitchPipeline(Pass->GetPipeline());

		if (bEmptyCommandLists)
		{
			FRDGBuilder::PushPreScopes(*RHICmdList, Pass);
			FRDGBuilder::ExecutePassPrologue(*RHICmdList, Pass);
		}
	}
	
	FRDGBuilder::ExecutePassEpilogue(*RHICmdList, Pass);
	FRDGBuilder::PopPreScopes(*RHICmdList, Pass);

	IF_RENDER_TRACING_ENABLED(RenderTracing::EndExecuteDispatchPass(*Pass, RHICmdList));

	RHICmdList->FinishRecording();
}

FRDGBarrierBatchBegin::FRDGBarrierBatchBegin(ERHIPipeline InPipelineToBegin, ERHIPipeline InPipelinesToEnd, const TCHAR* InName, FRDGPass* InPass)
	: PipelinesToBegin(InPipelineToBegin)
	, PipelinesToEnd(InPipelinesToEnd)
#if RDG_ENABLE_DEBUG
	, DebugPasses(InPlace, nullptr)
	, DebugName(InName)
#endif
{
#if RDG_ENABLE_DEBUG
	DebugPasses[InPipelineToBegin] = InPass;
#endif
}

FRDGBarrierBatchBegin::FRDGBarrierBatchBegin(ERHIPipeline InPipelinesToBegin, ERHIPipeline InPipelinesToEnd, const TCHAR* InName, FRDGPassesByPipeline InPasses)
	: PipelinesToBegin(InPipelinesToBegin)
	, PipelinesToEnd(InPipelinesToEnd)
#if RDG_ENABLE_DEBUG
	, DebugPasses(InPasses)
	, DebugName(InName)
#endif
{}

void FRDGBarrierBatchBegin::AddTransition(FRDGViewableResource* Resource, FRDGTransitionInfo Info)
{
	Transitions.Add(Info);
	bTransitionNeeded = true;

#if RDG_STATS
	GRDGStatTransitionCount++;
#endif

#if RDG_ENABLE_DEBUG
	DebugTransitionResources.Add(Resource);
#endif
}

void FRDGBarrierBatchBegin::AddAlias(FRDGViewableResource* Resource, const FRHITransientAliasingInfo& Info)
{
	Aliases.Add(Info);
	bTransitionNeeded = true;

#if RDG_STATS
	GRDGStatAliasingCount++;
#endif

#if RDG_ENABLE_DEBUG
	DebugAliasingResources.Add(Resource);
#endif
}

void FRDGBarrierBatchBegin::CreateTransition(TConstArrayView<FRHITransitionInfo> TransitionsRHI)
{
	check(bTransitionNeeded && !Transition);
	Transition = RHICreateTransition(FRHITransitionCreateInfo(PipelinesToBegin, PipelinesToEnd, TransitionFlags, TransitionsRHI, Aliases));

	if (bSeparateFenceTransitionNeeded)
	{
		SeparateFenceTransition = RHICreateTransition(FRHITransitionCreateInfo(PipelinesToBegin, PipelinesToEnd));
	}
}

void FRDGBarrierBatchBegin::Submit(FRHIComputeCommandList& RHICmdList, ERHIPipeline Pipeline, FRDGTransitionQueue& TransitionsToBegin)
{
	if (SeparateFenceTransition)
	{
		TransitionsToBegin.Emplace(SeparateFenceTransition);
	}

	if (Transition)
	{
		TransitionsToBegin.Emplace(Transition);
	}

#if RDG_STATS
	GRDGStatTransitionBatchCount.fetch_add(1, std::memory_order_relaxed);
#endif
}

FRDGBarrierBatchEndId FRDGBarrierBatchEnd::GetId() const
{
	return FRDGBarrierBatchEndId(Pass->GetHandle(), BarrierLocation);
}

bool FRDGBarrierBatchEnd::IsPairedWith(const FRDGBarrierBatchBegin& BeginBatch) const
{
	return GetId() == BeginBatch.BarriersToEnd[Pass->GetPipeline()];
}

void FRDGBarrierBatchBegin::Submit(FRHIComputeCommandList& RHICmdList, ERHIPipeline Pipeline)
{
	FRDGTransitionQueue TransitionsToBegin;
	Submit(RHICmdList, Pipeline, TransitionsToBegin);

	if (!TransitionsToBegin.IsEmpty())
	{
		RHICmdList.BeginTransitions(TransitionsToBegin);
	}
}

void FRDGBarrierBatchEnd::AddDependency(FRDGBarrierBatchBegin* BeginBatch)
{
#if RDG_ENABLE_DEBUG
	check(BeginBatch);

	for (ERHIPipeline Pipeline : MakeFlagsRange(ERHIPipeline::All))
	{
		const FRDGPass* BeginPass = BeginBatch->DebugPasses[Pipeline];
		if (BeginPass)
		{
			checkf(BeginPass->GetHandle() <= Pass->GetHandle(), TEXT("A transition end batch for pass %s is dependent on begin batch for pass %s."), Pass->GetName(), BeginPass->GetName());
		}
	}
#endif

	{
		const FRDGBarrierBatchEndId Id = GetId();

		FRDGBarrierBatchEndId& EarliestEndId = BeginBatch->BarriersToEnd[Pass->GetPipeline()];

		if (EarliestEndId == Id)
		{
			return;
		}
		const FRDGBarrierBatchEndId MinId(
			FRDGPassHandle::Min(EarliestEndId.PassHandle, Id.PassHandle),
			(ERDGBarrierLocation)FMath::Min((int32)EarliestEndId.BarrierLocation, (int32)Id.BarrierLocation));

		if (MinId == Id)
		{
			Dependencies.Add(BeginBatch);
			EarliestEndId = MinId;
		}
	}
}

void FRDGBarrierBatchEnd::Submit(FRHIComputeCommandList& RHICmdList, ERHIPipeline Pipeline)
{
	const FRDGBarrierBatchEndId Id(Pass->GetHandle(), BarrierLocation);

	FRDGTransitionQueue Transitions;
	Transitions.Reserve(Dependencies.Num());

	for (FRDGBarrierBatchBegin* Dependent : Dependencies)
	{
		if (Dependent->BarriersToEnd[Pipeline] == Id)
		{
			if (Dependent->SeparateFenceTransition)
			{
				Transitions.Emplace(Dependent->SeparateFenceTransition);
			}

			Transitions.Emplace(Dependent->Transition);
		}
	}

	if (!Transitions.IsEmpty())
	{
		RHICmdList.EndTransitions(Transitions);
	}
}

FRDGBarrierBatchBegin& FRDGPass::GetPrologueBarriersToBegin(FRDGAllocator& Allocator, FRDGTransitionCreateQueue& CreateQueue)
{
	if (!PrologueBarriersToBegin)
	{
		PrologueBarriersToBegin = Allocator.AllocNoDestruct<FRDGBarrierBatchBegin>(Pipeline, Pipeline, TEXT("Prologue"), this);
		CreateQueue.Emplace(PrologueBarriersToBegin);
	}
	return *PrologueBarriersToBegin;
}

FRDGBarrierBatchBegin& FRDGPass::GetEpilogueBarriersToBeginForGraphics(FRDGAllocator& Allocator, FRDGTransitionCreateQueue& CreateQueue)
{
	if (!EpilogueBarriersToBeginForGraphics)
	{
		EpilogueBarriersToBeginForGraphics = Allocator.AllocNoDestruct<FRDGBarrierBatchBegin>(Pipeline, ERHIPipeline::Graphics, GetEpilogueBarriersToBeginDebugName(ERHIPipeline::Graphics), this);
		if (!EpilogueBarriersToBeginForGraphics->IsTransitionNeeded())
		{
			EpilogueBarriersToBeginForGraphics->Reserve(NumTransitionsToReserve);
			CreateQueue.Emplace(EpilogueBarriersToBeginForGraphics);
		}
	}
	return *EpilogueBarriersToBeginForGraphics;
}

FRDGBarrierBatchBegin& FRDGPass::GetEpilogueBarriersToBeginForAsyncCompute(FRDGAllocator& Allocator, FRDGTransitionCreateQueue& CreateQueue)
{
	if (!EpilogueBarriersToBeginForAsyncCompute)
	{
		EpilogueBarriersToBeginForAsyncCompute = Allocator.AllocNoDestruct<FRDGBarrierBatchBegin>(Pipeline, ERHIPipeline::AsyncCompute, GetEpilogueBarriersToBeginDebugName(ERHIPipeline::AsyncCompute), this);
		CreateQueue.Emplace(EpilogueBarriersToBeginForAsyncCompute);
	}
	return *EpilogueBarriersToBeginForAsyncCompute;
}

FRDGBarrierBatchBegin& FRDGPass::GetEpilogueBarriersToBeginForAll(FRDGAllocator& Allocator, FRDGTransitionCreateQueue& CreateQueue)
{
	if (!EpilogueBarriersToBeginForAll)
	{
		EpilogueBarriersToBeginForAll = Allocator.AllocNoDestruct<FRDGBarrierBatchBegin>(Pipeline, ERHIPipeline::All, GetEpilogueBarriersToBeginDebugName(ERHIPipeline::All), this);
		CreateQueue.Emplace(EpilogueBarriersToBeginForAll);
	}
	return *EpilogueBarriersToBeginForAll;
}

FRDGBarrierBatchEnd& FRDGPass::GetPrologueBarriersToEnd(FRDGAllocator& Allocator)
{
	if (!PrologueBarriersToEnd)
	{
		PrologueBarriersToEnd = Allocator.AllocNoDestruct<FRDGBarrierBatchEnd>(this, ERDGBarrierLocation::Prologue);
	}
	return *PrologueBarriersToEnd;
}

FRDGBarrierBatchEnd& FRDGPass::GetEpilogueBarriersToEnd(FRDGAllocator& Allocator)
{
	if (!EpilogueBarriersToEnd)
	{
		EpilogueBarriersToEnd = Allocator.AllocNoDestruct<FRDGBarrierBatchEnd>(this, ERDGBarrierLocation::Epilogue);
	}
	return *EpilogueBarriersToEnd;
}

FRDGPass::FRDGPass(
	FRDGEventName&& InName,
	FRDGParameterStruct InParameterStruct,
	ERDGPassFlags InFlags,
	ERDGPassTaskMode InTaskMode)
	: Name(Forward<FRDGEventName&&>(InName))
	, ParameterStruct(InParameterStruct)
	, Flags(InFlags)
	, TaskMode(InTaskMode)
	, Pipeline(EnumHasAnyFlags(Flags, ERDGPassFlags::AsyncCompute) ? ERHIPipeline::AsyncCompute : ERHIPipeline::Graphics)
{}

#if RDG_ENABLE_DEBUG
const TCHAR* FRDGPass::GetName() const
{
	// When in debug runtime mode, use the full path name.
	if (!FullPathIfDebug.IsEmpty())
	{
		return *FullPathIfDebug;
	}
	else
	{
		return Name.GetTCHAR();
	}
}
#endif
