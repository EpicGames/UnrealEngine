// Copyright Epic Games, Inc. All Rights Reserved.

#include "RenderGraphPrivate.h"
#include "RenderGraphEvent.h"
#include "RenderGraphTrace.h"
#include "RenderGraphBuilder.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "Misc/CommandLine.h"
#include "RHICommandList.h"
#include "DumpGPU.h"

#if RDG_ENABLE_DEBUG

int32 GRDGDumpGraphUnknownCount = 0;

int32 GRDGImmediateMode = 0;
FAutoConsoleVariableRef CVarImmediateMode(
	TEXT("r.RDG.ImmediateMode"),
	GRDGImmediateMode,
	TEXT("Executes passes as they get created. Useful to have a callstack of the wiring code when crashing in the pass' lambda."),
	ECVF_RenderThreadSafe);

int32 GRDGValidation = 1;
FAutoConsoleVariableRef CVarRDGValidation(
	TEXT("r.RDG.Validation"),
	GRDGValidation,
	TEXT("Enables validation of correctness in API calls and pass parameter dependencies.\n")
	TEXT(" 0: disabled;\n")
	TEXT(" 1: enabled (default);\n"),
	ECVF_RenderThreadSafe);

int32 GRDGDebugFlushGPU = 0;
FAutoConsoleVariableRef CVarRDGDebugFlushGPU(
	TEXT("r.RDG.Debug.FlushGPU"),
	GRDGDebugFlushGPU,
	TEXT("Enables flushing the GPU after every pass. Disables async compute (r.RDG.AsyncCompute=0) and parallel execute (r.RDG.ParallelExecute=0) when set.\n")
	TEXT(" 0: disabled (default);\n")
	TEXT(" 1: enabled."),
	ECVF_RenderThreadSafe);

int32 GRDGDebugExtendResourceLifetimes = 0;
FAutoConsoleVariableRef CVarRDGDebugExtendResourceLifetimes(
	TEXT("r.RDG.Debug.ExtendResourceLifetimes"),
	GRDGDebugExtendResourceLifetimes,
	TEXT("Extends the resource lifetimes of resources (or a specific resource filter specified by r.RDG.Debug.ResourceFilter) ")
	TEXT("so that they cannot overlap memory with any other resource within the graph. Useful to debug if transient aliasing is causing issues.\n")
	TEXT(" 0: disabled (default);\n")
	TEXT(" 1: enabled;\n"),
	ECVF_RenderThreadSafe);

int32 GRDGDebugDisableTransientResources = 0;
FAutoConsoleVariableRef CVarRDGDebugDisableTransientResource(
	TEXT("r.RDG.Debug.DisableTransientResources"),
	GRDGDebugDisableTransientResources,
	TEXT("Filters out transient resources from the transient allocator. Use r.rdg.debug.resourcefilter to specify the filter. Defaults to all resources if enabled."),
	ECVF_RenderThreadSafe);

int32 GRDGClobberResources = 0;
FAutoConsoleVariableRef CVarRDGClobberResources(
	TEXT("r.RDG.ClobberResources"),
	GRDGClobberResources,
	TEXT("Clears all render targets and texture / buffer UAVs with the requested clear color at allocation time. Useful for debugging.\n")
	TEXT(" 0:off (default);\n")
	TEXT(" 1: 1000 on RGBA channels;\n")
	TEXT(" 2: NaN on RGBA channels;\n")
	TEXT(" 3: +INFINITY on RGBA channels.\n"),
	ECVF_Cheat | ECVF_RenderThreadSafe);

int32 GRDGOverlapUAVs = 1;
FAutoConsoleVariableRef CVarRDGOverlapUAVs(
	TEXT("r.RDG.OverlapUAVs"), GRDGOverlapUAVs,
	TEXT("RDG will overlap UAV work when requested; if disabled, UAV barriers are always inserted."),
	ECVF_RenderThreadSafe);

int32 GRDGTransitionLog = 0;
FAutoConsoleVariableRef CVarRDGTransitionLog(
	TEXT("r.RDG.TransitionLog"), GRDGTransitionLog,
	TEXT("Logs resource transitions to the console.\n")
	TEXT(" 0: disabled(default);\n")
	TEXT(">0: enabled for N frames;\n")
	TEXT("<0: enabled;\n"),
	ECVF_RenderThreadSafe);

TAutoConsoleVariable<FString> CVarRDGDebugGraphFilter(
	TEXT("r.RDG.Debug.GraphFilter"), TEXT(""),
	TEXT("Filters certain debug events to a specific graph. Set to 'None' to reset.\n"),
	ECVF_Default);

FString GRDGDebugGraphFilterName;

inline FString GetDebugFilterString(const FString& InputString)
{
	if (!InputString.Compare(TEXT("None"), ESearchCase::IgnoreCase))
	{
		return {};
	}
	return InputString;
}

FAutoConsoleVariableSink CVarRDGDebugGraphSink(FConsoleCommandDelegate::CreateLambda([]()
{
	GRDGDebugGraphFilterName = GetDebugFilterString(CVarRDGDebugGraphFilter.GetValueOnGameThread());
}));

inline bool IsDebugAllowed(const FString& FilterString, const TCHAR* Name)
{
	if (FilterString.IsEmpty())
	{
		return true;
	}

	const bool bInverted = FilterString[0] == '!';
	if (FilterString.Len() == 1 && bInverted)
	{
		return true;
	}

	const TCHAR* FilterStringRaw = *FilterString;

	if (bInverted)
	{
		FilterStringRaw++;
	}

	const bool bFound = FCString::Strifind(Name, FilterStringRaw) != nullptr;
	return bFound ^ bInverted;
}

bool IsDebugAllowedForGraph(const TCHAR* GraphName)
{
	return IsDebugAllowed(GRDGDebugGraphFilterName, GraphName);
}

TAutoConsoleVariable<FString> CVarRDGDebugPassFilter(
	TEXT("r.RDG.Debug.PassFilter"), TEXT(""),
	TEXT("Filters certain debug events to specific passes. Set to 'None' to reset.\n"),
	ECVF_Default);

FString GRDGDebugPassFilterName;

FAutoConsoleVariableSink CVarRDGDebugPassSink(FConsoleCommandDelegate::CreateLambda([]()
{
	GRDGDebugPassFilterName = GetDebugFilterString(CVarRDGDebugPassFilter.GetValueOnGameThread());
}));

bool IsDebugAllowedForPass(const TCHAR* PassName)
{
	return IsDebugAllowed(GRDGDebugPassFilterName, PassName);
}

TAutoConsoleVariable<FString> CVarRDGDebugResourceFilter(
	TEXT("r.RDG.Debug.ResourceFilter"), TEXT(""),
	TEXT("Filters certain debug events to a specific resource. Set to 'None' to reset.\n"),
	ECVF_Default);

FString GRDGDebugResourceFilterName;

FAutoConsoleVariableSink CVarRDGDebugResourceSink(FConsoleCommandDelegate::CreateLambda([]()
{
	GRDGDebugResourceFilterName = GetDebugFilterString(CVarRDGDebugResourceFilter.GetValueOnGameThread());
}));

bool IsDebugAllowedForResource(const TCHAR* ResourceName)
{
	return IsDebugAllowed(GRDGDebugResourceFilterName, ResourceName);
}

static float GetClobberValue()
{
	switch (GRDGClobberResources)
	{
	case 1:
		return 1000.0f;
	case 2:
		return NAN;
	case 3:
		return std::numeric_limits<float>::infinity();
	}
	return 0.0f;
}

FLinearColor GetClobberColor()
{
	float ClobberValue = GetClobberValue();
	return FLinearColor(ClobberValue, ClobberValue, ClobberValue, ClobberValue);
}

uint32 GetClobberBufferValue()
{
	float ClobberValue = GetClobberValue();
	uint32 ClobberValueUint;
	FMemory::Memcpy(&ClobberValueUint, &ClobberValue, sizeof(ClobberValueUint));
	return ClobberValueUint;
}

float GetClobberDepth()
{
	return 0.123456789f;
}

uint8 GetClobberStencil()
{
	return 123;
}

std::atomic<bool> GRDGAllowRHIAccess{false};
std::atomic<bool> GRDGAllowRHIAccessAsync{false};

#endif

int32 GRDGAsyncCompute = 1;
TAutoConsoleVariable<int32> CVarRDGAsyncCompute(
	TEXT("r.RDG.AsyncCompute"),
	RDG_ASYNC_COMPUTE_ENABLED,
	TEXT("Controls the async compute policy.\n")
	TEXT(" 0:disabled, no async compute is used;\n")
	TEXT(" 1:enabled for passes tagged for async compute (default);\n")
	TEXT(" 2:enabled for all compute passes implemented to use the compute command list;\n"),
	ECVF_RenderThreadSafe);

FAutoConsoleVariableSink CVarRDGAsyncComputeSink(FConsoleCommandDelegate::CreateLambda([]()
{
	GRDGAsyncCompute = CVarRDGAsyncCompute.GetValueOnGameThread();

	if (GRDGDebugFlushGPU)
	{
		GRDGAsyncCompute = 0;
	}
}));

int32 GRDGCullPasses = 1;
FAutoConsoleVariableRef CVarRDGCullPasses(
	TEXT("r.RDG.CullPasses"),
	GRDGCullPasses,
	TEXT("The graph will cull passes with unused outputs.\n")
	TEXT(" 0:off;\n")
	TEXT(" 1:on(default);\n"),
	ECVF_RenderThreadSafe);

int32 GRDGMergeRenderPasses = 1;
FAutoConsoleVariableRef CVarRDGMergeRenderPasses(
	TEXT("r.RDG.MergeRenderPasses"),
	GRDGMergeRenderPasses,
	TEXT("The graph will merge identical, contiguous render passes into a single render pass.\n")
	TEXT(" 0:off;\n")
	TEXT(" 1:on(default);\n"),
	ECVF_RenderThreadSafe);

int32 GRDGTransientAllocator = 1;
FAutoConsoleVariableRef CVarRDGUseTransientAllocator(
	TEXT("r.RDG.TransientAllocator"), GRDGTransientAllocator,
	TEXT("RDG will use the RHITransientResourceAllocator to allocate all transient resources.")
	TEXT(" 0: disables the transient allocator;")
	TEXT(" 1: enables the transient allocator (default);")
	TEXT(" 2: enables the transient allocator for resources with FastVRAM flag only"),
	ECVF_RenderThreadSafe);

int32 GRDGTransientExtractedResources = 1;
FAutoConsoleVariableRef CVarRDGTransientExtractedResource(
	TEXT("r.RDG.TransientExtractedResources"), GRDGTransientExtractedResources,
	TEXT("RDG will allocate extracted resources as transient, unless explicitly marked non-transient by the user.")
	TEXT(" 0: disables external transient resources;")
	TEXT(" 1: enables external transient resources (default);")
	TEXT(" 2: force enables all external transient resources (not recommended);"),
	ECVF_RenderThreadSafe);

int32 GRDGAsyncComputeTransientAliasing = 1;
FAutoConsoleVariableRef CVarRDGAsyncComputeTransientAliasing(
	TEXT("r.RDG.AsyncComputeTransientAliasing"), GRDGAsyncComputeTransientAliasing,
	TEXT("RDG will alias async compute resources on the same heap as graphics resources using fences. This must also be supported by the RHI.")
	TEXT(" 0: disables transient async compute aliasing;")
	TEXT(" 1: enables transient async compute aliasing (default);"),
	ECVF_RenderThreadSafe);

#if RDG_EVENTS
TAutoConsoleVariable<int32> CVarRDGEvents(
	TEXT("r.RDG.Events"),
	1,
	TEXT("Controls how RDG events are emitted.\n")
	TEXT(" 0: off;\n")
	TEXT(" 1: events are enabled and RDG_EVENT_SCOPE_FINAL is respected; (default)\n")
	TEXT(" 2: all events are enabled (RDG_EVENT_SCOPE_FINAL is ignored);\n")
	TEXT(" 3: same as 2, but RDG pass names are also included."),
	ECVF_RenderThreadSafe);
#endif

#if RDG_ENABLE_PARALLEL_TASKS

int32 GRDGParallelDestruction = 1;
FAutoConsoleVariableRef CVarRDGParallelDestruction(
	TEXT("r.RDG.ParallelDestruction"), GRDGParallelDestruction,
	TEXT("RDG will destruct the graph using an async task.")
	TEXT(" 0: graph destruction is done synchronously;")
	TEXT(" 1: graph destruction may be done asynchronously (default);"),
	ECVF_RenderThreadSafe);

int32 GRDGParallelSetup = 1;
FAutoConsoleVariableRef CVarRDGParallelSetup(
	TEXT("r.RDG.ParallelSetup"), GRDGParallelSetup,
	TEXT("RDG will launch setup tasks from AddSetupTask.")
	TEXT(" 0: setup tasks are executed inline;")
	TEXT(" 1: setup tasks are launched (default);"),
	ECVF_RenderThreadSafe);

int32 GRDGParallelCompile = 1;
FAutoConsoleVariableRef CVarRDGParallelCompile(
	TEXT("r.RDG.ParallelCompile"), GRDGParallelCompile,
	TEXT("RDG will launch tasks to compile the RDG graph.")
	TEXT(" 0: compile tasks are executed inline;")
	TEXT(" 1: compile tasks are launched (default);"),
	ECVF_RenderThreadSafe);

int32 GRDGAsyncSetupQueue = 1;
FAutoConsoleVariableRef CVarRDGAsyncSetupQueue(
	TEXT("r.RDG.AsyncSetupQueue"), GRDGAsyncSetupQueue,
	TEXT("RDG will setup passes in parallel when prompted by calls to FRDGBuilder::FlushSetupQueue.")
	TEXT(" 0: pass setup is done synchronously in AddPass;")
	TEXT(" 1: pass setup is done asynchronously (default);"),
	ECVF_RenderThreadSafe);

int32 GRDGParallelSetupTaskPriorityBias = 1;
FAutoConsoleVariableRef CVarRDGParallelSetupTaskPriorityBias(
	TEXT("r.RDG.ParallelSetup.TaskPriorityBias"), GRDGParallelSetupTaskPriorityBias,
	TEXT("Biases the task priority of all setup tasks. Useful as a tweak when contention from game thread tasks is high."),
	ECVF_RenderThreadSafe);

int32 GRDGParallelExecute = 2;
FAutoConsoleVariableRef CVarRDGParallelExecute(
	TEXT("r.RDG.ParallelExecute"), GRDGParallelExecute,
	TEXT("Whether to enable parallel execution of passes when supported.")
	TEXT(" 0: off")
	TEXT(" 1: parallel with all tasks awaited")
	TEXT(" 2: parallel with async tasks (default)"),
	FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* Variable)
	{
		if (Variable->GetInt())
		{
			if (GRDGParallelExecutePassMax <= 1)
			{
				GRDGParallelExecutePassMax = 1;
			}

			if (GRDGParallelExecutePassMax < GRDGParallelExecutePassMin)
			{
				GRDGParallelExecutePassMin = GRDGParallelExecutePassMax;
			}
		}
	}),
	ECVF_RenderThreadSafe);

int32 GRDGParallelExecutePassMin = 1;
FAutoConsoleVariableRef CVarRDGParallelExecutePassMin(
	TEXT("r.RDG.ParallelExecute.PassMin"), GRDGParallelExecutePassMin,
	TEXT("The minimum span of contiguous passes eligible for parallel execution for the span to be offloaded to a task."),
	ECVF_RenderThreadSafe);

int32 GRDGParallelExecutePassMax = 32;
FAutoConsoleVariableRef CVarRDGParallelExecutePassMax(
	TEXT("r.RDG.ParallelExecute.PassMax"), GRDGParallelExecutePassMax,
	TEXT("The maximum span of contiguous passes eligible for parallel execution for the span to be offloaded to a task."),
	ECVF_RenderThreadSafe);

int32 GRDGParallelExecutePassTaskModeThreshold = 2;
FAutoConsoleVariableRef CVarRDGParallelExecutePassTaskModeThreshold(
	TEXT("r.RDG.ParallelExecute.PassTaskModeThreshold"), GRDGParallelExecutePassTaskModeThreshold,
	TEXT(" 0: A pass that is not marked async will mark the entire parallel pass set as awaited.")
	TEXT(" 1: A pass that does not match the task mode of the current batch will always flush the current batch.")
    TEXT(">1: Same as the above, but only if the current batch is larger than the threshold."),
	ECVF_RenderThreadSafe);

int32 GRDGParallelExecuteStress = 0;
FAutoConsoleVariableRef CVarRDGDebugParallelExecute(
	TEXT("r.RDG.ParallelExecuteStress"),
	GRDGParallelExecuteStress,
	TEXT("Stress tests the parallel execution path by launching one task per pass. Render pass merging is also disabled."),
	FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* Variable)
	{
		static int32 GRDGMergeRenderPassesHistory = GRDGMergeRenderPasses;
		static int32 GRDGParallelExecutePassMinHistory = GRDGParallelExecutePassMin;
		static int32 GRDGParallelExecutePassMaxHistory = GRDGParallelExecutePassMax;

		const int32 CurrentValue = Variable->GetInt();

		if (GRDGParallelExecuteStress == CurrentValue)
		{
			return;
		}

		if (CurrentValue)
		{
			GRDGMergeRenderPassesHistory = GRDGMergeRenderPasses;
			GRDGParallelExecutePassMinHistory = GRDGParallelExecutePassMin;
			GRDGParallelExecutePassMaxHistory = GRDGParallelExecutePassMax;

			GRDGMergeRenderPasses = 0;
			GRDGParallelExecutePassMin = 1;
			GRDGParallelExecutePassMax = 1;
		}
		else
		{
			GRDGMergeRenderPasses = GRDGMergeRenderPassesHistory;
			GRDGParallelExecutePassMin = GRDGParallelExecutePassMinHistory;
			GRDGParallelExecutePassMax = GRDGParallelExecutePassMaxHistory;
		}
	}),
	ECVF_RenderThreadSafe);

int32 GRDGParallelMobile = 0;
FAutoConsoleVariableRef CVarRDGParallelMobile(
	TEXT("r.RDG.ParallelMobile"), GRDGParallelMobile,
	TEXT("Enable RDG parallel execution on mobile renderer (disabled by default)."),
	ECVF_RenderThreadSafe);

#endif //!RDG_ENABLE_PARALLEL_TASKS

// Fix for random GPU crashes on draw indirects on multiple IHVs. Force all indirect arg buffers as non transient (see UE-115982)
int32 GRDGTransientIndirectArgBuffers = 0;
FAutoConsoleVariableRef CVarRDGIndirectArgBufferTransientAllocated(
	TEXT("r.RDG.TransientAllocator.IndirectArgumentBuffers"), GRDGTransientIndirectArgBuffers,
	TEXT("Whether indirect argument buffers should use transient resource allocator. Default: 0"),
	ECVF_RenderThreadSafe);

#if CSV_PROFILER_STATS
int32 GRDGVerboseCSVStats = 0;
FAutoConsoleVariableRef CVarRDGVerboseCSVStats(
	TEXT("r.RDG.VerboseCSVStats"),
	GRDGVerboseCSVStats,
	TEXT("Controls the verbosity of CSV profiling stats for RDG.\n")
	TEXT(" 0: emits one CSV profile for graph execution;\n")
	TEXT(" 1: emits a CSV profile for each phase of graph execution."),
	ECVF_RenderThreadSafe);
#endif

#if RDG_STATS
int32 GRDGStatPassCount = 0;
int32 GRDGStatPassWithParameterCount = 0;
int32 GRDGStatPassCullCount = 0;
int32 GRDGStatPassDependencyCount = 0;
int32 GRDGStatRenderPassMergeCount = 0;
int32 GRDGStatTextureCount = 0;
int32 GRDGStatTextureReferenceCount = 0;
int32 GRDGStatBufferCount = 0;
int32 GRDGStatBufferReferenceCount = 0;
int32 GRDGStatViewCount = 0;
int32 GRDGStatTransientTextureCount = 0;
int32 GRDGStatTransientBufferCount = 0;
int32 GRDGStatTransitionCount = 0;
int32 GRDGStatAliasingCount = 0;
std::atomic_int32_t GRDGStatTransitionBatchCount{ 0 };
int32 GRDGStatMemoryWatermark = 0;
#endif

CSV_DEFINE_CATEGORY(RDGCount, true);

TRACE_DECLARE_INT_COUNTER(COUNTER_RDG_PassCount, TEXT("RDG/PassCount"));
TRACE_DECLARE_INT_COUNTER(COUNTER_RDG_PassWithParameterCount, TEXT("RDG/PassWithParameterCount"));
TRACE_DECLARE_INT_COUNTER(COUNTER_RDG_PassCullCount, TEXT("RDG/PassCullCount"));
TRACE_DECLARE_INT_COUNTER(COUNTER_RDG_RenderPassMergeCount, TEXT("RDG/RenderPassMergeCount"));
TRACE_DECLARE_INT_COUNTER(COUNTER_RDG_PassDependencyCount, TEXT("RDG/PassDependencyCount"));
TRACE_DECLARE_INT_COUNTER(COUNTER_RDG_TextureCount, TEXT("RDG/TextureCount"));
TRACE_DECLARE_INT_COUNTER(COUNTER_RDG_TextureReferenceCount, TEXT("RDG/TextureReferenceCount"));
TRACE_DECLARE_FLOAT_COUNTER(COUNTER_RDG_TextureReferenceAverage, TEXT("RDG/TextureReferenceAverage"));
TRACE_DECLARE_INT_COUNTER(COUNTER_RDG_BufferCount, TEXT("RDG/BufferCount"));
TRACE_DECLARE_INT_COUNTER(COUNTER_RDG_BufferReferenceCount, TEXT("RDG/BufferReferenceCount"));
TRACE_DECLARE_FLOAT_COUNTER(COUNTER_RDG_BufferReferenceAverage, TEXT("RDG/BufferReferenceAverage"));
TRACE_DECLARE_INT_COUNTER(COUNTER_RDG_ViewCount, TEXT("RDG/ViewCount"));
TRACE_DECLARE_INT_COUNTER(COUNTER_RDG_TransientTextureCount, TEXT("RDG/TransientTextureCount"));
TRACE_DECLARE_INT_COUNTER(COUNTER_RDG_TransientBufferCount, TEXT("RDG/TransientBufferCount"));
TRACE_DECLARE_INT_COUNTER(COUNTER_RDG_TransitionCount, TEXT("RDG/TransitionCount"));
TRACE_DECLARE_INT_COUNTER(COUNTER_RDG_AliasingCount, TEXT("RDG/AliasingCount"));
TRACE_DECLARE_INT_COUNTER(COUNTER_RDG_TransitionBatchCount, TEXT("RDG/TransitionBatchCount"));
TRACE_DECLARE_MEMORY_COUNTER(COUNTER_RDG_MemoryWatermark, TEXT("RDG/MemoryWatermark"));

DEFINE_STAT(STAT_RDG_PassCount);
DEFINE_STAT(STAT_RDG_PassWithParameterCount);
DEFINE_STAT(STAT_RDG_PassCullCount);
DEFINE_STAT(STAT_RDG_RenderPassMergeCount);
DEFINE_STAT(STAT_RDG_PassDependencyCount);
DEFINE_STAT(STAT_RDG_TextureCount);
DEFINE_STAT(STAT_RDG_TextureReferenceCount);
DEFINE_STAT(STAT_RDG_TextureReferenceAverage);
DEFINE_STAT(STAT_RDG_BufferCount);
DEFINE_STAT(STAT_RDG_BufferReferenceCount);
DEFINE_STAT(STAT_RDG_BufferReferenceAverage);
DEFINE_STAT(STAT_RDG_ViewCount);
DEFINE_STAT(STAT_RDG_TransientTextureCount);
DEFINE_STAT(STAT_RDG_TransientBufferCount);
DEFINE_STAT(STAT_RDG_TransitionCount);
DEFINE_STAT(STAT_RDG_AliasingCount);
DEFINE_STAT(STAT_RDG_TransitionBatchCount);
DEFINE_STAT(STAT_RDG_SetupTime);
DEFINE_STAT(STAT_RDG_CompileTime);
DEFINE_STAT(STAT_RDG_ExecuteTime);
DEFINE_STAT(STAT_RDG_CollectResourcesTime);
DEFINE_STAT(STAT_RDG_CollectBarriersTime);
DEFINE_STAT(STAT_RDG_ClearTime);
DEFINE_STAT(STAT_RDG_FlushRHIResources);
DEFINE_STAT(STAT_RDG_MemoryWatermark);

void InitRenderGraph()
{
#if RDG_ENABLE_DEBUG_WITH_ENGINE
	if (FParse::Param(FCommandLine::Get(), TEXT("rdgimmediate")))
	{
		GRDGImmediateMode = 1;
	}

	int32 ValidationValue = 0;
	if (FParse::Value(FCommandLine::Get(), TEXT("rdgvalidation="), ValidationValue))
	{
		GRDGValidation = ValidationValue;
	}

	if (FParse::Param(FCommandLine::Get(), TEXT("rdgdebugextendresourcelifetimes")))
	{
		GRDGDebugExtendResourceLifetimes = 1;
	}

	if (FParse::Param(FCommandLine::Get(), TEXT("rdgtransitionlog")))
	{
		// Set to -1 to specify infinite number of frames.
		GRDGTransitionLog = -1;
	}

	if (FParse::Param(FCommandLine::Get(), TEXT("rdgclobberresources")))
	{
		GRDGClobberResources = 1;
	}

	int32 OverlapUAVsValue = 0;
	if (FParse::Value(FCommandLine::Get(), TEXT("rdgoverlapuavs="), OverlapUAVsValue))
	{
		GRDGOverlapUAVs = OverlapUAVsValue;
	}

	FString GraphFilter;
	if (FParse::Value(FCommandLine::Get(), TEXT("rdgdebuggraphfilter="), GraphFilter))
	{
		CVarRDGDebugGraphFilter->Set(*GraphFilter);
	}

	FString PassFilter;
	if (FParse::Value(FCommandLine::Get(), TEXT("rdgdebugpassfilter="), PassFilter))
	{
		CVarRDGDebugPassFilter->Set(*PassFilter);
	}

	FString ResourceFilter;
	if (FParse::Value(FCommandLine::Get(), TEXT("rdgdebugresourcefilter="), ResourceFilter))
	{
		CVarRDGDebugResourceFilter->Set(*ResourceFilter);
	}
#endif

	int32 TransientAllocatorValue = 0;
	if (FParse::Value(FCommandLine::Get(), TEXT("rdgtransientallocator="), TransientAllocatorValue))
	{
		GRDGTransientAllocator = TransientAllocatorValue;
	}

	int32 CullPassesValue = 0;
	if (FParse::Value(FCommandLine::Get(), TEXT("rdgcullpasses="), CullPassesValue))
	{
		GRDGCullPasses = CullPassesValue;
	}

#if RDG_ENABLE_PARALLEL_TASKS
	int32 ParallelSetupValue = 0;
	if (FParse::Value(FCommandLine::Get(), TEXT("rdgparallelsetup="), ParallelSetupValue))
	{
		GRDGParallelSetup = ParallelSetupValue;
	}

	int32 ParallelExecuteValue = 0;
	if (FParse::Value(FCommandLine::Get(), TEXT("rdgparallelexecute="), ParallelExecuteValue))
	{
		GRDGParallelExecute = ParallelExecuteValue;
	}
#endif

	int32 MergeRenderPassesValue = 0;
	if (FParse::Value(FCommandLine::Get(), TEXT("rdgmergerenderpasses="), MergeRenderPassesValue))
	{
		GRDGMergeRenderPasses = MergeRenderPassesValue;
	}

	int32 AsyncComputeValue = 0;
	if (FParse::Value(FCommandLine::Get(), TEXT("rdgasynccompute="), AsyncComputeValue))
	{
		CVarRDGAsyncCompute->Set(AsyncComputeValue);
	}

#if RDG_EVENTS
	int32 RDGEventValue = 0;
	if (FParse::Value(FCommandLine::Get(), TEXT("rdgevents="), RDGEventValue))
	{
		CVarRDGEvents->Set(RDGEventValue);
	}
#endif
}

void ShutdownRenderGraph()
{
	FRDGBuilder::WaitForAsyncDeleteTask();
}

FORCEINLINE bool IsParallelMobileEnabled()
{
#if RDG_ENABLE_PARALLEL_TASKS
	return (GRDGParallelMobile > 0);
#else
	return false;
#endif
}

bool IsParallelExecuteEnabled(EShaderPlatform ShaderPlatform)
{
	return GRDGParallelExecute > 0
		&& !GRHICommandList.Bypass()
		&& !IsImmediateMode()
		&& !GRDGDebugFlushGPU
		&& !GRDGTransitionLog
		&& (!IsMobilePlatform(ShaderPlatform) || IsParallelMobileEnabled())
		&& !IsOpenGLPlatform(ShaderPlatform)
		&& !IsVulkanMobileSM5Platform(ShaderPlatform)
		&& GRHISupportsMultithreadedShaderCreation
#if WITH_DUMPGPU
		&& !UE::RenderCore::DumpGPU::IsDumpingFrame()
#endif
		// Only run parallel RDG if we have a rendering thread.
		&& IsInActualRenderingThread()
		;
}

bool IsParallelSetupEnabled(EShaderPlatform ShaderPlatform)
{
	return GRDGParallelSetup > 0
		&& !GRHICommandList.Bypass()
		&& !IsImmediateMode()
		&& !GRDGTransitionLog
		&& (!IsMobilePlatform(ShaderPlatform) || IsParallelMobileEnabled())
		&& !IsOpenGLPlatform(ShaderPlatform)
		&& !IsVulkanMobileSM5Platform(ShaderPlatform)
		&& GRHISupportsMultithreadedShaderCreation
#if WITH_DUMPGPU
		&& !UE::RenderCore::DumpGPU::IsDumpingFrame()
#endif
		// Only run parallel RDG if we have a rendering thread.
		&& IsInActualRenderingThread()
		;
}

FRDGScopeState::FState::FState(bool bInImmediate, bool bInParallelExecute)
	: bImmediate(bInImmediate),
	  bParallelExecute(bInParallelExecute)
#if RDG_EVENTS
	, ScopeMode([]
	{
		bool bRDGChannelEnabled = false;
		IF_RDG_ENABLE_TRACE(bRDGChannelEnabled = UE_TRACE_CHANNELEXPR_IS_ENABLED(RDGChannel));

		if (FRDGBuilder::IsDumpingFrame() || UE::RHI::GPUProfiler::IsProfiling())
		{
			// We want all possible scope and pass names in a DumpGPU/profilegpu trace.
			return ERDGScopeMode::AllEventsAndPassNames;
		}

		// This is polled once as a workaround for a race condition since the underlying global is not always changed on the render thread.
		ERDGScopeMode LocalScopeMode = static_cast<ERDGScopeMode>(CVarRDGEvents.GetValueOnRenderThread());

		switch (LocalScopeMode)
		{
		case ERDGScopeMode::Disabled:
		case ERDGScopeMode::TopLevelOnly:
		case ERDGScopeMode::AllEvents:
			// Override to a higher level in some cases
			if (bRDGChannelEnabled != 0) //-V547
			{
				LocalScopeMode = ERDGScopeMode::AllEventsAndPassNames;
			}
			break;

		case ERDGScopeMode::AllEventsAndPassNames:
			break;

		default:
			LocalScopeMode = ERDGScopeMode::Disabled;
			break;
		}

		return LocalScopeMode;
	}())
#endif // RDG_EVENTS
{}

bool IsExtendedLifetimeResource(FRDGViewableResource* Resource)
{
#if RDG_ENABLE_DEBUG
	return IsDebugAllowedForResource(Resource->Name) && Resource->ReferenceCount != 0 && Resource->ReferenceCount != FRDGViewableResource::DeallocatedReferenceCount;
#else
	return false;
#endif
}