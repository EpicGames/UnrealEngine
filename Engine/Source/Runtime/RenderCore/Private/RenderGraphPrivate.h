// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/ScopeExit.h"
#include "RenderGraphDefinitions.h"
#include "ProfilingDebugging/CountersTrace.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "RHI.h"
#include "DataDrivenShaderPlatformInfo.h"

#define RDG_DUMP_GRAPH_PRODUCERS 1
#define RDG_DUMP_GRAPH_RESOURCES 2
#define RDG_DUMP_GRAPH_TRACKS 3

#define RDG_ASYNC_COMPUTE_DISABLED 0
#define RDG_ASYNC_COMPUTE_ENABLED 1
#define RDG_ASYNC_COMPUTE_FORCE_ENABLED 2

#ifndef RDG_ENABLE_PARALLEL_TASKS
#define RDG_ENABLE_PARALLEL_TASKS 1
#endif

#define RDG_RECURSION_COUNTER_SCOPE(Counter) Counter++; ON_SCOPE_EXIT { Counter--; }

#if RDG_ENABLE_DEBUG
extern int32 GRDGAsyncCompute;
extern int32 GRDGClobberResources;
extern int32 GRDGValidation;
extern int32 GRDGDebugFlushGPU;
extern int32 GRDGDebugExtendResourceLifetimes;
extern int32 GRDGDebugDisableTransientResources;
extern int32 GRDGTransitionLog;
extern int32 GRDGImmediateMode;
extern int32 GRDGOverlapUAVs;
extern std::atomic<bool> GRDGAllowRHIAccess;
extern std::atomic<bool> GRDGAllowRHIAccessAsync;

class FRDGAllowRHIAccessScope
{
public:
	FRDGAllowRHIAccessScope()
	{
		check(!GRDGAllowRHIAccess.load(std::memory_order_relaxed));
		GRDGAllowRHIAccess.store(true, std::memory_order_relaxed);
	}

	~FRDGAllowRHIAccessScope()
	{
		check(GRDGAllowRHIAccess.load(std::memory_order_relaxed));
		GRDGAllowRHIAccess.store(false, std::memory_order_relaxed);
	}
};

#define RDG_ALLOW_RHI_ACCESS_SCOPE() FRDGAllowRHIAccessScope RDGAllowRHIAccessScopeRAII;

// Colors for texture / buffer clobbering.
FLinearColor GetClobberColor();
uint32 GetClobberBufferValue();
float GetClobberDepth();
uint8 GetClobberStencil();

bool IsDebugAllowedForGraph(const TCHAR* GraphName);
bool IsDebugAllowedForPass(const TCHAR* PassName);
bool IsDebugAllowedForResource(const TCHAR* ResourceName);

void EmitRDGWarning(const FString& WarningMessage);

#else // !RDG_ENABLE_DEBUG

const int32 GRDGClobberResources = 0;
const int32 GRDGValidation = 0;
const int32 GRDGDebugFlushGPU = 0;
const int32 GRDGDebugExtendResourceLifetimes = 0;
const int32 GRDGDebugDisableTransientResources = 0;
const int32 GRDGTransitionLog = 0;
const int32 GRDGImmediateMode = 0;
const int32 GRDGOverlapUAVs = 1;

#define RDG_ALLOW_RHI_ACCESS_SCOPE()

#endif

extern int32 GRDGAsyncCompute;
extern int32 GRDGCullPasses;
extern int32 GRDGMergeRenderPasses;
extern int32 GRDGTransientAllocator;
extern int32 GRDGAsyncComputeTransientAliasing;
extern int32 GRDGTransientExtractedResources;
extern int32 GRDGTransientIndirectArgBuffers;

#if RDG_ENABLE_PARALLEL_TASKS

extern int32 GRDGParallelDestruction;
extern int32 GRDGParallelSetup;
extern int32 GRDGParallelCompile;
extern int32 GRDGParallelSetupTaskPriorityBias;
extern int32 GRDGParallelExecute;
extern int32 GRDGParallelExecutePassMin;
extern int32 GRDGParallelExecutePassMax;
extern int32 GRDGParallelExecutePassTaskModeThreshold;
extern int32 GRDGAsyncSetupQueue;

#else

const int32 GRDGParallelDestruction = 0;
const int32 GRDGParallelSetup = 0;
const int32 GRDGParallelCompile = 0;
const int32 GRDGParallelExecute = 0;
const int32 GRDGParallelExecutePassMin = 0;
const int32 GRDGParallelExecutePassMax = 0;
const int32 GRDGParallelExecutePassTaskModeThreshold = 0;
const int32 GRDGAsyncSetupQueue = 0;

#endif

#if CSV_PROFILER_STATS
extern int32 GRDGVerboseCSVStats;
#else
const int32 GRDGVerboseCSVStats = 0;
#endif

CSV_DECLARE_CATEGORY_EXTERN(RDGCount);

#define RDG_STATS STATS || COUNTERSTRACE_ENABLED

#if RDG_STATS
extern int32 GRDGStatPassCount;
extern int32 GRDGStatPassCullCount;
extern int32 GRDGStatRenderPassMergeCount;
extern int32 GRDGStatPassDependencyCount;
extern int32 GRDGStatTextureCount;
extern int32 GRDGStatTextureReferenceCount;
extern int32 GRDGStatBufferCount;
extern int32 GRDGStatBufferReferenceCount;
extern int32 GRDGStatViewCount;
extern int32 GRDGStatTransientTextureCount;
extern int32 GRDGStatTransientBufferCount;
extern int32 GRDGStatTransitionCount;
extern int32 GRDGStatAliasingCount;
extern std::atomic_int32_t GRDGStatTransitionBatchCount;
extern int32 GRDGStatMemoryWatermark;
#endif

TRACE_DECLARE_INT_COUNTER_EXTERN(COUNTER_RDG_PassCount);
TRACE_DECLARE_INT_COUNTER_EXTERN(COUNTER_RDG_PassWithParameterCount);
TRACE_DECLARE_INT_COUNTER_EXTERN(COUNTER_RDG_PassCullCount);
TRACE_DECLARE_INT_COUNTER_EXTERN(COUNTER_RDG_RenderPassMergeCount);
TRACE_DECLARE_INT_COUNTER_EXTERN(COUNTER_RDG_PassDependencyCount);
TRACE_DECLARE_INT_COUNTER_EXTERN(COUNTER_RDG_TextureCount);
TRACE_DECLARE_INT_COUNTER_EXTERN(COUNTER_RDG_TextureReferenceCount);
TRACE_DECLARE_FLOAT_COUNTER_EXTERN(COUNTER_RDG_TextureReferenceAverage);
TRACE_DECLARE_INT_COUNTER_EXTERN(COUNTER_RDG_BufferCount);
TRACE_DECLARE_INT_COUNTER_EXTERN(COUNTER_RDG_BufferReferenceCount);
TRACE_DECLARE_FLOAT_COUNTER_EXTERN(COUNTER_RDG_BufferReferenceAverage);
TRACE_DECLARE_INT_COUNTER_EXTERN(COUNTER_RDG_ViewCount);
TRACE_DECLARE_INT_COUNTER_EXTERN(COUNTER_RDG_TransientTextureCount);
TRACE_DECLARE_INT_COUNTER_EXTERN(COUNTER_RDG_TransientBufferCount);
TRACE_DECLARE_INT_COUNTER_EXTERN(COUNTER_RDG_TransitionCount);
TRACE_DECLARE_INT_COUNTER_EXTERN(COUNTER_RDG_AliasingCount);
TRACE_DECLARE_INT_COUNTER_EXTERN(COUNTER_RDG_TransitionBatchCount);
TRACE_DECLARE_MEMORY_COUNTER_EXTERN(COUNTER_RDG_MemoryWatermark);

DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Passes"), STAT_RDG_PassCount, STATGROUP_RDG, RENDERCORE_API);
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Passes With Parameters"), STAT_RDG_PassWithParameterCount, STATGROUP_RDG, RENDERCORE_API);
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Passes Culled"), STAT_RDG_PassCullCount, STATGROUP_RDG, RENDERCORE_API);
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Render Passes Merged"), STAT_RDG_RenderPassMergeCount, STATGROUP_RDG, RENDERCORE_API);
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Pass Dependencies"), STAT_RDG_PassDependencyCount, STATGROUP_RDG, RENDERCORE_API);
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Textures"), STAT_RDG_TextureCount, STATGROUP_RDG, RENDERCORE_API);
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Texture References"), STAT_RDG_TextureReferenceCount, STATGROUP_RDG, RENDERCORE_API);
DECLARE_FLOAT_COUNTER_STAT_EXTERN(TEXT("Texture References Average"), STAT_RDG_TextureReferenceAverage, STATGROUP_RDG, RENDERCORE_API);
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Buffers"), STAT_RDG_BufferCount, STATGROUP_RDG, RENDERCORE_API);
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Buffer References"), STAT_RDG_BufferReferenceCount, STATGROUP_RDG, RENDERCORE_API);
DECLARE_FLOAT_COUNTER_STAT_EXTERN(TEXT("Buffer References Average"), STAT_RDG_BufferReferenceAverage, STATGROUP_RDG, RENDERCORE_API);
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Views"), STAT_RDG_ViewCount, STATGROUP_RDG, RENDERCORE_API);
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Transient Textures"), STAT_RDG_TransientTextureCount, STATGROUP_RDG, RENDERCORE_API);
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Transient Buffers"), STAT_RDG_TransientBufferCount, STATGROUP_RDG, RENDERCORE_API);
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Resource Transitions"), STAT_RDG_TransitionCount, STATGROUP_RDG, RENDERCORE_API);
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Resource Acquires and Discards"), STAT_RDG_AliasingCount, STATGROUP_RDG, RENDERCORE_API);
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Resource Transition Batches"), STAT_RDG_TransitionBatchCount, STATGROUP_RDG, RENDERCORE_API);

DECLARE_CYCLE_STAT_EXTERN(TEXT("Setup"), STAT_RDG_SetupTime, STATGROUP_RDG, RENDERCORE_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Compile"), STAT_RDG_CompileTime, STATGROUP_RDG, RENDERCORE_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Execute"), STAT_RDG_ExecuteTime, STATGROUP_RDG, RENDERCORE_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Collect Resources"), STAT_RDG_CollectResourcesTime, STATGROUP_RDG, RENDERCORE_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Collect Barriers"), STAT_RDG_CollectBarriersTime, STATGROUP_RDG, RENDERCORE_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Clear"), STAT_RDG_ClearTime, STATGROUP_RDG, RENDERCORE_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Flush RHI Resources"), STAT_RDG_FlushRHIResources, STATGROUP_RDG, RENDERCORE_API);

DECLARE_MEMORY_STAT_EXTERN(TEXT("Builder Watermark"), STAT_RDG_MemoryWatermark, STATGROUP_RDG, RENDERCORE_API);

inline const TCHAR* GetEpilogueBarriersToBeginDebugName(ERHIPipeline Pipelines)
{
#if RDG_ENABLE_DEBUG
	switch (Pipelines)
	{
	case ERHIPipeline::Graphics:
		return TEXT("Epilogue (For Graphics)");
	case ERHIPipeline::AsyncCompute:
		return TEXT("Epilogue (For AsyncCompute)");
	case ERHIPipeline::All:
		return TEXT("Epilogue (For All)");
	}
#endif
	return TEXT("");
}

inline bool SkipUAVBarrier(FRDGViewHandle PreviousHandle, FRDGViewHandle NextHandle)
{
	// Barrier if previous / next don't have a matching valid skip-barrier UAV handle.
	if (GRDGOverlapUAVs != 0 && NextHandle.IsValid() && PreviousHandle == NextHandle)
	{
		return true;
	}

	return false;
}

FORCEINLINE bool IsImmediateMode()
{
	return GRDGImmediateMode != 0;
}

FORCEINLINE bool IsRenderPassMergeEnabled(EShaderPlatform ShaderPlatform)
{
	return GRDGMergeRenderPasses != 0 && !IsImmediateMode() && FDataDrivenShaderPlatformInfo::IsValid(ShaderPlatform) && RHIHasTiledGPU(ShaderPlatform) && !GRDGDebugFlushGPU;
}

FORCEINLINE bool IsAsyncComputeSupported(EShaderPlatform ShaderPlatform)
{
	// Render pass merging and async compute are mutually exclusive since they are mutually dependent on each other (i.e. we can't
	// merge render passes that involve async compute fences, since transitions would be relocated to the ends of the merged region).
	return GRDGAsyncCompute > 0 && !IsImmediateMode() && !IsRenderPassMergeEnabled(ShaderPlatform) && GSupportsEfficientAsyncCompute && GRHISupportsSeparateDepthStencilCopyAccess;
}

extern bool IsParallelExecuteEnabled(EShaderPlatform ShaderPlatform);
extern bool IsParallelSetupEnabled(EShaderPlatform ShaderPlatform);
extern bool IsExtendedLifetimeResource(FRDGViewableResource* Resource);

inline bool IsAsyncComputeTransientAliasingEnabled()
{
	return GRDGAsyncComputeTransientAliasing && GRHIGlobals.SupportsAsyncComputeTransientAliasing;
}

template <typename ResourceRegistryType, typename FunctionType>
void EnumerateExtendedLifetimeResources(ResourceRegistryType& Registry, FunctionType Function)
{
#if RDG_ENABLE_DEBUG
	if (GRDGDebugExtendResourceLifetimes)
	{
		for (auto Handle = Registry.Begin(); Handle != Registry.End(); ++Handle)
		{
			auto* Resource = Registry[Handle];

			if (IsExtendedLifetimeResource(Resource))
			{
				Function(Resource);
			}
		}
	}
#endif
}