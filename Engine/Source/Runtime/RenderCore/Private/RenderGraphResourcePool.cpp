// Copyright Epic Games, Inc. All Rights Reserved.

#include "RenderGraphResourcePool.h"
#include "RHICommandList.h"
#include "RenderGraphResources.h"
#include "RHITransientResourceAllocator.h"
#include "Trace/Trace.inl"
#include "ProfilingDebugging/CountersTrace.h"
#include "RenderCore.h"

TRACE_DECLARE_INT_COUNTER(BufferPoolCount, TEXT("BufferPool/BufferCount"));
TRACE_DECLARE_INT_COUNTER(BufferPoolCreateCount, TEXT("BufferPool/BufferCreateCount"));
TRACE_DECLARE_INT_COUNTER(BufferPoolReleaseCount, TEXT("BufferPool/BufferReleaseCount"));
TRACE_DECLARE_MEMORY_COUNTER(BufferPoolSize, TEXT("BufferPool/Size"));

UE_TRACE_EVENT_BEGIN(Cpu, FRDGBufferPool_CreateBuffer, NoSync)
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, Name)
	UE_TRACE_EVENT_FIELD(uint32, SizeInBytes)
UE_TRACE_EVENT_END()

RENDERCORE_API void DumpBufferPoolMemory(FOutputDevice& OutputDevice)
{
	GRenderGraphResourcePool.DumpMemoryUsage(OutputDevice);
}

static FAutoConsoleCommandWithOutputDevice GDumpBufferPoolMemoryCmd(
	TEXT("r.DumpBufferPoolMemory"),
	TEXT("Dump allocation information for the buffer pool."),
	FConsoleCommandWithOutputDeviceDelegate::CreateStatic(DumpBufferPoolMemory)
);

static FRDGBufferDesc GetAlignedBufferDesc(const FRDGBufferDesc& Desc, const TCHAR* InDebugName, ERDGPooledBufferAlignment Alignment)
{
	const uint64 BufferPageSize = 64 * 1024;

	FRDGBufferDesc AlignedDesc = Desc;

	switch (Alignment)
	{
	case ERDGPooledBufferAlignment::PowerOfTwo:
		AlignedDesc.NumElements = FMath::RoundUpToPowerOfTwo(AlignedDesc.BytesPerElement * AlignedDesc.NumElements) / AlignedDesc.BytesPerElement;
		// Fall through to align up to page size for small buffers; helps with reuse.

	case ERDGPooledBufferAlignment::Page:
		AlignedDesc.NumElements = Align(AlignedDesc.BytesPerElement * AlignedDesc.NumElements, BufferPageSize) / AlignedDesc.BytesPerElement;
	}

	if (!ensureMsgf(AlignedDesc.NumElements >= Desc.NumElements, TEXT("Alignment caused buffer size overflow for buffer '%s' (AlignedDesc.NumElements: %d < Desc.NumElements: %d)"), InDebugName, AlignedDesc.NumElements, Desc.NumElements))
	{
		// Use the unaligned desc since we apparently overflowed when rounding up.
		AlignedDesc = Desc;
	}

	return AlignedDesc;
}

void FRDGBufferPool::DumpMemoryUsage(FOutputDevice& OutputDevice)
{
	OutputDevice.Logf(TEXT("Pooled Buffers:"));

	Mutex.Lock();
	TArray<TRefCountPtr<FRDGPooledBuffer>> BuffersBySize = AllocatedBuffers;
	Mutex.Unlock();

	Algo::Sort(BuffersBySize, [](const TRefCountPtr<FRDGPooledBuffer>& LHS, const TRefCountPtr<FRDGPooledBuffer>& RHS)
	{
		return LHS->GetAlignedSize() > RHS->GetAlignedSize();
	});

	for (const TRefCountPtr<FRDGPooledBuffer>& Buffer : BuffersBySize)
	{
		const uint32 BufferSize = Buffer->GetAlignedSize();
		const uint32 UnusedForNFrames = FrameCounter - Buffer->LastUsedFrame;

		OutputDevice.Logf(
			TEXT("  %6.3fMB Name: %s, NumElements: %u, BytesPerElement: %u, UAV: %s, Frames Since Requested: %u"),
			(float)BufferSize / (1024.0f * 1024.0f),
			Buffer->Name,
			Buffer->NumAllocatedElements,
			Buffer->Desc.BytesPerElement,
			EnumHasAnyFlags(Buffer->Desc.Usage, EBufferUsageFlags::UnorderedAccess) ? TEXT("Yes") : TEXT("No"),
			UnusedForNFrames);
	}
}

template <typename T>
FRDGPooledBuffer* FRDGBufferPool::TryFindPooledBuffer(const FRDGBufferDesc& Desc, uint32 DescHash, T&& Predicate)
{
	for (int32 Index = 0; Index < AllocatedBufferHashes.Num(); ++Index)
	{
		if (AllocatedBufferHashes[Index] != DescHash)
		{
			continue;
		}

		FRDGPooledBuffer* Found = AllocatedBuffers[Index];

		// Still being used outside the pool.
		if (Found->GetRefCount() > 1 || !Predicate(Found))
		{
			continue;
		}

		check(Found->GetAlignedDesc() == Desc);
		return Found;
	}
	return nullptr;
}

FRDGPooledBuffer* FRDGBufferPool::ScheduleAllocation(
	FRHICommandListBase& RHICmdList,
	const FRDGBufferDesc& Desc,
	const TCHAR* Name,
	ERDGPooledBufferAlignment Alignment,
	const FRHITransientAllocationFences& Fences)
{
	const FRDGBufferDesc AlignedDesc = GetAlignedBufferDesc(Desc, Name, Alignment);
	const uint32 DescHash = GetTypeHash(AlignedDesc);

	FRDGPooledBuffer* PooledBuffer = TryFindPooledBuffer(AlignedDesc, DescHash, [&](FRDGPooledBuffer* PooledBuffer)
	{
		return PooledBuffer->Fences && !FRHITransientAllocationFences::Contains(*PooledBuffer->Fences, Fences);
	});

	if (!PooledBuffer)
	{
		PooledBuffer = CreateBuffer(RHICmdList, AlignedDesc, DescHash, Name);
	}

	// We need the external-facing desc to match what the user requested.
	const_cast<FRDGBufferDesc&>(PooledBuffer->Desc).NumElements = Desc.NumElements;
	PooledBuffer->Fences.Reset();
	PooledBuffer->LastUsedFrame = FrameCounter;
	return PooledBuffer;
}

void FRDGBufferPool::ScheduleDeallocation(FRDGPooledBuffer* PooledBuffer, const FRHITransientAllocationFences& Fences)
{
	PooledBuffer->Fences = Fences;
}

void FRDGBufferPool::FinishSchedule(FRHICommandListBase& RHICmdList, FRDGPooledBuffer* PooledBuffer)
{
	PooledBuffer->Fences.Emplace();
	PooledBuffer->SetDebugLabelName(RHICmdList, PooledBuffer->Name);
}

TRefCountPtr<FRDGPooledBuffer> FRDGBufferPool::FindFreeBuffer(FRHICommandListBase& RHICmdList, const FRDGBufferDesc& Desc, const TCHAR* InDebugName, ERDGPooledBufferAlignment Alignment)
{
	const FRDGBufferDesc AlignedDesc = GetAlignedBufferDesc(Desc, InDebugName, Alignment);
	const uint32 DescHash = GetTypeHash(AlignedDesc);

	UE::TScopeLock Lock(Mutex);

	FRDGPooledBuffer* PooledBuffer = TryFindPooledBuffer(AlignedDesc, DescHash);

	if (!PooledBuffer)
	{
		PooledBuffer = CreateBuffer(RHICmdList, AlignedDesc, DescHash, InDebugName);
	}

	// We need the external-facing desc to match what the user requested.
	const_cast<FRDGBufferDesc&>(PooledBuffer->Desc).NumElements = Desc.NumElements;
	PooledBuffer->LastUsedFrame = FrameCounter;
	PooledBuffer->SetDebugLabelName(RHICmdList, InDebugName);
	return PooledBuffer;
}

FRDGPooledBuffer* FRDGBufferPool::CreateBuffer(FRHICommandListBase& RHICmdList, const FRDGBufferDesc& Desc, uint32 DescHash, const TCHAR* InDebugName)
{
	const uint32 NumBytes = Desc.GetSize();

#if CPUPROFILERTRACE_ENABLED
	UE_TRACE_LOG_SCOPED_T(Cpu, FRDGBufferPool_CreateBuffer, CpuChannel)
		<< FRDGBufferPool_CreateBuffer.Name(InDebugName)
		<< FRDGBufferPool_CreateBuffer.SizeInBytes(NumBytes);
#endif

	TRACE_COUNTER_ADD(BufferPoolCount, 1);
	TRACE_COUNTER_ADD(BufferPoolCreateCount, 1);
	TRACE_COUNTER_ADD(BufferPoolSize, NumBytes);
	
	LLM_SCOPE_BYNAME(TEXT("RHIMisc/BufferPool"));
	UE_TRACE_METADATA_CLEAR_SCOPE(); // Do not associate a pooled buffer with specific asset
	LLM_TAGSET_SCOPE_CLEAR(ELLMTagSet::Assets);
	LLM_TAGSET_SCOPE_CLEAR(ELLMTagSet::AssetClasses);

	const FRHIBufferCreateDesc CreateDesc =
		FRHIBufferCreateDesc::Create(InDebugName, NumBytes, Desc.BytesPerElement, Desc.Usage)
		.SetInitialState(RHIGetDefaultResourceState(Desc.Usage, false));

	TRefCountPtr<FRHIBuffer> BufferRHI = RHICmdList.CreateBuffer(CreateDesc);

	FRDGPooledBuffer* PooledBuffer = new FRDGPooledBuffer(RHICmdList, MoveTemp(BufferRHI), Desc, Desc.NumElements, InDebugName);
	AllocatedBuffers.Add(PooledBuffer);
	AllocatedBufferHashes.Add(DescHash);

	if (EnumHasAllFlags(Desc.Usage, EBufferUsageFlags::ReservedResource))
	{
		PooledBuffer->CommittedSizeInBytes = 0;
	}

	return PooledBuffer;
}

void FRDGBufferPool::ReleaseRHI()
{
	AllocatedBuffers.Empty();
	AllocatedBufferHashes.Empty();
}

void FRDGBufferPool::TickPoolElements()
{
	const uint32 kFramesUntilRelease = 30;

	int32 BufferIndex = 0;
	int32 NumReleasedBuffers = 0;
	int64 NumReleasedBufferBytes = 0;

	UE::TScopeLock Lock(Mutex);

	while (BufferIndex < AllocatedBuffers.Num())
	{
		TRefCountPtr<FRDGPooledBuffer>& Buffer = AllocatedBuffers[BufferIndex];

		const bool bIsUnused = Buffer.GetRefCount() == 1;

		const bool bNotRequestedRecently = (FrameCounter - Buffer->LastUsedFrame) > kFramesUntilRelease;

		if (bIsUnused && bNotRequestedRecently)
		{
			NumReleasedBufferBytes += Buffer->GetAlignedDesc().GetSize();

			AllocatedBuffers.RemoveAtSwap(BufferIndex);
			AllocatedBufferHashes.RemoveAtSwap(BufferIndex);

			++NumReleasedBuffers;
		}
		else
		{
			++BufferIndex;
		}
	}

	TRACE_COUNTER_SUBTRACT(BufferPoolSize, NumReleasedBufferBytes);
	TRACE_COUNTER_SUBTRACT(BufferPoolCount, NumReleasedBuffers);
	TRACE_COUNTER_SET(BufferPoolReleaseCount, NumReleasedBuffers);
	TRACE_COUNTER_SET(BufferPoolCreateCount, 0);

	++FrameCounter;
}

TGlobalResource<FRDGBufferPool> GRenderGraphResourcePool;

uint32 FRDGTransientRenderTarget::AddRef() const
{
	check(LifetimeState == ERDGTransientResourceLifetimeState::Allocated);
	return uint32(FPlatformAtomics::InterlockedIncrement(&RefCount));
}

uint32 FRDGTransientRenderTarget::Release()
{
	const int32 Refs = FPlatformAtomics::InterlockedDecrement(&RefCount);
	check(Refs >= 0);
	if (Refs == 0)
	{
		check(LifetimeState == ERDGTransientResourceLifetimeState::Allocated);
		if (GRDGTransientResourceAllocator.IsValid())
		{
			GRDGTransientResourceAllocator.AddPendingDeallocation(this);
		}
		else
		{
			delete this;
		}
	}
	return Refs;
}

void FRDGTransientResourceAllocator::InitRHI(FRHICommandListBase&)
{
	Allocator = RHICreateTransientResourceAllocator();
}

void FRDGTransientResourceAllocator::ReleaseRHI()
{
	if (Allocator)
	{
		FRHICommandListImmediate& RHICmdList = FRHICommandListImmediate::Get();

		ReleasePendingDeallocations();
		PendingDeallocationList.Empty();

		for (FRDGTransientRenderTarget* RenderTarget : DeallocatedList)
		{
			delete RenderTarget;
		}
		DeallocatedList.Empty();

		Allocator->Flush(RHICmdList);
		
		// Allocator->Flush() enqueues some lambdas on the command list, so make sure they are executed
		// before the allocator is deleted.
		RHICmdList.ImmediateFlush(EImmediateFlushType::FlushRHIThread);

		Allocator->Release(RHICmdList);
		Allocator = nullptr;
	}
}

TRefCountPtr<FRDGTransientRenderTarget> FRDGTransientResourceAllocator::AllocateRenderTarget(FRHITransientTexture* Texture)
{
	check(Texture);

	FRDGTransientRenderTarget* RenderTarget = nullptr;

	if (!FreeList.IsEmpty())
	{
		RenderTarget = FreeList.Pop();
	}
	else
	{
		RenderTarget = new FRDGTransientRenderTarget();
	}

	RenderTarget->Texture = Texture;
	RenderTarget->Desc = Translate(Texture->CreateInfo);
	RenderTarget->Desc.DebugName = Texture->GetName();
	RenderTarget->LifetimeState = ERDGTransientResourceLifetimeState::Allocated;
	RenderTarget->RenderTargetItem.TargetableTexture = Texture->GetRHI();
	RenderTarget->RenderTargetItem.ShaderResourceTexture = Texture->GetRHI();
	return RenderTarget;
}

void FRDGTransientResourceAllocator::Release(TRefCountPtr<FRDGTransientRenderTarget>&& RenderTarget, const FRHITransientAllocationFences& Fences)
{
	check(RenderTarget);

	// If this is true, we hold the final reference in the RenderTarget argument. We want to zero out its
	// members before dereferencing to zero so that it gets marked as deallocated rather than pending.
	if (RenderTarget->GetRefCount() == 1)
	{
		Allocator->DeallocateMemory(RenderTarget->Texture, Fences);
		RenderTarget->Reset();
		RenderTarget = nullptr;
	}
}

void FRDGTransientResourceAllocator::AddPendingDeallocation(FRDGTransientRenderTarget* RenderTarget)
{
	check(RenderTarget);
	check(RenderTarget->GetRefCount() == 0);

	FScopeLock Lock(&CS);

	if (RenderTarget->Texture)
	{
		RenderTarget->LifetimeState = ERDGTransientResourceLifetimeState::PendingDeallocation;
		PendingDeallocationList.Emplace(RenderTarget);
	}
	else
	{
		RenderTarget->LifetimeState = ERDGTransientResourceLifetimeState::Deallocated;
		DeallocatedList.Emplace(RenderTarget);
	}
}

void FRDGTransientResourceAllocator::ReleasePendingDeallocations()
{
	FScopeLock Lock(&CS);

	if (!PendingDeallocationList.IsEmpty())
	{
		TArray<FRHITrackedAccessInfo, SceneRenderingAllocator> EpilogueResourceAccesses;
		EpilogueResourceAccesses.Reserve(PendingDeallocationList.Num());

		TArray<FRHITransitionInfo, SceneRenderingAllocator> Transitions;
		Transitions.Reserve(PendingDeallocationList.Num());

		TArray<FRHITransientAliasingInfo, SceneRenderingAllocator> Aliases;
		Aliases.Reserve(PendingDeallocationList.Num());

		for (FRDGTransientRenderTarget* RenderTarget : PendingDeallocationList)
		{
			FRHITransientAllocationFences Fences(ERHIPipeline::Graphics);
			Fences.SetGraphics(0);
			Allocator->DeallocateMemory(RenderTarget->Texture, Fences);
			Transitions.Emplace(RenderTarget->Texture->GetRHI(), ERHIAccess::Unknown, ERHIAccess::Discard);
			EpilogueResourceAccesses.Emplace(RenderTarget->Texture->GetRHI(), ERHIAccess::Discard, ERHIPipeline::Graphics);

			RenderTarget->Reset();
			RenderTarget->LifetimeState = ERDGTransientResourceLifetimeState::Deallocated;
		}

		{
			const FRHITransition* Transition = RHICreateTransition(FRHITransitionCreateInfo(ERHIPipeline::Graphics, ERHIPipeline::Graphics, ERHITransitionCreateFlags::None, Transitions, Aliases));

			FRHICommandListImmediate& RHICmdList = FRHICommandListImmediate::Get();
			RHICmdList.BeginTransition(Transition);
			RHICmdList.EndTransition(Transition);
			RHICmdList.SetTrackedAccess(EpilogueResourceAccesses);
		}

		FreeList.Append(PendingDeallocationList);
		PendingDeallocationList.Reset();
	}

	if (!DeallocatedList.IsEmpty())
	{
		FreeList.Append(DeallocatedList);
		DeallocatedList.Reset();
	}
}

TGlobalResource<FRDGTransientResourceAllocator, FRenderResource::EInitPhase::Pre> GRDGTransientResourceAllocator;
