// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	RenderTargetPool.h: Scene render target pool manager.
=============================================================================*/

#pragma once

#include "RenderResource.h"
#include "RendererInterface.h"
#include "RenderGraphResources.h"
#include "Async/RecursiveMutex.h"

class FRDGBufferPool : public FRenderResource
{
public:
	FRDGBufferPool() = default;

	/** Call once per frame to trim elements from the pool. */
	RENDERCORE_API void TickPoolElements();

	RENDERCORE_API TRefCountPtr<FRDGPooledBuffer> FindFreeBuffer(FRHICommandListBase& RHICmdList, const FRDGBufferDesc& Desc, const TCHAR* InDebugName, ERDGPooledBufferAlignment Alignment = ERDGPooledBufferAlignment::Page);

	TRefCountPtr<FRDGPooledBuffer> FindFreeBuffer(const FRDGBufferDesc& Desc, const TCHAR* InDebugName, ERDGPooledBufferAlignment Alignment = ERDGPooledBufferAlignment::Page)
	{
		return FindFreeBuffer(FRHICommandListImmediate::Get(), Desc, InDebugName, Alignment);
	}

	RENDERCORE_API void DumpMemoryUsage(FOutputDevice& OutputDevice);

private:
	RENDERCORE_API void ReleaseRHI() override;

	FRDGPooledBuffer* CreateBuffer(FRHICommandListBase& RHICmdList, const FRDGBufferDesc& Desc, uint32 DescHash, const TCHAR* InDebugName);

	FRDGPooledBuffer* ScheduleAllocation(
		FRHICommandListBase& RHICmdList,
		const FRDGBufferDesc& Desc,
		const TCHAR* Name,
		ERDGPooledBufferAlignment Alignment,
		const FRHITransientAllocationFences& Fences);

	void ScheduleDeallocation(FRDGPooledBuffer* PooledBuffer, const FRHITransientAllocationFences& Fences);

	void FinishSchedule(FRHICommandListBase& RHICmdList, FRDGPooledBuffer* PooledBuffer);

	template <typename T>
	FRDGPooledBuffer* TryFindPooledBuffer(const FRDGBufferDesc& Desc, uint32 DescHash, T&& Predicate);

	FRDGPooledBuffer* TryFindPooledBuffer(const FRDGBufferDesc& Desc, uint32 DescHash)
	{
		return TryFindPooledBuffer(Desc, DescHash, [](FRDGPooledBuffer*) { return true; });
	}

	mutable UE::FRecursiveMutex Mutex;

	/** Elements can be 0, we compact the buffer later. */
	TArray<TRefCountPtr<FRDGPooledBuffer>> AllocatedBuffers;
	TArray<uint32> AllocatedBufferHashes;

	uint32 FrameCounter = 0;

	friend class FRDGBuilder;
};

/** The global render targets for easy shading. */
extern RENDERCORE_API TGlobalResource<FRDGBufferPool> GRenderGraphResourcePool;

enum class ERDGTransientResourceLifetimeState
{
	Deallocated,
	Allocated,
	PendingDeallocation
};

class FRDGTransientRenderTarget final : public IPooledRenderTarget
{
public:
	uint32 AddRef() const override;
	uint32 Release() override;
	uint32 GetRefCount() const override { return RefCount; }

	bool IsFree() const override { return false; }
	bool IsTracked() const override { return true; }
	uint32 ComputeMemorySize() const override { return 0; }

	const FPooledRenderTargetDesc& GetDesc() const override { return Desc; }

	FRHITransientTexture* GetTransientTexture() const override
	{
		check(LifetimeState == ERDGTransientResourceLifetimeState::Allocated);
		return Texture;
	}

	void Reset()
	{
		Texture = nullptr;
		RenderTargetItem.ShaderResourceTexture = nullptr;
		RenderTargetItem.TargetableTexture = nullptr;
	}

private:
	FRDGTransientRenderTarget() = default;

	FRHITransientTexture* Texture;
	FPooledRenderTargetDesc Desc;
	ERDGTransientResourceLifetimeState LifetimeState;
	mutable int32 RefCount = 0;

	friend class FRDGTransientResourceAllocator;
};

class FRDGTransientResourceAllocator : public FRenderResource
{
public:
	IRHITransientResourceAllocator* Get() { return Allocator; }

	TRefCountPtr<FRDGTransientRenderTarget> AllocateRenderTarget(FRHITransientTexture* Texture);

	void Release(TRefCountPtr<FRDGTransientRenderTarget>&& RenderTarget, const FRHITransientAllocationFences& Fences);

	void ReleasePendingDeallocations();

	bool IsValid() const { return Allocator != nullptr; }

private:
	void InitRHI(FRHICommandListBase& RHICmdList) override;
	void ReleaseRHI() override;

	void AddPendingDeallocation(FRDGTransientRenderTarget* RenderTarget);

	IRHITransientResourceAllocator* Allocator = nullptr;

	FCriticalSection CS;
	TArray<FRDGTransientRenderTarget*> FreeList;
	TArray<FRDGTransientRenderTarget*> PendingDeallocationList;
	TArray<FRDGTransientRenderTarget*> DeallocatedList;

	friend class FRDGTransientRenderTarget;
};

extern RENDERCORE_API TGlobalResource<FRDGTransientResourceAllocator, FRenderResource::EInitPhase::Pre> GRDGTransientResourceAllocator;
