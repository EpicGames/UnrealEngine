// Copyright Epic Games, Inc. All Rights Reserved.

#include "RenderGraphResources.h"
#include "RenderGraphPrivate.h"

inline bool SkipUAVBarrier(const FRDGSubresourceState& Previous, const FRDGSubresourceState& Next)
{
	return SkipUAVBarrier(Previous.NoUAVBarrierFilter.GetUniqueHandle(), Next.NoUAVBarrierFilter.GetUniqueHandle());
}

FRDGTexture::FRDGTexture(const TCHAR* InName, const FRDGTextureDesc& InDesc, ERDGTextureFlags InFlags)
	: FRDGViewableResource(InName, ERDGViewableResourceType::Texture, EnumHasAnyFlags(InFlags, ERDGTextureFlags::SkipTracking), !EnumHasAnyFlags(InFlags, ERDGTextureFlags::ForceImmediateFirstBarrier) && !EnumHasAnyFlags(InDesc.Flags, ETextureCreateFlags::Presentable))
	, Desc(InDesc)
	, Flags(InFlags)
	, Layout(InDesc)
	, WholeRange(Layout)
	, SubresourceCount(Layout.GetSubresourceCount())
{
	if (EnumHasAnyFlags(Desc.Flags, ETextureCreateFlags::Foveation))
	{
		EpilogueAccess = ERHIAccess::ShadingRateSource;
	}

	State.SetNum(SubresourceCount);
	FirstState.SetNum(SubresourceCount);
	MergeState.SetNum(SubresourceCount);
	LastProducers.SetNum(SubresourceCount);
}

FRDGBuffer::FRDGBuffer(const TCHAR* InName, const FRDGBufferDesc& InDesc, ERDGBufferFlags InFlags)
	: FRDGViewableResource(InName, ERDGViewableResourceType::Buffer, EnumHasAnyFlags(InFlags, ERDGBufferFlags::SkipTracking), !EnumHasAnyFlags(InFlags, ERDGBufferFlags::ForceImmediateFirstBarrier))
	, Desc(InDesc)
	, Flags(InFlags)
{
}

FRDGBuffer::FRDGBuffer(const TCHAR* InName, const FRDGBufferDesc& InDesc, ERDGBufferFlags InFlags, FRDGBufferNumElementsCallback* InNumElementsCallback)
	: FRDGBuffer(InName, InDesc, InFlags)
{
	NumElementsCallback = InNumElementsCallback;
}

FRDGViewableResource::FRDGViewableResource(const TCHAR* InName, const ERDGViewableResourceType InType, bool bSkipTracking, bool bInSplitFirstTransition)
	: FRDGResource(InName)
	, Type(InType)
	, bTransient(0)
	, bForceNonTransient(0)
	, bSkipLastTransition(0)
	, bSplitFirstTransition(bInSplitFirstTransition)
	, bQueuedForUpload(0)
	, bCollectForAllocate(1)
#if RHI_USE_RESOURCE_DEBUG_NAME
	, bHeapAllocatedDebugName(0)
#endif
#if RDG_ENABLE_DEBUG
	, bClobbered(0)
#endif
	, TransientExtractionHint(ETransientExtractionHint::None)
	, ReferenceCount(IsImmediateMode() ? 1 : 0)
{
	if (bSkipTracking)
	{
		SetExternalAccessMode(ERHIAccess::Mask, ERHIPipeline::All);
		AccessModeState.bLocked = 1;
		AccessModeState.ActiveMode = AccessModeState.Mode;
	}
}

bool FRDGSubresourceState::IsMergeAllowed(ERDGViewableResourceType ResourceType, const FRDGSubresourceState& Previous, const FRDGSubresourceState& Next)
{
	/** State merging occurs during compilation and before resource transitions are collected. It serves to remove the bulk
	 *  of unnecessary transitions by looking ahead in the resource usage chain. A resource transition cannot occur within
	 *  a merged state, so a merge is not allowed to proceed if a barrier might be required. Merging is also where multi-pipe
	 *  transitions are determined, if supported by the platform.
	 */

	const ERHIAccess AccessUnion = Previous.Access | Next.Access;
	const ERHIAccess DSVMask = ERHIAccess::DSVRead | ERHIAccess::DSVWrite;

	// If we have the same access between the two states, we don't need to check for invalid access combinations.
	if (Previous.Access != Next.Access)
	{
		// Not allowed to merge read-only and writable states.
		if (EnumHasAnyFlags(Previous.Access, ERHIAccess::ReadOnlyExclusiveMask) && EnumHasAnyFlags(Next.Access, ERHIAccess::WritableMask))
		{
			return false;
		}

		// Not allowed to merge write-only and readable states.
		if (EnumHasAnyFlags(Previous.Access, ERHIAccess::WriteOnlyExclusiveMask) && EnumHasAnyFlags(Next.Access, ERHIAccess::ReadableMask))
		{
			return false;
		}

		// UAVs will filter through the above checks because they are both read and write. UAV can only merge it itself.
		if (EnumHasAnyFlags(AccessUnion, ERHIAccess::UAVMask) && EnumHasAnyFlags(AccessUnion, ~ERHIAccess::UAVMask))
		{
			return false;
		}

		// Depth Read / Write should never merge with anything other than itself.
		if (EnumHasAllFlags(AccessUnion, DSVMask) && EnumHasAnyFlags(AccessUnion, ~DSVMask))
		{
			return false;
		}

		// Filter out platform-specific unsupported mergeable states.
		if (EnumHasAnyFlags(AccessUnion, ~GRHIMergeableAccessMask))
		{
			return false;
		}
	}

	// Not allowed if the resource is being used as a UAV and needs a barrier.
	if (EnumHasAnyFlags(Next.Access, ERHIAccess::UAVMask) && !SkipUAVBarrier(Previous, Next))
	{
		return false;
	}

	// Filter out unsupported platform-specific multi-pipeline merged accesses.
	if (EnumHasAnyFlags(AccessUnion, ~GRHIMultiPipelineMergeableAccessMask) && Previous.GetPipelines() != Next.GetPipelines())
	{
		return false;
	}

	// Not allowed to merge differing flags.
	if (Previous.Flags != Next.Flags)
	{
		return false;
	}

	return true;
}

bool FRDGSubresourceState::IsTransitionRequired(const FRDGSubresourceState& Previous, const FRDGSubresourceState& Next)
{
	// This function only needs to filter out identical states and handle UAV barriers.
	check(Next.Access != ERHIAccess::Unknown);

	if (Previous.Access != Next.Access || Previous.GetPipelines() != Next.GetPipelines() || Previous.Flags != Next.Flags)
	{
		return true;
	}

	// UAV is a special case as a barrier may still be required even if the states match.
	if (EnumHasAnyFlags(Next.Access, ERHIAccess::UAVMask) && !SkipUAVBarrier(Previous, Next))
	{
		return true;
	}

	return false;
}

FRDGPooledBuffer::FRDGPooledBuffer(TRefCountPtr<FRHIBuffer> InBuffer, const FRDGBufferDesc& InDesc, uint32 InNumAllocatedElements, const TCHAR* InName)
	: FRDGPooledBuffer(FRHICommandListImmediate::Get(), MoveTemp(InBuffer), InDesc, InNumAllocatedElements, InName)
{}

void FRDGPooledBuffer::SetDebugLabelName(FRHICommandListBase& RHICmdList, const TCHAR* InName)
{
#if RHI_USE_RESOURCE_DEBUG_NAME
	// For performance, avoid updating name if it happens to be the same (true 80% of the time in testing)
	bool bNameUpdated = false;
	if (!InName || FCString::Strcmp(Name, InName))
	{
		// Name changed, need to update it
		Name = InName;

		RHICmdList.BindDebugLabelName(GetRHI(), InName);

		bNameUpdated = true;
	}

	// Propagate the debug name to ViewCache if the name was updated, or if any items were added to ViewCache since the debug name was set
	int32 ViewCacheNum = ViewCache.NumItems();
	if (bNameUpdated || NameUpdatedViewCacheNum != ViewCacheNum)
	{
		NameUpdatedViewCacheNum = ViewCacheNum;
		ViewCache.SetDebugName(RHICmdList, InName);
	}
#else
	Name = InName;
#endif
}

FRDGUniformBuffer::~FRDGUniformBuffer() = default;

void FRDGUniformBuffer::InitRHI()
{
	check(!HasRHI());

	const EUniformBufferValidation Validation =
#if RDG_ENABLE_DEBUG
		EUniformBufferValidation::ValidateResources;
#else
		EUniformBufferValidation::None;
#endif

	const FRDGParameterStruct& PassParameters = GetParameters();
	UniformBufferRHI = RHICreateUniformBuffer(PassParameters.GetContents(), PassParameters.GetLayoutPtr(), UniformBuffer_SingleFrame, Validation);
	ResourceRHI = UniformBufferRHI;
}

FRDGTextureSubresourceRange FRDGTexture::GetSubresourceRangeSRV() const
{
	FRDGTextureSubresourceRange Range = GetSubresourceRange();

	// When binding a whole texture for shader read (SRV), we only use the first plane.
	// Other planes like stencil require a separate view to access for read in the shader.
	Range.PlaneSlice = FRHITransitionInfo::kDepthPlaneSlice;
	Range.NumPlaneSlices = 1;

	return Range;
}

void FRDGBuffer::FinalizeDesc()
{
	if (NumElementsCallback)
	{
		Desc.NumElements = FMath::Max((*NumElementsCallback)(), 1u);
		*NumElementsCallback = {};
		NumElementsCallback = nullptr;
	}
}