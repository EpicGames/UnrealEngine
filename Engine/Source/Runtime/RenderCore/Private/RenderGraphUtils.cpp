// Copyright Epic Games, Inc. All Rights Reserved.

#include "RenderGraphUtils.h"

#include "ClearQuad.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "GlobalShader.h"
#include "PixelShaderUtils.h"
#include "RenderGraphResourcePool.h"
#include "RenderTargetPool.h"
#include "RHIGPUReadback.h"

#include <initializer_list>

static TAutoConsoleVariable<bool> CVarRDGBarrierPass(
	TEXT("r.RDG.BarrierPass"),
	true,
	TEXT("Whether to allow AddBarrierPass. AddBarrierPass is only used on plaforms which don't support split barriers and allows to manually batch barriers for optimal GPU dispatch overlap."),
	ECVF_RenderThreadSafe
);

void ClearUnusedGraphResourcesImpl(
	const FShaderParameterBindings& ShaderBindings,
	const FShaderParametersMetadata* ParametersMetadata,
	void* InoutParameters,
	std::initializer_list<FRDGResourceRef> ExcludeList)
{
	const auto& GraphResources = ParametersMetadata->GetLayout().GraphResources;

	int32 ShaderResourceIndex = 0;
	int32 BindlessResourceIndex = 0;
	int32 GraphUniformBufferId = 0;
	uint8* const Base = reinterpret_cast<uint8*>(InoutParameters);

	for (int32 GraphResourceIndex = 0, GraphResourceCount = GraphResources.Num(); GraphResourceIndex < GraphResourceCount; GraphResourceIndex++)
	{
		const EUniformBufferBaseType Type = GraphResources[GraphResourceIndex].MemberType;
		const uint16 ByteOffset = GraphResources[GraphResourceIndex].MemberOffset;

		if (Type == UBMT_RDG_TEXTURE ||
			Type == UBMT_RDG_TEXTURE_SRV ||
			Type == UBMT_RDG_TEXTURE_NON_PIXEL_SRV ||
			Type == UBMT_RDG_TEXTURE_UAV ||
			Type == UBMT_RDG_BUFFER_SRV ||
			Type == UBMT_RDG_BUFFER_UAV ||
			Type == UBMT_RESOURCE_COLLECTION)
		{
			const TMemoryImageArray<FShaderParameterBindings::FResourceParameter>& ResourceParameters = ShaderBindings.ResourceParameters;
			const int32 ShaderResourceCount = ResourceParameters.Num();
			for (; ShaderResourceIndex < ShaderResourceCount && ResourceParameters[ShaderResourceIndex].ByteOffset < ByteOffset; ++ShaderResourceIndex)
			{
			}

			if (ShaderResourceIndex < ShaderResourceCount && ResourceParameters[ShaderResourceIndex].ByteOffset == ByteOffset)
			{
				continue;
			}

			const TMemoryImageArray<FShaderParameterBindings::FBindlessResourceParameter>& BindlessResourceParameters = ShaderBindings.BindlessResourceParameters;
			const int32 BindlessResourceCount = BindlessResourceParameters.Num();
			for (; BindlessResourceIndex < BindlessResourceCount && BindlessResourceParameters[BindlessResourceIndex].ByteOffset < ByteOffset; BindlessResourceIndex++)
			{
			}

			if (BindlessResourceIndex < BindlessResourceCount && BindlessResourceParameters[BindlessResourceIndex].ByteOffset == ByteOffset)
			{
				continue;
			}
		}
		else if (Type == UBMT_RDG_UNIFORM_BUFFER)
		{
			if (GraphUniformBufferId < ShaderBindings.GraphUniformBuffers.Num() && ByteOffset == ShaderBindings.GraphUniformBuffers[GraphUniformBufferId].ByteOffset)
			{
				GraphUniformBufferId++;
				continue;
			}

			const FRDGUniformBufferBinding& UniformBuffer = *reinterpret_cast<FRDGUniformBufferBinding*>(Base + ByteOffset);
			if (!UniformBuffer || UniformBuffer.IsStatic())
			{
				continue;
			}
		}
		else
		{
			continue;
		}

		FRDGResourceRef* ResourcePtr = reinterpret_cast<FRDGResourceRef*>(Base + ByteOffset);

		for (FRDGResourceRef ExcludeResource : ExcludeList)
		{
			if (*ResourcePtr == ExcludeResource)
			{
				continue;
			}
		}

		*ResourcePtr = nullptr;
	}
}

void ClearUnusedGraphResourcesImpl(
	TArrayView<const FShaderParameterBindings*> ShaderBindingsList,
	const FShaderParametersMetadata* ParametersMetadata,
	void* InoutParameters,
	std::initializer_list<FRDGResourceRef> ExcludeList)
{
	const auto& GraphResources = ParametersMetadata->GetLayout().GraphResources;

	TArray<int32, TInlineAllocator<SF_NumFrequencies>> ShaderResourceIds;
	TArray<int32, TInlineAllocator<SF_NumFrequencies>> BindlessResourceIds;
	TArray<int32, TInlineAllocator<SF_NumFrequencies>> GraphUniformBufferIds;
	ShaderResourceIds.SetNumZeroed(ShaderBindingsList.Num());
	BindlessResourceIds.SetNumZeroed(ShaderBindingsList.Num());
	GraphUniformBufferIds.SetNumZeroed(ShaderBindingsList.Num());

	auto Base = reinterpret_cast<uint8*>(InoutParameters);

	for (int32 GraphResourceIndex = 0, GraphResourceCount = GraphResources.Num(); GraphResourceIndex < GraphResourceCount; GraphResourceIndex++)
	{
		EUniformBufferBaseType Type = GraphResources[GraphResourceIndex].MemberType;
		uint16 ByteOffset = GraphResources[GraphResourceIndex].MemberOffset;
		bool bResourceIsUsed = false;

		if (Type == UBMT_RDG_TEXTURE ||
			Type == UBMT_RDG_TEXTURE_SRV ||
			Type == UBMT_RDG_TEXTURE_NON_PIXEL_SRV ||
			Type == UBMT_RDG_TEXTURE_UAV ||
			Type == UBMT_RDG_BUFFER_SRV ||
			Type == UBMT_RDG_BUFFER_UAV ||
			Type == UBMT_RESOURCE_COLLECTION)
		{
			for (int32 Index = 0; Index < ShaderBindingsList.Num(); ++Index)
			{
				{
					const auto& ResourceParameters = ShaderBindingsList[Index]->ResourceParameters;
					int32& ShaderResourceId = ShaderResourceIds[Index];
					for (; ShaderResourceId < ResourceParameters.Num() && ResourceParameters[ShaderResourceId].ByteOffset < ByteOffset; ++ShaderResourceId)
					{
					}
					bResourceIsUsed |= ShaderResourceId < ResourceParameters.Num() && ByteOffset == ResourceParameters[ShaderResourceId].ByteOffset;
				}

				if (!bResourceIsUsed)
				{
					const auto& BindlessResourceParameters = ShaderBindingsList[Index]->BindlessResourceParameters;
					int32& BindlessResourceId = BindlessResourceIds[Index];
					for (; BindlessResourceId < BindlessResourceParameters.Num() && BindlessResourceParameters[BindlessResourceId].ByteOffset < ByteOffset; ++BindlessResourceId)
					{
					}
					bResourceIsUsed |= BindlessResourceId < BindlessResourceParameters.Num() && ByteOffset == BindlessResourceParameters[BindlessResourceId].ByteOffset;
				}
			}
		}
		else if (Type == UBMT_RDG_UNIFORM_BUFFER)
		{
			for (int32 Index = 0; Index < ShaderBindingsList.Num(); ++Index)
			{
				const auto& GraphUniformBuffers = ShaderBindingsList[Index]->GraphUniformBuffers;
				int32& GraphUniformBufferId = GraphUniformBufferIds[Index];
				for (; GraphUniformBufferId < GraphUniformBuffers.Num() && GraphUniformBuffers[GraphUniformBufferId].ByteOffset < ByteOffset; ++GraphUniformBufferId)
				{
				}
				bResourceIsUsed |= GraphUniformBufferId < GraphUniformBuffers.Num() && ByteOffset == GraphUniformBuffers[GraphUniformBufferId].ByteOffset;
			}

			const FRDGUniformBufferBinding& UniformBuffer = *reinterpret_cast<const FRDGUniformBufferBinding*>(Base + ByteOffset);
			if (!UniformBuffer || UniformBuffer.IsStatic())
			{
				continue;
			}
		}
		else
		{
			// Not a resource we care about.
			continue;
		}

		if (bResourceIsUsed)
		{
			continue;
		}

		FRDGResourceRef* ResourcePtr = reinterpret_cast<FRDGResourceRef*>(Base + ByteOffset);

		for (FRDGResourceRef ExcludeResource : ExcludeList)
		{
			if (*ResourcePtr == ExcludeResource)
			{
				continue;
			}
		}

		*ResourcePtr = nullptr;
	}
}

FRDGTextureRef RegisterExternalTextureWithFallback(
	FRDGBuilder& GraphBuilder,
	const TRefCountPtr<IPooledRenderTarget>& ExternalPooledTexture,
	const TRefCountPtr<IPooledRenderTarget>& FallbackPooledTexture)
{
	ensureMsgf(FallbackPooledTexture.IsValid(), TEXT("RegisterExternalTextureWithDummyFallback() requires a valid fallback pooled texture."));
	if (ExternalPooledTexture.IsValid())
	{
		return GraphBuilder.RegisterExternalTexture(ExternalPooledTexture);
	}
	else
	{
		return GraphBuilder.RegisterExternalTexture(FallbackPooledTexture);
	}
}

RENDERCORE_API FRDGTextureMSAA CreateTextureMSAA(
	FRDGBuilder& GraphBuilder,
	FRDGTextureDesc Desc,
	const TCHAR* NameMultisampled, const TCHAR* NameResolved,
	ETextureCreateFlags ResolveFlagsToAdd)
{
	const bool bForceSeparateTargetAndShaderResource = Desc.NumSamples > 1 && RHISupportsSeparateMSAAAndResolveTextures(GMaxRHIShaderPlatform);

	if (LIKELY(bForceSeparateTargetAndShaderResource))
	{
		FRDGTextureMSAA Texture(GraphBuilder.CreateTexture(Desc, NameMultisampled));

		Desc.NumSamples = 1;
		ETextureCreateFlags ResolveFlags = TexCreate_ShaderResource;
		if (EnumHasAnyFlags(Desc.Flags, TexCreate_DepthStencilTargetable))
		{
			ResolveFlags |= TexCreate_DepthStencilResolveTarget;
		}
		else
		{
			ResolveFlags |= TexCreate_ResolveTargetable;
		}
        ResolveFlags &= ~(TexCreate_Memoryless);
        
		Desc.Flags = ResolveFlags | ResolveFlagsToAdd;
		Texture.Resolve = GraphBuilder.CreateTexture(Desc, NameResolved);

		return Texture;
	}

	Desc.Flags |= TexCreate_ShaderResource;
	return FRDGTextureMSAA(GraphBuilder.CreateTexture(Desc, NameResolved));
}

BEGIN_SHADER_PARAMETER_STRUCT(FCopyTextureParameters, )
	RDG_TEXTURE_ACCESS(Input,  ERHIAccess::CopySrc)
	RDG_TEXTURE_ACCESS(Output, ERHIAccess::CopyDest)
END_SHADER_PARAMETER_STRUCT()

class FDrawTexturePS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FDrawTexturePS);
	SHADER_USE_PARAMETER_STRUCT(FDrawTexturePS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, InputTexture)
		SHADER_PARAMETER(FIntPoint, InputOffset)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()
};

IMPLEMENT_GLOBAL_SHADER(FDrawTexturePS, "/Engine/Private/Tools/DrawTexture.usf", "DrawTexturePS", SF_Pixel);

void AddCopyTexturePass(
	FRDGBuilder& GraphBuilder,
	FRDGTextureRef InputTexture,
	FRDGTextureRef OutputTexture,
	const FRHICopyTextureInfo& CopyInfo)
{
	if (InputTexture == OutputTexture)
	{
		return;
	}

	const FRDGTextureDesc& InputDesc = InputTexture->Desc;
	const FRDGTextureDesc& OutputDesc = OutputTexture->Desc;

	FCopyTextureParameters* Parameters = GraphBuilder.AllocParameters<FCopyTextureParameters>();
	Parameters->Input = InputTexture;
	Parameters->Output = OutputTexture;

	GraphBuilder.AddPass(
		RDG_EVENT_NAME("CopyTexture(%s -> %s)", InputTexture->Name, OutputTexture->Name),
		Parameters,
		ERDGPassFlags::Copy,
		[InputTexture, OutputTexture, CopyInfo](FRDGAsyncTask, FRHICommandList& RHICmdList)
	{
		RHICmdList.CopyTexture(InputTexture->GetRHI(), OutputTexture->GetRHI(), CopyInfo);
	});
}

RENDERCORE_API void AddDrawTexturePass(
	FRDGBuilder& GraphBuilder,
	const FGlobalShaderMap* ShaderMap,
	FRDGTextureRef InputTexture,
	FRDGTextureRef OutputTexture,
	const FRDGDrawTextureInfo& DrawInfo)
{
	if (InputTexture == OutputTexture)
	{
		return;
	}

	const FRDGTextureDesc& InputDesc = InputTexture->Desc;
	const FRDGTextureDesc& OutputDesc = OutputTexture->Desc;

	// Use a hardware copy if formats match.
	if (InputDesc.Format == OutputDesc.Format)
	{
		FRHICopyTextureInfo CopyInfo;

		// Translate the draw texture info into a copy info.
		CopyInfo.Size = FIntVector(DrawInfo.Size.X, DrawInfo.Size.Y, 0);
		CopyInfo.SourcePosition = FIntVector(DrawInfo.SourcePosition.X, DrawInfo.SourcePosition.Y, 0);
		CopyInfo.DestPosition = FIntVector(DrawInfo.DestPosition.X, DrawInfo.DestPosition.Y, 0);
		CopyInfo.SourceSliceIndex = DrawInfo.SourceSliceIndex;
		CopyInfo.DestSliceIndex = DrawInfo.DestSliceIndex;
		CopyInfo.NumSlices = DrawInfo.NumSlices;
		CopyInfo.SourceMipIndex = DrawInfo.SourceMipIndex;
		CopyInfo.DestMipIndex = DrawInfo.DestMipIndex;
		CopyInfo.NumMips = DrawInfo.NumMips;

		AddCopyTexturePass(GraphBuilder, InputTexture, OutputTexture, CopyInfo);
	}
	else
	{
		const FIntPoint DrawSize = DrawInfo.Size == FIntPoint::ZeroValue ? OutputDesc.Extent : DrawInfo.Size;

		// Don't load color data if the whole texture is being overwritten.
		const ERenderTargetLoadAction LoadAction = (DrawInfo.DestPosition == FIntPoint::ZeroValue && DrawSize == OutputDesc.Extent)
			? ERenderTargetLoadAction::ENoAction
			: ERenderTargetLoadAction::ELoad;

		TShaderMapRef<FDrawTexturePS> PixelShader(ShaderMap);

		for (uint32 MipIndex = 0; MipIndex < DrawInfo.NumMips; ++MipIndex)
		{
			const int32 SourceMipIndex = MipIndex + DrawInfo.SourceMipIndex;
			const int32 DestMipIndex = MipIndex + DrawInfo.DestMipIndex;
			FRDGTextureSRVDesc SRVDesc = FRDGTextureSRVDesc::CreateForMipLevel(InputTexture, SourceMipIndex);

			// CreateForMipLevel leaves NumArraySlices at ArraySize for array sources; narrow to the single slice we
			// bind so FirstArraySlice + NumArraySlices stays in range, and view as Texture2D to match FDrawTexturePS.
			if (InputTexture->Desc.IsTextureArray())
			{
				SRVDesc.NumArraySlices = 1;
				SRVDesc.DimensionOverride = ETextureDimension::Texture2D;
			}

			for (uint32 SliceIndex = 0; SliceIndex < DrawInfo.NumSlices; ++SliceIndex)
			{
				const int32 SourceSliceIndex = SliceIndex + DrawInfo.SourceSliceIndex;
				const int32 DestSliceIndex = SliceIndex + DrawInfo.DestSliceIndex;

				SRVDesc.FirstArraySlice = SourceSliceIndex;

				auto* PassParameters = GraphBuilder.AllocParameters<FDrawTexturePS::FParameters>();
				PassParameters->InputTexture = GraphBuilder.CreateSRV(SRVDesc);
				PassParameters->InputOffset = DrawInfo.SourcePosition;
				PassParameters->RenderTargets[0] = FRenderTargetBinding(OutputTexture, LoadAction, DestMipIndex, DestSliceIndex);

				const FIntRect ViewRect(DrawInfo.DestPosition, DrawInfo.DestPosition + DrawSize);

				FPixelShaderUtils::AddFullscreenPass(
					GraphBuilder,
					ShaderMap,
					RDG_EVENT_NAME("DrawTexture ([%s, Mip: %d, Slice: %d] -> [%s, Mip: %d, Slice: %d])", InputTexture->Name, SourceMipIndex, SourceSliceIndex, OutputTexture->Name, DestMipIndex, DestSliceIndex),
					PixelShader,
					PassParameters,
					ViewRect
				);
			}
		}
	}
}

BEGIN_SHADER_PARAMETER_STRUCT(FCopyBufferParameters, )
	RDG_BUFFER_ACCESS(SrcBuffer, ERHIAccess::CopySrc)
	RDG_BUFFER_ACCESS(DstBuffer, ERHIAccess::CopyDest)
END_SHADER_PARAMETER_STRUCT()

void AddCopyBufferPass(FRDGBuilder& GraphBuilder, FRDGBufferRef DstBuffer, uint64 DstOffset, FRDGBufferRef SrcBuffer, uint64 SrcOffset, uint64 NumBytes)
{
	check(SrcBuffer);
	check(DstBuffer);

	FCopyBufferParameters* Parameters = GraphBuilder.AllocParameters<FCopyBufferParameters>();
	Parameters->SrcBuffer = SrcBuffer;
	Parameters->DstBuffer = DstBuffer;

	GraphBuilder.AddPass(
		RDG_EVENT_NAME("CopyBuffer(%s Size=%ubytes)", SrcBuffer->Name, SrcBuffer->Desc.GetSize()),
		Parameters,
		ERDGPassFlags::Copy,
		[&Parameters, SrcBuffer, DstBuffer, SrcOffset, DstOffset, NumBytes](FRDGAsyncTask, FRHICommandList& RHICmdList)
		{
			RHICmdList.CopyBufferRegion(DstBuffer->GetRHI(), DstOffset, SrcBuffer->GetRHI(), SrcOffset, NumBytes);
		});
}

void AddCopyBufferPass(FRDGBuilder& GraphBuilder, FRDGBufferRef DstBuffer, FRDGBufferRef SrcBuffer)
{
	check(SrcBuffer);
	check(DstBuffer);

	const uint64 NumBytes = SrcBuffer->Desc.NumElements * SrcBuffer->Desc.BytesPerElement;

	AddCopyBufferPass(GraphBuilder, DstBuffer, 0, SrcBuffer, 0, NumBytes);
}

BEGIN_SHADER_PARAMETER_STRUCT(FClearBufferUAVParameters, )
	SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, BufferUAV)
END_SHADER_PARAMETER_STRUCT()

void AddClearUAVPass(FRDGBuilder& GraphBuilder, FRDGBufferUAVRef BufferUAV, uint32 Value, ERDGPassFlags ComputePassFlags)
{
	check(BufferUAV);

	FClearBufferUAVParameters* Parameters = GraphBuilder.AllocParameters<FClearBufferUAVParameters>();
	Parameters->BufferUAV = BufferUAV;

	GraphBuilder.AddPass(
		RDG_EVENT_NAME("ClearBuffer(%s Size=%ubytes)", BufferUAV->GetParent()->Name, BufferUAV->GetParent()->Desc.GetSize()),
		Parameters,
		ComputePassFlags,
		[&Parameters, BufferUAV, Value](FRDGAsyncTask, FRHIComputeCommandList& RHICmdList)
		{
			RHICmdList.ClearUAVUint(BufferUAV->GetRHI(), FUintVector4(Value, Value, Value, Value));
			BufferUAV->MarkResourceAsUsed();
		});
}

void AddClearUAVFloatPass(FRDGBuilder& GraphBuilder, FRDGBufferUAVRef BufferUAV, float Value, ERDGPassFlags ComputePassFlags)
{
	FClearBufferUAVParameters* Parameters = GraphBuilder.AllocParameters<FClearBufferUAVParameters>();
	Parameters->BufferUAV = BufferUAV;

	GraphBuilder.AddPass(
		RDG_EVENT_NAME("ClearBuffer(%s Size=%ubytes)", BufferUAV->GetParent()->Name, BufferUAV->GetParent()->Desc.GetSize()),
		Parameters,
		ComputePassFlags,
		[&Parameters, BufferUAV, Value](FRDGAsyncTask, FRHIComputeCommandList& RHICmdList)
		{
			RHICmdList.ClearUAVFloat(BufferUAV->GetRHI(), FVector4f(Value, Value, Value, Value));
			BufferUAV->MarkResourceAsUsed();
		});
}

BEGIN_SHADER_PARAMETER_STRUCT(FClearTextureUAVParameters, )
	SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, TextureUAV)
END_SHADER_PARAMETER_STRUCT()

void AddClearUAVPass(FRDGBuilder& GraphBuilder, FRDGTextureUAVRef TextureUAV, const FUintVector4& ClearValues, ERDGPassFlags ComputePassFlags)
{
	check(TextureUAV);

	FClearTextureUAVParameters* Parameters = GraphBuilder.AllocParameters<FClearTextureUAVParameters>();
	Parameters->TextureUAV = TextureUAV;

	FRDGTextureRef Texture = TextureUAV->GetParent();

	GraphBuilder.AddPass(
		RDG_EVENT_NAME("ClearTextureUint(%s %s %dx%d Mip=%d)",
			Texture->Name,
			GPixelFormats[Texture->Desc.Format].Name,
			Texture->Desc.Extent.X, Texture->Desc.Extent.Y,
			int32(TextureUAV->Desc.MipLevel)),
		Parameters,
		ComputePassFlags,
		[&Parameters, TextureUAV, ClearValues](FRDGAsyncTask, FRHIComputeCommandList& RHICmdList)
		{
			const FRDGTextureDesc& LocalTextureDesc = TextureUAV->GetParent()->Desc;

			FRHIUnorderedAccessView* RHITextureUAV = TextureUAV->GetRHI();

			RHICmdList.ClearUAVUint(RHITextureUAV, ClearValues);
			TextureUAV->MarkResourceAsUsed();
		});
}

void AddClearUAVPass(FRDGBuilder& GraphBuilder, FRDGTextureUAVRef TextureUAV, const FVector4f& ClearValues, ERDGPassFlags ComputePassFlags)
{
	check(TextureUAV);

	FClearTextureUAVParameters* Parameters = GraphBuilder.AllocParameters<FClearTextureUAVParameters>();
	Parameters->TextureUAV = TextureUAV;

	const FRDGTextureDesc& TextureDesc = TextureUAV->GetParent()->Desc;

	GraphBuilder.AddPass(
		RDG_EVENT_NAME("ClearTextureFloat(%s) %dx%d", TextureUAV->GetParent()->Name, TextureDesc.Extent.X, TextureDesc.Extent.Y),
		Parameters,
		ComputePassFlags,
		[&Parameters, TextureUAV, ClearValues](FRDGAsyncTask, FRHIComputeCommandList& RHICmdList)
		{
			const FRDGTextureDesc& LocalTextureDesc = TextureUAV->GetParent()->Desc;

			FRHIUnorderedAccessView* RHITextureUAV = TextureUAV->GetRHI();

			RHICmdList.ClearUAVFloat(RHITextureUAV, ClearValues);
			TextureUAV->MarkResourceAsUsed();
		});
}

void AddClearUAVPass(FRDGBuilder& GraphBuilder, FRDGTextureUAVRef TextureUAV, const uint32(&ClearValues)[4], ERDGPassFlags ComputePassFlags)
{
	AddClearUAVPass(GraphBuilder, TextureUAV, FUintVector4(ClearValues[0], ClearValues[1], ClearValues[2], ClearValues[3]), ComputePassFlags);
}

void AddClearUAVPass(FRDGBuilder& GraphBuilder, FRDGTextureUAVRef TextureUAV, const float(&ClearValues)[4], ERDGPassFlags ComputePassFlags)
{
	AddClearUAVPass(GraphBuilder, TextureUAV, FVector4f(ClearValues[0], ClearValues[1], ClearValues[2], ClearValues[3]), ComputePassFlags);
}

void AddClearUAVPass(FRDGBuilder& GraphBuilder, FRDGTextureUAVRef TextureUAV, const FLinearColor& ClearColor, ERDGPassFlags ComputePassFlags)
{
	AddClearUAVPass(GraphBuilder, TextureUAV, FVector4f(ClearColor.R, ClearColor.G, ClearColor.B, ClearColor.A), ComputePassFlags);
}

void AddClearUAVPass(FRDGBuilder& GraphBuilder, FRDGTextureUAVRef TextureUAV, uint32 Value, ERDGPassFlags ComputePassFlags)
{
	AddClearUAVPass(GraphBuilder, TextureUAV, { Value, Value , Value , Value }, ComputePassFlags);
}

void AddClearUAVPass(FRDGBuilder& GraphBuilder, FRDGTextureUAVRef TextureUAV, float Value, ERDGPassFlags ComputePassFlags)
{
	AddClearUAVPass(GraphBuilder, TextureUAV, { Value, Value , Value , Value }, ComputePassFlags);
}

void AddClearUAVPass(FRDGBuilder& GraphBuilder, FRDGTextureUAVRef TextureUAV, const FVector& Value, ERDGPassFlags ComputePassFlags)
{
	AddClearUAVPass(GraphBuilder, TextureUAV, { (float)Value.X, (float)Value.Y , (float)Value.Z , 0.f }, ComputePassFlags);	// LWC_TODO: Precision loss?
}

void AddClearUAVPass(FRDGBuilder& GraphBuilder, FRDGTextureUAVRef TextureUAV, const FIntPoint& Value, ERDGPassFlags ComputePassFlags)
{
	AddClearUAVPass(GraphBuilder, TextureUAV, { uint32(Value.X), uint32(Value.Y), 0u, 0u }, ComputePassFlags);
}

void AddClearUAVPass(FRDGBuilder& GraphBuilder, FRDGTextureUAVRef TextureUAV, const FVector2D& Value, ERDGPassFlags ComputePassFlags)
{
	AddClearUAVPass(GraphBuilder, TextureUAV, { (float)Value.X,(float)Value.Y , 0.f, 0.f }, ComputePassFlags);	// LWC_TODO: Precision loss?
}

void AddClearUAVPass(FRDGBuilder& GraphBuilder, FRDGTextureUAVRef TextureUAV, const FVector4d& Value, ERDGPassFlags ComputePassFlags)
{
	AddClearUAVPass(GraphBuilder, TextureUAV, FVector4f(Value), ComputePassFlags);								// LWC_TODO: Precision loss?
}

class FClearUAVRectsPS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FClearUAVRectsPS);
	SHADER_USE_PARAMETER_STRUCT(FClearUAVRectsPS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(FUintVector4, ClearValue)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, ClearResource)
	END_SHADER_PARAMETER_STRUCT()

	using FPermutationDomain = TShaderPermutationDomain<>;

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		int32 ResourceType = RHIGetPreferredClearUAVRectPSResourceType(Parameters.Platform);

		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("ENABLE_CLEAR_VALUE"), 1);
		OutEnvironment.SetDefine(TEXT("RESOURCE_TYPE"), ResourceType);
		OutEnvironment.SetDefine(TEXT("VALUE_TYPE"), TEXT("uint4"));
	}
};

IMPLEMENT_GLOBAL_SHADER(FClearUAVRectsPS, "/Engine/Private/ClearReplacementShaders.usf", "ClearTextureRWPS", SF_Pixel);

BEGIN_SHADER_PARAMETER_STRUCT(FClearUAVRectsParameters, )
	SHADER_PARAMETER_STRUCT_INCLUDE(FPixelShaderUtils::FRasterizeToRectsVS::FParameters, VS)
	SHADER_PARAMETER_STRUCT_INCLUDE(FClearUAVRectsPS::FParameters, PS)
	RENDER_TARGET_BINDING_SLOTS()
END_SHADER_PARAMETER_STRUCT()

void AddClearUAVPass(FRDGBuilder& GraphBuilder, ERHIFeatureLevel::Type FeatureLevel, FRDGTextureUAVRef TextureUAV, const uint32(&ClearValues)[4], FRDGBufferSRVRef RectCoordBufferSRV, uint32 NumRects)
{
	if (NumRects == 0)
	{
		AddClearUAVPass(GraphBuilder, TextureUAV, ClearValues);
		return;
	}

	check(TextureUAV && RectCoordBufferSRV);

	const FRDGTextureRef Texture = TextureUAV->GetParent();
	const FIntPoint TextureSize = Texture->Desc.Extent;

	// Create a R32G32 view of the R64 instead of adding a permutation to the clear shader
	if (Texture->Desc.Format == PF_R64_UINT)
	{
		TextureUAV = GraphBuilder.CreateUAV(Texture, ERDGUnorderedAccessViewFlags::None, PF_R32G32_UINT);
	}

	FClearUAVRectsParameters* PassParameters = GraphBuilder.AllocParameters<FClearUAVRectsParameters>();

	PassParameters->PS.ClearValue.X = ClearValues[0];
	PassParameters->PS.ClearValue.Y = ClearValues[1];
	PassParameters->PS.ClearValue.Z = ClearValues[2];
	PassParameters->PS.ClearValue.W = ClearValues[3];
	PassParameters->PS.ClearResource = TextureUAV;

	auto* ShaderMap = GetGlobalShaderMap(FeatureLevel);
	auto PixelShader = ShaderMap->GetShader<FClearUAVRectsPS>();

	const ERDGPassFlags AdditionalRenderPassFlags = (PassParameters->RenderTargets.GetActiveCount() == 0) ? ERDGPassFlags::SkipRenderPass : ERDGPassFlags::None;

	FPixelShaderUtils::AddRasterizeToRectsPass<FClearUAVRectsPS>(GraphBuilder,
		ShaderMap,
		RDG_EVENT_NAME("ClearTextureRects(%s %s %dx%d Mip=%d)",
			Texture->Name,
			GPixelFormats[Texture->Desc.Format].Name,
			Texture->Desc.Extent.X, Texture->Desc.Extent.Y,
			int32(TextureUAV->Desc.MipLevel)),
		PixelShader,
		PassParameters,
		TextureSize,
		RectCoordBufferSRV,
		NumRects,
		/*BlendState*/ nullptr,
		/*RasterizerState*/ nullptr,
		/*DepthStencilState*/ nullptr,
		/*StencilRef*/ 0,
		/*TextureSize*/ TextureSize,
		/*RectUVBufferSRV*/ nullptr,
		/*DownsampleFactor*/ 1,
		AdditionalRenderPassFlags);
}

void AddClearRenderTargetPass(FRDGBuilder& GraphBuilder, FRDGTextureRef Texture)
{
	// Single mip, single slice, same clear color as what is specified in the render target :
	AddClearRenderTargetPass(GraphBuilder, Texture, FRDGTextureClearInfo());
}

void AddClearRenderTargetPass(FRDGBuilder& GraphBuilder, FRDGTextureRef Texture, const FLinearColor& ClearColor)
{
	// Single mip, single slice, custom clear color :
	FRDGTextureClearInfo TextureClearInfo;
	TextureClearInfo.ClearColor = ClearColor;
	AddClearRenderTargetPass(GraphBuilder, Texture, TextureClearInfo);
}

void AddClearRenderTargetPass(FRDGBuilder& GraphBuilder, FRDGTextureRef Texture, const FLinearColor& ClearColor, FIntRect Viewport)
{
	// Single mip, single slice, custom viewport, custom clear color :
	FRDGTextureClearInfo TextureClearInfo;
	TextureClearInfo.ClearColor = ClearColor;
	TextureClearInfo.Viewport = Viewport;
	AddClearRenderTargetPass(GraphBuilder, Texture, TextureClearInfo);
}

void AddClearRenderTargetPass(FRDGBuilder& GraphBuilder, FRDGTextureRef Texture, const FRDGTextureClearInfo& TextureClearInfo)
{
	check(Texture);

	bool bUseCustomViewport = (TextureClearInfo.Viewport.Area() > 0);
	FLinearColor ClearColor = TextureClearInfo.ClearColor.IsSet() ? TextureClearInfo.ClearColor.GetValue() : Texture->Desc.ClearValue.GetClearColor();
	uint16 TextureNumSlicesOrDepth = Texture->Desc.IsTexture3D() ? Texture->Desc.Depth : Texture->Desc.ArraySize;

	checkf((TextureClearInfo.FirstMipIndex < Texture->Desc.NumMips) && ((TextureClearInfo.FirstMipIndex + TextureClearInfo.NumMips) <= Texture->Desc.NumMips),
		TEXT("Invalid mip range [%d, %d] for texture %s (%d mips)"), TextureClearInfo.FirstMipIndex, TextureClearInfo.FirstMipIndex + TextureClearInfo.NumMips - 1, Texture->Name, Texture->Desc.NumMips);
	checkf(((TextureClearInfo.FirstSliceIndex == 0) && (TextureClearInfo.NumSlices == 1)) || (Texture->Desc.IsTextureArray() && (EnumHasAnyFlags(Texture->Desc.Flags, ETextureCreateFlags::TargetArraySlicesIndependently) || GRHISupportsBindingTexArrayPerSlice)),
		TEXT("Per-slice clear (outside of slice 0, i.e. clearing any other slice than the first one) is only supported on 2DArray at the moment and ETextureCreateFlags::TargetArraySlicesIndependently must be passed when creating the texture (texture %s)."), Texture->Name);
	checkf((TextureClearInfo.FirstSliceIndex < TextureNumSlicesOrDepth) && ((TextureClearInfo.FirstSliceIndex + TextureClearInfo.NumSlices) <= TextureNumSlicesOrDepth),
		TEXT("Invalid slice range [%d, %d] for texture %s (%d slices)"), TextureClearInfo.FirstSliceIndex, TextureClearInfo.FirstSliceIndex + TextureClearInfo.NumSlices - 1, Texture->Name, TextureNumSlicesOrDepth);

	// Use clear action if no viewport specified and clear color is not passed or matches the fast clear color :
	if (!bUseCustomViewport
		&& (Texture->Desc.ClearValue.ColorBinding == EClearBinding::EColorBound)
		&& (Texture->Desc.ClearValue.GetClearColor() == ClearColor))
	{
		for (uint32 SliceIndex = 0; SliceIndex < TextureClearInfo.NumSlices; ++SliceIndex)
		{
			uint16 CurrentSliceIndex = IntCastChecked<uint16>(TextureClearInfo.FirstSliceIndex + SliceIndex);
			for (uint32 MipIndex = 0; MipIndex < TextureClearInfo.NumMips; ++MipIndex)
			{
				uint8 CurrentMipIndex = IntCastChecked<uint8>(TextureClearInfo.FirstMipIndex + MipIndex);

				FRenderTargetParameters* Parameters = GraphBuilder.AllocParameters<FRenderTargetParameters>();
				Parameters->RenderTargets[0] = FRenderTargetBinding(Texture, ERenderTargetLoadAction::EClear, CurrentMipIndex, CurrentSliceIndex);

				GraphBuilder.AddPass(
					RDG_EVENT_NAME("ClearRenderTarget(%s, slice %d, mip %d) %dx%d ClearAction", Texture->Name, CurrentSliceIndex, CurrentMipIndex, Texture->Desc.Extent.X, Texture->Desc.Extent.Y),
					Parameters,
					ERDGPassFlags::Raster,
					[](FRDGAsyncTask, FRHICommandList& RHICmdList) {});
			}
		}
	}
	else
	{
		FIntRect OriginalViewport = bUseCustomViewport ? TextureClearInfo.Viewport : FIntRect(FIntPoint::ZeroValue, Texture->Desc.Extent);
		checkf((OriginalViewport.Max.X <= Texture->Desc.Extent.X) && (OriginalViewport.Max.Y <= Texture->Desc.Extent.Y), TEXT("Invalid custom viewport ((%d, %d) - (%d, %d)) for texture %s (size (%d, %d))"),
			OriginalViewport.Min.X, OriginalViewport.Min.Y, OriginalViewport.Max.X, OriginalViewport.Max.Y, Texture->Name, Texture->Desc.Extent.X, Texture->Desc.Extent.Y);

		for (uint32 SliceIndex = 0; SliceIndex < TextureClearInfo.NumSlices; ++SliceIndex)
		{
			uint16 CurrentSliceIndex = IntCastChecked<uint16>(TextureClearInfo.FirstSliceIndex + SliceIndex);
			for (uint32 MipIndex = 0; MipIndex < TextureClearInfo.NumMips; ++MipIndex)
			{
				FIntRect CurrentViewport(
					(uint32)OriginalViewport.Min.X >> MipIndex, 
					(uint32)OriginalViewport.Min.Y >> MipIndex,
					FMath::Max(1u, (uint32)OriginalViewport.Max.X >> MipIndex),
					FMath::Max(1u, (uint32)OriginalViewport.Max.Y >> MipIndex));

				uint8 CurrentMipIndex = IntCastChecked<uint8>(TextureClearInfo.FirstMipIndex + MipIndex);
				if (CurrentViewport.Area() > 0)
				{
					FRenderTargetParameters* Parameters = GraphBuilder.AllocParameters<FRenderTargetParameters>();
					Parameters->RenderTargets[0] = FRenderTargetBinding(Texture, ERenderTargetLoadAction::ENoAction, CurrentMipIndex, CurrentSliceIndex);

					GraphBuilder.AddPass(
						RDG_EVENT_NAME("ClearRenderTarget(%s, slice %d, mip %d) [(%d, %d), (%d, %d)] ClearQuad", Texture->Name, CurrentSliceIndex, CurrentMipIndex, CurrentViewport.Min.X, CurrentViewport.Min.Y, CurrentViewport.Max.X, CurrentViewport.Max.Y),
						Parameters,
						ERDGPassFlags::Raster,
						[Parameters, ClearColor, CurrentViewport](FRDGAsyncTask, FRHICommandList& RHICmdList)
					{
						RHICmdList.SetViewport((float)CurrentViewport.Min.X, (float)CurrentViewport.Min.Y, 0.0f, (float)CurrentViewport.Max.X, (float)CurrentViewport.Max.Y, 1.0f);
						DrawClearQuad(RHICmdList, ClearColor);
					});

					CurrentViewport = CurrentViewport / 2;
				}
			}
		}
	}
}

void AddClearDepthStencilPass(
	FRDGBuilder& GraphBuilder,
	FRDGTextureRef Texture,
	bool bClearDepth,
	float Depth,
	bool bClearStencil,
	uint8 Stencil)
{
	check(Texture);

	FExclusiveDepthStencil ExclusiveDepthStencil;
	ERenderTargetLoadAction DepthLoadAction = ERenderTargetLoadAction::ELoad;
	ERenderTargetLoadAction StencilLoadAction = ERenderTargetLoadAction::ENoAction;

	const bool bHasStencil = Texture->Desc.Format == PF_DepthStencil;

	// We can't clear stencil if we don't have it.
	bClearStencil &= bHasStencil;

	if (bClearDepth)
	{
		ExclusiveDepthStencil.SetDepthWrite();
		DepthLoadAction = ERenderTargetLoadAction::ENoAction;
	}

	if (bHasStencil)
	{
		if (bClearStencil)
		{
			ExclusiveDepthStencil.SetStencilWrite();
			StencilLoadAction = ERenderTargetLoadAction::ENoAction;
		}
		else
		{
			// Preserve stencil contents.
			StencilLoadAction = ERenderTargetLoadAction::ELoad;
		}
	}

	FRenderTargetParameters* Parameters = GraphBuilder.AllocParameters<FRenderTargetParameters>();
	Parameters->RenderTargets.DepthStencil = FDepthStencilBinding(Texture, DepthLoadAction, StencilLoadAction, ExclusiveDepthStencil);

	GraphBuilder.AddPass(
		RDG_EVENT_NAME("ClearDepthStencil(%s) %dx%d", Texture->Name, Texture->Desc.Extent.X, Texture->Desc.Extent.Y),
		Parameters,
		ERDGPassFlags::Raster,
		[Parameters, bClearDepth, Depth, bClearStencil, Stencil](FRDGAsyncTask, FRHICommandList& RHICmdList)
	{
		DrawClearQuad(RHICmdList, false, FLinearColor(), bClearDepth, Depth, bClearStencil, Stencil);
	});
}

void AddClearDepthStencilPass(FRDGBuilder& GraphBuilder, FRDGTextureRef Texture, ERenderTargetLoadAction DepthLoadAction, ERenderTargetLoadAction StencilLoadAction)
{
	auto* PassParameters = GraphBuilder.AllocParameters<FRenderTargetParameters>();
	PassParameters->RenderTargets.DepthStencil = FDepthStencilBinding(Texture, DepthLoadAction, StencilLoadAction, FExclusiveDepthStencil::DepthWrite_StencilWrite);
	GraphBuilder.AddPass(RDG_EVENT_NAME("ClearDepthStencil (%s)", Texture->Name), PassParameters, ERDGPassFlags::Raster, [](FRDGAsyncTask, FRHICommandList&) {});
}

void AddClearStencilPass(FRDGBuilder& GraphBuilder, FRDGTextureRef Texture)
{
	auto* PassParameters = GraphBuilder.AllocParameters<FRenderTargetParameters>();
	PassParameters->RenderTargets.DepthStencil = FDepthStencilBinding(Texture, ERenderTargetLoadAction::ELoad, ERenderTargetLoadAction::EClear, FExclusiveDepthStencil::DepthRead_StencilWrite);
	GraphBuilder.AddPass(RDG_EVENT_NAME("ClearStencil (%s)", Texture->Name), PassParameters, ERDGPassFlags::Raster, [](FRDGAsyncTask, FRHICommandList&) {});
}

void AddResummarizeHTilePass(FRDGBuilder& GraphBuilder, FRDGTextureRef Texture)
{
	auto* PassParameters = GraphBuilder.AllocParameters<FRenderTargetParameters>();
	const bool bHasStencil = Texture->Desc.Format == PF_DepthStencil;
	PassParameters->RenderTargets.DepthStencil = bHasStencil ? 
		FDepthStencilBinding(Texture, ERenderTargetLoadAction::ELoad, ERenderTargetLoadAction::ELoad, FExclusiveDepthStencil::DepthWrite_StencilWrite) :
		FDepthStencilBinding(Texture, ERenderTargetLoadAction::ELoad, ERenderTargetLoadAction::ENoAction, FExclusiveDepthStencil::DepthWrite_StencilNop);
	GraphBuilder.AddPass(RDG_EVENT_NAME("ResummarizeHTile (%s)", Texture->Name), PassParameters, ERDGPassFlags::Raster,
		[Texture](FRDGAsyncTask, FRHICommandList& RHICmdList)
	{
		RHICmdList.ResummarizeHTile(static_cast<FRHITexture*>(Texture->GetRHI()));
	});
}

BEGIN_SHADER_PARAMETER_STRUCT(FEnqueueCopyTexturePass, )
	RDG_TEXTURE_ACCESS(Texture, ERHIAccess::CopySrc)
END_SHADER_PARAMETER_STRUCT()

void AddEnqueueCopyPass(FRDGBuilder& GraphBuilder, FRHIGPUTextureReadback* Readback, FRDGTextureRef SourceTexture, FResolveRect Rect, uint32 SourceSlice)
{
	FEnqueueCopyTexturePass* PassParameters = GraphBuilder.AllocParameters<FEnqueueCopyTexturePass>();
	PassParameters->Texture = SourceTexture;

	GraphBuilder.AddPass(
		RDG_EVENT_NAME("EnqueueCopy(%s)", SourceTexture->Name),
		PassParameters,
		ERDGPassFlags::Readback,
		[Readback, SourceTexture, Rect, SourceSlice](FRDGAsyncTask, FRHICommandList& RHICmdList)
	{
		Readback->EnqueueCopy(RHICmdList, SourceTexture->GetRHI(), Rect, SourceSlice);
	});
}

BEGIN_SHADER_PARAMETER_STRUCT(FEnqueueCopyBufferPass, )
	RDG_BUFFER_ACCESS(Buffer, ERHIAccess::CopySrc)
END_SHADER_PARAMETER_STRUCT()

void AddEnqueueCopyPass(FRDGBuilder& GraphBuilder, FRHIGPUBufferReadback* Readback, FRDGBufferRef SourceBuffer, uint32 NumBytes)
{
	FEnqueueCopyBufferPass* PassParameters = GraphBuilder.AllocParameters<FEnqueueCopyBufferPass>();
	PassParameters->Buffer = SourceBuffer;

	GraphBuilder.AddPass(
		RDG_EVENT_NAME("EnqueueCopy(%s)", SourceBuffer->Name),
		PassParameters,
		ERDGPassFlags::Readback,
		[Readback, SourceBuffer, NumBytes](FRDGAsyncTask, FRHICommandList& RHICmdList)
	{
		Readback->EnqueueCopy(RHICmdList, SourceBuffer->GetRHI(), NumBytes);
	});
}

class FInitIndirectArgs1DCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FInitIndirectArgs1DCS);
	SHADER_USE_PARAMETER_STRUCT(FInitIndirectArgs1DCS, FGlobalShader)

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer< uint >, InputCountBuffer)
		SHADER_PARAMETER(uint32, Multiplier)
		SHADER_PARAMETER(uint32, Divisor)
		SHADER_PARAMETER(uint32, InputCountOffset)

		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer< uint >, IndirectDispatchArgsOut)
	END_SHADER_PARAMETER_STRUCT()
};

IMPLEMENT_GLOBAL_SHADER(FInitIndirectArgs1DCS, "/Engine/Private/Tools/SetupIndirectArgs.usf", "InitIndirectArgs1DCS", SF_Compute);

FRDGBufferRef FComputeShaderUtils::AddIndirectArgsSetupCsPass1D(FRDGBuilder& GraphBuilder, ERHIFeatureLevel::Type FeatureLevel, FRDGBuffer* InputCountBuffer, const TCHAR* OutputBufferName, uint32 Divisor, uint32 InputCountOffset, uint32 Multiplier)
{
	// 1. Add setup pass
	FRDGBufferRef IndirectArgsBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateIndirectDesc<FRHIDispatchIndirectParameters>(), OutputBufferName);
	{
		FInitIndirectArgs1DCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FInitIndirectArgs1DCS::FParameters>();
		PassParameters->InputCountBuffer = GraphBuilder.CreateSRV(InputCountBuffer);
		PassParameters->Multiplier = Multiplier;
		PassParameters->Divisor = Divisor;
		PassParameters->InputCountOffset = InputCountOffset;
		PassParameters->IndirectDispatchArgsOut = GraphBuilder.CreateUAV(IndirectArgsBuffer, PF_R32_UINT);

		auto ComputeShader = GetGlobalShaderMap(FeatureLevel)->GetShader<FInitIndirectArgs1DCS>();
		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("InitIndirectArgs1D"),
			ComputeShader,
			PassParameters,
			FIntVector(1, 1, 1)
		);
	}

	return IndirectArgsBuffer;
}

FRDGBufferRef CreateStructuredBuffer(
	FRDGBuilder& GraphBuilder,
	const TCHAR* Name,
	uint32 BytesPerElement,
	uint32 NumElements,
	const void* InitialData,
	uint64 InitialDataSize,
	ERDGInitialDataFlags InitialDataFlags)
{
	FRDGBufferRef Buffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateStructuredDesc(BytesPerElement, NumElements), Name);
	GraphBuilder.QueueBufferUpload(Buffer, InitialData, InitialDataSize, InitialDataFlags);
	return Buffer;
}

FRDGBufferRef CreateStructuredBuffer(
	FRDGBuilder& GraphBuilder,
	const TCHAR* Name,
	uint32 BytesPerElement,
	FRDGBufferNumElementsCallback&& NumElementsCallback,
	FRDGBufferInitialDataCallback&& InitialDataCallback,
	FRDGBufferInitialDataSizeCallback&& InitialDataSizeCallback)
{
	FRDGBufferRef Buffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateStructuredDesc(BytesPerElement, 1), Name, MoveTemp(NumElementsCallback));
	GraphBuilder.QueueBufferUpload(Buffer, MoveTemp(InitialDataCallback), MoveTemp(InitialDataSizeCallback));
	return Buffer;
}

FRDGBufferRef CreateByteAddressBuffer(
	FRDGBuilder& GraphBuilder,
	const TCHAR* Name,
	uint32 NumBytes,
	const void* InitialData,
	uint64 InitialDataSize,
	ERDGInitialDataFlags InitialDataFlags)
{
	FRDGBufferRef Buffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateByteAddressDesc(NumBytes), Name);
	GraphBuilder.QueueBufferUpload(Buffer, InitialData, InitialDataSize, InitialDataFlags);
	return Buffer;
}

FRDGBufferRef CreateByteAddressBuffer(
	FRDGBuilder& GraphBuilder,
	const TCHAR* Name,
	FRDGBufferNumElementsCallback&& NumElementsCallback,
	FRDGBufferInitialDataCallback&& InitialDataCallback,
	FRDGBufferInitialDataSizeCallback&& InitialDataSizeCallback)
{
	FRDGBufferRef Buffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateByteAddressDesc(4), Name, MoveTemp(NumElementsCallback));
	GraphBuilder.QueueBufferUpload(Buffer, MoveTemp(InitialDataCallback), MoveTemp(InitialDataSizeCallback));
	return Buffer;
}

FRDGBufferRef CreateUploadBuffer(
	FRDGBuilder& GraphBuilder,
	const TCHAR* Name,
	uint32 BytesPerElement,
	uint32 NumElements,
	const void* InitialData,
	uint64 InitialDataSize,
	ERDGInitialDataFlags InitialDataFlags)
{
	FRDGBufferRef Buffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateUploadDesc(BytesPerElement, NumElements), Name);
	if (InitialData != nullptr && InitialDataSize > 0)
	{
		GraphBuilder.QueueBufferUpload(Buffer, InitialData, InitialDataSize, InitialDataFlags);
	}
	return Buffer;
}

FRDGBufferRef CreateVertexBuffer(
	FRDGBuilder& GraphBuilder,
	const TCHAR* Name,
	const FRDGBufferDesc& Desc,
	const void* InitialData,
	uint64 InitialDataSize,
	ERDGInitialDataFlags InitialDataFlags)
{
	checkf(Name!=nullptr, TEXT("Buffer must have a name."));
	checkf(EnumHasAnyFlags(Desc.Usage, EBufferUsageFlags::VertexBuffer), TEXT("CreateVertexBuffer called with an FRDGBufferDesc underlying type that is not 'VertexBuffer'. Buffer: %s"), Name);

	FRDGBufferRef Buffer = GraphBuilder.CreateBuffer(Desc, Name);
	GraphBuilder.QueueBufferUpload(Buffer, InitialData, InitialDataSize, InitialDataFlags);
	return Buffer;
}

void FRDGExternalAccessQueue::Submit(FRDGBuilder& GraphBuilder)
{
	for (FResource Resource : Resources)
	{
		GraphBuilder.UseExternalAccessMode(Resource.Resource, Resource.Access, Resource.Pipelines);
	}
	Resources.Empty();
}

bool AllocatePooledBuffer(
	const FRDGBufferDesc& Desc,
	TRefCountPtr<FRDGPooledBuffer>& Out,
	const TCHAR* Name,
	ERDGPooledBufferAlignment Alignment)
{
	if (Out && Out->Desc == Desc)
	{
		// Kept current allocation.
		return false;
	}

	// New allocation.
	Out = GRenderGraphResourcePool.FindFreeBuffer(Desc, Name, Alignment);
	return true;
}

TRefCountPtr<FRDGPooledBuffer> AllocatePooledBuffer(const FRDGBufferDesc& Desc, const TCHAR* Name, ERDGPooledBufferAlignment Alignment)
{
	return GRenderGraphResourcePool.FindFreeBuffer(Desc, Name, Alignment);
}

bool AllocatePooledTexture(const FRDGTextureDesc& Desc, TRefCountPtr<IPooledRenderTarget>& Out, const TCHAR* Name)
{
	return GRenderTargetPool.FindFreeElement(Desc, Out, Name);
}

TRefCountPtr<IPooledRenderTarget> AllocatePooledTexture(const FRDGTextureDesc& Desc, const TCHAR* Name)
{
	return GRenderTargetPool.FindFreeElement(Desc, Name);
}

#if ENABLE_LOW_LEVEL_MEM_TRACKER || UE_MEMORY_TRACE_ENABLED
TRefCountPtr<FRDGPooledBuffer> AllocatePooledBufferCurrentLLMTag(FRHICommandListBase& RHICmdList, const FRDGBufferDesc& Desc, const TCHAR* Name, ERDGPooledBufferAlignment Alignment)
{
	TRefCountPtr<FRDGPooledBuffer> RDGPooledBuffer =  AllocatePooledBuffer(Desc, Name, Alignment);
	if (RDGPooledBuffer && RDGPooledBuffer->GetRHI())
	{
		RHICmdList.UpdateAllocationTags(RDGPooledBuffer->GetRHI());
	}
	return RDGPooledBuffer;
}
#endif

BEGIN_SHADER_PARAMETER_STRUCT(FBarrierPassParameters, )
	RDG_TEXTURE_ACCESS_ARRAY(Textures)
	RDG_BUFFER_ACCESS_ARRAY(Buffers)
END_SHADER_PARAMETER_STRUCT()

void AddBarrierPass(FRDGBuilder& GraphBuilder, ERDGPassFlags PassFlags, FRDGTextureAccessArray Textures, FRDGBufferAccessArray Buffers)
{
	// Skip adding the barrier pass when split barriers are supported, as they intrinsically support overlap.
	if (!GRHIGlobals.SupportsSplitBarriers && CVarRDGBarrierPass.GetValueOnRenderThread())
	{
		FBarrierPassParameters* Parameters = GraphBuilder.AllocParameters<FBarrierPassParameters>();
		Parameters->Textures = MoveTemp(Textures);
		Parameters->Buffers  = MoveTemp(Buffers);
		GraphBuilder.AddPass(RDG_EVENT_NAME("BarrierPass"), Parameters, ERDGPassFlags::NeverCull | PassFlags, [&Parameters](FRDGAsyncTask, FRHICommandList& RHICmdList) {});
	}
}