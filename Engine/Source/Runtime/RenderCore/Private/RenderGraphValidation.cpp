// Copyright Epic Games, Inc. All Rights Reserved.

#include "RenderGraphValidation.h"
#include "RenderGraphEvent.h"
#include "RenderGraphPrivate.h"
#include "MultiGPU.h"
#include "RenderGraphPass.h"

DEFINE_LOG_CATEGORY_STATIC(LogRDG, Log, All);

#if RDG_ENABLE_DEBUG

namespace
{

const ERHIAccess AccessMaskCopy    = ERHIAccess::CopySrc | ERHIAccess::CopyDest | ERHIAccess::CPURead;
const ERHIAccess AccessMaskCompute = ERHIAccess::SRVCompute | ERHIAccess::UAVCompute;
const ERHIAccess AccessMaskRaster  = ERHIAccess::ResolveSrc | ERHIAccess::ResolveDst | ERHIAccess::DSVRead | ERHIAccess::DSVWrite | ERHIAccess::RTV | ERHIAccess::SRVGraphics | ERHIAccess::UAVGraphics | ERHIAccess::Present | ERHIAccess::VertexOrIndexBuffer;
const ERHIAccess AccessMaskComputeOrRaster = ERHIAccess::IndirectArgs;

/** Validates that only one builder instance exists at any time. This is currently a requirement for state tracking and allocation lifetimes. */
bool GRDGBuilderActive = false;

} //! namespace

void FRDGResource::MarkResourceAsUsed()
{
	if (!GRDGValidation)
	{
		return;
	}

	ValidateRHIAccess();
}

void FRDGResource::ValidateRHIAccess() const
{
	if (!GRDGValidation)
	{
		return;
	}

	checkf(bAllowRHIAccess || GRDGAllowRHIAccess.load(std::memory_order_relaxed) || GRDGAllowRHIAccessAsync.load(std::memory_order_relaxed),
		TEXT("Accessing the RHI resource of %s at this time is not allowed. If you hit this check in pass, ")
		TEXT("that is due to this resource not being referenced in the parameters of your pass."),
		Name);
}

void FRDGUniformBuffer::MarkResourceAsUsed()
{
	if (!GRDGValidation)
	{
		return;
	}

	FRDGResource::MarkResourceAsUsed();

	// Individual resources can't be culled from a uniform buffer, so we have to mark them all as used.
	ParameterStruct.Enumerate([](FRDGParameter Parameter)
	{
		if (FRDGResourceRef Resource = Parameter.GetAsResource())
		{
			Resource->MarkResourceAsUsed();
		}
	});
}

FRDGUserValidation::FRDGUserValidation(FRDGAllocator& InAllocator)
	: Allocator(InAllocator)
{
	checkf(!GRDGBuilderActive, TEXT("Another FRDGBuilder already exists on the stack. Only one builder can be created at a time. This builder instance should be merged into the parent one."));
	GRDGBuilderActive = true;
}

FRDGUserValidation::~FRDGUserValidation()
{
	checkf(bHasExecuted, TEXT("Render graph execution is required to ensure consistency with immediate mode."));
}

void FRDGUserValidation::ExecuteGuard(const TCHAR* Operation, const TCHAR* ResourceName)
{
	checkf(!bHasExecuted, TEXT("Render graph operation '%s' with resource '%s' must be performed prior to graph execution."), Operation, ResourceName);
}

void FRDGUserValidation::ValidateCreateResource(FRDGResourceRef Resource)
{
	check(Resource);

	bool bAlreadyInSet = false;
	ResourceMap.Emplace(Resource, &bAlreadyInSet);
	check(!bAlreadyInSet);
}

void FRDGUserValidation::ValidateCreateViewableResource(FRDGViewableResource* Resource)
{
	ValidateCreateResource(Resource);
}

void FRDGUserValidation::ValidateCreateTexture(FRDGTextureRef Texture)
{
	if (!GRDGValidation)
	{
		return;
	}

	ValidateCreateViewableResource(Texture);
}

void FRDGUserValidation::ValidateCreateBuffer(FRDGBufferRef Buffer)
{
	if (!GRDGValidation)
	{
		return;
	}

	ValidateCreateViewableResource(Buffer);

	checkf(Buffer->Desc.GetSize() > 0 || Buffer->NumElementsCallback, TEXT("Creating buffer '%s' is zero bytes in size."), Buffer->Name);
}

void FRDGUserValidation::ValidateCreateSRV(FRDGTextureSRVRef SRV)
{
	if (!GRDGValidation)
	{
		return;
	}

	ValidateCreateResource(SRV);
}

void FRDGUserValidation::ValidateCreateSRV(FRDGBufferSRVRef SRV)
{
	if (!GRDGValidation)
	{
		return;
	}

	ValidateCreateResource(SRV);
}

void FRDGUserValidation::ValidateCreateUAV(FRDGTextureUAVRef UAV)
{
	if (!GRDGValidation)
	{
		return;
	}

	ValidateCreateResource(UAV);
}

void FRDGUserValidation::ValidateCreateUAV(FRDGBufferUAVRef UAV)
{
	if (!GRDGValidation)
	{
		return;
	}

	ValidateCreateResource(UAV);
}

void FRDGUserValidation::ValidateCreateUniformBuffer(FRDGUniformBufferRef UniformBuffer)
{
	if (!GRDGValidation)
	{
		return;
	}

	ValidateCreateResource(UniformBuffer);
}

void FRDGUserValidation::ValidateRegisterExternalTexture(
	const TRefCountPtr<IPooledRenderTarget>& ExternalPooledTexture,
	const TCHAR* Name,
	ERDGTextureFlags Flags)
{
	if (!GRDGValidation)
	{
		return;
	}

	checkf(Name!=nullptr, TEXT("Attempted to register external texture with NULL name."));
	checkf(ExternalPooledTexture.IsValid(), TEXT("Attempted to register NULL external texture."));
	ExecuteGuard(TEXT("RegisterExternalTexture"), Name);
}

void FRDGUserValidation::ValidateRegisterExternalBuffer(const TRefCountPtr<FRDGPooledBuffer>& ExternalPooledBuffer, const TCHAR* Name, ERDGBufferFlags Flags)
{
	if (!GRDGValidation)
	{
		return;
	}

	checkf(Name!=nullptr, TEXT("Attempted to register external buffer with NULL name."));
	checkf(ExternalPooledBuffer.IsValid(), TEXT("Attempted to register NULL external buffer."));
	ExecuteGuard(TEXT("RegisterExternalBuffer"), Name);
}

void FRDGUserValidation::ValidateRegisterExternalTexture(FRDGTextureRef Texture)
{
	if (!GRDGValidation)
	{
		return;
	}

	ValidateCreateTexture(Texture);
}

void FRDGUserValidation::ValidateRegisterExternalBuffer(FRDGBufferRef Buffer)
{
	if (!GRDGValidation)
	{
		return;
	}

	ValidateCreateBuffer(Buffer);
}

void FRDGUserValidation::ValidateCreateTexture(const FRDGTextureDesc& Desc, const TCHAR* Name, ERDGTextureFlags Flags)
{
	if (!GRDGValidation)
	{
		return;
	}

	checkf(Name!=nullptr, TEXT("Creating a texture requires a valid debug name."));
	ExecuteGuard(TEXT("CreateTexture"), Name);

	// Make sure the descriptor is supported by the RHI.
	check(FRDGTextureDesc::CheckValidity(Desc, Name));

	// Can't create back buffer textures
	checkf(!EnumHasAnyFlags(Desc.Flags, ETextureCreateFlags::Presentable), TEXT("Illegal to create texture %s with presentable flag."), Name);

	const bool bCanHaveUAV = EnumHasAnyFlags(Desc.Flags, TexCreate_UAV);
	const bool bIsMSAA = Desc.NumSamples > 1;

	// D3D11 doesn't allow creating a UAV on MSAA texture.
	const bool bIsUAVForMSAATexture = bIsMSAA && bCanHaveUAV;
	checkf(!bIsUAVForMSAATexture, TEXT("TexCreate_UAV is not allowed on MSAA texture %s."), Name);

	checkf(!EnumHasAnyFlags(Flags, ERDGTextureFlags::SkipTracking), TEXT("Cannot create texture %s with the SkipTracking flag. Only registered textures can use this flag."), Name);
}

void FRDGUserValidation::ValidateCreateBuffer(const FRDGBufferDesc& Desc, const TCHAR* Name, ERDGBufferFlags Flags)
{
	if (!GRDGValidation)
	{
		return;
	}

	checkf(Name!=nullptr, TEXT("Creating a buffer requires a valid debug name."));
	ExecuteGuard(TEXT("CreateBuffer"), Name);

	if (EnumHasAllFlags(Desc.Usage, EBufferUsageFlags::StructuredBuffer | EBufferUsageFlags::ByteAddressBuffer))
	{
		checkf(Desc.BytesPerElement == 4, TEXT("Creating buffer '%s' as a structured buffer that is also byte addressable, BytesPerElement must be 4! Instead it is %d"), Name, Desc.BytesPerElement);
	}

	checkf(!EnumHasAnyFlags(Flags, ERDGBufferFlags::SkipTracking), TEXT("Cannot create buffer %s with the SkipTracking flag. Only registered buffers can use this flag."), Name);
}

void FRDGUserValidation::ValidateCreateSRV(const FRDGTextureSRVDesc& Desc)
{
	if (!GRDGValidation)
	{
		return;
	}

	FRDGTextureRef Texture = Desc.Texture;
	checkf(Texture, TEXT("Texture SRV created with a null texture."));
	ExecuteGuard(TEXT("CreateSRV"), Texture->Name);
	check(FRDGTextureSRVDesc::CheckValidity(Texture->Desc, Desc, Texture->Name));
}

void FRDGUserValidation::ValidateCreateSRV(const FRDGBufferSRVDesc& Desc)
{
	if (!GRDGValidation)
	{
		return;
	}

	FRDGBufferRef Buffer = Desc.Buffer;
	checkf(Buffer, TEXT("Buffer SRV created with a null buffer."));
	ExecuteGuard(TEXT("CreateSRV"), Buffer->Name);
}

void FRDGUserValidation::ValidateCreateUAV(const FRDGTextureUAVDesc& Desc)
{
	if (!GRDGValidation)
	{
		return;
	}

	FRDGTextureRef Texture = Desc.Texture;

	checkf(Texture, TEXT("Texture UAV created with a null texture."));
	ExecuteGuard(TEXT("CreateUAV"), Texture->Name);

	checkf(Texture->Desc.Flags & TexCreate_UAV, TEXT("Attempted to create UAV from texture %s which was not created with TexCreate_UAV"), Texture->Name);
	checkf(Desc.MipLevel < Texture->Desc.NumMips, TEXT("Failed to create UAV at mip %d: the texture %s has only %d mip levels."), Desc.MipLevel, Texture->Name, Texture->Desc.NumMips);
}

void FRDGUserValidation::ValidateCreateUAV(const FRDGBufferUAVDesc& Desc)
{
	if (!GRDGValidation)
	{
		return;
	}

	FRDGBufferRef Buffer = Desc.Buffer;
	checkf(Buffer, TEXT("Buffer UAV created with a null buffer."));
	ExecuteGuard(TEXT("CreateUAV"), Buffer->Name);
}

void FRDGUserValidation::ValidateCreateUniformBuffer(const void* ParameterStruct, const FShaderParametersMetadata* Metadata)
{
	if (!GRDGValidation)
	{
		return;
	}

	check(Metadata);
	const TCHAR* Name = Metadata->GetShaderVariableName();
	checkf(ParameterStruct, TEXT("Uniform buffer '%s' created with null parameters."), Name);
	ExecuteGuard(TEXT("CreateUniformBuffer"), Name);
}

void FRDGUserValidation::ValidateUploadBuffer(FRDGBufferRef Buffer, const void* InitialData, uint64 InitialDataSize)
{
	if (!GRDGValidation)
	{
		return;
	}

	check(Buffer);
	checkf(!Buffer->bQueuedForUpload, TEXT("Buffer %s already has an upload queued. Only one upload can be done for each graph."), Buffer->Name);
	check(InitialData || InitialDataSize == 0);
}

void FRDGUserValidation::ValidateUploadBuffer(FRDGBufferRef Buffer, const void* InitialData, uint64 InitialDataSize, const FRDGBufferInitialDataFreeCallback& InitialDataFreeCallback)
{
	if (!GRDGValidation)
	{
		return;
	}

	check(Buffer);
	checkf(!Buffer->bQueuedForUpload, TEXT("Buffer %s already has an upload queued. Only one upload can be done for each graph."), Buffer->Name);
	check((InitialData || InitialDataSize == 0) && InitialDataFreeCallback);
}

void FRDGUserValidation::ValidateUploadBuffer(FRDGBufferRef Buffer, const FRDGBufferInitialDataFillCallback& InitialDataFillCallback)
{
	if (!GRDGValidation)
	{
		return;
	}

	check(Buffer);
	checkf(!Buffer->bQueuedForUpload, TEXT("Buffer %s already has an upload queued. Only one upload can be done for each graph."), Buffer->Name);
	check(InitialDataFillCallback);
}

void FRDGUserValidation::ValidateUploadBuffer(FRDGBufferRef Buffer, const FRDGBufferInitialDataCallback& InitialDataCallback, const FRDGBufferInitialDataSizeCallback& InitialDataSizeCallback)
{
	if (!GRDGValidation)
	{
		return;
	}

	check(Buffer);
	checkf(!Buffer->bQueuedForUpload, TEXT("Buffer %s already has an upload queued. Only one upload can be done for each graph."), Buffer->Name);
	check(InitialDataCallback && InitialDataSizeCallback);
}

void FRDGUserValidation::ValidateUploadBuffer(FRDGBufferRef Buffer, const FRDGBufferInitialDataCallback& InitialDataCallback, const FRDGBufferInitialDataSizeCallback& InitialDataSizeCallback, const FRDGBufferInitialDataFreeCallback& InitialDataFreeCallback)
{
	if (!GRDGValidation)
	{
		return;
	}

	check(Buffer);
	checkf(!Buffer->bQueuedForUpload, TEXT("Buffer %s already has an upload queued. Only one upload can be done for each graph."), Buffer->Name);
	check(InitialDataCallback && InitialDataSizeCallback && InitialDataFreeCallback);
}

void FRDGUserValidation::ValidateCommitBuffer(FRDGBufferRef Buffer, uint64 CommitSizeInBytes)
{
	if (!GRDGValidation)
	{
		return;
	}

	check(Buffer);
	checkf(EnumHasAnyFlags(Buffer->Desc.Usage, BUF_ReservedResource), TEXT("Buffer %s is not marked as reserved and thus cannot be queued for reserved resource commit."), Buffer->Name);
	checkf(Buffer->IsExternal(), TEXT("Only external buffers support commit operation. It is expected that reserved resource commit mechanism is only used when perserving buffer contents is required."));
	checkf(CommitSizeInBytes > 0, TEXT("Attempted to set a reserved buffer commit size of 0 for buffer %s"), Buffer->Name);
}

void FRDGUserValidation::ValidateExtractTexture(FRDGTextureRef Texture, TRefCountPtr<IPooledRenderTarget>* OutTexturePtr)
{
	if (!GRDGValidation)
	{
		return;
	}

	ValidateExtractResource(Texture);
	checkf(OutTexturePtr, TEXT("Texture %s was extracted, but the output texture pointer is null."), Texture->Name);
}

void FRDGUserValidation::ValidateExtractBuffer(FRDGBufferRef Buffer, TRefCountPtr<FRDGPooledBuffer>* OutBufferPtr)
{
	if (!GRDGValidation)
	{
		return;
	}

	ValidateExtractResource(Buffer);
	checkf(OutBufferPtr, TEXT("Texture %s was extracted, but the output texture pointer is null."), Buffer->Name);
}

void FRDGUserValidation::ValidateExtractResource(FRDGViewableResource* Resource)
{
	check(Resource);

	checkf(Resource->bProduced || Resource->bExternal || Resource->bQueuedForUpload,
		TEXT("Unable to queue the extraction of the resource %s because it has not been produced by any pass."),
		Resource->Name);
}

void FRDGUserValidation::ValidateConvertToExternalResource(FRDGViewableResource* Resource)
{
	if (!GRDGValidation)
	{
		return;
	}

	check(Resource);
	checkf(!bHasExecuteBegun || !Resource->bTransient,
		TEXT("Unable to convert resource %s to external because passes in the graph have already executed."),
		Resource->Name);
}

void FRDGUserValidation::ValidateConvertToExternalUniformBuffer(FRDGUniformBuffer* UniformBuffer)
{
	if (!GRDGValidation)
	{
		return;
	}

	check(UniformBuffer);
	checkf(!bHasExecuteBegun,
		TEXT("Unable to convert uniform buffer %s to external because passes in the graph have already executed."),
		UniformBuffer->GetLayoutName());
}

bool FRDGUserValidation::TryMarkForClobber(FRDGViewableResource* Resource) const
{
	if (!GRDGValidation)
	{
		return false;
	}

	check(Resource);
	const bool bClobber = !Resource->bClobbered && !Resource->bExternal && !Resource->bQueuedForUpload && IsDebugAllowedForResource(Resource->Name);

	if (bClobber)
	{
		Resource->bClobbered = true;
	}

	return bClobber;
}

void FRDGUserValidation::ValidateGetPooledTexture(FRDGTextureRef Texture) const
{
	if (!GRDGValidation)
	{
		return;
	}

	check(Texture);
	checkf(Texture->bExternal, TEXT("GetPooledTexture called on texture %s, but it is not external. Call PreallocateTexture or register as an external texture instead."), Texture->Name);
}

void FRDGUserValidation::ValidateGetPooledBuffer(FRDGBufferRef Buffer) const
{
	if (!GRDGValidation)
	{
		return;
	}

	check(Buffer);
	checkf(Buffer->bExternal, TEXT("GetPooledBuffer called on buffer %s, but it is not external. Call PreallocateBuffer or register as an external buffer instead."), Buffer->Name);
}

void FRDGUserValidation::ValidateSetAccessFinal(FRDGViewableResource* Resource, ERHIAccess AccessFinal)
{
	if (!GRDGValidation)
	{
		return;
	}

	check(Resource);
	check(AccessFinal != ERHIAccess::Unknown && IsValidAccess(AccessFinal));
	checkf(Resource->bExternal || Resource->bExtracted, TEXT("Cannot set final access on non-external resource '%s' unless it is first extracted or preallocated."), Resource->Name);
	checkf(Resource->AccessModeState.Mode == FRDGViewableResource::EAccessMode::Internal, TEXT("Cannot set final access on a resource in external access mode: %s."), Resource->Name);
}

void FRDGUserValidation::ValidateUseExternalAccessMode(FRDGViewableResource* Resource, ERHIAccess ReadOnlyAccess, ERHIPipeline Pipelines)
{
	if (!GRDGValidation)
	{
		return;
	}

	check(Resource);
	check(Pipelines != ERHIPipeline::None);

	const auto& AccessModeState = Resource->AccessModeState;
	checkf(!AccessModeState.bLocked, TEXT("Resource is locked in external access mode by the SkipTracking resource flag: %s."), Resource->Name);
	checkf(IsReadOnlyExclusiveAccess(ReadOnlyAccess), TEXT("A read only access is required when use external access mode. (Resource: %s, Access: %s, Pipelines: %s)"), Resource->Name, *GetRHIAccessName(ReadOnlyAccess), *GetRHIPipelineName(Pipelines));

	checkf(
		AccessModeState.Mode == FRDGViewableResource::EAccessMode::Internal ||
		(AccessModeState.Access == ReadOnlyAccess && AccessModeState.Pipelines == Pipelines),
		TEXT("UseExternalAccessMode called on a resource that is already in external access mode, but different parameters were used.\n")
		TEXT("Resource: %s\n")
		TEXT("\tExisting Access: %s, Requested Access: %s\n")
		TEXT("\tExisting Pipelines: %s, Requested Pipelines: %s\n"),
		Resource->Name, *GetRHIAccessName(AccessModeState.Access), *GetRHIAccessName(ReadOnlyAccess), *GetRHIPipelineName(AccessModeState.Pipelines), *GetRHIPipelineName(Pipelines));

	checkf(EnumHasAnyFlags(Pipelines, ERHIPipeline::Graphics) || !EnumHasAnyFlags(ReadOnlyAccess, AccessMaskRaster),
		TEXT("Raster access flags were specified for external access but the graphics pipe was not specified.\n")
		TEXT("Resource: %s\n")
		TEXT("\tRequested Access: %s\n")
		TEXT("\tRequested Pipelines: %s\n"),
		Resource->Name, *GetRHIAccessName(ReadOnlyAccess), *GetRHIPipelineName(Pipelines));
}

void FRDGUserValidation::ValidateUseInternalAccessMode(FRDGViewableResource* Resource)
{
	if (!GRDGValidation)
	{
		return;
	}

	check(Resource);

	const auto& AccessModeState = Resource->AccessModeState;
	checkf(!AccessModeState.bLocked, TEXT("Resource is locked in external access mode by the SkipTracking resource flag: %s."), Resource->Name);
}

void FRDGUserValidation::ValidateExternalAccess(FRDGViewableResource* Resource, ERHIAccess Access, const FRDGPass* Pass)
{
	if (!GRDGValidation)
	{
		return;
	}

	const auto& AccessModeState = Resource->AccessModeState;

	ensureMsgf(EnumHasAnyFlags(AccessModeState.Access, Access),
		TEXT("Resource %s is in external access mode and is valid for access with the following states: %s, but is being used in pass %s with access %s."),
		Resource->Name, *GetRHIAccessName(AccessModeState.Access), Pass->GetName(), *GetRHIAccessName(Access));
 
	ensureMsgf(EnumHasAnyFlags(AccessModeState.Pipelines, Pass->GetPipeline()),
		TEXT("Resource %s is in external access mode and is valid for access on the following pipelines: %s, but is being used on the %s pipe in pass %s."),
		Resource->Name, *GetRHIPipelineName(AccessModeState.Pipelines), *GetRHIPipelineName(Pass->GetPipeline()), Pass->GetName());
}

void FRDGUserValidation::ValidateAddSubresourceAccess(FRDGViewableResource* Resource, const FRDGSubresourceState& Subresource, ERHIAccess Access)
{
	if (!GRDGValidation)
	{
		return;
	}

	const bool bOldAccessMergeable = EnumHasAnyFlags(Subresource.Access, GRHIMergeableAccessMask) || Subresource.Access == ERHIAccess::Unknown;
	const bool bNewAccessMergeable = EnumHasAnyFlags(Access, GRHIMergeableAccessMask);

	checkf(bOldAccessMergeable || bNewAccessMergeable || Subresource.Access == Access,
		TEXT("Resource %s has incompatible access states specified for the same subresource. AccessBefore: %s, AccessAfter: %s."),
		Resource->Name, *GetRHIAccessName(Subresource.Access), *GetRHIAccessName(Access));
}

void FRDGUserValidation::ValidateAddPass(const FRDGEventName& Name, ERDGPassFlags Flags)
{
	if (!GRDGValidation)
	{
		return;
	}

	ExecuteGuard(TEXT("AddPass"), Name.GetTCHAR());

	checkf(!EnumHasAnyFlags(Flags, ERDGPassFlags::Copy | ERDGPassFlags::Compute | ERDGPassFlags::AsyncCompute | ERDGPassFlags::Raster),
		TEXT("Pass %s may not specify any of the (Copy, Compute, AsyncCompute, Raster) flags, because it has no parameters. Use None instead."), Name.GetTCHAR());
}

void FRDGUserValidation::ValidateAddPass(const void* ParameterStruct, const FShaderParametersMetadata* Metadata, const FRDGEventName& Name, ERDGPassFlags Flags)
{
	if (!GRDGValidation)
	{
		return;
	}

	checkf(ParameterStruct, TEXT("Pass '%s' created with null parameters."), Name.GetTCHAR());
	ExecuteGuard(TEXT("AddPass"), Name.GetTCHAR());

	checkf(EnumHasAnyFlags(Flags, ERDGPassFlags::Raster | ERDGPassFlags::Compute | ERDGPassFlags::AsyncCompute | ERDGPassFlags::Copy),
		TEXT("Pass %s must specify at least one of the following flags: (Copy, Compute, AsyncCompute, Raster)"), Name.GetTCHAR());

	checkf(!EnumHasAllFlags(Flags, ERDGPassFlags::Compute | ERDGPassFlags::AsyncCompute),
		TEXT("Pass %s specified both Compute and AsyncCompute. They are mutually exclusive."), Name.GetTCHAR());

	checkf(!EnumHasAllFlags(Flags, ERDGPassFlags::Raster | ERDGPassFlags::AsyncCompute),
		TEXT("Pass %s specified both Raster and AsyncCompute. They are mutually exclusive."), Name.GetTCHAR());

	checkf(!EnumHasAllFlags(Flags, ERDGPassFlags::SkipRenderPass) || EnumHasAllFlags(Flags, ERDGPassFlags::Raster),
		TEXT("Pass %s specified SkipRenderPass without Raster. Only raster passes support this flag."), Name.GetTCHAR());

	checkf(!EnumHasAllFlags(Flags, ERDGPassFlags::NeverMerge) || EnumHasAllFlags(Flags, ERDGPassFlags::Raster),
		TEXT("Pass %s specified NeverMerge without Raster. Only raster passes support this flag."), Name.GetTCHAR());
}

void FRDGUserValidation::ValidateAddPass(const FRDGPass* Pass)
{
	if (!GRDGValidation)
	{
		return;
	}

	const FRenderTargetBindingSlots* RenderTargetBindingSlots = nullptr;

	// Pass flags are validated as early as possible by the builder in AddPass.
	const ERDGPassFlags PassFlags = Pass->GetFlags();
	const FRDGParameterStruct PassParameters = Pass->GetParameters();

	const TCHAR* PassName = Pass->GetName();
	const bool bIsRaster = EnumHasAnyFlags(PassFlags, ERDGPassFlags::Raster);
	const bool bIsCopy = EnumHasAnyFlags(PassFlags, ERDGPassFlags::Copy);
	const bool bIsAnyCompute = EnumHasAnyFlags(PassFlags, ERDGPassFlags::Compute | ERDGPassFlags::AsyncCompute);
	const bool bSkipRenderPass = EnumHasAnyFlags(PassFlags, ERDGPassFlags::SkipRenderPass);

	const auto MarkAsProduced = [&](FRDGViewableResource* Resource)
	{
		// Currently a no-op.
	};

	const auto MarkAsConsumed = [&] (FRDGViewableResource* Resource)
	{
		ensureMsgf(Resource->bProduced || Resource->bExternal || Resource->bQueuedForUpload,
			TEXT("Pass %s has a read dependency on %s, but it was never written to."),
			PassName, Resource->Name);
	};

	const auto CheckValidResource = [&](FRDGResourceRef Resource)
	{
		checkf(ResourceMap.Contains(Resource), TEXT("Resource at %p registered with pass %s is not part of the graph and is likely a dangling pointer or garbage value."), Resource, Pass->GetName());
	};

	const auto CheckComputeOrRaster = [&](FRDGResourceRef Resource)
	{
		ensureMsgf(bIsAnyCompute || bIsRaster, TEXT("Pass %s, parameter %s is valid for Raster or (Async)Compute, but neither flag is set."), PassName, Resource->Name);
	};

	bool bCanProduce = false;

	const auto CheckResourceAccess = [&](FRDGViewableResource* Resource, ERHIAccess Access)
	{
		checkf(bIsCopy || !EnumHasAnyFlags(Access, AccessMaskCopy), TEXT("Pass '%s' uses resource '%s' with access '%s' containing states which require the 'ERDGPass::Copy' flag."), Pass->GetName(), Resource->Name, *GetRHIAccessName(Access));
		checkf(bIsAnyCompute || !EnumHasAnyFlags(Access, AccessMaskCompute), TEXT("Pass '%s' uses resource '%s' with access '%s' containing states which require the 'ERDGPass::Compute' or 'ERDGPassFlags::AsyncCompute' flag."), Pass->GetName(), Resource->Name, *GetRHIAccessName(Access));
		checkf(bIsRaster || !EnumHasAnyFlags(Access, AccessMaskRaster), TEXT("Pass '%s' uses resource '%s' with access '%s' containing states which require the 'ERDGPass::Raster' flag."), Pass->GetName(), Resource->Name, *GetRHIAccessName(Access));
		checkf(bIsAnyCompute || bIsRaster || !EnumHasAnyFlags(Access, AccessMaskComputeOrRaster), TEXT("Pass '%s' uses resource '%s' with access '%s' containing states which require the 'ERDGPassFlags::Compute' or 'ERDGPassFlags::AsyncCompute' or 'ERDGPass::Raster' flag."), Pass->GetName(), Resource->Name, *GetRHIAccessName(Access));
	};

	const auto CheckBufferAccess = [&](FRDGBufferRef Buffer, ERHIAccess Access)
	{
		CheckResourceAccess(Buffer, Access);

		if (IsWritableAccess(Access))
		{
			bCanProduce = true;
		}
	};

	const auto CheckTextureAccess = [&](FRDGTextureRef Texture, ERHIAccess Access)
	{
		CheckResourceAccess(Texture, Access);

		if (IsWritableAccess(Access))
		{
			bCanProduce = true;
		}
	};

	PassParameters.Enumerate([&](FRDGParameter Parameter)
	{
		if (Parameter.IsResource())
		{
			if (FRDGResourceRef Resource = Parameter.GetAsResource())
			{
				CheckValidResource(Resource);
			}
		}

		switch (Parameter.GetType())
		{
		case UBMT_RDG_TEXTURE:
		{
			if (FRDGTextureRef Texture = Parameter.GetAsTexture())
			{
				MarkAsConsumed(Texture);
			}
		}
		break;
		case UBMT_RDG_TEXTURE_SRV:
		case UBMT_RDG_TEXTURE_NON_PIXEL_SRV:
		{
			if (FRDGTextureSRVRef SRV = Parameter.GetAsTextureSRV())
			{
				FRDGTextureRef Texture = SRV->GetParent();
				CheckComputeOrRaster(Texture);
				MarkAsConsumed(Texture);
			}
		}
		break;
		case UBMT_RDG_TEXTURE_UAV:
		{
			bCanProduce = true;
			if (FRDGTextureUAVRef UAV = Parameter.GetAsTextureUAV())
			{
				FRDGTextureRef Texture = UAV->GetParent();
				CheckComputeOrRaster(Texture);
				MarkAsProduced(Texture);
			}
		}
		break;
		case UBMT_RDG_BUFFER_SRV:
		{
			if (FRDGBufferSRVRef SRV = Parameter.GetAsBufferSRV())
			{
				FRDGBufferRef Buffer = SRV->GetParent();
				CheckComputeOrRaster(Buffer);
				MarkAsConsumed(Buffer);
			}
		}
		break;
		case UBMT_RDG_BUFFER_UAV:
		{
			bCanProduce = true;
			if (FRDGBufferUAVRef UAV = Parameter.GetAsBufferUAV())
			{
				FRDGBufferRef Buffer = UAV->GetParent();
				CheckComputeOrRaster(Buffer);
				MarkAsProduced(Buffer);
			}
		}
		break;
		case UBMT_RDG_TEXTURE_ACCESS:
		{
			FRDGTextureAccess TextureAccess = Parameter.GetAsTextureAccess();
			bCanProduce |= IsWritableAccess(TextureAccess.GetAccess());

			if (TextureAccess)
			{
				CheckTextureAccess(TextureAccess.GetTexture(), TextureAccess.GetAccess());
			}
		}
		break;
		case UBMT_RDG_TEXTURE_ACCESS_ARRAY:
		{
			const FRDGTextureAccessArray& TextureAccessArray = Parameter.GetAsTextureAccessArray();

			for (FRDGTextureAccess TextureAccess : TextureAccessArray)
			{
				CheckTextureAccess(TextureAccess.GetTexture(), TextureAccess.GetAccess());
			}
		}
		break;
		case UBMT_RDG_BUFFER_ACCESS:
		{
			FRDGBufferAccess BufferAccess = Parameter.GetAsBufferAccess();

			if (BufferAccess)
			{
				CheckBufferAccess(BufferAccess.GetBuffer(), BufferAccess.GetAccess());
			}
		}
		break;
		case UBMT_RDG_BUFFER_ACCESS_ARRAY:
		{
			const FRDGBufferAccessArray& BufferAccessArray = Parameter.GetAsBufferAccessArray();

			for (FRDGBufferAccess BufferAccess : BufferAccessArray)
			{
				CheckBufferAccess(BufferAccess.GetBuffer(), BufferAccess.GetAccess());
			}
		}
		break;
		case UBMT_RENDER_TARGET_BINDING_SLOTS:
		{
			RenderTargetBindingSlots = &Parameter.GetAsRenderTargetBindingSlots();
			bCanProduce = true;
		}
		break;
		}
	});

	checkf(bCanProduce || EnumHasAnyFlags(PassFlags, ERDGPassFlags::NeverCull) || PassParameters.HasExternalOutputs(),
		TEXT("Pass '%s' has no graph parameters defined on its parameter struct and did not specify 'NeverCull'. The pass will always be culled."), PassName);

	/** Validate that raster passes have render target binding slots and compute passes don't. */
	if (RenderTargetBindingSlots)
	{
		checkf(bIsRaster, TEXT("Pass '%s' has render target binding slots but is not set to 'Raster'."), PassName);
	}
	else
	{
		checkf(!bIsRaster || bSkipRenderPass, TEXT("Pass '%s' is set to 'Raster' but is missing render target binding slots. Use 'SkipRenderPass' if this is desired."), PassName);
	}

	/** Validate render target / depth stencil binding usage. */
	if (RenderTargetBindingSlots)
	{
		const auto& RenderTargets = RenderTargetBindingSlots->Output;
		{
			if (FRDGTextureRef Texture = RenderTargetBindingSlots->ShadingRateTexture)
			{
				CheckValidResource(Texture);
				MarkAsConsumed(Texture);
			}

			const auto& DepthStencil = RenderTargetBindingSlots->DepthStencil;

			const auto CheckDepthStencil = [&](FRDGTextureRef Texture)
			{
				// Depth stencil only supports one mip, since there isn't actually a way to select the mip level.
				check(Texture->Desc.NumMips == 1);
				CheckValidResource(Texture);
				if (DepthStencil.GetDepthStencilAccess().IsAnyWrite())
				{
					MarkAsProduced(Texture);
				}
				else
				{
					MarkAsConsumed(Texture);
				}
			};

			FRDGTextureRef Texture = DepthStencil.GetTexture();

			if (Texture)
			{
				checkf(
					EnumHasAnyFlags(Texture->Desc.Flags, TexCreate_DepthStencilTargetable | TexCreate_DepthStencilResolveTarget),
					TEXT("Pass '%s' attempted to bind texture '%s' as a depth stencil render target, but the texture has not been created with TexCreate_DepthStencilTargetable."),
					PassName, Texture->Name);

				CheckDepthStencil(Texture);
			}
		}

		const uint32 RenderTargetCount = RenderTargets.Num();

		{
			/** Tracks the number of contiguous, non-null textures in the render target output array. */
			uint32 ValidRenderTargetCount = RenderTargetCount;

			for (uint32 RenderTargetIndex = 0; RenderTargetIndex < RenderTargetCount; ++RenderTargetIndex)
			{
				const FRenderTargetBinding& RenderTarget = RenderTargets[RenderTargetIndex];

				FRDGTextureRef Texture = RenderTarget.GetTexture();
				FRDGTextureRef ResolveTexture = RenderTarget.GetResolveTexture();

				if (ResolveTexture && ResolveTexture != Texture)
				{
					checkf(RenderTarget.GetTexture(), TEXT("Pass %s specified resolve target '%s' with a null render target."), PassName, ResolveTexture->Name);

					ensureMsgf(
						EnumHasAnyFlags(ResolveTexture->Desc.Flags, TexCreate_ResolveTargetable),
						TEXT("Pass '%s' attempted to bind texture '%s' as a render target, but the texture has not been created with TexCreate_ResolveTargetable."),
						PassName, ResolveTexture->Name);

					CheckValidResource(Texture);
					MarkAsProduced(ResolveTexture);
				}

				if (Texture)
				{
					ensureMsgf(
						EnumHasAnyFlags(Texture->Desc.Flags, TexCreate_RenderTargetable | TexCreate_ResolveTargetable),
						TEXT("Pass '%s' attempted to bind texture '%s' as a render target, but the texture has not been created with TexCreate_RenderTargetable."),
						PassName, Texture->Name);

					CheckValidResource(Texture);

					/** Mark the pass as a producer for render targets with a store action. */
					MarkAsProduced(Texture);
				}
				else
				{
					/** Found end of contiguous interval of valid render targets. */
					ValidRenderTargetCount = RenderTargetIndex;
					break;
				}
			}

			/** Validate that no holes exist in the render target output array. Render targets must be bound contiguously. */
			for (uint32 RenderTargetIndex = ValidRenderTargetCount; RenderTargetIndex < RenderTargetCount; ++RenderTargetIndex)
			{
				const FRenderTargetBinding& RenderTarget = RenderTargets[RenderTargetIndex];
				checkf(RenderTarget.GetTexture() == nullptr && RenderTarget.GetResolveTexture() == nullptr, TEXT("Render targets must be packed. No empty spaces in the array."));
			}
		}
	}
}

void FRDGUserValidation::ValidateExecuteBegin()
{
	checkf(!bHasExecuted, TEXT("Render graph execution should only happen once to ensure consistency with immediate mode."));
	check(!bHasExecuteBegun);
	bHasExecuteBegun = true;
}

void FRDGUserValidation::ValidateExecuteEnd()
{
	check(bHasExecuteBegun);

	bHasExecuted = true;
	GRDGBuilderActive = false;
}

void FRDGUserValidation::ValidateExecutePassBegin(const FRDGPass* Pass)
{
	if (bParallelExecuteEnabled || !GRDGValidation)
	{
		return;
	}

	SetAllowRHIAccess(Pass, true);
}

void FRDGUserValidation::ValidateExecutePassEnd(const FRDGPass* Pass)
{
	if (bParallelExecuteEnabled || !GRDGValidation)
	{
		return;
	}

	SetAllowRHIAccess(Pass, false);
}

void FRDGUserValidation::SetAllowRHIAccess(const FRDGPass* Pass, bool bAllowAccess)
{
	if (!GRDGValidation)
	{
		return;
	}

	Pass->GetParameters().Enumerate([&](FRDGParameter Parameter)
	{
		if (Parameter.IsResource())
		{
			if (FRDGResourceRef Resource = Parameter.GetAsResource())
			{
				Resource->bAllowRHIAccess = bAllowAccess;
			}
		}
		else if (Parameter.IsBufferAccessArray())
		{
			for (FRDGBufferAccess BufferAccess : Parameter.GetAsBufferAccessArray())
			{
				BufferAccess->bAllowRHIAccess = bAllowAccess;
			}
		}
		else if (Parameter.IsTextureAccessArray())
		{
			for (FRDGTextureAccess TextureAccess : Parameter.GetAsTextureAccessArray())
			{
				TextureAccess->bAllowRHIAccess = bAllowAccess;
			}
		}
		else if (Parameter.IsRenderTargetBindingSlots())
		{
			const FRenderTargetBindingSlots& RenderTargets = Parameter.GetAsRenderTargetBindingSlots();

			RenderTargets.Enumerate([&](FRenderTargetBinding RenderTarget)
			{
				RenderTarget.GetTexture()->bAllowRHIAccess = bAllowAccess;

				if (FRDGTextureRef ResolveTexture = RenderTarget.GetResolveTexture())
				{
					ResolveTexture->bAllowRHIAccess = bAllowAccess;
				}
			});

			if (FRDGTextureRef Texture = RenderTargets.DepthStencil.GetTexture())
			{
				Texture->bAllowRHIAccess = bAllowAccess;
			}

			if (FRDGTextureRef ResolveTexture = RenderTargets.DepthStencil.GetResolveTexture())
			{
				ResolveTexture->bAllowRHIAccess = bAllowAccess;
			}

			if (FRDGTexture* Texture = RenderTargets.ShadingRateTexture)
			{
				Texture->bAllowRHIAccess = bAllowAccess;
			}
		}
	});
}

FRDGBarrierValidation::FRDGBarrierValidation(const FRDGPassRegistry* InPasses, const FRDGEventName& InGraphName)
	: Passes(InPasses)
	, GraphName(InGraphName.GetTCHAR())
{
	check(Passes);
}

void FRDGBarrierValidation::ValidateBarrierBatchBegin(const FRDGPass* Pass, const FRDGBarrierBatchBegin& Batch)
{
	if (!GRDGTransitionLog)
	{
		return;
	}

	FResourceMap* ResourceMap = BatchMap.Find(&Batch);

	if (!ResourceMap)
	{
		ResourceMap = &BatchMap.Emplace(&Batch);

		for (int32 Index = 0; Index < Batch.Transitions.Num(); ++Index)
		{
			FRDGViewableResource* Resource = Batch.DebugTransitionResources[Index];
			const FRDGTransitionInfo& Transition = Batch.Transitions[Index];

			if (Resource->Type == ERDGViewableResourceType::Texture)
			{
				ResourceMap->Textures.FindOrAdd(static_cast<FRDGTextureRef>(Resource)).Add(Transition);
			}
			else
			{
				check(Resource->Type == ERDGViewableResourceType::Buffer);
				ResourceMap->Buffers.Emplace(static_cast<FRDGBufferRef>(Resource), Transition);
			}
		}

		for (int32 Index = 0; Index < Batch.Aliases.Num(); ++Index)
		{
			ResourceMap->Aliases.Emplace(Batch.DebugAliasingResources[Index], Batch.Aliases[Index]);
		}
	}

	if (!IsDebugAllowedForGraph(GraphName) || !IsDebugAllowedForPass(Pass->GetName()))
	{
		return;
	}

	bool bFoundFirst = false;

	const auto LogHeader = [&]()
	{
		if (!bFoundFirst)
		{
			bFoundFirst = true;
			UE_LOGF(LogRDG, Display, "[%ls(Index: %d, Pipeline: %ls): Batch: (%p) %ls] (Begin):", Pass->GetName(), Pass->GetHandle().GetIndex(), *GetRHIPipelineName(Pass->GetPipeline()), &Batch, Batch.DebugName);
		}
	};

	for (const auto& KeyValue : ResourceMap->Aliases)
	{
		const FRHITransientAliasingInfo& Info = KeyValue.Value;
		if (Info.IsAcquire())
		{
			FRDGViewableResource* Resource = KeyValue.Key;

			if (IsDebugAllowedForResource(Resource->Name))
			{
				LogHeader();
				UE_LOGF(LogRDG, Display, "\tRDG(%p) RHI(%p) %ls - Acquire", Resource, Resource->GetRHIUnchecked(), Resource->Name);
			}
		}
	}

	for (const auto& Pair : ResourceMap->Textures)
	{
		FRDGTextureRef Texture = Pair.Key;

		if (!IsDebugAllowedForResource(Texture->Name))
		{
			continue;
		}

		const auto& Transitions = Pair.Value;
		if (Transitions.Num())
		{
			LogHeader();
			UE_LOGF(LogRDG, Display, "\tRDG(%p) RHI(%p) %ls:", Texture, Texture->GetRHIUnchecked(), Texture->Name);
		}

		const FRDGTextureSubresourceLayout SubresourceLayout = Texture->GetSubresourceLayout();

		for (const FRDGTransitionInfo& Transition : Transitions)
		{
			UE_LOGF(LogRDG, Display, "\t\tMip(%d), Array(%d), Slice(%d): [%ls, %ls] -> [%ls, %ls]",
				Transition.Texture.MipIndex, Transition.Texture.ArraySlice, Transition.Texture.PlaneSlice,
				*GetRHIAccessName((ERHIAccess)Transition.AccessBefore),
				*GetRHIPipelineName(Batch.PipelinesToBegin),
				*GetRHIAccessName((ERHIAccess)Transition.AccessAfter),
				*GetRHIPipelineName(Batch.PipelinesToEnd));
		}
	}

	for (const auto& Pair : ResourceMap->Buffers)
	{
		FRDGBufferRef Buffer = Pair.Key;
		const FRDGTransitionInfo& Transition = Pair.Value;

		if (!IsDebugAllowedForResource(Buffer->Name))
		{
			continue;
		}

		LogHeader();

		UE_LOGF(LogRDG, Display, "\tRDG(%p) RHI(%p) %ls: [%ls, %ls] -> [%ls, %ls]",
			Buffer,
			Buffer->GetRHIUnchecked(),
			Buffer->Name,
			*GetRHIAccessName((ERHIAccess)Transition.AccessBefore),
			*GetRHIPipelineName(Batch.PipelinesToBegin),
			*GetRHIAccessName((ERHIAccess)Transition.AccessAfter),
			*GetRHIPipelineName(Batch.PipelinesToEnd));
	}
}

void FRDGBarrierValidation::ValidateBarrierBatchEnd(const FRDGPass* Pass, const FRDGBarrierBatchEnd& Batch)
{
	if (!GRDGTransitionLog || !IsDebugAllowedForGraph(GraphName) || !IsDebugAllowedForPass(Pass->GetName()))
	{
		return;
	}

	const bool bAllowedForPass = IsDebugAllowedForGraph(GraphName) && IsDebugAllowedForPass(Pass->GetName());

	bool bFoundFirst = false;

	const FRDGBarrierBatchEndId Id = Batch.GetId();

	for (const FRDGBarrierBatchBegin* Dependent : Batch.Dependencies)
	{
		if (!Batch.IsPairedWith(*Dependent))
		{
			continue;
		}

		const FResourceMap& ResourceMap = BatchMap.FindChecked(Dependent);

		TArray<FRDGTextureRef> Textures;
		if (ResourceMap.Textures.Num())
		{
			ResourceMap.Textures.GetKeys(Textures);
		}

		TArray<FRDGBufferRef> Buffers;
		if (ResourceMap.Buffers.Num())
		{
			ResourceMap.Buffers.GetKeys(Buffers);
		}

		const auto LogHeader = [&]()
		{
			if (!bFoundFirst)
			{
				bFoundFirst = true;
				UE_LOGF(LogRDG, Display, "[%ls(Index: %d, Pipeline: %ls) Batch: (%p) %ls] (End):", Pass->GetName(), Pass->GetHandle().GetIndex(), *GetRHIPipelineName(Pass->GetPipeline()), Dependent, Dependent->DebugName);
			}
		};

		for (FRDGTextureRef Texture : Textures)
		{
			if (IsDebugAllowedForResource(Texture->Name))
			{
				LogHeader();
				UE_LOGF(LogRDG, Display, "\tRDG(%p) RHI(%p) %ls - End:", Texture, Texture->GetRHIUnchecked(), Texture->Name);
			}
		}

		for (FRDGBufferRef Buffer : Buffers)
		{
			if (IsDebugAllowedForResource(Buffer->Name))
			{
				LogHeader();
				UE_LOGF(LogRDG, Display, "\tRDG(%p) RHI(%p) %ls - End", Buffer, Buffer->GetRHIUnchecked(), Buffer->Name);
			}
		}

		bFoundFirst = false;
	}
}

#endif
