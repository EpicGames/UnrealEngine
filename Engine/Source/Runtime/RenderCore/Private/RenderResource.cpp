// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	RenderResource.cpp: Render resource implementation.
=============================================================================*/

#include "RenderResource.h"
#include "Misc/App.h"
#include "RHI.h"
#include "RenderingThread.h"
#include "ProfilingDebugging/LoadTimeTracker.h"
#include "CoreGlobals.h"
#include "RenderGraphResources.h"
#include "Containers/ResourceArray.h"
#include "RenderCore.h"
#include "Async/RecursiveMutex.h"

static thread_local FName GRenderResourceScopeName = NAME_None;

class FRenderResourceList
{
public:
	template <FRenderResource::EInitPhase>
	static FRenderResourceList& Get()
	{
		static FRenderResourceList Instance;
		return Instance;
	}

	static FRenderResourceList& Get(FRenderResource::EInitPhase InitPhase)
	{
		switch (InitPhase)
		{
		case FRenderResource::EInitPhase::Pre:
			return Get<FRenderResource::EInitPhase::Pre>();
		default:
			return Get<FRenderResource::EInitPhase::Default>();
		}
	}

	int32 Allocate(FRenderResource* Resource)
	{
		Mutex.Lock();
		int32 Index;
		if (FreeIndexList.IsEmpty())
		{
			Index = ResourceList.Num();
			ResourceList.Emplace(Resource);
		}
		else
		{
			Index = FreeIndexList.Pop(EAllowShrinking::No);
			ResourceList[Index] = Resource;
		}
		Mutex.Unlock();
		return Index;
	}

	void Deallocate(int32 Index)
	{
		Mutex.Lock();
		FreeIndexList.Emplace(Index);
		ResourceList[Index] = nullptr;
		Mutex.Unlock();
	}

	void Clear()
	{
		Mutex.Lock();
		FreeIndexList.Empty();
		ResourceList.Empty();
		Mutex.Unlock();
	}

	template<typename FunctionType>
	void ForEach(const FunctionType& Function)
	{
		Mutex.Lock();
		for (int32 Index = 0; Index < ResourceList.Num(); ++Index)
		{
			FRenderResource* Resource = ResourceList[Index];
			if (Resource)
			{
				check(Resource->GetListIndex() == Index);
				Function(Resource);
			}
		}
		Mutex.Unlock();
	}

	template<typename FunctionType>
	void ForEachReverse(const FunctionType& Function)
	{
		Mutex.Lock();
		for (int32 Index = ResourceList.Num() - 1; Index >= 0; --Index)
		{
			FRenderResource* Resource = ResourceList[Index];
			if (Resource)
			{
				check(Resource->GetListIndex() == Index);
				Function(Resource);
			}
		}
		Mutex.Unlock();
	}

private:
	UE::FRecursiveMutex Mutex;
	TArray<int32> FreeIndexList;
	TArray<FRenderResource*> ResourceList;
};

void FRenderResource::ReleaseRHIForAllResources()
{
	const auto Lambda = [](FRenderResource* Resource) { check(Resource->IsInitialized()); Resource->ReleaseRHI(); };
	FRenderResourceList::Get<FRenderResource::EInitPhase::Default>().ForEachReverse(Lambda);
	FRenderResourceList::Get<FRenderResource::EInitPhase::Pre>().ForEachReverse(Lambda);
}

/** Initialize all resources initialized before the RHI was initialized */
void FRenderResource::InitPreRHIResources()
{
	FRHICommandListImmediate& RHICmdList = FRHICommandListImmediate::Get();
	SCOPED_GPU_MASK(RHICmdList, FRHIGPUMask::All());

	RHICmdList.InitializeImmediateContexts();

	FRenderResourceList& PreResourceList = FRenderResourceList::Get<FRenderResource::EInitPhase::Pre>();
	FRenderResourceList& DefaultResourceList = FRenderResourceList::Get<FRenderResource::EInitPhase::Default>();

	// Notify all initialized FRenderResources that there's a valid RHI device to create their RHI resources for now.
	const auto Lambda = [&](FRenderResource* Resource) { Resource->InitRHI(RHICmdList); };
	PreResourceList.ForEach(Lambda);
	DefaultResourceList.ForEach(Lambda);

#if !PLATFORM_NEEDS_RHIRESOURCELIST
	PreResourceList.Clear();
	DefaultResourceList.Clear();
#endif
}

void FRenderResource::ChangeFeatureLevel(ERHIFeatureLevel::Type NewFeatureLevel)
{
	ENQUEUE_RENDER_COMMAND(FRenderResourceChangeFeatureLevel)(
		[NewFeatureLevel](FRHICommandList& RHICmdList)
	{
		const auto Lambda = [NewFeatureLevel, &RHICmdList](FRenderResource* Resource)
		{
			// Only resources configured for a specific feature level need to be updated
			if (Resource->HasValidFeatureLevel() && (Resource->FeatureLevel != NewFeatureLevel))
			{
				Resource->ReleaseRHI();
				Resource->FeatureLevel = NewFeatureLevel;
				Resource->InitRHI(RHICmdList);
			}
		};

		FRenderResourceList::Get<FRenderResource::EInitPhase::Pre>().ForEach(Lambda);
		FRenderResourceList::Get<FRenderResource::EInitPhase::Default>().ForEach(Lambda);
	});
}

FName FRenderResource::SetScopeName(FName Name)
{
	FName Old = GRenderResourceScopeName;
	GRenderResourceScopeName = Name;
	return Old;
}

FRHICommandListBase& FRenderResource::GetImmediateCommandList()
{
	return FRHICommandListImmediate::Get();
}

void FRenderResource::InitResource(FRHICommandListBase& RHICmdList)
{
	if (ListIndex == INDEX_NONE)
	{
		int32 LocalListIndex = INDEX_NONE;

		if (PLATFORM_NEEDS_RHIRESOURCELIST || !GIsRHIInitialized)
		{
			LLM_SCOPE(ELLMTag::SceneRender);
			LocalListIndex = FRenderResourceList::Get(InitPhase).Allocate(this);
		}
		else
		{
			// Mark this resource as initialized
			LocalListIndex = 0;
		}

		if (GIsRHIInitialized)
		{
			CSV_SCOPED_TIMING_STAT_EXCLUSIVE(InitRenderResource);
			InitRHI(RHICmdList);
		}

		FPlatformMisc::MemoryBarrier(); // there are some multithreaded reads of ListIndex
		ListIndex = LocalListIndex;
	}
}

void FRenderResource::ReleaseResource()
{
	if ( !GIsCriticalError )
	{
		if(ListIndex != INDEX_NONE)
		{
			if(GIsRHIInitialized)
			{
				ReleaseRHI();
			}

#if PLATFORM_NEEDS_RHIRESOURCELIST
			FRenderResourceList::Get(InitPhase).Deallocate(ListIndex);
#endif
			ListIndex = INDEX_NONE;
		}
	}
}

void FRenderResource::UpdateRHI(FRHICommandListBase& RHICmdList)
{
	if (IsInitialized() && GIsRHIInitialized)
	{
		ReleaseRHI();
		InitRHI(RHICmdList);
	}
}
FRenderResource::FRenderResource()
	: ListIndex(INDEX_NONE)
	, FeatureLevel(ERHIFeatureLevel::Num)
{
}

FRenderResource::FRenderResource(ERHIFeatureLevel::Type InFeatureLevel)
	: ListIndex(INDEX_NONE)
	, FeatureLevel(InFeatureLevel)
{
}

FRenderResource::FRenderResource(const FRenderResource&) = default;
FRenderResource::FRenderResource(FRenderResource&&) = default;
FRenderResource& FRenderResource::operator=(const FRenderResource& Other) = default;
FRenderResource& FRenderResource::operator=(FRenderResource&& Other) = default;

FRenderResource::~FRenderResource()
{
	checkf(ResourceState == ERenderResourceState::Default, TEXT(" Invalid Resource State: %s"), ResourceState == ERenderResourceState::BatchReleased ? TEXT("BatchReleased") : TEXT("Deleted"));
	ResourceState = ERenderResourceState::Deleted;
	if (IsInitialized() && !GIsCriticalError)
	{
		// Deleting an initialized FRenderResource will result in a crash later since it is still linked
		UE_LOGF(LogRendererCore, Fatal, "A FRenderResource was deleted without being released first! [Resource=%ls, Owner=%ls, Scope=%ls]", *GetResourceName().ToString(), *GetOwnerName().ToString(), *GRenderResourceScopeName.ToString());
	}
}

bool FRenderResource::ShouldFreeResourceObject(void* ResourceObject, FResourceArrayInterface* ResourceArray)
{
	return ResourceObject && (!ResourceArray || !ResourceArray->GetResourceDataSize());
}

FBufferRHIRef FRenderResource::CreateRHIBufferInternal(
	FRHICommandListBase& RHICmdList,
	const TCHAR* InDebugName,
	const FName& InOwnerName,
	uint32 ResourceCount,
	EBufferUsageFlags InBufferUsageFlags,
	FResourceArrayInterface* ResourceArray,
	bool bWithoutNativeResource)
{
	const uint32 SizeInBytes = ResourceArray ? ResourceArray->GetResourceDataSize() : 0;

	FRHIBufferCreateDesc CreateDesc = bWithoutNativeResource
		? FRHIBufferCreateDesc::CreateNull(InDebugName)
		: (FRHIBufferCreateDesc::CreateVertex(InDebugName, SizeInBytes).AddUsage(InBufferUsageFlags));

	CreateDesc.SetClassName(FName(InDebugName))
		.SetOwnerName(InOwnerName)
		.SetInitialState(ERHIAccess::VertexOrIndexBuffer | ERHIAccess::SRVMask);

	if (ResourceArray)
	{
		CreateDesc.SetInitActionResourceArray(ResourceArray);
	}

	return RHICmdList.CreateBuffer(CreateDesc);
}

void BeginInitResource(FRenderResource* Resource, FRenderCommandPipe* RenderCommandPipe)
{
	ENQUEUE_RENDER_COMMAND(InitCommand)(RenderCommandPipe,
		[Resource](FRHICommandListBase& RHICmdList)
		{
			Resource->InitResource(RHICmdList);
		});
}

void BeginUpdateResourceRHI(FRenderResource* Resource, FRenderCommandPipe* RenderCommandPipe)
{
	ENQUEUE_RENDER_COMMAND(UpdateCommand)(RenderCommandPipe,
		[Resource](FRHICommandListBase& RHICmdList)
		{
			Resource->UpdateRHI(RHICmdList);
		});
}

struct FBatchedReleaseResources
{
	enum 
	{
		NumPerBatch = 16
	};
	TArray<FRenderResource*, TInlineAllocator<NumPerBatch>> Resources;

	void Flush()
	{
		if (Resources.Num())
		{
			ENQUEUE_RENDER_COMMAND(BatchReleaseCommand)(
			[BatchedReleaseResources = MoveTemp(Resources)](FRHICommandList& RHICmdList)
			{
				for (FRenderResource* Resource : BatchedReleaseResources)
				{
					check(Resource->ResourceState == ERenderResourceState::BatchReleased);
					Resource->ReleaseResource();
					Resource->ResourceState = ERenderResourceState::Default;
				}
			});
		}
	}

	void Add(FRenderResource* Resource)
	{
		if (Resources.Num() >= NumPerBatch)
		{
			Flush();
		}
		check(Resources.Num() < NumPerBatch);
		check(Resource->ResourceState == ERenderResourceState::Default);
		Resource->ResourceState = ERenderResourceState::BatchReleased;
		Resources.Push(Resource);
	}

	bool IsEmpty()
	{
		return Resources.Num() == 0;
	}
};

static bool GBatchedReleaseIsActive = false;
static FBatchedReleaseResources GBatchedRelease;

void StartBatchedRelease()
{
	check(IsInGameThread() && !GBatchedReleaseIsActive && GBatchedRelease.IsEmpty());
	GBatchedReleaseIsActive = true;
}
void EndBatchedRelease()
{
	check(IsInGameThread() && GBatchedReleaseIsActive);
	GBatchedRelease.Flush();
	GBatchedReleaseIsActive = false;
}

void BeginReleaseResource(FRenderResource* Resource, FRenderCommandPipe* RenderCommandPipe)
{
	if (GBatchedReleaseIsActive && IsInGameThread())
	{
		GBatchedRelease.Add(Resource);
		return;
	}
	ENQUEUE_RENDER_COMMAND(ReleaseCommand)(RenderCommandPipe,
		[Resource]
		{
			Resource->ReleaseResource();
		});
}

void ReleaseResourceAndFlush(FRenderResource* Resource)
{
	// Send the release message.
	ENQUEUE_RENDER_COMMAND(ReleaseCommand)(
		[Resource](FRHICommandList& RHICmdList)
		{
			Resource->ReleaseResource();
		});

	FlushRenderingCommands();
}

//////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// FTextureSamplerStateCache

class FTextureSamplerStateCache : public FRenderResource
{
public:
	TMap<FSamplerStateInitializerRHI, FRHISamplerState*> Samplers;

	virtual void ReleaseRHI() override
	{
		for (auto Pair : Samplers)
		{
			Pair.Value->Release();
		}
		Samplers.Empty();
	}
};

TGlobalResource<FTextureSamplerStateCache> GTextureSamplerStateCache;

//////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// FTexture

FTexture::FTexture() = default;
FTexture::~FTexture() = default;
FTexture::FTexture(const FTexture&) = default;
FTexture::FTexture(FTexture&&) = default;
FTexture& FTexture::operator=(const FTexture& Other) = default;
FTexture& FTexture::operator=(FTexture&& Other) = default;

uint32 FTexture::GetSizeX() const
{
	return 0;
}

uint32 FTexture::GetSizeY() const
{
	return 0;
}

uint32 FTexture::GetSizeZ() const
{
	return 0;
}

void FTexture::ReleaseRHI()
{
	TextureRHI.SafeRelease();
	SamplerStateRHI.SafeRelease();
	DeferredPassSamplerStateRHI.SafeRelease();
}

FString FTexture::GetFriendlyName() const
{
	return TEXT("FTexture");
}

FRHISamplerState* FTexture::GetOrCreateSamplerState(const FSamplerStateInitializerRHI& Initializer)
{
	// This sampler cache is supposed to be used only from RT
	// Add a lock here if it's used from multiple threads
	check(IsInRenderingThread());

	FRHISamplerState** Found = GTextureSamplerStateCache.Samplers.Find(Initializer);
	if (Found)
	{
		return *Found;
	}

	FSamplerStateRHIRef NewState = RHICreateSamplerState(Initializer);

	// Add an extra reference so we don't have TRefCountPtr in the maps
	NewState->AddRef();
	GTextureSamplerStateCache.Samplers.Add(Initializer, NewState);
	return NewState;
}

//////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// FTextureWithSRV

FTextureWithSRV::FTextureWithSRV() = default;
FTextureWithSRV::~FTextureWithSRV() = default;

void FTextureWithSRV::ReleaseRHI()
{
	ShaderResourceViewRHI.SafeRelease();
	FTexture::ReleaseRHI();
}

//////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// FTextureReference

FTextureReference::FTextureReference()
	: TextureReferenceRHI(NULL)
{
}

FTextureReference::~FTextureReference()
{
}

void FTextureReference::BeginInit_GameThread()
{
	bInitialized_GameThread = true;
	BeginInitResource(this);
}

void FTextureReference::BeginRelease_GameThread()
{
	BeginReleaseResource(this);
	bInitialized_GameThread = false;
}

double FTextureReference::GetLastRenderTime() const
{
	if (bInitialized_GameThread && TextureReferenceRHI)
	{
		return TextureReferenceRHI->GetLastRenderTime();
	}

	return FRHITexture::EmptyLastRenderTime;
}

void FTextureReference::InvalidateLastRenderTime()
{
	if (bInitialized_GameThread && TextureReferenceRHI)
	{
		TextureReferenceRHI->SetLastRenderTime(-FLT_MAX);
	}
}

void FTextureReference::InitRHI(FRHICommandListBase& RHICmdList)
{
	SCOPED_LOADTIMER(FTextureReference_InitRHI);
	TextureReferenceRHI = RHICmdList.CreateTextureReference();
}
	
int32 GTextureReferenceRevertsLastRenderContainer = 1;
FAutoConsoleVariableRef CVarTextureReferenceRevertsLastRenderContainer(
	TEXT("r.TextureReferenceRevertsLastRenderContainer"),
	GTextureReferenceRevertsLastRenderContainer,
	TEXT(""));

void FTextureReference::ReleaseRHI()
{
	TextureReferenceRHI.SafeRelease();
}

FString FTextureReference::GetFriendlyName() const
{
	return TEXT("FTextureReference");
}

//////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// FVertexBuffer

FVertexBuffer::FVertexBuffer() = default;
FVertexBuffer::FVertexBuffer(const FVertexBuffer&) = default;
FVertexBuffer& FVertexBuffer::operator=(const FVertexBuffer& Other) = default;
FVertexBuffer::~FVertexBuffer() = default;

void FVertexBuffer::ReleaseRHI()
{
	VertexBufferRHI.SafeRelease();
}

FString FVertexBuffer::GetFriendlyName() const
{
	return TEXT("FVertexBuffer");
}

void FVertexBuffer::SetRHI(const FBufferRHIRef& BufferRHI)
{
	VertexBufferRHI = BufferRHI;
}

//////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// FVertexBufferWithSRV

FVertexBufferWithSRV::FVertexBufferWithSRV() = default;
FVertexBufferWithSRV::~FVertexBufferWithSRV() = default;

void FVertexBufferWithSRV::ReleaseRHI()
{
	ShaderResourceViewRHI.SafeRelease();
	UnorderedAccessViewRHI.SafeRelease();
	FVertexBuffer::ReleaseRHI();
}

//////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// FIndexBuffer

FIndexBuffer::FIndexBuffer() = default;
FIndexBuffer::FIndexBuffer(const FIndexBuffer&) = default;
FIndexBuffer& FIndexBuffer::operator=(const FIndexBuffer& Other) = default;
FIndexBuffer::~FIndexBuffer() = default;

void FIndexBuffer::ReleaseRHI()
{
	IndexBufferRHI.SafeRelease();
}

FString FIndexBuffer::GetFriendlyName() const
{
	return TEXT("FIndexBuffer");
}

void FIndexBuffer::SetRHI(const FBufferRHIRef& BufferRHI)
{
	IndexBufferRHI = BufferRHI;
}
//////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// FBufferWithRDG

FBufferWithRDG::FBufferWithRDG() = default;
FBufferWithRDG::FBufferWithRDG(const FBufferWithRDG& Other) = default;
FBufferWithRDG& FBufferWithRDG::operator=(const FBufferWithRDG& Other) = default;
FBufferWithRDG::~FBufferWithRDG() = default;

void FBufferWithRDG::ReleaseRHI()
{
	Buffer = nullptr;
	FRenderResource::ReleaseRHI();
}
