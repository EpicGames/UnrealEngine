// Copyright Epic Games, Inc. All Rights Reserved.

#include "RenderTargetPool.h"
#include "Hash/CityHash.h"
#include "Misc/CoreMisc.h"
#include "Trace/Trace.inl"
#include "ProfilingDebugging/CountersTrace.h"
#include "RHI.h"
#include "RenderCore.h"
#include "RHICommandList.h"

/** The global render targets pool. */
TGlobalResource<FRenderTargetPool> GRenderTargetPool;

DEFINE_LOG_CATEGORY_STATIC(LogRenderTargetPool, Warning, All);

CSV_DEFINE_CATEGORY(RenderTargetPool, !UE_SERVER);

TRACE_DECLARE_INT_COUNTER(RenderTargetPoolCount, TEXT("RenderTargetPool/Count"));
TRACE_DECLARE_MEMORY_COUNTER(RenderTargetPoolSize, TEXT("RenderTargetPool/Size"));

UE_TRACE_EVENT_BEGIN(Cpu, FRenderTargetPool_CreateTexture, NoSync)
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, Name)
	UE_TRACE_EVENT_FIELD(uint32, Width)
	UE_TRACE_EVENT_FIELD(uint32, Height)
UE_TRACE_EVENT_END()


///////////////////////////////////////////////////////////////////////////////

static int32 GRenderTargetPoolLogCreationSizes = 0;
static FAutoConsoleVariableRef CVarRenderTargetPoolLogCreationSizes(
	TEXT("r.RenderTargetPool.LogCreationSizes"),
	GRenderTargetPoolLogCreationSizes,
	TEXT("Enable/disable warning log for render target pool creation sizes."));

static int32 GRenderTargetPoolLogCreationSizesSpamInterval = 4; // 4 per frames
static FAutoConsoleVariableRef CVarRenderTargetPoolLogCreationSpamInterval(
	TEXT("r.RenderTargetPool.LogCreationSizesSpamInterval"),
	GRenderTargetPoolLogCreationSizesSpamInterval,
	TEXT("The number of logs in a given frame that are generated before we force a print."));

static int32 GRenderTargetPoolLogCreationSizesSizeInterval = 1024 * 1024; // 1 MB
static FAutoConsoleVariableRef CVarRenderTargetPoolLogCreationSizeInterval(
	TEXT("r.RenderTargetPool.LogCreationSizesSizeInterval"),
	GRenderTargetPoolLogCreationSizesSizeInterval,
	TEXT("The total size of accumulated logs during a frame before we force a print."));

static void LogRenderTargetCreationSizeAndFrequencies(const FRHITextureCreateInfo& Desc, const TCHAR* Name)
{
#if !UE_BUILD_SHIPPING && !UE_SERVER
	static FString AccumulatedLogs;
	static uint64 AccumulatedCreatedTargets = 0;
	static uint64 AccumulatedCreatedTargetSizes = 0;
	static uint64 TrackingCurrentFrame = 0;
	
	if (GRenderTargetPoolLogCreationSizes <= 0)
	{
		AccumulatedLogs.Empty();
		AccumulatedCreatedTargets = 0;
		AccumulatedCreatedTargetSizes = 0;
		TrackingCurrentFrame = 0;
		return;
	}

	// Log last frame's results IFF we are over one or more of our target
	// thresholds. Either way, clear the accumulators for the current frame.
	const uint64 CurrentRTFrame = GFrameCounterRenderThread;
	if (TrackingCurrentFrame != CurrentRTFrame)
	{
		if (AccumulatedCreatedTargets >= GRenderTargetPoolLogCreationSizesSpamInterval ||
			AccumulatedCreatedTargetSizes >= GRenderTargetPoolLogCreationSizesSizeInterval)
		{
			UE_LOGF(LogRenderTargetPool, Warning, "FRenderTargetPool::CreateRenderTarget(s) since last frame:\n%ls", *AccumulatedLogs);
		}

		AccumulatedLogs.Empty();
		AccumulatedCreatedTargets = 0;
		AccumulatedCreatedTargetSizes = 0;
		TrackingCurrentFrame = CurrentRTFrame;
	}

	const uint64 CreationSize = RHICalcTexturePlatformSize(Desc).Size;
	AccumulatedCreatedTargets += 1;
	AccumulatedCreatedTargetSizes += CreationSize;
	AccumulatedLogs += FString::Printf(
		TEXT("\tRenderTargetCreation { \"%s\", Size(%.2f KB) }\n"), Name, CreationSize / 1024.0);
#endif
}

///////////////////////////////////////////////////////////////////////////////

TRefCountPtr<IPooledRenderTarget> CreateRenderTarget(FRHITexture* Texture, const TCHAR* Name)
{
	check(Texture);

	FSceneRenderTargetItem Item;
	Item.TargetableTexture = Texture;
	Item.ShaderResourceTexture = Texture;

	FPooledRenderTargetDesc Desc = Translate(Texture->GetDesc());
	Desc.DebugName = Name;

	TRefCountPtr<IPooledRenderTarget> PooledRenderTarget;
	GRenderTargetPool.CreateUntrackedElement(Desc, PooledRenderTarget, Item);
	return MoveTemp(PooledRenderTarget);
}

bool CacheRenderTarget(FRHITexture* Texture, const TCHAR* Name, TRefCountPtr<IPooledRenderTarget>& OutPooledRenderTarget)
{
	if (!OutPooledRenderTarget || OutPooledRenderTarget->GetRHI() != Texture)
	{
		OutPooledRenderTarget = CreateRenderTarget(Texture, Name);
		return true;
	}
	return false;
}

RENDERCORE_API void DumpRenderTargetPoolMemory(FOutputDevice& OutputDevice)
{
	GRenderTargetPool.DumpMemoryUsage(OutputDevice);
}

static FAutoConsoleCommandWithOutputDevice GDumpRenderTargetPoolMemoryCmd(
	TEXT("r.DumpRenderTargetPoolMemory"),
	TEXT("Dump allocation information for the render target pool."),
	FConsoleCommandWithOutputDeviceDelegate::CreateStatic(DumpRenderTargetPoolMemory)
);

static uint32 ComputeSizeInKB(FPooledRenderTarget& Element)
{
	return (Element.ComputeMemorySize() + 1023) / 1024;
}

FPooledRenderTarget* FRenderTargetPool::CreateRenderTarget(FRHICommandListBase& RHICmdList, const FRHITextureCreateInfo& Desc, uint32 DescHash, const TCHAR* Name)
{
#if CPUPROFILERTRACE_ENABLED
	UE_TRACE_LOG_SCOPED_T(Cpu, FRenderTargetPool_CreateTexture, CpuChannel)
		<< FRenderTargetPool_CreateTexture.Name(Name)
		<< FRenderTargetPool_CreateTexture.Width(Desc.Extent.X)
		<< FRenderTargetPool_CreateTexture.Height(Desc.Extent.Y);
#endif

	LogRenderTargetCreationSizeAndFrequencies(Desc, Name);

	FRHITextureCreateDesc CreateDesc(Desc, ERHIAccess::SRVMask, Name);
	const static FLazyName ClassName(TEXT("FPooledRenderTarget"));
	CreateDesc.SetClassName(ClassName);

	FPooledRenderTarget* Result = new FPooledRenderTarget(
		RHICmdList.CreateTexture(CreateDesc),
		Translate(CreateDesc),
		this);

	PooledRenderTargets.Add(Result);
	PooledRenderTargetHashes.Add(DescHash);

	if (EnumHasAnyFlags(Desc.Flags, TexCreate_UAV))
	{
		if (GRHISupportsUAVFormatAliasing)
		{
			EPixelFormat AliasFormat = Desc.UAVFormat != PF_Unknown
				? Desc.UAVFormat
				: Desc.Format;

			Result->RenderTargetItem.UAV = RHICmdList.CreateUnorderedAccessView(
				Result->GetRHI(),
				FRHIViewDesc::CreateTextureUAV()
					.SetDimensionFromTexture(Result->GetRHI())
					.SetFormat(AliasFormat));
		}
		else
		{
			checkf(Desc.UAVFormat == PF_Unknown || Desc.UAVFormat == Desc.Format, TEXT("UAV aliasing is not supported by the current RHI."));
			Result->RenderTargetItem.UAV = RHICmdList.CreateUnorderedAccessView(Result->GetRHI(), FRHIViewDesc::CreateTextureUAV().SetDimensionFromTexture(Result->GetRHI()));
		}
	}

	AllocationLevelInKB += ComputeSizeInKB(*Result);
	TRACE_COUNTER_ADD(RenderTargetPoolCount, 1);
	TRACE_COUNTER_SET(RenderTargetPoolSize, (int64)AllocationLevelInKB * 1024);
	return Result;
}

template <typename T>
FPooledRenderTarget* FRenderTargetPool::TryFindRenderTarget(const FRHITextureCreateInfo& Desc, uint32 DescHash, T&& Predicate) const
{
	for (uint32 Index = 0, Num = (uint32)PooledRenderTargets.Num(); Index < Num; ++Index)
	{
		if (PooledRenderTargetHashes[Index] == DescHash)
		{
			FPooledRenderTarget* Element = PooledRenderTargets[Index];

			checkf(Element && Element->GetRHI(), TEXT("Hash was not cleared from the list."));

			// Compare the descriptors to avoid hash collisions.
			if (Element->GetRHI()->GetDesc() != Desc)
			{
				continue;
			}

			if (Element->IsFree() && Predicate(Element))
			{
				return Element;
			}
		}
	}
	return nullptr;
}

FPooledRenderTarget* FRenderTargetPool::ScheduleAllocation(FRHICommandListBase& RHICmdList, FRHITextureCreateInfo Desc, const TCHAR* Name, const FRHITransientAllocationFences& Fences)
{
	// FastVRAM is no longer supported by the render target pool.
	EnumRemoveFlags(Desc.Flags, ETextureCreateFlags::FastVRAM | ETextureCreateFlags::FastVRAMPartialAlloc);

	// We always want SRV access
	Desc.Flags |= TexCreate_ShaderResource;

	const uint32 DescHash = GetTypeHash(Desc);
	
	FPooledRenderTarget* Found = TryFindRenderTarget(Desc, DescHash, [&](FPooledRenderTarget* Element)
	{
		return Element->PooledTexture.Fences && !FRHITransientAllocationFences::Contains(*Element->PooledTexture.Fences, Fences);
	});

	if (!Found)
	{
		Found = CreateRenderTarget(RHICmdList, Desc, DescHash, Name);
	}

	Found->PooledTexture.Fences.Reset();
	Found->UnusedForNFrames = 0;
	return Found;
}

void FRenderTargetPool::ScheduleDeallocation(FPooledRenderTarget* RenderTarget, const FRHITransientAllocationFences& Fences)
{
	RenderTarget->PooledTexture.Fences = Fences;
}

void FRenderTargetPool::FinishSchedule(FRHICommandListBase& RHICmdList, FPooledRenderTarget* RenderTarget, const TCHAR* Name)
{
	RenderTarget->PooledTexture.Fences.Emplace();
	RenderTarget->SetDebugLabelName(RHICmdList, Name);
}

TRefCountPtr<IPooledRenderTarget> FRenderTargetPool::FindFreeElement(FRHICommandListBase& RHICmdList, FRHITextureCreateInfo Desc, const TCHAR* Name)
{
	// FastVRAM is no longer supported by the render target pool.
	EnumRemoveFlags(Desc.Flags, ETextureCreateFlags::FastVRAM | ETextureCreateFlags::FastVRAMPartialAlloc);

	// We always want SRV access
	Desc.Flags |= TexCreate_ShaderResource;

	const uint32 DescHash = GetTypeHash(Desc);

	UE::TScopeLock Lock(Mutex);

	FPooledRenderTarget* Found = TryFindRenderTarget(Desc, DescHash);

	if (!Found)
	{
		Found = CreateRenderTarget(RHICmdList, Desc, DescHash, Name);
	}

	Found->SetDebugLabelName(RHICmdList, Name);
	Found->UnusedForNFrames = 0;

	return TRefCountPtr<IPooledRenderTarget>(MoveTemp(Found));
}

bool FRenderTargetPool::FindFreeElement(FRHICommandListBase& RHICmdList, const FRHITextureCreateInfo& Desc, TRefCountPtr<IPooledRenderTarget>& Out, const TCHAR* Name)
{
	if (!Desc.IsValid())
	{
		// no need to do anything
		return true;
	}

	// Querying a render target that have no mip levels makes no sens.
	check(Desc.NumMips > 0);

	FPooledRenderTarget* Current = nullptr;
	bool bFreeCurrent = false;

	// if we can keep the current one, do that
	if (Out)
	{
		Current = (FPooledRenderTarget*)Out.GetReference();

		if (Out->GetRHI() && Out->GetRHI()->GetDesc() == Desc)
		{
			// we can reuse the same, but the debug name might have changed
			Current->Desc.DebugName = Name;
#if RHI_USE_RESOURCE_DEBUG_NAME
			RHICmdList.BindDebugLabelName(Current->GetRHI(), Name);
#endif
			check(!Out->IsFree());
			return true;
		}
		else
		{
			// release old reference, it might free a RT we can use
			Out = 0;
			bFreeCurrent = Current->IsFree();
		}
	}

	UE::TScopeLock Lock(Mutex);

	if (bFreeCurrent)
	{
		AllocationLevelInKB -= ComputeSizeInKB(*Current);
		TRACE_COUNTER_SUBTRACT(RenderTargetPoolCount, 1);
		TRACE_COUNTER_SET(RenderTargetPoolSize, (int64)AllocationLevelInKB * 1024);
		int32 Index = FindIndex(Current);
		check(Index >= 0);
		FreeElementAtIndex(Index);
	}

	Out = FindFreeElement(RHICmdList, Desc, Name);
	return false;
}

void FRenderTargetPool::CreateUntrackedElement(const FPooledRenderTargetDesc& Desc, TRefCountPtr<IPooledRenderTarget>& Out, const FSceneRenderTargetItem& Item)
{
	FPooledRenderTarget* Result = new FPooledRenderTarget(Item.GetRHI(), Desc, nullptr);
	Result->RenderTargetItem = Item;
	Out = Result;
}

void FRenderTargetPool::GetStats(uint32& OutWholeCount, uint32& OutWholePoolInKB, uint32& OutUsedInKB) const
{
	UE::TScopeLock Lock(Mutex);
	OutWholeCount = (uint32)PooledRenderTargets.Num();
	OutUsedInKB = 0;
	OutWholePoolInKB = 0;

	for (uint32 i = 0; i < (uint32)PooledRenderTargets.Num(); ++i)
	{
		FPooledRenderTarget* Element = PooledRenderTargets[i];

		if (Element)
		{
			uint32 SizeInKB = ComputeSizeInKB(*Element);

			OutWholePoolInKB += SizeInKB;

			if (!Element->IsFree())
			{
				OutUsedInKB += SizeInKB;
			}
		}
	}

	// if this triggers uncomment the code in VerifyAllocationLevel() and debug the issue, we might leak memory or not release when we could
	ensure(AllocationLevelInKB == OutWholePoolInKB);
}

void FRenderTargetPool::TickPoolElements()
{
	UE::TScopeLock Lock(Mutex);

	uint32 DeferredAllocationLevelInKB = 0;
	for (FPooledRenderTarget* Element : DeferredDeleteArray)
	{
		DeferredAllocationLevelInKB += ComputeSizeInKB(*Element);
	}

	DeferredDeleteArray.Reset();

	uint32 MinimumPoolSizeInKB;
	{
		static const auto CVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.RenderTargetPoolMin"));

		MinimumPoolSizeInKB = FMath::Max(CVar->GetValueOnRenderThread(), 0) * 1024;
	}

	CompactPool();

	uint32 UnusedAllocationLevelInKB = 0;
	for (uint32 i = 0; i < (uint32)PooledRenderTargets.Num(); ++i)
	{
		FPooledRenderTarget* Element = PooledRenderTargets[i];

		if (Element)
		{
			Element->OnFrameStart();
			if (Element->UnusedForNFrames > 2)
			{
				UnusedAllocationLevelInKB += ComputeSizeInKB(*Element);
			}
		}
	}

	uint32 TotalFrameUsageInKb = AllocationLevelInKB + DeferredAllocationLevelInKB ;

	CSV_CUSTOM_STAT(RenderTargetPool, UnusedMB, UnusedAllocationLevelInKB / 1024.0f, ECsvCustomStatOp::Set);
	CSV_CUSTOM_STAT(RenderTargetPool, PeakUsedMB, (TotalFrameUsageInKb - UnusedAllocationLevelInKB) / 1024.f, ECsvCustomStatOp::Set);
	
	// we need to release something, take the oldest ones first
	while (AllocationLevelInKB > MinimumPoolSizeInKB)
	{
		// -1: not set
		int32 OldestElementIndex = -1;

		// find oldest element we can remove
		for (uint32 i = 0, Num = (uint32)PooledRenderTargets.Num(); i < Num; ++i)
		{
			FPooledRenderTarget* Element = PooledRenderTargets[i];

			if (Element && Element->UnusedForNFrames > 2)
			{
				if (OldestElementIndex != -1)
				{
					if (PooledRenderTargets[OldestElementIndex]->UnusedForNFrames < Element->UnusedForNFrames)
					{
						OldestElementIndex = i;
					}
				}
				else
				{
					OldestElementIndex = i;
				}
			}
		}

		if (OldestElementIndex != -1)
		{
			AllocationLevelInKB -= ComputeSizeInKB(*PooledRenderTargets[OldestElementIndex]);
			TRACE_COUNTER_SUBTRACT(RenderTargetPoolCount, 1);
			TRACE_COUNTER_SET(RenderTargetPoolSize, (int64)AllocationLevelInKB * 1024);

			// we assume because of reference counting the resource gets released when not needed any more
			// we don't use Remove() to not shuffle around the elements for better transparency on RenderTargetPoolEvents
			FreeElementAtIndex(OldestElementIndex);
		}
		else
		{
			// There is no element we can remove but we are over budget, better we log that.
			// Options:
			//   * Increase the pool
			//   * Reduce rendering features or resolution
			//   * Investigate allocations, order or reusing other render targets can help
			//   * Ignore (editor case, might start using slow memory which can be ok)
			if (!bCurrentlyOverBudget)
			{
				UE_CLOGF(IsRunningClientOnly() && MinimumPoolSizeInKB != 0, LogRenderTargetPool, Warning, "r.RenderTargetPoolMin exceeded %d/%d MB (ok in editor, bad on fixed memory platform)", (AllocationLevelInKB + 1023) / 1024, MinimumPoolSizeInKB / 1024);
				bCurrentlyOverBudget = true;
			}
			// at this point we need to give up
			break;
		}
	}

	if (AllocationLevelInKB <= MinimumPoolSizeInKB)
	{
		if (bCurrentlyOverBudget)
		{
			UE_CLOGF(MinimumPoolSizeInKB != 0, LogRenderTargetPool, Display, "r.RenderTargetPoolMin resolved %d/%d MB", (AllocationLevelInKB + 1023) / 1024, MinimumPoolSizeInKB / 1024);
			bCurrentlyOverBudget = false;
		}
	}

#if STATS
	uint32 Count, SizeKB, UsedKB;
	GetStats(Count, SizeKB, UsedKB);
	CSV_CUSTOM_STAT_GLOBAL(RenderTargetPoolSize, float(SizeKB) / 1024.0f, ECsvCustomStatOp::Set);
	CSV_CUSTOM_STAT_GLOBAL(RenderTargetPoolUsed, float(UsedKB) / 1024.0f, ECsvCustomStatOp::Set);
	CSV_CUSTOM_STAT_GLOBAL(RenderTargetPoolCount, int32(Count), ECsvCustomStatOp::Set);
	SET_MEMORY_STAT(STAT_RenderTargetPoolSize, int64(SizeKB) * 1024ll);
	SET_MEMORY_STAT(STAT_RenderTargetPoolUsed, int64(UsedKB) * 1024ll);
	SET_DWORD_STAT(STAT_RenderTargetPoolCount, Count);
#endif // STATS
}

int32 FRenderTargetPool::FindIndex(IPooledRenderTarget* In) const
{
	UE::TScopeLock Lock(Mutex);

	if (In)
	{
		for (uint32 i = 0, Num = (uint32)PooledRenderTargets.Num(); i < Num; ++i)
		{
			const FPooledRenderTarget* Element = PooledRenderTargets[i];

			if (Element == In)
			{
				return i;
			}
		}
	}

	// not found
	return -1;
}

void FRenderTargetPool::FreeElementAtIndex(int32 Index)
{
	// we don't use Remove() to not shuffle around the elements for better transparency on RenderTargetPoolEvents
	PooledRenderTargets[Index] = 0;
	PooledRenderTargetHashes[Index] = 0;
}

void FRenderTargetPool::FreeUnusedResource(TRefCountPtr<IPooledRenderTarget>& In)
{
	UE::TScopeLock Lock(Mutex);

	int32 Index = FindIndex(In);

	if (Index != -1)
	{
		FPooledRenderTarget* Element = PooledRenderTargets[Index];

		// Ref count will always be at least 2
		ensure(Element->GetRefCount() >= 2);
		In = nullptr;

		if (Element->IsFree())
		{
			AllocationLevelInKB -= ComputeSizeInKB(*Element);
			TRACE_COUNTER_SUBTRACT(RenderTargetPoolCount, 1);
			TRACE_COUNTER_SET(RenderTargetPoolSize, (int64)AllocationLevelInKB * 1024);

			// we assume because of reference counting the resource gets released when not needed any more
			DeferredDeleteArray.Add(PooledRenderTargets[Index]);
			FreeElementAtIndex(Index);
		}
	}
}

void FRenderTargetPool::FreeUnusedResources()
{
	UE::TScopeLock Lock(Mutex);

	for (uint32 i = 0, Num = (uint32)PooledRenderTargets.Num(); i < Num; ++i)
	{
		FPooledRenderTarget* Element = PooledRenderTargets[i];

		if (Element && Element->IsFree())
		{
			AllocationLevelInKB -= ComputeSizeInKB(*Element);
			TRACE_COUNTER_SUBTRACT(RenderTargetPoolCount, 1);
			TRACE_COUNTER_SET(RenderTargetPoolSize, (int64)AllocationLevelInKB * 1024);

			// we assume because of reference counting the resource gets released when not needed any more
			// we don't use Remove() to not shuffle around the elements for better transparency on RenderTargetPoolEvents
			DeferredDeleteArray.Add(PooledRenderTargets[i]);
			FreeElementAtIndex(i);
		}
	}
}

struct FRenderTargetWithSize
{
	const FPooledRenderTarget* RenderTarget;
	uint32 SizeInKB;

	bool operator<(const FRenderTargetWithSize& Other) const
	{
		if (SizeInKB != Other.SizeInKB)
		{
			// Sort larger sizes first
			return SizeInKB > Other.SizeInKB;
		}
		int32 NameOrdinalCompare = FCString::Strcmp(RenderTarget->GetDesc().DebugName, Other.RenderTarget->GetDesc().DebugName);
		if (NameOrdinalCompare != 0)
		{
			return NameOrdinalCompare < 0;
		}
		return RenderTarget < Other.RenderTarget;
	}
};

void FRenderTargetPool::DumpMemoryUsage(FOutputDevice& OutputDevice)
{
	UE::TScopeLock Lock(Mutex);

	uint32 UnusedAllocationInKB = 0;

	TArray<FRenderTargetWithSize> SortedRenderTargets;
	SortedRenderTargets.Reserve(PooledRenderTargets.Num());

	for (int32 i = 0; i < PooledRenderTargets.Num(); ++i)
	{
		FPooledRenderTarget* Element = PooledRenderTargets[i];

		if (Element)
		{
			SortedRenderTargets.Add({ Element, ComputeSizeInKB(*Element) });
		}
	}

	SortedRenderTargets.Sort();

	OutputDevice.Logf(TEXT("Pooled Render Targets:"));
	for (FRenderTargetWithSize& SortedRenderTarget : SortedRenderTargets)
	{
		if (SortedRenderTarget.RenderTarget->UnusedForNFrames > 2)
		{
			UnusedAllocationInKB += SortedRenderTarget.SizeInKB;
		}

		const FPooledRenderTarget* Element = SortedRenderTarget.RenderTarget;
		OutputDevice.Logf(
			TEXT("  %6.3fMB %4dx%4d%s%s %2dmip(s) %s (%s) Unused frames: %d"),
			SortedRenderTarget.SizeInKB / 1024.0f,
			Element->Desc.Extent.X,
			Element->Desc.Extent.Y,
			Element->Desc.Depth > 1 ? *FString::Printf(TEXT("x%3d"), Element->Desc.Depth) : (Element->Desc.IsCubemap() ? TEXT("cube") : TEXT("    ")),
			Element->Desc.bIsArray ? *FString::Printf(TEXT("[%3d]"), Element->Desc.ArraySize) : TEXT("     "),
			Element->Desc.NumMips,
			Element->Desc.DebugName,
			GPixelFormats[Element->Desc.Format].Name,
			Element->UnusedForNFrames
		);
	}
	uint32 NumTargets = 0;
	uint32 UsedKB = 0;
	uint32 PoolKB = 0;
	GetStats(NumTargets, PoolKB, UsedKB);
	OutputDevice.Logf(TEXT("%.3fMB total, %.3fMB used, %.3fMB unused, %d render targets"), PoolKB / 1024.f, UsedKB / 1024.f, UnusedAllocationInKB / 1024.f, NumTargets);

	uint32 DeferredTotal = 0;
	OutputDevice.Logf(TEXT("Deferred Render Targets:"));
	for (int32 i = 0; i < DeferredDeleteArray.Num(); ++i)
	{
		FPooledRenderTarget* Element = DeferredDeleteArray[i];

		if (Element)
		{
			OutputDevice.Logf(
				TEXT("  %6.3fMB %4dx%4d%s%s %2dmip(s) %s (%s)"),
				ComputeSizeInKB(*Element) / 1024.0f,
				Element->Desc.Extent.X,
				Element->Desc.Extent.Y,
				Element->Desc.Depth > 1 ? *FString::Printf(TEXT("x%3d"), Element->Desc.Depth) : (Element->Desc.IsCubemap() ? TEXT("cube") : TEXT("    ")),
				Element->Desc.bIsArray ? *FString::Printf(TEXT("[%3d]"), Element->Desc.ArraySize) : TEXT("     "),
				Element->Desc.NumMips,
				Element->Desc.DebugName,
				GPixelFormats[Element->Desc.Format].Name
			);
			uint32 SizeInKB = ComputeSizeInKB(*Element);
			DeferredTotal += SizeInKB;
		}
	}
	OutputDevice.Logf(TEXT("%.3fMB Deferred total"), DeferredTotal / 1024.f);
}

void FRenderTargetPool::ReleaseRHI()
{
	UE::TScopeLock Lock(Mutex);
	DeferredDeleteArray.Empty();
	PooledRenderTargets.Empty();
}

// for debugging purpose
FPooledRenderTarget* FRenderTargetPool::GetElementById(uint32 Id) const
{
	// is used in game and render thread

	if (Id >= (uint32)PooledRenderTargets.Num())
	{
		return 0;
	}

	return PooledRenderTargets[Id];
}

void FRenderTargetPool::CompactPool()
{
	for (uint32 i = 0, Num = (uint32)PooledRenderTargets.Num(); i < Num; )
	{
		FPooledRenderTarget* Element = PooledRenderTargets[i];

		if (!Element)
		{
			PooledRenderTargets.RemoveAtSwap(i);
			PooledRenderTargetHashes.RemoveAtSwap(i);
			--Num;
		}
		else
		{
			++i;
		}
	}
}

bool FPooledRenderTarget::OnFrameStart()
{
	// If there are any references to the pooled render target other than the pool itself, then it may not be freed.
	if (!IsFree())
	{
		check(!UnusedForNFrames);
		return false;
	}

	++UnusedForNFrames;

	// this logic can be improved
	if (UnusedForNFrames > 10)
	{
		// release
		return true;
	}

	return false;
}

void FPooledRenderTarget::SetDebugLabelName(FRHICommandListBase& RHICmdList, const TCHAR* Name)
{
#if RHI_USE_RESOURCE_DEBUG_NAME
	// For performance, avoid updating name if it happens to be the same (true 90% of the time in testing)
	if (FCString::Strcmp(Desc.DebugName, Name))
	{
		Desc.DebugName = Name;
		RHICmdList.BindDebugLabelName(GetRHI(), Name);
	}
#else
	Desc.DebugName = Name;
#endif
}

uint32 FPooledRenderTarget::ComputeMemorySize() const
{
	uint32 Size = 0;
	if (Desc.Is2DTexture())
	{
		Size += RHIComputeMemorySize(RenderTargetItem.TargetableTexture);
		if (RenderTargetItem.ShaderResourceTexture != RenderTargetItem.TargetableTexture)
		{
			Size += RHIComputeMemorySize(RenderTargetItem.ShaderResourceTexture);
		}
	}
	else if (Desc.Is3DTexture())
	{
		Size += RHIComputeMemorySize(RenderTargetItem.TargetableTexture);
		if (RenderTargetItem.ShaderResourceTexture != RenderTargetItem.TargetableTexture)
		{
			Size += RHIComputeMemorySize(RenderTargetItem.ShaderResourceTexture);
		}
	}
	else
	{
		Size += RHIComputeMemorySize(RenderTargetItem.TargetableTexture);
		if (RenderTargetItem.ShaderResourceTexture != RenderTargetItem.TargetableTexture)
		{
			Size += RHIComputeMemorySize(RenderTargetItem.ShaderResourceTexture);
		}
	}
	return Size;
}

bool FPooledRenderTarget::IsFree() const
{
	uint32 RefCount = GetRefCount();
	check(RefCount >= 1);

	// If the only reference to the pooled render target is from the pool, then it's unused.
	return RefCount == 1;
}
