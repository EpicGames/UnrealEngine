// Copyright Epic Games, Inc. All Rights Reserved.

#include "RenderTransform.h"

FRenderTransform FRenderTransform::Identity(FVector3f(1.0f, 0.0f, 0.0f), FVector3f(0.0f, 1.0f, 0.0f), FVector3f(0.0f, 0.0f, 1.0f), FVector3f(0.0f, 0.0f, 0.0f));

FRenderBounds FRenderBounds::TransformBy(const FMatrix44f& M) const
{
	FVector3f Origin = GetCenter();
	FVector3f Extent = GetExtent();

	const VectorRegister VecOrigin = VectorLoadFloat3(&Origin);
	const VectorRegister VecExtent = VectorLoadFloat3(&Extent);

	const VectorRegister m0 = VectorLoadAligned(M.M[0]);
	const VectorRegister m1 = VectorLoadAligned(M.M[1]);
	const VectorRegister m2 = VectorLoadAligned(M.M[2]);
	const VectorRegister m3 = VectorLoadAligned(M.M[3]);

	VectorRegister NewOrigin = VectorMultiply(VectorReplicate(VecOrigin, 0), m0);
	NewOrigin = VectorMultiplyAdd(VectorReplicate(VecOrigin, 1), m1, NewOrigin);
	NewOrigin = VectorMultiplyAdd(VectorReplicate(VecOrigin, 2), m2, NewOrigin);
	NewOrigin = VectorAdd(NewOrigin, m3);

	VectorRegister NewExtent = VectorAbs(VectorMultiply(VectorReplicate(VecExtent, 0), m0));
	NewExtent = VectorAdd(NewExtent, VectorAbs(VectorMultiply(VectorReplicate(VecExtent, 1), m1)));
	NewExtent = VectorAdd(NewExtent, VectorAbs(VectorMultiply(VectorReplicate(VecExtent, 2), m2)));

	VectorStoreFloat3(NewExtent, &(Extent.X));
	VectorStoreFloat3(NewOrigin, &(Origin.X));

	FRenderBounds Result;
	Result.Min = Origin - Extent;
	Result.Max = Origin + Extent;
	return Result;
}

FRenderBounds FRenderBounds::TransformBy(const FMatrix44d& M) const
{
	FVector3f Origin = GetCenter();
	FVector3f Extent = GetExtent();

	const VectorRegister VecOrigin = VectorLoadFloat3(&Origin);
	const VectorRegister VecExtent = VectorLoadFloat3(&Extent);

	const VectorRegister4Double m0 = VectorLoadAligned(M.M[0]);
	const VectorRegister4Double m1 = VectorLoadAligned(M.M[1]);
	const VectorRegister4Double m2 = VectorLoadAligned(M.M[2]);
	const VectorRegister4Double m3 = VectorLoadAligned(M.M[3]);

	VectorRegister4Double NewOrigin = VectorMultiply(VectorReplicate(VecOrigin, 0), m0);
	NewOrigin = VectorMultiplyAdd(VectorReplicate(VecOrigin, 1), m1, NewOrigin);
	NewOrigin = VectorMultiplyAdd(VectorReplicate(VecOrigin, 2), m2, NewOrigin);
	NewOrigin = VectorAdd(NewOrigin, m3);

	VectorRegister4Double NewExtent = VectorAbs(VectorMultiply(VectorReplicate(VecExtent, 0), m0));
	NewExtent = VectorAdd(NewExtent, VectorAbs(VectorMultiply(VectorReplicate(VecExtent, 1), m1)));
	NewExtent = VectorAdd(NewExtent, VectorAbs(VectorMultiply(VectorReplicate(VecExtent, 2), m2)));

	VectorStoreFloat3(NewExtent, &(Extent.X));
	VectorStoreFloat3(NewOrigin, &(Origin.X));

	FRenderBounds Result;
	Result.Min = Origin - Extent;
	Result.Max = Origin + Extent;
	return Result;
}

FRenderBounds FRenderBounds::TransformBy(const FRenderTransform& T) const
{
	return TransformBy(T.ToMatrix44f());
}

