// Copyright Epic Games, Inc. All Rights Reserved.

#include "RenderUtils.h"

#include "Containers/DynamicRHIResourceArray.h"
#include "Containers/StaticBitArray.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/CoreMisc.h"
#include "Misc/DataDrivenPlatformInfoRegistry.h" // IWYU pragma: keep
#include "PackedNormal.h"
#include "PipelineStateCache.h"
#include "RenderResource.h"
#include "RHI.h"
#include "RHIResourceUtils.h"
#include "Shader.h"
#include "ShaderPlatformCachedIniValue.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "RenderCore.h"
#include "SubstrateDefinitions.h"
#include "Animation/MeshDeformerProvider.h"
#include "Interfaces/ITargetPlatform.h"
#include "ShaderPlatformConfig.h"

#if WITH_EDITORONLY_DATA
#include "Interfaces/ITargetPlatformManagerModule.h"
#include "RHIShaderFormatDefinitions.inl"
#endif

// This is a per-project main switch for Nanite, that influences the shader permutations compiled. Changing it will cause shaders to be recompiled.
int32 GNaniteProjectEnabled = 1;
FAutoConsoleVariableRef CVarAllowNanite(
	TEXT("r.Nanite.ProjectEnabled"),
	GNaniteProjectEnabled,
	TEXT("This setting allows you to disable Nanite on platforms that support it to reduce the number of shaders. It cannot be used to force Nanite on on unsupported platforms.\n"),
	ECVF_ReadOnly | ECVF_RenderThreadSafe
);

int32 GAllowTranslucencyShadowsInProject = 0;
FAutoConsoleVariableRef CVarAllowTranslucencyShadowsInProject(
	TEXT("r.Shadow.TranslucentPerObject.ProjectEnabled"),
	GAllowTranslucencyShadowsInProject,
	TEXT("Enable/Disable translucency shadows on a per-project basis. Turning off can significantly reduce the number of permutations if your project has many translucent materials.\n"),
	ECVF_ReadOnly | ECVF_RenderThreadSafe
);

static int32 GRayTracingEnableOnDemand = 1;
static FAutoConsoleVariableRef CVarRayTracingEnableOnDemand(
	TEXT("r.RayTracing.EnableOnDemand"),
	GRayTracingEnableOnDemand,
	TEXT("Controls whether ray tracing features can be toggled on demand at runtime without restarting the game.\n")
	TEXT("Requires r.RayTracing=1. Has a small performance and memory overhead.\n")
	TEXT(" 0: off\n")
	TEXT(" 1: on"),
	ECVF_RenderThreadSafe | ECVF_ReadOnly);

static bool GRayTracingProxiesProjectEnabled = false;
static FAutoConsoleVariableRef CVarRayTracingProxiesProjectEnabled(
	TEXT("r.RayTracing.RayTracingProxies.ProjectEnabled"),
	GRayTracingProxiesProjectEnabled,
	TEXT("Whether ray tracing proxies are generated by default for this project."),
	ECVF_ReadOnly);

// This is a per-project main switch for Virtual Shadow Maps support
// Disabling this can remove some shader permutations if virtual shadow maps are not needed
int32 GVirtualShadowMapsProjectEnabled = 1;
FAutoConsoleVariableRef CVarAllowVirtualShadowMaps(
	TEXT("r.Shadow.Virtual.ProjectEnabled"),
	GVirtualShadowMapsProjectEnabled,
	TEXT("This setting allows you to disable Virtual Shadow Maps on platforms that support it to reduce the number of shaders.\n"),
	ECVF_ReadOnly | ECVF_RenderThreadSafe
);

// This is a per-project main switch for Non-Nanite support in Virtual Shadow Maps.
// Disabling this can remove shader permutations and slightly improve performance, but Non-Nanite objects
// will not cast shadows when Virtual Shadow Maps are enabled.
int32 GVirtualShadowMapNonNaniteProjectEnabled = 1;
FAutoConsoleVariableRef CVarVirtualShadowMapNonNaniteProjectEnabled(
	TEXT("r.Shadow.Virtual.NonNanite.ProjectEnabled"),
	GVirtualShadowMapNonNaniteProjectEnabled,
	TEXT("This can be used to disable non-nanite rendering into Virtual Shadow Maps and related shader permutations.\n")
	TEXT("Only use this if you do not need any non-nanite geometry to cast shadows in VSMs."),
	ECVF_RenderThreadSafe | ECVF_ReadOnly
);

// This is a per-project main switch for supporting the classic shadow map techniques in Unreal for Non-Nanite geometry
// This includes cascaded shadow maps, (classic, non-VSM) spot light shadow maps and (classic, non-VSM) point light shadows
// Disabling this can remove shader permutations if these techniques are not needed
int32 GClassicShadowMapsProjectEnabled = 1;
FAutoConsoleVariableRef CVarClassicShadowMapsProjectEnabled(
	TEXT("r.Shadow.Classic.ProjectEnabled"),
	GClassicShadowMapsProjectEnabled,
	TEXT("This can be used to disable classic shadow maps (cascaded shadow maps, classic spot/point/rect light shadow maps).\n")
	TEXT("This will remove a significant number of shader permutations if these techniques are not needed."),
	ECVF_RenderThreadSafe | ECVF_ReadOnly
);

const uint16 GCubeIndices[12*3] =
{
	0, 2, 3,
	0, 3, 1,
	4, 5, 7,
	4, 7, 6,
	0, 1, 5,
	0, 5, 4,
	2, 6, 7,
	2, 7, 3,
	0, 4, 6,
	0, 6, 2,
	1, 3, 7,
	1, 7, 5,
};

//
// FPackedNormal serializer
//
FArchive& operator<<(FArchive& Ar, FDeprecatedSerializedPackedNormal& N)
{
	Ar << N.Vector.Packed;
	return Ar;
}

FArchive& operator<<(FArchive& Ar, FPackedNormal& N)
{
	Ar << N.Vector.Packed;
	return Ar;
}

FArchive& operator<<(FArchive& Ar, FPackedRGBA16N& N)
{
	Ar << N.X;
	Ar << N.Y;
	Ar << N.Z;
	Ar << N.W;
	return Ar;
}

/*
	3 XYZ packed in 4 bytes. (11:11:10 for X:Y:Z)
*/

/**
*	operator FVector - unpacked to -1 to 1
*/
FPackedPosition::operator FVector3f() const
{

	return FVector3f(Vector.X/1023.f, Vector.Y/1023.f, Vector.Z/511.f);
}

/**
* operator VectorRegister
*/
VectorRegister FPackedPosition::GetVectorRegister() const
{
	FVector3f UnpackedVect = *this;

	VectorRegister VectorToUnpack = VectorLoadFloat3_W0(&UnpackedVect);

	return VectorToUnpack;
}

/**
* Pack this vector(-1 to 1 for XYZ) to 4 bytes XYZ(11:11:10)
*/
void FPackedPosition::Set( const FVector3f& InVector )
{
	check (FMath::Abs<float>(InVector.X) <= 1.f && FMath::Abs<float>(InVector.Y) <= 1.f &&  FMath::Abs<float>(InVector.Z) <= 1.f);
	
#if !WITH_EDITORONLY_DATA
	// This should not happen in Console - this should happen during Cooking in PC
	check (false);
#else
	// Too confusing to use .5f - wanted to use the last bit!
	// Change to int for easier read
	Vector.X = FMath::Clamp<int32>(FMath::TruncToInt(InVector.X * 1023.0f),-1023,1023);
	Vector.Y = FMath::Clamp<int32>(FMath::TruncToInt(InVector.Y * 1023.0f),-1023,1023);
	Vector.Z = FMath::Clamp<int32>(FMath::TruncToInt(InVector.Z * 511.0f),-511,511);
#endif
}

// LWC_TODO: Perf pessimization
void FPackedPosition::Set(const FVector3d& InVector)
{
	Set(FVector3f(InVector));
}

/**
* operator << serialize
*/
FArchive& operator<<(FArchive& Ar,FPackedPosition& N)
{
	// Save N.Packed
	return Ar << N.Packed;
}

void CalcMipMapExtent3D( uint32 TextureSizeX, uint32 TextureSizeY, uint32 TextureSizeZ, EPixelFormat Format, uint32 MipIndex, uint32& OutXExtent, uint32& OutYExtent, uint32& OutZExtent )
{
	// UE-159189 to explain/fix why this forces min size of block size
	OutXExtent = FMath::Max<uint32>(TextureSizeX >> MipIndex, GPixelFormats[Format].BlockSizeX);
	OutYExtent = FMath::Max<uint32>(TextureSizeY >> MipIndex, GPixelFormats[Format].BlockSizeY);
	OutZExtent = FMath::Max<uint32>(TextureSizeZ >> MipIndex, GPixelFormats[Format].BlockSizeZ);
}

SIZE_T CalcTextureMipMapSize3D( uint32 TextureSizeX, uint32 TextureSizeY, uint32 TextureSizeZ, EPixelFormat Format, uint32 MipIndex )
{
	return GPixelFormats[Format].Get3DTextureMipSizeInBytes(TextureSizeX, TextureSizeY, TextureSizeZ, MipIndex);
}

SIZE_T CalcTextureSize3D( uint32 SizeX, uint32 SizeY, uint32 SizeZ, EPixelFormat Format, uint32 MipCount )
{
	return GPixelFormats[Format].Get3DTextureSizeInBytes(SizeX, SizeY, SizeZ, MipCount);
}

FIntPoint CalcMipMapExtent( uint32 TextureSizeX, uint32 TextureSizeY, EPixelFormat Format, uint32 MipIndex )
{
	// UE-159189 to explain/fix why this forces min size of block size
	return FIntPoint(FMath::Max<uint32>(TextureSizeX >> MipIndex, GPixelFormats[Format].BlockSizeX), FMath::Max<uint32>(TextureSizeY >> MipIndex, GPixelFormats[Format].BlockSizeY));
}

SIZE_T CalcTextureMipMapSize( uint32 TextureSizeX, uint32 TextureSizeY, EPixelFormat Format, uint32 MipIndex )
{
	return GPixelFormats[Format].Get2DTextureMipSizeInBytes(TextureSizeX, TextureSizeY, MipIndex);
}

SIZE_T CalcTextureSize( uint32 SizeX, uint32 SizeY, EPixelFormat Format, uint32 MipCount )
{
	return GPixelFormats[Format].Get2DTextureSizeInBytes(SizeX, SizeY, MipCount);
}

void CopyTextureData2D(const void* Source,void* Dest,uint32 SizeY,EPixelFormat Format,uint32 SourceStride,uint32 DestStride)
{
	const uint32 BlockSizeY = GPixelFormats[Format].BlockSizeY;
	const uint32 NumBlocksY = (SizeY + BlockSizeY - 1) / BlockSizeY;

	// a DestStride of 0 means to use the SourceStride
	if(SourceStride == DestStride || DestStride == 0)
	{
		// If the source and destination have the same stride, copy the data in one block.
		if (ensure(Source))
		{
			FMemory::ParallelMemcpy(Dest,Source,NumBlocksY * SourceStride, EMemcpyCachePolicy::StoreUncached);
		}
		else
		{
			FMemory::Memzero(Dest,NumBlocksY * SourceStride);
		}
	}
	else
	{
		// If the source and destination have different strides, copy each row of blocks separately.
		const uint32 NumBytesPerRow = FMath::Min<uint32>(SourceStride, DestStride);
		for(uint32 BlockY = 0;BlockY < NumBlocksY;++BlockY)
		{
			if (ensure(Source))
			{
				FMemory::ParallelMemcpy(
					(uint8*)Dest   + DestStride   * BlockY,
					(uint8*)Source + SourceStride * BlockY,
					NumBytesPerRow,
					EMemcpyCachePolicy::StoreUncached
					);
			}
			else
			{
				FMemory::Memzero((uint8*)Dest + DestStride * BlockY, NumBytesPerRow);
			}
		}
	}
}

EPixelFormatChannelFlags GetPixelFormatValidChannels(EPixelFormat InPixelFormat)
{
	static constexpr EPixelFormatChannelFlags PixelFormatToChannelFlags[] =
	{
		EPixelFormatChannelFlags::None,		// PF_Unknown,
		EPixelFormatChannelFlags::RGBA,		// PF_A32B32G32R32F
		EPixelFormatChannelFlags::RGBA,		// PF_B8G8R8A8
		EPixelFormatChannelFlags::R,		// PF_G8
		EPixelFormatChannelFlags::R,		// PF_G16
		EPixelFormatChannelFlags::RGB,		// PF_DXT1
		EPixelFormatChannelFlags::RGBA,		// PF_DXT3
		EPixelFormatChannelFlags::RGBA,		// PF_DXT5
		EPixelFormatChannelFlags::RG,		// PF_UYVY
		EPixelFormatChannelFlags::RGB,		// PF_FloatRGB
		EPixelFormatChannelFlags::RGBA,		// PF_FloatRGBA
		EPixelFormatChannelFlags::None,		// PF_DepthStencil
		EPixelFormatChannelFlags::None,		// PF_ShadowDepth
		EPixelFormatChannelFlags::R,		// PF_R32_FLOAT
		EPixelFormatChannelFlags::RG,		// PF_G16R16
		EPixelFormatChannelFlags::RG,		// PF_G16R16F
		EPixelFormatChannelFlags::RG,		// PF_G16R16F_FILTER
		EPixelFormatChannelFlags::RG,		// PF_G32R32F
		EPixelFormatChannelFlags::RGBA,		// PF_A2B10G10R10
		EPixelFormatChannelFlags::RGBA,		// PF_A16B16G16R16
		EPixelFormatChannelFlags::None,		// PF_D24
		EPixelFormatChannelFlags::R,		// PF_R16F
		EPixelFormatChannelFlags::R,		// PF_R16F_FILTER
		EPixelFormatChannelFlags::RG,		// PF_BC5
		EPixelFormatChannelFlags::RG,		// PF_V8U8
		EPixelFormatChannelFlags::A,		// PF_A1
		EPixelFormatChannelFlags::RGB,		// PF_FloatR11G11B10
		EPixelFormatChannelFlags::A,		// PF_A8
		EPixelFormatChannelFlags::R,		// PF_R32_UINT
		EPixelFormatChannelFlags::RGBA,		// PF_R32_SINT
		EPixelFormatChannelFlags::RGBA,		// PF_PVRTC2
		EPixelFormatChannelFlags::RGBA,		// PF_PVRTC4
		EPixelFormatChannelFlags::R,		// PF_R16_UINT
		EPixelFormatChannelFlags::R,		// PF_R16_SINT
		EPixelFormatChannelFlags::RGBA,		// PF_R16G16B16A16_UINT
		EPixelFormatChannelFlags::RGBA,		// PF_R16G16B16A16_SINT
		EPixelFormatChannelFlags::RGB,		// PF_R5G6B5_UNORM
		EPixelFormatChannelFlags::RGBA,		// PF_R8G8B8A8
		EPixelFormatChannelFlags::RGBA,		// PF_A8R8G8B8
		EPixelFormatChannelFlags::R,		// PF_BC4
		EPixelFormatChannelFlags::RG,		// PF_R8G8
		EPixelFormatChannelFlags::RGB,		// PF_ATC_RGB
		EPixelFormatChannelFlags::RGBA,		// PF_ATC_RGBA_E
		EPixelFormatChannelFlags::RGBA,		// PF_ATC_RGBA_I
		EPixelFormatChannelFlags::G,		// PF_X24_G8		
		EPixelFormatChannelFlags::RGB,		// PF_ETC1
		EPixelFormatChannelFlags::RGB,		// PF_ETC2_RGB
		EPixelFormatChannelFlags::RGBA,		// PF_ETC2_RGBA
		EPixelFormatChannelFlags::RGBA,		// PF_R32G32B32A32_UINT
		EPixelFormatChannelFlags::RG,		// PF_R16G16_UINT
		EPixelFormatChannelFlags::RGB,		// PF_ASTC_4x4
		EPixelFormatChannelFlags::RGB,		// PF_ASTC_6x6
		EPixelFormatChannelFlags::RGB,		// PF_ASTC_8x8
		EPixelFormatChannelFlags::RGB,		// PF_ASTC_10x10
		EPixelFormatChannelFlags::RGB,		// PF_ASTC_12x12
		EPixelFormatChannelFlags::RGB,		// PF_BC6H
		EPixelFormatChannelFlags::RGBA,		// PF_BC7
		EPixelFormatChannelFlags::R,		// PF_R8_UINT
		EPixelFormatChannelFlags::None,		// PF_L8
		EPixelFormatChannelFlags::RGBA,		// PF_XGXR8
		EPixelFormatChannelFlags::RGBA,		// PF_R8G8B8A8_UINT
		EPixelFormatChannelFlags::RGBA,		// PF_R8G8B8A8_SNORM
		EPixelFormatChannelFlags::RGBA,		// PF_R16G16B16A16_UNORM
		EPixelFormatChannelFlags::RGBA,		// PF_R16G16B16A16_SNORM
		EPixelFormatChannelFlags::RGBA,		// PF_PLATFORM_HDR_0
		EPixelFormatChannelFlags::RGBA,		// PF_PLATFORM_HDR_1
		EPixelFormatChannelFlags::RGBA,		// PF_PLATFORM_HDR_2
		EPixelFormatChannelFlags::None,		// PF_NV12
		EPixelFormatChannelFlags::RG,		// PF_R32G32_UINT
		EPixelFormatChannelFlags::R,		// PF_ETC2_R11_EAC
		EPixelFormatChannelFlags::RG,		// PF_ETC2_RG11_EAC
		EPixelFormatChannelFlags::R,		// PF_R8
		EPixelFormatChannelFlags::RGBA,		// PF_B5G5R5A1_UNORM
		EPixelFormatChannelFlags::RGB,		// PF_ASTC_4x4_HDR
		EPixelFormatChannelFlags::RGB,		// PF_ASTC_6x6_HDR
		EPixelFormatChannelFlags::RGB,		// PF_ASTC_8x8_HDR
		EPixelFormatChannelFlags::RGB,		// PF_ASTC_10x10_HDR
		EPixelFormatChannelFlags::RGB,		// PF_ASTC_12x12_HDR
		EPixelFormatChannelFlags::RG,		// PF_G16R16_SNORM
		EPixelFormatChannelFlags::RG,		// PF_R8G8_UINT
		EPixelFormatChannelFlags::RGB,		// PF_R32G32B32_UINT
		EPixelFormatChannelFlags::RGB,		// PF_R32G32B32_SINT
		EPixelFormatChannelFlags::RGB,		// PF_R32G32B32F
		EPixelFormatChannelFlags::R,		// PF_R8_SINT
		EPixelFormatChannelFlags::R,		// PF_R64_UINT
		EPixelFormatChannelFlags::RGB,		// PF_R9G9B9EXP5
		EPixelFormatChannelFlags::None,		// PF_P010
		EPixelFormatChannelFlags::RG,		// PF_ASTC_4x4_NORM_RG
		EPixelFormatChannelFlags::RG,		// PF_ASTC_6x6_NORM_RG
		EPixelFormatChannelFlags::RG,		// PF_ASTC_8x8_NORM_RG
		EPixelFormatChannelFlags::RG,		// PF_ASTC_10x10_NORM_RG
		EPixelFormatChannelFlags::RG,		// PF_ASTC_12x12_NORM_RG
		EPixelFormatChannelFlags::RG,		// PF_R16G16_SINT
		EPixelFormatChannelFlags::RGB,		// PF_R8G8B8
	};
	static_assert(UE_ARRAY_COUNT(PixelFormatToChannelFlags) == (uint8)PF_MAX, "Missing pixel format");
	return (InPixelFormat < PF_MAX) ? PixelFormatToChannelFlags[(uint8)InPixelFormat] : EPixelFormatChannelFlags::None;
}

const TCHAR* GetCubeFaceName(ECubeFace Face)
{
	switch(Face)
	{
	case CubeFace_PosX:
		return TEXT("PosX");
	case CubeFace_NegX:
		return TEXT("NegX");
	case CubeFace_PosY:
		return TEXT("PosY");
	case CubeFace_NegY:
		return TEXT("NegY");
	case CubeFace_PosZ:
		return TEXT("PosZ");
	case CubeFace_NegZ:
		return TEXT("NegZ");
	default:
		return TEXT("");
	}
}

ECubeFace GetCubeFaceFromName(const FString& Name)
{
	// not fast but doesn't have to be
	if(Name.EndsWith(TEXT("PosX")))
	{
		return CubeFace_PosX;
	}
	else if(Name.EndsWith(TEXT("NegX")))
	{
		return CubeFace_NegX;
	}
	else if(Name.EndsWith(TEXT("PosY")))
	{
		return CubeFace_PosY;
	}
	else if(Name.EndsWith(TEXT("NegY")))
	{
		return CubeFace_NegY;
	}
	else if(Name.EndsWith(TEXT("PosZ")))
	{
		return CubeFace_PosZ;
	}
	else if(Name.EndsWith(TEXT("NegZ")))
	{
		return CubeFace_NegZ;
	}

	return CubeFace_MAX;
}

class FVector4VertexDeclaration : public FRenderResource
{
public:
	FVertexDeclarationRHIRef VertexDeclarationRHI;
	virtual void InitRHI(FRHICommandListBase& RHICmdList) override
	{
		FVertexDeclarationElementList Elements;
		Elements.Add(FVertexElement(0, 0, VET_Float4, 0, sizeof(FVector4f)));
		VertexDeclarationRHI = PipelineStateCache::GetOrCreateVertexDeclaration(Elements);
	}
	virtual void ReleaseRHI() override
	{
		VertexDeclarationRHI.SafeRelease();
	}
};

TGlobalResource<FVector4VertexDeclaration> FVector4VertexDeclaration;

RENDERCORE_API FVertexDeclarationRHIRef& GetVertexDeclarationFVector4()
{
	return FVector4VertexDeclaration.VertexDeclarationRHI;
}

class FVector3VertexDeclaration : public FRenderResource
{
public:
	FVertexDeclarationRHIRef VertexDeclarationRHI;
	virtual void InitRHI(FRHICommandListBase& RHICmdList) override
	{
		FVertexDeclarationElementList Elements;
		Elements.Add(FVertexElement(0, 0, VET_Float3, 0, sizeof(FVector3f)));
		VertexDeclarationRHI = PipelineStateCache::GetOrCreateVertexDeclaration(Elements);
	}
	virtual void ReleaseRHI() override
	{
		VertexDeclarationRHI.SafeRelease();
	}
};

TGlobalResource<FVector3VertexDeclaration> GVector3VertexDeclaration;

RENDERCORE_API FVertexDeclarationRHIRef& GetVertexDeclarationFVector3()
{
	return GVector3VertexDeclaration.VertexDeclarationRHI;
}

class FVector2VertexDeclaration : public FRenderResource
{
public:
	FVertexDeclarationRHIRef VertexDeclarationRHI;
	virtual void InitRHI(FRHICommandListBase& RHICmdList) override
	{
		FVertexDeclarationElementList Elements;
		Elements.Add(FVertexElement(0, 0, VET_Float2, 0, sizeof(FVector2f)));
		VertexDeclarationRHI = PipelineStateCache::GetOrCreateVertexDeclaration(Elements);
	}
	virtual void ReleaseRHI() override
	{
		VertexDeclarationRHI.SafeRelease();
	}
};

TGlobalResource<FVector2VertexDeclaration> GVector2VertexDeclaration;

RENDERCORE_API FVertexDeclarationRHIRef& GetVertexDeclarationFVector2()
{
	return GVector2VertexDeclaration.VertexDeclarationRHI;
}

RENDERCORE_API bool PlatformGPUSceneUsesUniformBufferView(const FStaticShaderPlatform Platform)
{
	return IsMobilePlatform(Platform) && FDataDrivenShaderPlatformInfo::GetSupportsUniformBufferObjects(Platform);
}

RENDERCORE_API bool MobileAllowFramebufferFetch(const FStaticShaderPlatform Platform)
{
	return FReadOnlyCVARCache::MobileAllowFramebufferFetch(Platform);
}

RENDERCORE_API bool MobileRequiresSceneDepthAux(const FStaticShaderPlatform Platform)
{
	const bool bMobileHDR = IsMobileHDR();

	if (MobileAllowFramebufferFetch(Platform) && !MobileUsesFullDepthPrepass(Platform))
	{
		// SceneDepthAux is used on tiled GPUs (iOS, Android) with forward shading and on iOS only with deferred
		if (IsMetalMobilePlatform(Platform))
		{
			return true;
		}
		else if (bMobileHDR && !IsMobileDeferredShadingEnabled(Platform))
		{
			return (IsAndroidOpenGLESPlatform(Platform) || IsVulkanMobilePlatform(Platform));
		}
	}

	return false;
}

RENDERCORE_API bool SupportsTextureCubeArray(ERHIFeatureLevel::Type FeatureLevel)
{
	return FeatureLevel >= ERHIFeatureLevel::SM5 
		// requries ES3.2 feature set
		|| IsMobileDeferredShadingEnabled(GetFeatureLevelShaderPlatform(FeatureLevel))
		|| MobileForwardEnableClusteredReflections(GetFeatureLevelShaderPlatform(FeatureLevel));
}

RENDERCORE_API bool MaskedInEarlyPass(const FStaticShaderPlatform Platform)
{
	static IConsoleVariable* CVarEarlyZPassOnlyMaterialMasking = IConsoleManager::Get().FindConsoleVariable(TEXT("r.EarlyZPassOnlyMaterialMasking"));
	if (IsMobilePlatform(Platform))
	{
		return FReadOnlyCVARCache::MobileEarlyZPass(Platform) == 2 || MobileUsesFullDepthPrepass(Platform);
	}
	else
	{
		return (CVarEarlyZPassOnlyMaterialMasking && CVarEarlyZPassOnlyMaterialMasking->GetInt() != 0);
	}
}

RENDERCORE_API bool AllowPixelDepthOffset(const FStaticShaderPlatform Platform)
{
	if (IsMobilePlatform(Platform))
	{
		static const auto CVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.Mobile.AllowPixelDepthOffset"));
		return CVar->GetValueOnAnyThread() != 0;
	}
	return true;
}

RENDERCORE_API bool AllowPerPixelShadingModels(const FStaticShaderPlatform Platform)
{
	if (IsMobilePlatform(Platform))
	{
		static FShaderPlatformCachedIniValue<int32> CVarAllowPerPixelShadingModels(TEXT("r.Mobile.AllowPerPixelShadingModels"));
		return CVarAllowPerPixelShadingModels.Get(Platform) != 0;
	}
	return true;
}

RENDERCORE_API uint32 GetPlatformShadingModelsMask(const FStaticShaderPlatform Platform)
{
	if (IsMobilePlatform(Platform))
	{
		static FShaderPlatformCachedIniValue<int32> CVarMobileShadingModelsMask(TEXT("r.Mobile.ShadingModelsMask"));
		return static_cast<uint32>(CVarMobileShadingModelsMask.Get(Platform));
	}
	return 0xFFFFFFFF;
}

using ShaderPlatformMaskType = TStaticBitArray<EShaderPlatform::SP_NumPlatforms>;

// Only for a mobile ForwardShading
RENDERCORE_API bool IsMobileAmbientOcclusionEnabled(const FStaticShaderPlatform Platform)
{
	static FShaderPlatformCachedIniValue<bool> MobileAmbientOcclusionIniValue(TEXT("r.Mobile.AmbientOcclusion"));
	return IsMobilePlatform(Platform) && MobileAmbientOcclusionIniValue.Get(Platform);
}

// Only for a mobile ForwardShading
RENDERCORE_API bool AreMobileScreenSpaceReflectionsEnabled(const FStaticShaderPlatform Platform)
{
	static FShaderPlatformCachedIniValue<bool> MobileScreenSpaceReflectionsIniValue(TEXT("r.Mobile.ScreenSpaceReflections"));
	return IsMobilePlatform(Platform) && MobileScreenSpaceReflectionsIniValue.Get(Platform);
}

RENDERCORE_API bool IsMobileDistanceFieldEnabled(const FStaticShaderPlatform Platform)
{
	return IsMobilePlatform(Platform) && FDataDrivenShaderPlatformInfo::GetSupportsDistanceFields(Platform) && IsUsingDistanceFields(Platform);
}

RENDERCORE_API bool IsMobileMovableSpotlightShadowsEnabled(const FStaticShaderPlatform Platform)
{
	return FReadOnlyCVARCache::MobileEnableMovableSpotlightsShadow(Platform);
}

RENDERCORE_API bool IsMobileCapsuleShadowsEnabled(const FStaticShaderPlatform Platform)
{
	static FShaderPlatformCachedIniValue<bool> MobileCapsuleShadowsIniValue(TEXT("r.Mobile.EnableCapsuleShadows"));
	return IsMobilePlatform(Platform) && MobileCapsuleShadowsIniValue.Get(Platform);
}

RENDERCORE_API bool IsMobileCapsuleDirectShadowsEnabled(const FStaticShaderPlatform Platform)
{
	static FShaderPlatformCachedIniValue<bool> MobileCapsuleDirectShadowsIniValue(TEXT("r.Mobile.EnableCapsuleDirectShadows"));
	return IsMobilePlatform(Platform) && IsMobileCapsuleShadowsEnabled(Platform) && MobileCapsuleDirectShadowsIniValue.Get(Platform);
}

RENDERCORE_API bool MobileForwardEnableClusteredReflections(const FStaticShaderPlatform Platform)
{
	static FShaderPlatformCachedIniValue<bool> MobileForwardEnableClusteredReflectionsIniValue(TEXT("r.Mobile.Forward.EnableClusteredReflections"));
	return MobileForwardEnableClusteredReflectionsIniValue.Get(Platform);
}

RENDERCORE_API bool MobileUsesShadowMaskTexture(const FStaticShaderPlatform Platform)
{
	const bool bMoveablePointOrSpotLightShadowsEnabledForward = !IsMobileDeferredShadingEnabled(Platform) &&
		(IsMobileMovableSpotlightShadowsEnabled(Platform) || FReadOnlyCVARCache::EnablePointLightShadows(Platform)) && MobileForwardEnableLocalLights(Platform);
	const bool bMobileCapsuleShadowsEnabled = IsMobileCapsuleDirectShadowsEnabled(Platform);
	const bool bMobileDistanceFieldEnabled = IsMobileDistanceFieldEnabled(Platform);
	const bool bForcedDepthPrepass = FReadOnlyCVARCache::MobileEarlyZPass(Platform) == 1; 
	// In case any of these feature are used force shadowmask path
	return (bForcedDepthPrepass || bMobileDistanceFieldEnabled || bMobileCapsuleShadowsEnabled || bMoveablePointOrSpotLightShadowsEnabledForward);
}

// Whether to support more than 4 color attachments for GBuffer 
RENDERCORE_API bool MobileUsesExtendedGBuffer(FStaticShaderPlatform ShaderPlatform)
{
	// For a single pass deferred shading (r.Mobile.AllowFramebufferFetch = 1)
	// Android GLES: uses PLS for deferred shading and limited to 128 bits
	// Vulkan requires:
		// maxDescriptorSetInputAttachments > 4
		// maxColorAttachments > 4
	// iOS: A8+
	static FShaderPlatformCachedIniValue<int32> MobileExtendedGBufferIniValue(TEXT("r.Mobile.ExtendedGBuffer"));
	return MobileExtendedGBufferIniValue.Get(ShaderPlatform) != 0;
}

// Required for shading models with a custom data
RENDERCORE_API bool MobileUsesGBufferCustomData(FStaticShaderPlatform ShaderPlatform)
{
	// we can pack CustomData into static lighting related space
	return MobileUsesExtendedGBuffer(ShaderPlatform) || !IsStaticLightingAllowed();
}

RENDERCORE_API bool MobileBasePassAlwaysUsesCSM(const FStaticShaderPlatform Platform)
{
	static TConsoleVariableData<int32>* CVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.Mobile.Shadow.CSMShaderCullingMethod"));
	if (IsMobileDeferredShadingEnabled(Platform))
	{
		// deferred shading does not need CSM culling
		return true;
	}
	else
	{
		return CVar && (CVar->GetValueOnAnyThread() & 0xF) == 5 && IsMobileDistanceFieldEnabled(Platform);
	}
}

RENDERCORE_API bool MobileUsesFullDepthPrepass(const FStaticShaderPlatform Platform)
{
	const bool bForcedDepthPrepass = (FReadOnlyCVARCache::MobileEarlyZPass(Platform) == 1 || IsUsingDBuffers(Platform) || IsStencilForLODDitherEnabled(Platform));
	if (!bForcedDepthPrepass) 
	{
		// Forward or single-pass deferred
		if (!IsMobileDeferredShadingEnabled(Platform) || MobileAllowFramebufferFetch(Platform))
		{
			// Features that require full depth before lighting
			return MobileUsesShadowMaskTexture(Platform) || IsMobileAmbientOcclusionEnabled(Platform);
		}
	}
	return bForcedDepthPrepass;
}

RENDERCORE_API bool ShouldForceFullDepthPass(const FStaticShaderPlatform Platform)
{
	if (IsMobilePlatform(Platform))
	{
		return MobileUsesFullDepthPrepass(Platform);
	}
	else
	{
		const bool bNaniteEnabled = UseNanite(Platform);
		const bool bDBufferAllowed = IsUsingDBuffers(Platform);
		const bool bVirtualTextureEnabled = UseVirtualTexturing(Platform);
		const bool bStencilLODDither = IsStencilForLODDitherEnabled(Platform);

		static const auto AOComputeCVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.AmbientOcclusion.Compute"));
		const bool bAOCompute = AOComputeCVar->GetValueOnAnyThread() > 0;

		const bool bEarlyZMaterialMasking = MaskedInEarlyPass(Platform);

		// Note: ShouldForceFullDepthPass affects which static draw lists meshes go into, so nothing it depends on can change at runtime, unless you do a FGlobalComponentRecreateRenderStateContext to propagate the cvar change
		return bNaniteEnabled || bAOCompute || bDBufferAllowed || bVirtualTextureEnabled || bStencilLODDither || bEarlyZMaterialMasking || IsForwardShadingEnabled(Platform) || IsUsingSelectiveBasePassOutputs(Platform);
	}
}

RENDERCORE_API bool SupportsGen4TAA(const FStaticShaderPlatform Platform)
{
	if (IsMobilePlatform(Platform))
	{
		static FShaderPlatformCachedIniValue<bool> MobileSupportsGen4TAAIniValue(TEXT("r.Mobile.SupportsGen4TAA"));
		return (MobileSupportsGen4TAAIniValue.Get(Platform) != 0);
	}

	return true;
}

RENDERCORE_API bool SupportsTSR(const FStaticShaderPlatform Platform)
{
	return FDataDrivenShaderPlatformInfo::GetSupportsGen5TemporalAA(Platform);
}

EShaderPlatform GetEditorShaderPlatform(EShaderPlatform ShaderPlatform)
{
	EShaderPlatform ActualShaderPlatform = ShaderPlatform;

#if WITH_EDITOR
	if (GIsEditor)
	{
		if (FDataDrivenShaderPlatformInfo::GetIsPreviewPlatform(ActualShaderPlatform))
		{
			ActualShaderPlatform = FDataDrivenShaderPlatformInfo::GetPreviewShaderPlatformParent(ActualShaderPlatform);
		}
	}
#endif // WITH_EDITOR

	return ActualShaderPlatform;
}

RENDERCORE_API int32 GUseForwardShading = 0;
static FAutoConsoleVariableRef CVarForwardShading(
	TEXT("r.ForwardShading"),
	GUseForwardShading,
	TEXT("Whether to use forward shading on desktop platforms - requires Shader Model 5 hardware.\n")
	TEXT("Forward shading has lower constant cost, but fewer features supported. 0:off, 1:on\n")
	TEXT("This rendering path is a work in progress with many unimplemented features, notably only a single reflection capture is applied per object and no translucency dynamic shadow receiving."),
	ECVF_RenderThreadSafe | ECVF_ReadOnly
	); 

static TAutoConsoleVariable<int32> CVarGBufferDiffuseSampleOcclusion(
	TEXT("r.GBufferDiffuseSampleOcclusion"), 0,
	TEXT("Whether the gbuffer contain occlusion information for individual diffuse samples."),
	ECVF_RenderThreadSafe | ECVF_ReadOnly
	); 

static TAutoConsoleVariable<int32> CVarDistanceFields(
	TEXT("r.DistanceFields"),
	1,
	TEXT("Enables distance fields rendering.\n") \
	TEXT(" 0: Disabled.\n") \
	TEXT(" 1: Enabled."),
	ECVF_RenderThreadSafe | ECVF_ReadOnly
	); 


RENDERCORE_API ShaderPlatformMaskType GForwardShadingPlatformMask;
RENDERCORE_API ShaderPlatformMaskType GDBufferPlatformMask;
RENDERCORE_API ShaderPlatformMaskType GVelocityEncodeDepthPlatformMask;
RENDERCORE_API ShaderPlatformMaskType GSelectiveBasePassOutputsPlatformMask;
RENDERCORE_API ShaderPlatformMaskType GNaniteFallbackMeshesPlatformMask;
RENDERCORE_API ShaderPlatformMaskType GDistanceFieldsPlatformMask;
RENDERCORE_API ShaderPlatformMaskType GSimpleSkyDiffusePlatformMask;

// Specifies whether ray tracing *can* be enabled on a particular platform.
// This takes into account whether RT is globally enabled for the project and specifically enabled on a target platform.
// Safe to use to make cook-time decisions, such as whether to compile ray tracing shaders.
RENDERCORE_API ShaderPlatformMaskType GRayTracingPlatformMask;

// Specifies whether ray tracing shaders *can* be used on a particular platform.
// This takes into account whether RT is globally enabled for the project and specifically enabled on a target platform.
RENDERCORE_API ShaderPlatformMaskType GRayTracingShadersPlatformMask;

// Specifies whether ray tracing *is* enabled on the current running system (in current game or editor process).
// This takes into account additional factors, such as concrete current GPU/OS/Driver capability, user-set game graphics options, etc.
// Only safe to make run-time decisions, such as whether to build acceleration structures and render ray tracing effects.
// Value may be queried using IsRayTracingEnabled().
RENDERCORE_API ERayTracingMode GRayTracingMode = ERayTracingMode::Disabled;

#if WITH_EDITOR
void GetAllPossiblePreviewPlatformsForMainShaderFormat(TArray<EShaderPlatform>& OutPreviewPlatforms, FName MainShaderFormat)
{
	for (int i = 0; i < EShaderPlatform::SP_NumPlatforms; ++i)
	{
		EShaderPlatform ShaderPlatform = EShaderPlatform(i);
		if (FDataDrivenShaderPlatformInfo::IsValid(ShaderPlatform))
		{
			bool bIsPreviewPlatform = FDataDrivenShaderPlatformInfo::GetIsPreviewPlatform(ShaderPlatform);
			if (bIsPreviewPlatform)
			{
				EShaderPlatform ParentShaderPlatform = FDataDrivenShaderPlatformInfo::GetPreviewShaderPlatformParent(ShaderPlatform);
				if (FDataDrivenShaderPlatformInfo::IsValid(ParentShaderPlatform))
				{
					bool bIsSameShaderFormat = FDataDrivenShaderPlatformInfo::GetShaderFormat(ParentShaderPlatform) == MainShaderFormat;
					if (bIsSameShaderFormat)
					{
						OutPreviewPlatforms.Add(ShaderPlatform);
					}
				}
			}
		}
	}
}
#endif // WITH_EDITOR

RENDERCORE_API void RenderUtilsInit()
{
	checkf(GIsRHIInitialized, TEXT("RenderUtilsInit() may only be called once RHI is initialized."));

	FReadOnlyCVARCache::Initialize();
	FShaderPlatformConfig::Initialize();

	GForwardShadingPlatformMask.Fill(GUseForwardShading == 1);

	static IConsoleVariable* DBufferVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.DBuffer"));
	GDBufferPlatformMask.Fill(DBufferVar && DBufferVar->GetInt());

	static IConsoleVariable* SelectiveBasePassOutputsCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.SelectiveBasePassOutputs"));
	GSelectiveBasePassOutputsPlatformMask.Fill(SelectiveBasePassOutputsCVar && SelectiveBasePassOutputsCVar->GetInt());

	static IConsoleVariable* DistanceFieldsCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.DistanceFields")); 
	GDistanceFieldsPlatformMask.Fill(DistanceFieldsCVar && DistanceFieldsCVar->GetInt());

	static IConsoleVariable* RayTracingCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.RayTracing"));

	GSimpleSkyDiffusePlatformMask.Fill(false);
	GVelocityEncodeDepthPlatformMask.Fill(false);
	GNaniteFallbackMeshesPlatformMask.Fill(true);
	GRayTracingPlatformMask.Fill(false);
	GRayTracingShadersPlatformMask.Fill(false);

#if WITH_EDITOR
	ITargetPlatformManagerModule* TargetPlatformManager = GetTargetPlatformManager();
	if (TargetPlatformManager)
	{
		for (ITargetPlatformSettings* TargetPlatformSettings : TargetPlatformManager->GetTargetPlatformSettings())
		{
			TArray<FName> PlatformPossibleShaderFormats;
			TargetPlatformSettings->GetAllPossibleShaderFormats(PlatformPossibleShaderFormats);

			for (FName Format : PlatformPossibleShaderFormats)
			{
				EShaderPlatform ShaderPlatform = ShaderFormatNameToShaderPlatform(Format);
				TArray<EShaderPlatform> PossiblePreviewPlatformsAndMainPlatform;
				GetAllPossiblePreviewPlatformsForMainShaderFormat(PossiblePreviewPlatformsAndMainPlatform, Format);
				PossiblePreviewPlatformsAndMainPlatform.Add(ShaderPlatform);

				for (EShaderPlatform ShaderPlatformToEdit : PossiblePreviewPlatformsAndMainPlatform)
				{
					uint32 ShaderPlatformIndex = static_cast<uint32>(ShaderPlatformToEdit);

					uint64 Mask = 1ull << ShaderPlatformToEdit;

					if (!FDataDrivenShaderPlatformInfo::IsValid(ShaderPlatformToEdit))
					{
						continue;
					}

					GForwardShadingPlatformMask[ShaderPlatformIndex] = TargetPlatformSettings->UsesForwardShading();

					GDBufferPlatformMask[ShaderPlatformIndex] = IsMobilePlatform(ShaderPlatformToEdit) ? TargetPlatformSettings->UsesMobileDBuffer() : TargetPlatformSettings->UsesDBuffer();

					GSelectiveBasePassOutputsPlatformMask[ShaderPlatformIndex] = TargetPlatformSettings->UsesSelectiveBasePassOutputs();

					GNaniteFallbackMeshesPlatformMask[ShaderPlatformIndex] = !TargetPlatformSettings->ShouldStripNaniteFallbackMeshes();

					GDistanceFieldsPlatformMask[ShaderPlatformIndex] = TargetPlatformSettings->UsesDistanceFields();

					GSimpleSkyDiffusePlatformMask[ShaderPlatformIndex] = TargetPlatformSettings->ForcesSimpleSkyDiffuse();

					GVelocityEncodeDepthPlatformMask[ShaderPlatformIndex] = TargetPlatformSettings->VelocityEncodeDepth();
				}
			}
			
			if (TargetPlatformSettings->UsesRayTracing())
			{
				TArray<FName> PlatformRayTracingShaderFormats;
				TargetPlatformSettings->GetRayTracingShaderFormats(PlatformRayTracingShaderFormats);

				const bool bRayTracingShadersEnabled = TargetPlatformSettings->GetRayTracingMode() == ERayTracingRuntimeMode::Full;

				for (FName FormatName : PlatformRayTracingShaderFormats)
				{
					EShaderPlatform MainShaderPlatform = ShaderFormatNameToShaderPlatform(FormatName);
					TArray<EShaderPlatform> PossiblePreviewPlatformsAndMainPlatform;
					GetAllPossiblePreviewPlatformsForMainShaderFormat(PossiblePreviewPlatformsAndMainPlatform, FormatName);

					PossiblePreviewPlatformsAndMainPlatform.Add(MainShaderPlatform);

					for (EShaderPlatform ShaderPlatform : PossiblePreviewPlatformsAndMainPlatform)
					{
						uint32 ShaderPlatformIndex = static_cast<uint32>(ShaderPlatform);
						GRayTracingPlatformMask[ShaderPlatformIndex] = true;
						GRayTracingShadersPlatformMask[ShaderPlatformIndex] = bRayTracingShadersEnabled;
					}
				}
			}
		}
	}

#else // WITH_EDITOR
	if (IsMobilePlatform(GMaxRHIShaderPlatform))
	{
		static IConsoleVariable* MobileDBufferCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Mobile.DBuffer"));
		GDBufferPlatformMask.Fill(MobileDBufferCVar && MobileDBufferCVar->GetInt());
	}

	if (RayTracingCVar && RayTracingCVar->GetInt() && GRHISupportsRayTracing)
	{
		GRayTracingPlatformMask.Fill(true);

		if (GRHISupportsRayTracingShaders)
		{
			GRayTracingShadersPlatformMask.Fill(true);
		}
	}

	// Load runtime values from and *.ini file used by a current platform
	// Should be code shared between cook and game, but unfortunately can't be done before we untangle non data driven platforms
	const FString PlatformName(FPlatformProperties::IniPlatformName());
	const FDataDrivenPlatformInfo& PlatformInfo = FDataDrivenPlatformInfoRegistry::GetPlatformInfo(PlatformName);

	const FString CategoryName = PlatformInfo.TargetSettingsIniSectionName;
	if (!CategoryName.IsEmpty())
	{
		FConfigFile PlatformIniFile;
		if (FConfigCacheIni::LoadLocalIniFile(PlatformIniFile, TEXT("Engine"), /*bIsBaseIniName*/ true, *PlatformName))
		{
			bool bNaniteFallbackMeshes = false;
			if (PlatformIniFile.GetBool(*CategoryName, TEXT("bGenerateNaniteFallbackMeshes"), bNaniteFallbackMeshes))
			{
				GNaniteFallbackMeshesPlatformMask.Fill(bNaniteFallbackMeshes);
			}

			bool bDistanceFields = false;
			if (PlatformIniFile.GetBool(*CategoryName, TEXT("bEnableDistanceFields"), bDistanceFields) && !bDistanceFields)
			{
				GDistanceFieldsPlatformMask.Fill(false);
			}

			{
				FString RayTracingModeString;
				GConfig->GetString(FPlatformProperties::GetRuntimeSettingsClassName(), TEXT("RayTracingMode"), RayTracingModeString, GEngineIni);

				if (RayTracingModeString.Equals(TEXT("Inline"), ESearchCase::IgnoreCase))
				{
					GRayTracingShadersPlatformMask.Fill(false);
				}
				else if (RayTracingModeString.Equals(TEXT("Full"), ESearchCase::IgnoreCase))
				{
					// nothing
				}
				else if (RayTracingModeString.Equals(TEXT("Disabled"), ESearchCase::IgnoreCase))
				{
					GRayTracingShadersPlatformMask.Fill(false);
					GRayTracingPlatformMask.Fill(false);
				}
				else
				{
					bool bRayTracing = false;
					if (PlatformIniFile.GetBool(*CategoryName, TEXT("bEnableRayTracing"), bRayTracing) && !bRayTracing)
					{
						GRayTracingPlatformMask.Fill(false);
					}
				}
			}
		}
	}
#endif // WITH_EDITOR

	// Run-time ray tracing support depends on the following factors:
	// - Ray tracing must be enabled for the project
	// - Skin cache must be enabled for the project
	// - Current GPU, OS and driver must support ray tracing
	// - r.RayTracing.Enable = 1

	// When ray tracing is enabled, we must load additional shaders and build acceleration structures for meshes.
	// For this reason it is only possible to enable RT at startup and changing the state requires restart.
	// This is also the reason why IsRayTracingEnabled() lives in RenderCore module, as it controls creation of 
	// RT pipelines in ShaderPipelineCache.cpp.

	// There are 2 modes in which ray tracing can be enabled:
	// 1 - Everything necessary is built and available if platform supports ray tracing
	// 2 - Same as above but ray tracing can be toggled on/off at runtime

	auto GetRayTracingModeName = [](ERayTracingMode Mode)
	{
		switch (Mode)
		{
		case ERayTracingMode::Enabled:
			return TEXT("enabled");
		case ERayTracingMode::Dynamic:
			return TEXT("enabled (dynamic)");
		}
		return TEXT("disabled");
	};

	if (RayTracingCVar && RayTracingCVar->GetInt())
	{
		const int32 RayTracingInt = RayTracingCVar->GetInt();

		ERayTracingMode DesiredRayTracingMode = ERayTracingMode::Disabled;
		if (RayTracingInt != 0)
		{
			if (GRayTracingEnableOnDemand == 1)
			{
				DesiredRayTracingMode = ERayTracingMode::Dynamic;
			}
			else
			{
				// if not using EnableOnDemand, check r.Raytracing.Enable during initialization (changing the cvar at runtime will have no effect)
				static const auto RayTracingEnableCVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.Raytracing.Enable"));

				if (RayTracingEnableCVar && RayTracingEnableCVar->GetValueOnAnyThread() != 0)
				{
					DesiredRayTracingMode = ERayTracingMode::Enabled;
				}
			}
		}

		const bool bRayTracingAllowedOnCurrentPlatform = (GRayTracingPlatformMask[(int)GMaxRHIShaderPlatform]);
		if (GRHISupportsRayTracing && bRayTracingAllowedOnCurrentPlatform)
		{
			GRayTracingMode = DesiredRayTracingMode;

			UE_LOGF(LogRendererCore, Log, "Ray tracing is %ls. Reason: r.RayTracing=%d and r.RayTracing.EnableOnDemand=%d.",
				GetRayTracingModeName(GRayTracingMode),
				RayTracingInt,
				GRayTracingEnableOnDemand);

			// Sanity check: skin cache is *required* for ray tracing.
			// It can be dynamically enabled only when its shaders have been compiled.
			IConsoleVariable* SkinCacheCompileShadersCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.SkinCache.CompileShaders"));
			const bool bUseRayTracing = (int32)GRayTracingMode >= (int32)ERayTracingMode::Enabled;
			if (bUseRayTracing && SkinCacheCompileShadersCVar->GetInt() <= 0)
			{
				GRayTracingMode = ERayTracingMode::Disabled;

				UE_LOGF(LogRendererCore, Fatal, "Ray tracing requires skin cache to be enabled. Set r.SkinCache.CompileShaders>=1.");
			}

			const bool bRayTracingShadersAllowedOnCurrentPlatform = GRayTracingShadersPlatformMask[(int)GMaxRHIShaderPlatform];
			if (GRHISupportsRayTracingShaders && bRayTracingShadersAllowedOnCurrentPlatform)
			{
				UE_LOGF(LogRendererCore, Log, "Ray tracing shaders are enabled.");
			}
			else
			{
				UE_LOGF(LogRendererCore, Log, "Ray tracing shaders are disabled.");
			}
		}
		else
		{
			if (!GRHISupportsRayTracing)
			{
				UE_LOGF(LogRendererCore, Log, "Ray tracing is disabled. Reason: not supported by current RHI.");
			}
			else
			{
				UE_LOGF(LogRendererCore, Log, "Ray tracing is disabled. Reason: disabled on current platform.");
			}
		}
	}
	else
	{
		UE_LOGF(LogRendererCore, Log, "Ray tracing is disabled. Reason: disabled through project setting (r.RayTracing=0).");
	}

	if (!FReadOnlyCVARCache::MobileAllowFramebufferFetch(GMaxRHIShaderPlatform))
	{
		GSupportsShaderFramebufferFetch = false;
		GSupportsShaderMRTFramebufferFetch = false;
		GSupportsShaderDepthStencilFetch = false;
		GSupportsPixelLocalStorage = false;
		GSupportsShaderFramebufferFetchProgrammableBlending = false;
	}
}

class FUnitCubeVertexBuffer : public FVertexBuffer
{
public:
	/**
	* Initialize the RHI for this rendering resource
	*/
	void InitRHI(FRHICommandListBase& RHICmdList) override
	{
		const int32 NumVerts = 8;
		TArray<FVector4f> Verts;
		Verts.SetNumUninitialized(NumVerts);

		for (uint32 Z = 0; Z < 2; Z++)
		{
			for (uint32 Y = 0; Y < 2; Y++)
			{
				for (uint32 X = 0; X < 2; X++)
				{
					const FVector4f Vertex = FVector4f(
					  (X ? -1.f : 1.f),
					  (Y ? -1.f : 1.f),
					  (Z ? -1.f : 1.f),
					  1.f
					);

					Verts[GetCubeVertexIndex(X, Y, Z)] = Vertex;
				}
			}
		}

		// Create vertex buffer. Fill buffer with initial data upon creation
		VertexBufferRHI = UE::RHIResourceUtils::CreateVertexBufferFromArray(RHICmdList, TEXT("FUnitCubeVertexBuffer"), EBufferUsageFlags::Static, MakeConstArrayView(Verts));
	}
};

class FUnitCubeIndexBuffer : public FIndexBuffer
{
public:
	/**
	* Initialize the RHI for this rendering resource
	*/
	void InitRHI(FRHICommandListBase& RHICmdList) override
	{
		// Create index buffer. Fill buffer with initial data upon creation
		IndexBufferRHI = UE::RHIResourceUtils::CreateIndexBufferFromArray(RHICmdList, TEXT("FUnitCubeIndexBuffer"), EBufferUsageFlags::Static, MakeConstArrayView(GCubeIndices));
	}
};

#if RHI_RAYTRACING

class FUnitCubeAABBVertexBuffer : public FVertexBuffer
{
public:
	/**
	* Initialize the RHI for this rendering resource
	*/
	void InitRHI(FRHICommandListBase& RHICmdList) override
	{
		const FVector3f Vertices[] =
		{
			FVector3f(-0.5f, -0.5f, -0.5f),
			FVector3f( 0.5f,  0.5f,  0.5f),
		};

		// Create vertex buffer. Fill buffer with initial data upon creation
		VertexBufferRHI = UE::RHIResourceUtils::CreateVertexBufferFromArray(RHICmdList, TEXT("FUnitCubeAABBVertexBuffer"), EBufferUsageFlags::Static, MakeConstArrayView(Vertices));
	}
};
#endif // RHI_RAYTRACING

static TGlobalResource<FUnitCubeVertexBuffer> GUnitCubeVertexBuffer;
static TGlobalResource<FUnitCubeIndexBuffer> GUnitCubeIndexBuffer;
#if RHI_RAYTRACING
static TGlobalResource<FUnitCubeAABBVertexBuffer> GUnitCubeAABBVertexBuffer;
#endif // RHI_RAYTRACING

RENDERCORE_API FBufferRHIRef& GetUnitCubeVertexBuffer()
{
	return GUnitCubeVertexBuffer.VertexBufferRHI;
}

RENDERCORE_API FBufferRHIRef& GetUnitCubeIndexBuffer()
{
	return GUnitCubeIndexBuffer.IndexBufferRHI;
}

#if RHI_RAYTRACING
RENDERCORE_API FBufferRHIRef& GetUnitCubeAABBVertexBuffer()
{
	return GUnitCubeAABBVertexBuffer.VertexBufferRHI;
}
#endif // RHI_RAYTRACING

RENDERCORE_API void QuantizeSceneBufferSize(const FIntPoint& InBufferSize, FIntPoint& OutBufferSize, const uint32 SuggestedDivisor)
{
	// A lot of rendering features assume that buffer size is dividable by 8
	const uint32 RenderingDividableBy = 8;

	static_assert(RenderingDividableBy % SUBSTRATE_TILE_SIZE == 0, "Ensure sizes are dividable by SUBSTRATE_TILE_SIZE");

	const uint32 DividableBy = FMath::Max(RenderingDividableBy, SuggestedDivisor);

	checkf(DividableBy % RenderingDividableBy == 0, TEXT("A lot of rendering features assume that buffer size is dividable by %u"), RenderingDividableBy);

	const uint32 Mask = ~(DividableBy - 1);
	OutBufferSize.X = (InBufferSize.X + DividableBy - 1) & Mask;
	OutBufferSize.Y = (InBufferSize.Y + DividableBy - 1) & Mask;
}

RENDERCORE_API bool UseVirtualTexturing(const FStaticShaderPlatform InShaderPlatform)
{
	// Use FTargetPlatformCachedIniValue because the setting affects texture builds and must match across the target's shader platforms.
	static FTargetPlatformCachedIniValue<int32> PerTargetCVar(TEXT("r.VirtualTextures"));
	return PerTargetCVar.Get(InShaderPlatform) != 0;
}

RENDERCORE_API bool UseVirtualTexturing(const ITargetPlatformSettings* InTargetPlatform)
{
	if (InTargetPlatform != nullptr)
	{	
		static FTargetPlatformCachedIniValue<int32> PerTargetCVar(TEXT("r.VirtualTextures"));
		return PerTargetCVar.Get(FName(InTargetPlatform->IniPlatformName())) != 0;
	}
	return UseVirtualTexturing(GMaxRHIShaderPlatform);
}

RENDERCORE_API bool UseVirtualTexturing(const FStaticShaderPlatform InShaderPlatform, const ITargetPlatformSettings* InTargetPlatform)
{
	return InTargetPlatform != nullptr ? UseVirtualTexturing(InTargetPlatform) : UseVirtualTexturing(InShaderPlatform);
}

RENDERCORE_API bool UseVirtualTextureLightmap(const FStaticShaderPlatform InShaderPlatform, const ITargetPlatformSettings* InTargetPlatform)
{
	static FTargetPlatformCachedIniValue<int32> PerTargetCVar(TEXT("r.VirtualTexturedLightmaps"));

	if (InTargetPlatform != nullptr)
	{
		return UseVirtualTexturing(InTargetPlatform) && PerTargetCVar.Get(FName(InTargetPlatform->IniPlatformName())) != 0;
	}

	return UseVirtualTexturing(InShaderPlatform) && PerTargetCVar.Get(InShaderPlatform) != 0;
}

RENDERCORE_API bool UseNaniteLandscapeMesh(EShaderPlatform ShaderPlatform)
{
	return DoesPlatformSupportNanite(ShaderPlatform);
}

RENDERCORE_API bool UseShaderPipelines(EShaderPlatform ShaderPlatform)
{
	static FShaderPlatformCachedIniValue<int32> PerPlatformCVar(TEXT("r.ShaderPipelines"));
	bool bUseShaderPipelines = PerPlatformCVar.Get(ShaderPlatform) != 0;
	return bUseShaderPipelines;
}

RENDERCORE_API bool UseRemoveUnsedInterpolators(EShaderPlatform ShaderPlatform)
{
	static FShaderPlatformCachedIniValue<int32> PerPlatformCVar(TEXT("r.Shaders.RemoveUnusedInterpolators"));
	bool bRemoveUnusedInterpolators = PerPlatformCVar.Get(ShaderPlatform) != 0;
	return bRemoveUnusedInterpolators;
}

RENDERCORE_API bool ExcludeNonPipelinedShaderTypes(EShaderPlatform ShaderPlatform)
{
	if (RHISupportsShaderPipelines(ShaderPlatform))
	{
		bool bShaderPipelinesAreEnabled = UseShaderPipelines(ShaderPlatform);
		if (bShaderPipelinesAreEnabled)
		{
			static FShaderPlatformCachedIniValue<int32> PerPlatformCVar(TEXT("r.Material.ExcludeNonPipelinedShaders"));
			bool bExcludeNonPipelinedShaders = PerPlatformCVar.Get(ShaderPlatform) != 0;
			return bExcludeNonPipelinedShaders;
		}
	}

	return false;
}

RENDERCORE_API bool PlatformSupportsVelocityRendering(const FStaticShaderPlatform Platform)
{
	if (IsMobilePlatform(Platform))
	{
		// Enable velocity rendering if desktop Gen4 TAA is supported on mobile or we're performing direct OpenXR motion vector rendering for frame synthesis.
		return SupportsGen4TAA(Platform) || PlatformSupportsOpenXRMotionVectors(Platform);
	}

	return true;
}

bool DoesPlatformSupportNanite(EShaderPlatform Platform, bool bCheckForProjectSetting)
{
	// Nanite allowed for this project
	static FShaderPlatformCachedIniValue<int32> CVarPlatformNaniteProjectEnabled(TEXT("r.Nanite.ProjectEnabled"));
	bool bIsNaniteProjectEnable = CVarPlatformNaniteProjectEnabled.Get(Platform) != 0;

	if (UNLIKELY(!bIsNaniteProjectEnable))
	{
		return false;
	}
	
	// Make sure the current platform has DDPI definitions.
	const bool bValidPlatform = FDataDrivenShaderPlatformInfo::IsValid(Platform);

	// GPUScene is required for Nanite
	const bool bSupportGPUScene = FDataDrivenShaderPlatformInfo::GetSupportsGPUScene(Platform);

	// Nanite specific check
	const bool bSupportNanite = FDataDrivenShaderPlatformInfo::GetSupportsNanite(Platform);

	const bool bFullCheck = bValidPlatform && bSupportGPUScene && bSupportNanite;
	return bFullCheck;
}

bool NaniteAtomicsSupported()
{
	// Are 64bit image atomics supported by the GPU/Driver/OS/API?
	bool bAtomicsSupported = GRHISupportsAtomicUInt64;

#if PLATFORM_WINDOWS
	const ERHIInterfaceType RHIInterface = RHIGetInterfaceType();
	const bool bIsDx11 = RHIInterface == ERHIInterfaceType::D3D11;
	const bool bIsDx12 = RHIInterface == ERHIInterfaceType::D3D12;

	static const auto NaniteRequireDX12CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Nanite.RequireDX12"));
	static const uint32 NaniteRequireDX12 = (NaniteRequireDX12CVar != nullptr) ? NaniteRequireDX12CVar->GetInt() : 1;

	if (bAtomicsSupported && NaniteRequireDX12 != 0)
	{
		// Only allow Vulkan or D3D12
		bAtomicsSupported = !bIsDx11;

		// Disable DX12 vendor extensions unless DX12 SM6.6 is supported
		if (NaniteRequireDX12 == 1 && bIsDx12 && !GRHISupportsDX12AtomicUInt64)
		{
			// Vendor extensions currently support atomic64, but SM 6.6 and the DX12 Agility SDK are reporting that atomics are not supported.
			// Likely due to a pre-1909 Windows 10 version, or outdated drivers without SM 6.6 support.
			// See: https://devblogs.microsoft.com/directx/gettingstarted-dx12agility/
			bAtomicsSupported = false;
		}
	}
#endif

	return bAtomicsSupported;
}

bool NaniteWorkGraphMaterialsSupported()
{
	static const auto AllowWorkGraphMaterials = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.Nanite.AllowWorkGraphMaterials"));
	static const bool bAllowWorkGraphMaterials = (AllowWorkGraphMaterials && AllowWorkGraphMaterials->GetValueOnAnyThread() != 0);
	return bAllowWorkGraphMaterials;
}

bool UseNaniteFastTileClear()
{
	static const IConsoleVariable* CVarNaniteFastTileClear = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Nanite.FastTileClear"));
	return (CVarNaniteFastTileClear && CVarNaniteFastTileClear->GetInt() != 0);
}

bool NaniteMeshShadersSupported(EShaderPlatform Platform)
{
	static FShaderPlatformCachedIniValue<bool> PerPlatformCVar(TEXT("r.Nanite.AllowMeshShaders"));
	return PerPlatformCVar.Get(Platform) != 0;
}

bool NaniteSplineMeshesSupported()
{
	static const auto AllowSplineMeshes = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.Nanite.AllowSplineMeshes"));
	static const bool bAllowSplineMeshes = (AllowSplineMeshes && AllowSplineMeshes->GetValueOnAnyThread() != 0);
	return bAllowSplineMeshes;
}

bool NaniteSkinnedMeshesSupported()
{
	static const auto AllowSkinnedMeshes = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.Nanite.AllowSkinnedMeshes"));
	static const bool bAllowSkinnedMeshes = (AllowSkinnedMeshes && AllowSkinnedMeshes->GetValueOnAnyThread() != 0);
	return bAllowSkinnedMeshes;
}

bool NaniteTranslucencySupported()
{
	static const auto AllowTranslucency = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.Nanite.AllowTranslucency"));
	static const bool bAllowTranslucency = (AllowTranslucency && AllowTranslucency->GetValueOnAnyThread() != 0);
	return bAllowTranslucency;
}

bool NaniteAssembliesSupported()
{
	static const auto NaniteFoliage = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.Nanite.Foliage"));
	static const bool bAllowNaniteFoliage = (NaniteFoliage && NaniteFoliage->GetValueOnAnyThread() != 0);
	static const auto AllowAssemblies = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.Nanite.AllowAssemblies"));
	static const bool bAllowAssemblies = (AllowAssemblies && AllowAssemblies->GetValueOnAnyThread() != 0);
	return bAllowNaniteFoliage || bAllowAssemblies;
}

bool NaniteVoxelsSupported()
{
	static const auto NaniteFoliage = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.Nanite.Foliage"));
	static const bool bAllowNaniteFoliage = (NaniteFoliage && NaniteFoliage->GetValueOnAnyThread() != 0);
	static const auto AllowVoxels = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.Nanite.AllowVoxels"));
	static const bool bAllowVoxels = (AllowVoxels && AllowVoxels->GetValueOnAnyThread() != 0);
	return bAllowNaniteFoliage || bAllowVoxels;
}

bool NaniteCurvesSupported()
{
	static const auto AllowCurves = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.Nanite.AllowCurves"));
	static const bool bAllowCurves = (AllowCurves && AllowCurves->GetValueOnAnyThread() != 0);
	return bAllowCurves;
}

static bool BindlessConfigSupportsNanite(EShaderPlatform Platform)
{
	const ERHIBindlessConfiguration BindlessConfiguration = FShaderPlatformConfig::GetBindlessConfiguration(Platform);
	return IsBindlessEnabledForAnyGraphics(BindlessConfiguration);
}

bool UseNaniteBindlessShading(EShaderPlatform Platform)
{
	return BindlessConfigSupportsNanite(Platform) && FShaderPlatformConfig::GetEnableNaniteBindlessShading(Platform);
}

bool UseNaniteBindlessRasterization(EShaderPlatform Platform)
{
	return BindlessConfigSupportsNanite(Platform) && FShaderPlatformConfig::GetEnableNaniteBindlessRasterization(Platform);
}

bool UseNaniteTessellation()
{
	static const auto TessellationVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.Nanite.Tessellation"));
	static const auto ProgrammableVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.Nanite.ProgrammableRaster"));
	static const auto ComputeRasterVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.Nanite.ComputeRasterization"));

	const bool bTessellation = (TessellationVar && TessellationVar->GetValueOnAnyThread() != 0);
	const bool bProgrammable = (ProgrammableVar && ProgrammableVar->GetValueOnAnyThread() != 0);
	const bool bComputeRaster = (ComputeRasterVar && ComputeRasterVar->GetValueOnAnyThread() != 0);

	// Tessellation requires programmable and compute raster
	return bTessellation && bProgrammable && bComputeRaster;
}

bool DoesRuntimeSupportNanite(EShaderPlatform ShaderPlatform, bool bCheckForAtomicSupport, bool bCheckForProjectSetting)
{
	// Does the platform support Nanite?
	const bool bSupportedPlatform = DoesPlatformSupportNanite(ShaderPlatform, bCheckForProjectSetting);

	// Nanite is not supported with forward shading at this time.
	const bool bForwardShadingEnabled = IsForwardShadingEnabled(ShaderPlatform);

	return bSupportedPlatform && (!bCheckForAtomicSupport || NaniteAtomicsSupported()) && !bForwardShadingEnabled;
}

bool DoesTargetPlatformSupportNanite(const ITargetPlatform* TargetPlatform)
{
	if (TargetPlatform != nullptr)
	{
		TArray<FName> DesiredShaderFormats;
		TargetPlatform->GetAllTargetedShaderFormats(DesiredShaderFormats);
		for (int32 FormatIndex = 0; FormatIndex < DesiredShaderFormats.Num(); FormatIndex++)
		{
			const EShaderPlatform ShaderPlatform = ShaderFormatToLegacyShaderPlatform(DesiredShaderFormats[FormatIndex]);
			if (DoesPlatformSupportNanite(ShaderPlatform))
			{
				return true;
			}
		}
	}

	return false;
}

bool DoesMaterialQualitySupportNanite(EShaderPlatform ShaderPlatform, int32 MaterialQualityLevel)
{
	// Fix ordering of EMaterialQualityLevel Medium/High.
	if (MaterialQualityLevel == 1) 
	{
		MaterialQualityLevel = 2;
	}
	else if (MaterialQualityLevel == 2) 
	{
		MaterialQualityLevel = 1;
	}
	static FShaderPlatformCachedIniValue<int32> CVarPlatform(TEXT("r.Nanite.MinMaterialQuality"));
	const int32 MinNaniteMaterialQuality = CVarPlatform.Get(ShaderPlatform);
	return MaterialQualityLevel >= MinNaniteMaterialQuality;
}

bool DoesMaterialQualitySupportNanite(EShaderPlatform ShaderPlatform)
{
	static const auto MaterialQualityLevelCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.MaterialQualityLevel"));
	const int32 MaterialQualityLevel = MaterialQualityLevelCVar ? MaterialQualityLevelCVar->GetInt() : 4/**EMaterialQualityLevel::Num*/;
	return DoesMaterialQualitySupportNanite(ShaderPlatform, MaterialQualityLevel);
}

bool UseNanite(EShaderPlatform ShaderPlatform, bool bCheckForAtomicSupport /*= true*/, bool bCheckForProjectSetting /*= true*/)
{
	static const auto EnableNaniteCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Nanite"));
	const bool bNaniteEnabled = (EnableNaniteCVar != nullptr) ? (EnableNaniteCVar->GetInt() != 0) : true;
	return bNaniteEnabled && DoesMaterialQualitySupportNanite(ShaderPlatform) && DoesRuntimeSupportNanite(ShaderPlatform, bCheckForAtomicSupport, bCheckForProjectSetting);
}

bool AreNaniteFallbackMeshesEnabledForPlatform(const FStaticShaderPlatform Platform)
{
	return (GNaniteFallbackMeshesPlatformMask[(int)Platform]);
}

bool DoesPlatformSupportClassicShadowMaps(EShaderPlatform Platform)
{
	// Unconditionally supported unless disabled by the project setting
	static FShaderPlatformCachedIniValue<int32> CVarPlatformClassicShadowMapsProjectEnabled(TEXT("r.Shadow.Classic.ProjectEnabled"));
	bool bIsProjectEnable = CVarPlatformClassicShadowMapsProjectEnabled.Get(Platform) != 0;
	return bIsProjectEnable;
}

bool DoesPlatformSupportVirtualShadowMaps(EShaderPlatform Platform)
{
	// VSM allowed for this project/platform
	static FShaderPlatformCachedIniValue<int32> CVarPlatformVSMProjectEnabled(TEXT("r.Shadow.Virtual.ProjectEnabled"));
	bool bIsProjectEnable = CVarPlatformVSMProjectEnabled.Get(Platform) != 0;

	return bIsProjectEnable && DoesPlatformSupportNanite(Platform);
}

bool DoesPlatformSupportNonNaniteVirtualShadowMaps(EShaderPlatform ShaderPlatform)
{
	static FShaderPlatformCachedIniValue<int32> CVarPlatformNonNaniteVSMProjectEnabled(TEXT("r.Shadow.Virtual.NonNanite.ProjectEnabled"));
	bool bIsProjectEnable = CVarPlatformNonNaniteVSMProjectEnabled.Get(ShaderPlatform) != 0;

	return bIsProjectEnable && DoesPlatformSupportVirtualShadowMaps(ShaderPlatform);
}

bool UseNonNaniteVirtualShadowMaps(EShaderPlatform ShaderPlatform, const FStaticFeatureLevel FeatureLevel)
{
	return DoesPlatformSupportNonNaniteVirtualShadowMaps(ShaderPlatform) && UseVirtualShadowMaps(ShaderPlatform, FeatureLevel);
}

bool UseVirtualShadowMaps(EShaderPlatform ShaderPlatform)
{
	static const auto EnableVirtualSMCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Shadow.Virtual.Enable"));
	const bool bVirtualShadowMapsEnabled = EnableVirtualSMCVar ? (EnableVirtualSMCVar->GetInt() != 0) : false;
	return bVirtualShadowMapsEnabled &&
		DoesPlatformSupportVirtualShadowMaps(ShaderPlatform) &&
		DoesRuntimeSupportNanite(ShaderPlatform, true /* check for atomics */, true /* check project setting */);
}

bool UseVirtualShadowMaps(EShaderPlatform ShaderPlatform, const FStaticFeatureLevel FeatureLevel)
{
	return UseVirtualShadowMaps(ShaderPlatform);
}

bool IsWaterVirtualShadowMapFilteringEnabled(const FStaticShaderPlatform Platform)
{
	static const auto CVarWaterSingleLayerShaderSupportVSMFiltering = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Water.SingleLayer.ShadersSupportVSMFiltering"));
	const bool bWaterVSMFilteringSupported = CVarWaterSingleLayerShaderSupportVSMFiltering && (CVarWaterSingleLayerShaderSupportVSMFiltering->GetInt() > 0);

	const bool bVirtualShadowMapsSupported = DoesPlatformSupportVirtualShadowMaps(Platform);

	return !IsForwardShadingEnabled(Platform) && bWaterVSMFilteringSupported && bVirtualShadowMapsSupported;
}

bool DoesRuntimeSupportHeterogeneousVolumes(EShaderPlatform Platform)
{
	static const auto CVarHeterogeneousVolumes = IConsoleManager::Get().FindConsoleVariable(TEXT("r.HeterogeneousVolumes"));
	bool bHeterogeneousVolumesEnabled = CVarHeterogeneousVolumes && CVarHeterogeneousVolumes->GetInt() > 0;
	return DoesPlatformSupportHeterogeneousVolumes(Platform) && bHeterogeneousVolumesEnabled;
}

bool DoesPlatformSupportHeterogeneousVolumes(EShaderPlatform Platform)
{
	return IsFeatureLevelSupported(Platform, ERHIFeatureLevel::SM5)
		// TODO:
		// && FDataDrivenShaderPlatformInfo::GetSupportsHeterogeneousVolumes(Platform)
		&& !IsForwardShadingEnabled(Platform);
}

bool DoesPlatformSupportSparseVolumeTextures(EShaderPlatform Platform)
{
	// There are currently no hard platform restrictions for SVT support, but mobile is probably out of spec at this point.
	return IsFeatureLevelSupported(Platform, ERHIFeatureLevel::SM5);
}

bool UseSparseVolumeTextures(EShaderPlatform Platform)
{
	static const auto CVarSparseVolumeTexture = IConsoleManager::Get().FindConsoleVariable(TEXT("r.SparseVolumeTexture"));
	const bool bIsSVTEnabled = CVarSparseVolumeTexture && CVarSparseVolumeTexture->GetInt() != 0;
	const bool bIsSVTSupportedOnPlatform = DoesPlatformSupportSparseVolumeTextures(Platform);
	return bIsSVTEnabled && bIsSVTSupportedOnPlatform;
}

bool IsSingleLayerWaterDepthPrepassEnabled(const FStaticShaderPlatform Platform, const FStaticFeatureLevel FeatureLevel)
{
	static const auto CVarWaterSingleLayerDepthPrepass = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Water.SingleLayer.DepthPrepass"));
	const bool bPrepassEnabled = CVarWaterSingleLayerDepthPrepass && CVarWaterSingleLayerDepthPrepass->GetInt() > 0;
	// Currently VSM is the only feature dependent on the depth prepass which is why we only enable it if VSM could also be enabled.
	// VSM can be toggled at runtime, but we need a compile time value here, so we fall back to DoesPlatformSupportVirtualShadowMaps() to check if
	// VSM *could* be enabled.
	const bool bVSMSupported = DoesPlatformSupportVirtualShadowMaps(Platform);

	return bPrepassEnabled && bVSMSupported;

}

bool IsUsingSingleLayerWaterFallback(FStaticShaderPlatform Platform)
{
	static const auto CVarWaterSingleLayerMobileFallback = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Water.SingleLayer.MobileFallback"));

	if (!IsMobilePlatform(Platform))
	{
		return false;
	}

	// Only multi-pass deferred supports SLW rendering
	// Disabled for GL, as there is not enough samplers
	if (IsMobileDeferredShadingEnabled(Platform) 
		&& !FReadOnlyCVARCache::MobileAllowFramebufferFetch(Platform)
		&& !IsOpenGLPlatform(Platform))
	{
		return (!CVarWaterSingleLayerMobileFallback || CVarWaterSingleLayerMobileFallback->GetInt() != 0);
	}

	return true;
}


RENDERCORE_API bool IsUsingDBuffers(const FStaticShaderPlatform Platform)
{
	// Dbuffer is enabled:
	// * If project enabled it 
	// * If Substrate is enabled and the platform is not set to BlendableGBuffer. When Substrate uses material buffer, the material data are not blendable and requires DBuffer.
	return GDBufferPlatformMask[(int)Platform] || (Substrate::IsSubstrateEnabled() && !Substrate::IsSubstrateBlendableGBufferEnabled(Platform));
}

RENDERCORE_API bool AreInlineSkinningShadersEnabled(EShaderPlatform Platform)
{
	// Some mobile GPUs (MALI) has a 64K elements limitation on texel buffers
	// This results in meshes with more than 64k vertices having skin cache disabled at runtime.
	// So for mobile we must always support inline skinning shaders so that we have something to fall back to.
	if (IsMobilePlatform(Platform))
	{
		return true;
	}

	static FShaderPlatformCachedIniValue<bool> PerPlatformAllowCVar(TEXT("r.SkinCache.Allow"));
	static FShaderPlatformCachedIniValue<int32> PerPlatformCompileCVar(TEXT("r.SkinCache.CompileShaders"));
	return PerPlatformAllowCVar.Get(Platform) == 0 || PerPlatformCompileCVar.Get(Platform) != 2;
}

RENDERCORE_API bool AreSkinCacheShadersEnabled(EShaderPlatform Platform)
{
	static FShaderPlatformCachedIniValue<int32> PerPlatformCVar(TEXT("r.SkinCache.CompileShaders"));
	return (PerPlatformCVar.Get(Platform) != 0);
}

RENDERCORE_API bool IsGPUSkinCacheAllowed(EShaderPlatform Platform)
{
	static FShaderPlatformCachedIniValue<bool> PerPlatformCVar(TEXT("r.SkinCache.Allow"));
	static FShaderPlatformCachedIniValue<bool> PerPlatformMobileCVar(TEXT("r.Mobile.SkinCache.Allow"));
	return IsMobilePlatform(Platform) ? PerPlatformMobileCVar.Get(Platform) : PerPlatformCVar.Get(Platform);
}

RENDERCORE_API bool IsGPUSkinCacheAvailable(EShaderPlatform Platform)
{
	return AreSkinCacheShadersEnabled(Platform) != 0 && IsGPUSkinCacheAllowed(Platform);
}

RENDERCORE_API bool AreMeshDeformersAvailable(EShaderPlatform Platform)
{
	// Note that we rely on the IMeshDeformerProvider being loaded before any calls to this function.
	// OptimusSettings/ComputeFramework are PostConfigInit for that reason.
	static IMeshDeformerProvider* MeshDeformerProvider = IMeshDeformerProvider::Get();
	return MeshDeformerProvider && MeshDeformerProvider->IsSupported(Platform);
}

RENDERCORE_API bool IsGPUSkinPassThroughSupported(EShaderPlatform Platform)
{
	// Enable the GPUSkin passthrough path if we might use the GPUSkinCache or MeshDeformers.
	return IsGPUSkinCacheAvailable(Platform) || AreMeshDeformersAvailable(Platform);
}

RENDERCORE_API bool AreBufferSRVsAlwaysCreatedByDefault(EShaderPlatform ShaderPlatform)
{
	// If manual Vertex fetch is supported, vertex data is passed in via an SRV instead of using hardware vertex fetch
	// GPUSkinCache uses GPUSkinPassThrough + Compute shader and thus also needs SRV
	// Mesh Deformer is not included here because sometimes mesh deformer are only used on specific meshes so
	// we want to avoid creating SRVs for meshes that never need them.
	return RHISupportsManualVertexFetch(ShaderPlatform) || IsGPUSkinCacheAvailable(ShaderPlatform);
}

RENDERCORE_API bool DoesRuntimeSupportOnePassPointLightShadows(EShaderPlatform Platform)
{
	static const auto CVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.Shadow.DetectVertexShaderLayerAtRuntime"));

	return (RHISupportsVertexShaderLayer(Platform)
		|| (CVar->GetValueOnAnyThread() != 0 && GRHISupportsArrayIndexFromAnyShader != 0));
}

bool IsForwardShadingEnabled(const FStaticShaderPlatform Platform)
{
	return (GForwardShadingPlatformMask[(int)Platform])
		// Culling uses compute shader
		&& GetMaxSupportedFeatureLevel(Platform) >= ERHIFeatureLevel::SM5;
}

bool IsUsingGBuffers(const FStaticShaderPlatform Platform)
{
	if (IsMobilePlatform(Platform))
	{
		return IsMobileDeferredShadingEnabled(Platform);
	}
	else
	{
		return !IsForwardShadingEnabled(Platform);
	}
}

bool IsUsingBasePassVelocity(const FStaticShaderPlatform Platform)
{
	static FShaderPlatformCachedIniValue<int32> PerPlatformCVar(TEXT("r.VelocityOutputPass"));
	// Writing velocity in base pass is disabled for desktop forward because it may use MSAA (runtime-settable setting) and Velocity isn't a multisample texture.
	// TSR or vr.AllowMotionBlurInVR=1 case aren't recommended for Desktop Forward renderer, and if enabled, cause a separate Velocity pass.
	if (IsMobilePlatform(Platform) || IsForwardShadingEnabled(Platform))
	{
		return false;
	}
	else
	{
		return (PerPlatformCVar.Get(Platform) == 1);
	}
}

bool IsUsingSelectiveBasePassOutputs(const FStaticShaderPlatform Platform)
{
	return (GSelectiveBasePassOutputsPlatformMask[(int)Platform]);
}

bool IsUsingDistanceFields(const FStaticShaderPlatform Platform)
{
	return (GDistanceFieldsPlatformMask[(int)Platform]);
}

bool IsWaterDistanceFieldShadowEnabled(const FStaticShaderPlatform Platform)
{
	// Only deferred support such a feature. It is not possible to do that for water without a water depth pre-pass.
	static const auto CVarWaterSingleLayerShaderSupportDistanceFieldShadow = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Water.SingleLayer.ShadersSupportDistanceFieldShadow"));
	const bool bWaterSingleLayerShaderSupportDistanceFieldShadow = CVarWaterSingleLayerShaderSupportDistanceFieldShadow && (CVarWaterSingleLayerShaderSupportDistanceFieldShadow->GetInt() > 0);
	return !IsForwardShadingEnabled(Platform) && IsUsingDistanceFields(Platform) && bWaterSingleLayerShaderSupportDistanceFieldShadow;
}

bool IsWaterSeparateMainDirLightEnabled(const FStaticShaderPlatform Platform)
{
	return IsWaterDistanceFieldShadowEnabled(Platform) || IsWaterVirtualShadowMapFilteringEnabled(Platform);
}

bool IsMobileDistanceFieldAOEnabled(const FStaticShaderPlatform Platform)
{
	static FShaderPlatformCachedIniValue<int32> MobileDistanceFieldAO(TEXT("r.Mobile.DistanceFieldAO"));
	return MobileDistanceFieldAO.Get(Platform) > 0;
}

bool IsStencilForLODDitherEnabled(const FStaticShaderPlatform Platform)
{
	if (IsMobilePlatform(Platform) && !FReadOnlyCVARCache::MobileAllowDitheredLODTransition(Platform))
	{
		return false;
	}
	static FShaderPlatformCachedIniValue<bool> StencilForLODDitherIniValue(TEXT("r.StencilForLODDither"));
	return StencilForLODDitherIniValue.Get(Platform);
}

int32 GetMaterialShadingPathNodeOverride(const FStaticShaderPlatform Platform)
{
	static FShaderPlatformCachedIniValue<int32> MaterialShadingPathNodeOverride(TEXT("r.Material.ShadingPathNodeOverride"));
	return MaterialShadingPathNodeOverride.Get(Platform);
}

bool GetMaterialSimulateMobileForLowQuality(const FStaticShaderPlatform Platform)
{
	static FShaderPlatformCachedIniValue<bool> MaterialSimulateMobileForLowQuality(TEXT("r.Material.SimulateMobileForLowQuality"));
	return MaterialSimulateMobileForLowQuality.Get(Platform);
}

// This is a deprecated version of the function in 5.8 and should not be used.
bool UseGPUScene(const FStaticShaderPlatform Platform, const FStaticFeatureLevel FeatureLevel)
{
	if (FeatureLevel == ERHIFeatureLevel::ES3_1)
	{
		return MobileSupportsGPUScene();
	}
	return UseGPUScene(Platform);
}

bool UseGPUScene(const FStaticShaderPlatform Platform)
{
	if (IsMobilePlatform(Platform))
	{
		return MobileSupportsGPUScene();
	}

	return FDataDrivenShaderPlatformInfo::GetSupportsGPUScene(Platform);
}

bool ForceSimpleSkyDiffuse(const FStaticShaderPlatform Platform)
{
	return (GSimpleSkyDiffusePlatformMask[(int)Platform]);
}

bool VelocityEncodeDepth(const FStaticShaderPlatform Platform)
{
	return !IsMobilePlatform(Platform) && (GVelocityEncodeDepthPlatformMask[(int)Platform]);
}

bool VelocityEncodeHasPixelAnimation(const FStaticShaderPlatform Platform)
{
	// Should matches VELOCITY_ENCODE_HAS_PIXEL_ANIMATION.
	// But there is a bug (UE-198416) where VelocityEncodeDepth() always returns false on staged build since only depends on TargetPlatform->VelocityEncodeDepth();
	// which is completly disconnected to FVelocityRendering::GetFormat(). This ends up in the case where VelocityEncodeDepth() == true at cook time, but false at runtime
	// and yet end up with FVelocityRendering::GetFormat() return a format to encode the velocity.
	return !IsMobilePlatform(Platform);
}

bool VelocitySupportsTemporalResponsiveness(const FStaticShaderPlatform Platform)
{
	static FShaderPlatformCachedIniValue<int32> PerPlatformCVar(TEXT("r.Velocity.TemporalResponsiveness.Supported"));
	return (PerPlatformCVar.Get(Platform) != 0);
}

bool VelocitySupportsPixelShaderMotionVectorWorldOffset(const FStaticShaderPlatform Platform)
{
	static FShaderPlatformCachedIniValue<int32> PerPlatformCVar(TEXT("r.Velocity.PixelShaderMotionVectorWorldOffset.Supported"));
	return (PerPlatformCVar.Get(Platform) != 0);
}

bool PlatformSupportsOpenXRMotionVectors(const FStaticShaderPlatform Platform)
{
	if (IsVulkanMobilePlatform(Platform) && !IsMobileDeferredShadingEnabled(Platform))
	{
		static FShaderPlatformCachedIniValue<bool> PerPlatformCVar(TEXT("r.Velocity.DirectlyRenderOpenXRMotionVectors"));
		return PerPlatformCVar.Get(Platform);
	}

	return false;
}

bool VelocityIncludeStationaryPrimitives(const FStaticShaderPlatform Platform)
{
	// Directly rendering OpenXR motion vectors requires including all primitives in the velocity pass, even stationary primitives that would normally be excluded
	return PlatformSupportsOpenXRMotionVectors(Platform);
}

RENDERCORE_API bool AllowTranslucencyPerObjectShadows(const FStaticShaderPlatform Platform)
{
	return IsFeatureLevelSupported(Platform, ERHIFeatureLevel::SM5) && GAllowTranslucencyShadowsInProject != 0;
}

RENDERCORE_API bool Allow128BitBasePassPSCompilation(const FStaticShaderPlatform Platform)
{
	static FShaderPlatformCachedIniValue<bool> PerPlatformCVar(TEXT("r.128BitBPPSCompilation.Allow"));
	return PerPlatformCVar.Get(Platform);
}

RENDERCORE_API bool ShouldGenerateRayTracingProxiesByDefault()
{
	return GRayTracingProxiesProjectEnabled;
}

bool PlatformRequires128bitRT(EPixelFormat PixelFormat)
{
	switch (PixelFormat)
	{
	case PF_R32_FLOAT:
	case PF_G32R32F:
	case PF_A32B32G32R32F:
		return FDataDrivenShaderPlatformInfo::GetRequiresExplicit128bitRT(GMaxRHIShaderPlatform);
	default:
		return false;
	}
}

bool IsRayTracingEnabledForProject(EShaderPlatform ShaderPlatform)
{
	if (RHISupportsRayTracing(ShaderPlatform))
	{
		return (GRayTracingPlatformMask[(int)ShaderPlatform]);
	}
	else
	{
		return false;
	}
}

bool AreRayTracingShadersEnabledForProject(EShaderPlatform ShaderPlatform)
{
	if (RHISupportsRayTracing(ShaderPlatform) && RHISupportsRayTracingShaders(ShaderPlatform))
	{
		return GRayTracingShadersPlatformMask[(int)ShaderPlatform];
	}
	else
	{
		return false;
	}
}

bool ShouldCompileRayTracingShadersForProject(EShaderPlatform ShaderPlatform)
{
	if (RHISupportsRayTracingShaders(ShaderPlatform))
	{
		return IsRayTracingEnabledForProject(ShaderPlatform) && AreRayTracingShadersEnabledForProject(ShaderPlatform) && IsFeatureLevelSupported(ShaderPlatform, ERHIFeatureLevel::SM6);
	}
	else
	{
		return false;
	}
}

bool ShouldCompileRayTracingCallableShadersForProject(EShaderPlatform ShaderPlatform)
{
	return RHISupportsRayTracingCallableShaders(ShaderPlatform) && ShouldCompileRayTracingShadersForProject(ShaderPlatform);
}

bool IsRayTracingEnabled()
{
	bool bRayTracingEnabled = true;
	static const auto RayTracingEnableCVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.Raytracing.Enable"));
	if (GRayTracingMode == ERayTracingMode::Dynamic && RayTracingEnableCVar)
	{
		bRayTracingEnabled = RayTracingEnableCVar->GetValueOnAnyThread() != 0;
	}

	return IsRayTracingAllowed() && bRayTracingEnabled;
}

bool IsRayTracingAllowed()
{
	checkf(GIsRHIInitialized, TEXT("IsRayTracingAllowed() may only be called once RHI is initialized."));

#if DO_CHECK && WITH_EDITOR
	{
		// This function must not be called while cooking
		if (IsRunningCookCommandlet())
		{
			return false;
		}
	}
#endif // DO_CHECK && WITH_EDITOR

	return (int32)GRayTracingMode >= (int32)ERayTracingMode::Enabled;
}

bool IsRayTracingEnabled(EShaderPlatform ShaderPlatform)
{
	return IsRayTracingEnabled() && RHISupportsRayTracing(ShaderPlatform) && (IsFeatureLevelSupported(ShaderPlatform, ERHIFeatureLevel::SM6) || IsVulkanMobileSM5Platform(ShaderPlatform));
}

ERayTracingMode GetRayTracingMode()
{
	return IsRayTracingAllowed() ? GRayTracingMode : ERayTracingMode::Disabled;
}

bool IsRayTracingUsingReferenceBasedResidency()
{
	static const auto CVar = IConsoleManager::Get().FindTConsoleVariableDataBool(TEXT("r.RayTracing.UseReferenceBasedResidency"));

	return CVar && CVar->GetValueOnAnyThread();
}

bool IsRayTracingEnableOnDemandSupported()
{
	return GRayTracingEnableOnDemand != 0;
}

bool UseSplineMeshSceneResources(const FStaticShaderPlatform Platform)
{
	// This feature currently requires GPU Scene to be supported and enabled.
	// NOTE: Mobile GPU can't currently support this feature because it doesn't include the payload extension in its instance data
	if (UseGPUScene(Platform) && !IsMobilePlatform(Platform))
	{
		static FShaderPlatformCachedIniValue<int32> CVar(TEXT("r.SplineMesh.SceneTextures"));
		return CVar.Get(Platform) > 0;
	}

	return false;
}

bool RenderRectLightsAsSpotLights(const FStaticFeatureLevel FeatureLevel)
{
	if (FeatureLevel == ERHIFeatureLevel::ES3_1)
	{
		static const auto CVarRenderRectLightAsSpotLight = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.Mobile.Forward.RenderRectLightsAsSpotLights"));
		return (CVarRenderRectLightAsSpotLight && CVarRenderRectLightAsSpotLight->GetValueOnAnyThread() != 0);
	}
	return false;
}

static TAutoConsoleVariable<int32> CVarLightFunctionAtlasFormat(
	TEXT("r.LightFunctionAtlas.Format"),
	0,
	TEXT("0: grey scale in [0,1]. 1: colored in [0,1]"),
	ECVF_ReadOnly | ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarSingleLayerWaterUsesLightFunctionAtlas(
	TEXT("r.SingleLayerWater.UsesLightFunctionAtlas"),
	0,
	TEXT("Enable sampling of the light function atlas on SingleLAyer Water mateirals."),
	ECVF_ReadOnly | ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarTranslucentUsesLightFunctionAtlas(
	TEXT("r.Translucent.UsesLightFunctionAtlas"),
	0,
	TEXT("Enable sampling of the light function atlas on translucent materials using forward shading."),
	ECVF_ReadOnly | ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarRectLighTranslucent(
	TEXT("r.Translucent.UsesRectLights"),
	0,
	TEXT("Enable rect light support for translucent surfaces. When enabled, it will add an extra sampler to the pixel shader (limited to 16 on dx11 based system)"),
	ECVF_ReadOnly | ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarShadowedLocalLightsTranslucent(
	TEXT("r.Translucent.UsesShadowedLocalLights"),
	0,
	TEXT("Enable shadow support on local lights for translucent surfaces."),
	ECVF_ReadOnly | ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarIESProfileTranslucent(
	TEXT("r.Translucent.UsesIESProfiles"),
	0,
	TEXT("Enable IES profiles support for translucent surfaces. When enabled, it will add an extrat sampler to the pixel shader (limited to 16 on dx11 based system)"),
	ECVF_ReadOnly | ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarHairStrandsUsesTriangleStrips(
	TEXT("r.HairStrands.Strands.UseTriangleStrips"),
	1,
	TEXT("Enable triangle strip geometry for hair strands rendering. This improves performances, but removes the last segments of each curve."),
	ECVF_ReadOnly | ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarHairStrandsLODMode(
	TEXT("r.HairStrands.LODMode"),
	1,
	TEXT("Enable hair strands Auto LOD mode by default. Otherwise use Manual LOD mode. Auto LOD mode adapts hair curves based on screen coverage. Manual LOD mode relies on LODs manually setup per groom asset. This global behavior can be overridden per groom asset."),
	ECVF_RenderThreadSafe);

int32 GetLightFunctionAtlasFormat()
{
	return CVarLightFunctionAtlasFormat.GetValueOnAnyThread();
}

bool GetSingleLayerWaterUsesLightFunctionAtlas()
{
	return CVarSingleLayerWaterUsesLightFunctionAtlas.GetValueOnAnyThread() > 0;
}

bool GetTranslucentUsesLightFunctionAtlas()
{
	return CVarTranslucentUsesLightFunctionAtlas.GetValueOnAnyThread() > 0;
}

bool GetTranslucentUsesLightRectLights()
{
	return CVarRectLighTranslucent.GetValueOnAnyThread() > 0;
}

bool GetTranslucentUsesShadowedLocalLights()
{
	return CVarShadowedLocalLightsTranslucent.GetValueOnAnyThread() > 0;
}

bool GetTranslucentUsesLightIESProfiles()
{
	return CVarIESProfileTranslucent.GetValueOnAnyThread() > 0;
}

bool GetHairStrandsUsesTriangleStrips()
{
	return CVarHairStrandsUsesTriangleStrips.GetValueOnAnyThread() > 0;
}

uint32 GetHairStrandsLODMode()
{
	return FMath::Clamp(CVarHairStrandsLODMode.GetValueOnAnyThread(), 0, 1);
}

///////////////////////////////////////////////////////////////////////////////////////////////////
// Substrate settings interface

static TAutoConsoleVariable<int32> CVarSubstrate(
	TEXT("r.Substrate"),
	0,
	TEXT("Enable Substrate materials."),
	ECVF_ReadOnly | ECVF_RenderThreadSafe);

// Summary:
// * Existing project NOT using Substrate will continue to NOT use Substrate.
// * Existing project using Substrate will continue to use Substrate with Adaptive GBuffer
// * New project will using Substrate with Blendable GBuffer
//
// By default, Substrate uses Adaptive GBUffer. This allows existing project to continue using Adaptive format without updating their settings
// New projects created with 5.7, will have Substrate enabled automatically:
//  * With Blendable GBuffer for regular templates.
//  * With Adaptive Gbuffer for advanced templates.
// This is handled in GameProjectUtils.cpp.
static TAutoConsoleVariable<int32> CVarSubstrateProjectGBufferFormat(
	TEXT("r.Substrate.ProjectGBufferFormat"),
	1,
	TEXT("Define GBuffer format used for this project.\n \
	- 0: Substrate materials are fit into a blendable GBuffer. This is faster and DBuffer decals are no longer required. However, lots of visual effects and fidelity are lost. To be able to cast to blendable GBuffer, this command enforce ClosuresPerPixel=1.\n \
	- 1: Substrate materials are fit into a bitstream GBuffer. This stores more data to enable complex lighting behavior per pixel. It is more costly than the blendable GBuffer format. This is only support on certain platforms. Non-supported platforms will automatically fallback onto a blendable GBuffer format."),
	ECVF_ReadOnly | ECVF_RenderThreadSafe | ECVF_Preview);

static TAutoConsoleVariable<int32> CVarSubstrateBytesPerPixel(
	TEXT("r.Substrate.BytesPerPixel"),
	80,
	TEXT("Substrate allocated byte per pixel to store materials data. Higher value means more complex material can be represented."),
	ECVF_ReadOnly | ECVF_RenderThreadSafe | ECVF_Preview);

static TAutoConsoleVariable<int32> CVarSubstrateClosuresPerPixel(
	TEXT("r.Substrate.ClosuresPerPixel"),
	1,	// Substrate 60fps by default. Must be less or equal to SUBSTRATE_MAX_CLOSURE_COUNT. This value is set for each platform
	TEXT("Substrate closure count per pixel can be constrained. That is usefull to reduce the number of byte written durting the base pass, but also to limit the number of closures in Forward. Higher value means more complex material can be represented."),
	ECVF_ReadOnly | ECVF_RenderThreadSafe | ECVF_Preview);

static TAutoConsoleVariable<int32> CVarSubstrateProjectClosuresPerPixel(
	TEXT("r.Substrate.ProjectClosuresPerPixel"),
	4, // 4 closures by default in order to support advanced use case
	TEXT("Define the maximum number of closures evaluated per pixel for this project. Materials with a number of closures beyond this limit will be simplified until it fits the requirements. This only applies when Adaptive GBuffer is enabled. On platforms not supporting Adaptive GBuffer, the material will be simplified into a single closure."),
	ECVF_ReadOnly | ECVF_RenderThreadSafe | ECVF_Preview);

static TAutoConsoleVariable<FString> CVarSubstrateMaterialQualityLevelsClosuresPerPixel(
	TEXT("r.Substrate.MaterialQualityLevelsClosuresPerPixel"),
	TEXT(""),	// Empty => invalid so ignored.
	TEXT("Define the maximum number of closures evaluated per pixel for a given material quality level. Empty by default: no impact. Otherwise L1-M1-H1-E1 to set closure count for each of the material quality level (Low, Medium, High and Epic respecively)."),
	ECVF_ReadOnly | ECVF_Preview);

static TAutoConsoleVariable<int32> CVarSubstrateProjectClosuresPerPixelOverride(
	TEXT("r.Substrate.ProjectClosuresPerPixelOverride"),
	0,	// By default, use the mininum between the platform's closure count and the project's closure count (i.e., ESubstrateClosureConfig::Type::UseMinProjectAndPlatforClosuresPerPixel)
	TEXT("Select how the maximum number of closures evaluated per pixel is defined for each platforms.\n - Use Platform Closure Count: Use the mininum between the platform's closure count and the project's closure count.\n - Force Closure Count : Force the project's closure count on platforms supporting Adaptive GBuffer."),
	ECVF_ReadOnly | ECVF_RenderThreadSafe | ECVF_Preview);

static TAutoConsoleVariable<int32> CVarSubstrateOpaqueMaterialRoughRefraction(
	TEXT("r.Substrate.OpaqueMaterialRoughRefraction"),
	0, // Substrate 60fps by default
	TEXT("Enable Substrate opaque material rough refractions effect from top layers over layers below. (Experimental for now since this is not a path validated in production)"),
	ECVF_ReadOnly | ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarSubstrateShadingQuality(
	TEXT("r.Substrate.ShadingQuality"),
	0, // Substrate 60fps by default
	TEXT("Define Substrate shading quality (0: approximate lighting, 1: accurate lighting). This variable is read-only."),
	ECVF_ReadOnly | ECVF_RenderThreadSafe | ECVF_Preview);

static TAutoConsoleVariable<int32> CVarSubstrateDBufferPass(
	TEXT("r.Substrate.DBufferPass"),
	0,
	TEXT("Apply DBuffer after the base-pass as a separate pass. Read only because when this is changed, it will require the recompilation of all shaders."),
	ECVF_ReadOnly | ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarSubstrateEnableLayerSupport(
	TEXT("r.Substrate.EnableLayerSupport"),
	0,
	TEXT("Enable Substrate material layering and UI (Experimental). Note: This support is one way; the legacy layer materials auto-upgrade and can only be reversed manually after assets are re-saved."),
	ECVF_ReadOnly | ECVF_RenderThreadSafe);

// Transition render settings that will disappear when Substrate gets enabled
static TAutoConsoleVariable<int32> CVarMaterialRoughDiffuse(
	TEXT("r.Material.RoughDiffuse"),
	0,
	TEXT("Enable rough diffuse material."),
	ECVF_ReadOnly | ECVF_RenderThreadSafe | ECVF_Preview);

static TAutoConsoleVariable<int32> CVarSubstrateRoughDiffuse(
	TEXT("r.Substrate.RoughDiffuse"),
	0, // Substrate 60fps by default
	TEXT("Enable Substrate rough diffuse model (works only if r.Material.RoughDiffuse is enabled in the project settings). Togglable at runtime"),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarSubstrateGlints(
	TEXT("r.Substrate.Glints"),
	0, // Substrate 60fps by default
	TEXT("Enable Glint support for Substrate slabs. If changed, shaders needs to be recompiled."),
	ECVF_ReadOnly | ECVF_RenderThreadSafe | ECVF_Preview);

static TAutoConsoleVariable<int32> CVarSubstrateGlintsLUT(
	TEXT("r.Substrate.Glints.LUT"),
	1,
	TEXT("Select one of the glint rendering LUT for testing."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<float> CVarSubstrateGlintsLevelBias(
	TEXT("r.Substrate.Glints.LevelBias"),
	0.0f,
	TEXT("Constantly bias the glint level. A negative value will reinforce the glint effect while a positive value will attenuate it. Be careful as this might cause aliasing."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<float> CVarSubstrateGlintsLevelMin(
	TEXT("r.Substrate.Glints.LevelMin"),
	0.0f,
	TEXT("The minimum glint level used to render glints. Mostly affect the look of distance surfaces. A value greater than 0 will force glints to always be visible instead of using the analytic BSDF. Be careful as this might cause aliasing."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarSubstrateSpecularProfile(
	TEXT("r.Substrate.SpecularProfile"),
	1,
	TEXT("Enable Specular Profile support for Substrate slabs. If changed, shaders needs to be recompiled."),
	ECVF_ReadOnly | ECVF_RenderThreadSafe | ECVF_Preview | ECVF_MobileShaderChange);

static TAutoConsoleVariable<int32> CVarSubstrateDebugAdvancedVisualizationShaders(
	TEXT("r.Substrate.Debug.AdvancedVisualizationShaders"),
	0,
	TEXT("Enable advanced Substrate material debug visualization shaders. Base pass shaders can output such advanced data."),
	ECVF_ReadOnly | ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarSubstrateStochasticLighting(
	TEXT("r.Substrate.StochasticLighting"),
	0,
	TEXT("Enable stochastic lighting to get better performance."),
	ECVF_ReadOnly | ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarSubstrateSimpleTiledLightingPasses(
	TEXT("r.Substrate.SimpleTiledLightingPasses"),
	0,
	TEXT("Enabling simple tiled lighting passes when greater than 0, can help performance on low end platforms not supporting Rect primitives. Only stencil marking will be tiled. And tiled passes will use larger tiles. Not compatible with Lumen nor MegaLight."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarSubstrateExperimentalToonReflectionQuantizationEnabled(
	TEXT("r.Substrate.Experimental.ToonReflectionQuantizationEnabled"),
	0,
	TEXT("Enable experimental toon reflection quantization."),
	ECVF_ReadOnly | ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarSubstrateExperimentalToonUnifiedDiffuse(
	TEXT("r.Substrate.Experimental.ToonUnifiedDiffuse"),
	0,
	TEXT("Enable experimental toon unified diffuse evaluation."),
	ECVF_ReadOnly | ECVF_RenderThreadSafe);

namespace Substrate
{
	bool IsSubstrateEnabled()
	{
		return CVarSubstrate.GetValueOnAnyThread() > 0;
	}

	bool IsSubstrateBlendableGBufferEnabled(EShaderPlatform InPlatform)
	{
		const bool bPlatformSupportsAdaptiveGBuffer = FDataDrivenShaderPlatformInfo::GetSupportsAdaptiveGBuffer(InPlatform);
		const bool bProjectRequestsAdaptiveGBuffer = CVarSubstrateProjectGBufferFormat.GetValueOnAnyThread() > 0;
		
		// Mobile always uses blendable GBuffer, otherwise it is enabled on selected platforms.
		return (IsMobilePlatform(InPlatform) && IsMobileDeferredShadingEnabled(InPlatform)) || !(bPlatformSupportsAdaptiveGBuffer && bProjectRequestsAdaptiveGBuffer);
	}

	bool IsStochasticLightingEnabled(EShaderPlatform InPlatform)
	{
		return IsSubstrateEnabled() && CVarSubstrateStochasticLighting.GetValueOnAnyThread() > 0 && !IsSubstrateBlendableGBufferEnabled(InPlatform);
	}

	uint32 GetBytePerPixel(EShaderPlatform InPlatform)
	{
		// Variant for shader compilation per platform
		static FShaderPlatformCachedIniValue<int32> CVarByteBudget(TEXT("r.Substrate.BytesPerPixel"));
		uint32 CvarBytePerPixel = CVarByteBudget.Get(InPlatform);

		auto RoundUpValueToUInt = [](uint32 Value)
			{
				const uint32 Divisor = sizeof(uint32);
				return  FMath::DivideAndRoundUp<uint32>(Value, Divisor) * Divisor;
			};

		const bool bBlendableGBufferFormat = IsSubstrateBlendableGBufferEnabled(InPlatform);

		// We enforce at least 20 bytes per pixel because this is the minimal Substrate GBuffer footprint of the simplest material.
		const uint32 MinSubstrateBytePerPixel = 20u;
		const uint32 MaxSubstrateBytePerPixel = bBlendableGBufferFormat ? 20u : 256u;
		// Align byte per pixel count to 4 because the read/write unit is uint32
		check(MinSubstrateBytePerPixel == RoundUpValueToUInt(MinSubstrateBytePerPixel));
		check(MaxSubstrateBytePerPixel == RoundUpValueToUInt(MaxSubstrateBytePerPixel));
		CvarBytePerPixel = RoundUpValueToUInt(CvarBytePerPixel);
		return FMath::Clamp(CvarBytePerPixel, MinSubstrateBytePerPixel, MaxSubstrateBytePerPixel);
	}

	uint32 GetClosurePerPixel(EShaderPlatform InPlatform)
	{
		return GetClosurePerPixel(InPlatform, 0);
	}

	struct FSubstratePerMaterialLevelQualityClosureCount
	{
		bool bIsValid = false;	// Only true if the cvar has been specified explicitely, i.e. not empty.
		uint32 LowMaterialQualityMaxClosureCount	= 1;
		uint32 MediumMaterialQualityMaxClosureCount	= 1;
		uint32 HighMaterialQualityMaxClosureCount	= 1;
		uint32 EpicMaterialQualityMaxClosureCount	= 1;
	};

	const FSubstratePerMaterialLevelQualityClosureCount& GetPerMaterialLevelQualityClosureCount(EShaderPlatform InPlatform)
	{
		// Typically there is only one platform. In editor many can be evaluated for multiple preview in a session. So a Map should be enough.
		static TMap<EShaderPlatform, FSubstratePerMaterialLevelQualityClosureCount> CachedPlatformResults;

		static FCriticalSection CS;
		FScopeLock Lock(&CS);

		FSubstratePerMaterialLevelQualityClosureCount* FoundResult = CachedPlatformResults.Find(InPlatform);
		if (!FoundResult)
		{
			FSubstratePerMaterialLevelQualityClosureCount& Result = CachedPlatformResults.Add(InPlatform);

			static FShaderPlatformCachedIniValue<FString> CVarMaterialQualityLevelsClosuresPerPixel(TEXT("r.Substrate.MaterialQualityLevelsClosuresPerPixel"));
			FString CVARString = CVarMaterialQualityLevelsClosuresPerPixel.Get(InPlatform);
			if (!CVARString.IsEmpty())
			{
				// The format must be in style "L1-M1-H1-E1"
				int32 ValidRead =
#if PLATFORM_WINDOWS || PLATFORM_MICROSOFT
					sscanf_s(
#else
					sscanf(
#endif
					TCHAR_TO_ANSI(*CVARString), "L%u-M%u-H%u-E%u",
					&Result.LowMaterialQualityMaxClosureCount,
					&Result.MediumMaterialQualityMaxClosureCount,
					&Result.HighMaterialQualityMaxClosureCount,
					&Result.EpicMaterialQualityMaxClosureCount);
				Result.bIsValid = true;

				if (ValidRead != 4) // not all input have been read.
				{
					Result.bIsValid = false;
					Result.LowMaterialQualityMaxClosureCount = 1;
					Result.MediumMaterialQualityMaxClosureCount = 1;
					Result.HighMaterialQualityMaxClosureCount = 1;
					Result.EpicMaterialQualityMaxClosureCount = 1;
				}

				Result.LowMaterialQualityMaxClosureCount = FMath::Max(1u, Result.LowMaterialQualityMaxClosureCount);
				Result.MediumMaterialQualityMaxClosureCount = FMath::Max(1u, Result.MediumMaterialQualityMaxClosureCount);
				Result.HighMaterialQualityMaxClosureCount = FMath::Max(1u, Result.HighMaterialQualityMaxClosureCount);
				Result.EpicMaterialQualityMaxClosureCount = FMath::Max(1u, Result.EpicMaterialQualityMaxClosureCount);
			}
			return Result;
		}
		return *FoundResult;
	}

	uint32 GetClosurePerPixel(EShaderPlatform InPlatform, uint8 InMaterialQualityLevel)
	{
		if (Substrate::IsSubstrateBlendableGBufferEnabled(InPlatform))
		{
			return 1u; // When using the blendable GBuffer, we enforce a single closure so that a single slab can be read to export.
		}

		uint32 OutClosurePerPixel = FShaderPlatformConfig::GetSubstrateMaxClosuresPerPixel(InPlatform);

		// Take min between the platform's settings and the project's settings
		const int32 ProjectClosurePerPixel = FMath::Clamp(CVarSubstrateProjectClosuresPerPixel.GetValueOnAnyThread(), 1, SUBSTRATE_MAX_CLOSURE_COUNT);

		const FSubstratePerMaterialLevelQualityClosureCount& PerMaterialLevelQualityClosureCount = GetPerMaterialLevelQualityClosureCount(InPlatform);
		if (PerMaterialLevelQualityClosureCount.bIsValid && InMaterialQualityLevel != IgnoreMaterialQualityLevelsForClosurePerPixel)
		{
			check(0 <= InMaterialQualityLevel && InMaterialQualityLevel <= 3);
			uint32 SelectedMaterialQualityMaxClosureCount = PerMaterialLevelQualityClosureCount.LowMaterialQualityMaxClosureCount;
			switch (InMaterialQualityLevel)
			{
				//	case 0: // Low
			case 2: // Medium
				SelectedMaterialQualityMaxClosureCount = PerMaterialLevelQualityClosureCount.MediumMaterialQualityMaxClosureCount;
				break;
			case 1: // High
				SelectedMaterialQualityMaxClosureCount = PerMaterialLevelQualityClosureCount.HighMaterialQualityMaxClosureCount;
				break;
			case 3: // Epic
				SelectedMaterialQualityMaxClosureCount = PerMaterialLevelQualityClosureCount.EpicMaterialQualityMaxClosureCount;
				break;
			}
			// The per material quality level value can only restricts the platform configured capacity.
			OutClosurePerPixel = FMath::Min(SelectedMaterialQualityMaxClosureCount, OutClosurePerPixel);
		}
		OutClosurePerPixel = FMath::Min<uint32>(ProjectClosurePerPixel, OutClosurePerPixel);

		// Force max closure count if enabled
		// - ESubstrateClosureConfig::Type::UseMinProjectAndPlatforClosuresPerPixel = 0
		// - ESubstrateClosureConfig::Type::ForceProjectClosuresPerPixel = 1
		const uint32 bForceProjectClosureCount = CVarSubstrateProjectClosuresPerPixelOverride.GetValueOnAnyThread() > 0;
		if (bForceProjectClosureCount)
		{
			OutClosurePerPixel = ProjectClosurePerPixel;
		}

		return FMath::Max<uint32>(1, OutClosurePerPixel);
	}

	uint32 GetNormalQuality()
	{
		static const auto CVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.GBufferFormat"));
		return CVar && CVar->GetValueOnAnyThread() > 1 ? 1 : 0;
	}

	uint32 GetRayTracingMaterialPayloadSizeInBytes(bool bFullySimplifiedMaterial)
	{
		// The payload size represents FPackedMaterialClosestHitPayload containing FSubstrateRaytracingPayload composed of 
		//  (1)- top layer data using a count of SUBSTRATE_TOP_LAYER_TYPE uints
		//  (2)- packed data in a uint32 array of size SUBSTRATE_RT_PAYLOAD_NUM_UINTS

		uint32 PayloadSizeBytes = 0;

		// Account for (1)
		const uint32 SubstrateNormalQuality = Substrate::GetNormalQuality();
		check(SubstrateNormalQuality >= 0 && SubstrateNormalQuality <= 1);
		switch (SubstrateNormalQuality)
		{
		case 0:
			PayloadSizeBytes += 1 * sizeof(uint32);
			break;
		case 1:
			PayloadSizeBytes += 2 * sizeof(uint32);	// with high quality normal
			break;
		}

		// Account for (2)
		if (bFullySimplifiedMaterial)
		{
			PayloadSizeBytes += SUBSTRATE_FULLY_SIMPLIFIED_NUM_UINTS * sizeof(uint32);
		}
		else
		{
			// Not supported for now. Ray tracing payload size is not scalable this way.
			// See GetRaytracingMaterialPayloadSizeFullySimplified()
			check(false);
		}

		return PayloadSizeBytes;
	}
	
	bool IsRoughDiffuseEnabled()
	{
		return CVarSubstrateRoughDiffuse.GetValueOnAnyThread() > 0;
	}

	bool IsRoughDiffuseEnabled(EShaderPlatform InPlatform)
	{
		// Variant for shader compilation per platform
		static FShaderPlatformCachedIniValue<int32> CVar(TEXT("r.Substrate.RoughDiffuse"));
		return CVar.Get(InPlatform) > 0;
	}

	bool IsGlintEnabled()
	{
		return IsGlintEnabled(GMaxRHIShaderPlatform);
	}

	bool IsGlintEnabled(EShaderPlatform InPlatform)
	{
		static FShaderPlatformCachedIniValue<int32> CVar(TEXT("r.Substrate.Glints"));
		// Force disable glint on ES31/SM4 platforms as it causes compilation error (e.g. too large local light unroll in forward shaders).
		// Usually this effect is explicitly disabled in XXXPlatform.ini files. However when running preview (e.g. -featureleveles31), 
		// these settings are not taken into account causing compilation issue.
		// Also disable glint when blendable gbuffer is used as the data can't be stored
		const bool bSupported = FDataDrivenShaderPlatformInfo::GetMaxFeatureLevel(InPlatform) >= ERHIFeatureLevel::SM5 && !Substrate::IsSubstrateBlendableGBufferEnabled(InPlatform);
		return IsSubstrateEnabled() && CVar.Get(InPlatform) && bSupported;
	}

	uint32 GlintLUTIndex()
	{
		return CVarSubstrateGlintsLUT.GetValueOnAnyThread()<= 0 ? 0u : 1u;
	}

	float GlintLevelBias()
	{
		return CVarSubstrateGlintsLevelBias.GetValueOnAnyThread();
	}

	float GlintLevelMin()
	{
		const float MinLevel = CVarSubstrateGlintsLevelMin.GetValueOnAnyThread();
		return MinLevel < 0.0f ? 0.0f : MinLevel;
	}

	bool IsSpecularProfileEnabled()
	{
		return IsSubstrateEnabled() && CVarSubstrateSpecularProfile.GetValueOnAnyThread() > 0;
	}

	bool IsSpecularProfileEnabled(EShaderPlatform InPlatform)
	{
		static FShaderPlatformCachedIniValue<int32> CVar(TEXT("r.Substrate.SpecularProfile"));
		return IsSubstrateEnabled() && CVar.Get(InPlatform) > 0;
	}

	bool IsToonProfileEnabled()
	{
		return IsSubstrateEnabled();
	}

	uint32 GetSheenQuality()
	{
		return 1u;
	}

	uint32 GetSheenQuality(EShaderPlatform InPlatform)
	{
		return 1u;
	}

	uint32 GetShadingQuality()
	{
		return CVarSubstrateShadingQuality.GetValueOnAnyThread();
	}

	uint32 GetShadingQuality(EShaderPlatform InPlatform)
	{
		static FShaderPlatformCachedIniValue<int32> CVarSubstrateShadingQualityPlatform(TEXT("r.Substrate.ShadingQuality"));
		return CVarSubstrateShadingQualityPlatform.Get(InPlatform);
	}
	
	bool IsDBufferPassEnabled(EShaderPlatform InPlatform)
	{
		// DBuffer pass is only available if high quality normal is disabled, or if we are on a console.
		// That is because in high quality normals requires a uint2 UAV which is only supported on some graphic cards on PC 
		// and this is an unknown when compiling shaders at this stage.
		// !!! If this is changed, please update all sites reading r.Substrate.DBufferPass !!!
		const uint32 NormalQuality = GetNormalQuality();
		const bool bDBufferPassSupported = NormalQuality == 0 || (NormalQuality > 0 && IsConsolePlatform(InPlatform));
		
		static FShaderPlatformCachedIniValue<int32> CVarSubstrateDBufferPassPlatform(TEXT("r.Substrate.DBufferPass"));
		return IsSubstrateEnabled() && IsUsingDBuffers(InPlatform) && bDBufferPassSupported && CVarSubstrateDBufferPassPlatform.Get(InPlatform) > 0;
	}

	// Accessor to experimental variable static local here to not have have it exposed in public header RenderUtils.h for now.
	static bool IsExperimentalToonUnifiedDiffuseEnabled()
	{
		static IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Substrate.Experimental.ToonUnifiedDiffuse"));
		return CVar && CVar->GetInt() > 0;
	}

	bool IsOpaqueRoughRefractionEnabled(EShaderPlatform InPlatform)
	{
		static FShaderPlatformCachedIniValue<int32> CVar(TEXT("r.Substrate.OpaqueMaterialRoughRefraction"));

		const bool bIsSubstrateBlendableGBufferEnabled = IsSubstrateBlendableGBufferEnabled(InPlatform);

		// By default sample closures from cvars using highest material quality level. 
		// If even the highest level does not have many closures then we assume lower levels won't neither.
		bool bManyClosures = GetClosurePerPixel(InPlatform, Substrate::IgnoreMaterialQualityLevelsForClosurePerPixel) > 1;

		// If we have per material quality level closure count constraints, we should check them all and assume bManyClosures==true only when any of them is >1 since material/global shaders are all compile for only one case.
		const FSubstratePerMaterialLevelQualityClosureCount& PerMaterialLevelQualityClosureCount = GetPerMaterialLevelQualityClosureCount(InPlatform);
		if (bManyClosures && !bIsSubstrateBlendableGBufferEnabled && PerMaterialLevelQualityClosureCount.bIsValid)
		{
			if (PerMaterialLevelQualityClosureCount.LowMaterialQualityMaxClosureCount > 1
				|| PerMaterialLevelQualityClosureCount.MediumMaterialQualityMaxClosureCount > 1
				|| PerMaterialLevelQualityClosureCount.HighMaterialQualityMaxClosureCount > 1
				|| PerMaterialLevelQualityClosureCount.EpicMaterialQualityMaxClosureCount > 1)
			{
				bManyClosures = true;
			}
			else
			{
				// All material quality level will constrain the closure count to 1 so set bManyClosures to false.
				bManyClosures = false;
			}
		}

		return IsSubstrateEnabled() && CVar.Get(InPlatform) > 0 && !bIsSubstrateBlendableGBufferEnabled && bManyClosures && !IsExperimentalToonUnifiedDiffuseEnabled();
	}

	bool IsAdvancedVisualizationEnabled()
	{
		return IsAdvancedVisualizationEnabled(GMaxRHIShaderPlatform);
	}

	bool IsAdvancedVisualizationEnabled(EShaderPlatform InPlatform)
	{
		return CVarSubstrateDebugAdvancedVisualizationShaders.GetValueOnAnyThread() > 0 && !IsSubstrateBlendableGBufferEnabled(InPlatform);
	}

	bool IsMaterialLayeringSupportEnabled()
	{
		return false;
	}

	bool IsHiddenMaterialAssetConversionEnabled()
	{
		// Set this bool to false when we want to force all material to be converted to substrate.
		const bool bSubstrateHiddenMaterialAssetConversion = true;

		return IsSubstrateEnabled() && bSubstrateHiddenMaterialAssetConversion;
	}

	bool AreLightingPassesTiled()
	{
		return CVarSubstrateSimpleTiledLightingPasses.GetValueOnAnyThread() <= 0;
	}

	bool UseTileSize16()
	{
		return CVarSubstrateSimpleTiledLightingPasses.GetValueOnAnyThread() > 0;
	}
}

bool PlatformSupportsSeparatedSubsurfaceDiffuse(const FStaticShaderPlatform Platform)
{
	static FShaderPlatformCachedIniValue<int32> PerPlatformCVar(TEXT("r.SSS.SeparatedDiffuse.Supported"));
	return (PerPlatformCVar.Get(Platform) != 0) && !Substrate::IsOpaqueRoughRefractionEnabled(Platform);
}

bool HasFirstPersonGBufferBit(EShaderPlatform ShaderPlatform)
{
	// FirstPerson uses the ZERO_PRECSHADOW_MASK GBuffer bit, which is free if static lighting is disabled.
	return !IsMobilePlatform(ShaderPlatform) && !IsForwardShadingEnabled(ShaderPlatform) && !IsStaticLightingAllowed();
}


bool DoesProjectSupportLumenGI(EShaderPlatform Platform)
{
	static FShaderPlatformCachedIniValue<int32> CVarLumenSupported(TEXT("r.Lumen.Supported"));
	return CVarLumenSupported.Get(Platform) != 0;
}

bool DoesPlatformSupportLumenGI(EShaderPlatform Platform, bool bSkipProjectCheck)
{
	bool bLumenSupported = DoesProjectSupportLumenGI(Platform);

	EShaderPlatform ParentPreviewPlatform = SP_NumPlatforms;
	#if WITH_EDITOR
	if (GIsEditor)
	{
		if (FDataDrivenShaderPlatformInfo::GetIsPreviewPlatform(Platform))
		{
			ParentPreviewPlatform = FDataDrivenShaderPlatformInfo::GetPreviewShaderPlatformParent(Platform);
		}
	}
	#endif // WITH_EDITOR

	const bool bMetalSM5 = Platform == SP_METAL_SM5 || ParentPreviewPlatform == SP_METAL_SM5;
	const bool bVulkanSM5 = Platform == SP_VULKAN_SM5 || Platform == SP_VULKAN_SM5_ANDROID || ParentPreviewPlatform == SP_VULKAN_SM5 || ParentPreviewPlatform == SP_VULKAN_SM5_ANDROID;
	return (bSkipProjectCheck || bLumenSupported)
		&& FDataDrivenShaderPlatformInfo::GetSupportsLumenGI(Platform)
		&& (IsFeatureLevelSupported(Platform, ERHIFeatureLevel::SM6) || bVulkanSM5 || bMetalSM5) // Android and Mac can't rely on SM6 yet, but want to run Lumen
		&& !IsForwardShadingEnabled(Platform);
}

static TAutoConsoleVariable<int32> CVarLumenReflectionsHardwareRayTracingTranslucentRefractionEnableForProject(
	TEXT("r.Lumen.Reflections.HardwareRayTracing.Translucent.Refraction.EnableForProject"),
	1,
	TEXT("Whether to use Lumen refraction tracing from surfaces when using harware ray tracing and hit lighting. This will require shader recompilation to compile of translucent card capture Lumen shaders. Increases GPU cost when enabled."),
	ECVF_ReadOnly | ECVF_RenderThreadSafe
);

bool DoesProjectSupportLumenRayTracedTranslucentRefraction()
{
	int32 LumenTranslucentRefractionEnabled = CVarLumenReflectionsHardwareRayTracingTranslucentRefractionEnableForProject.GetValueOnAnyThread();
	return LumenTranslucentRefractionEnabled > 0;
}

bool ShouldCompileMegaLightsShaders(EShaderPlatform ShaderPlatform)
{
	if (IsMobilePlatform(ShaderPlatform))
	{
		return false;
	}

	static const TConsoleVariableData<bool>* CVar = IConsoleManager::Get().FindTConsoleVariableDataBool(TEXT("r.MegaLights.Supported"));
	const bool bMegaLightsSupported = CVar && CVar->GetValueOnAnyThread();

	if (!bMegaLightsSupported)
	{
		return false;
	}

	// SM6 because it uses typed loads to accumulate lights
	return IsFeatureLevelSupported(ShaderPlatform, ERHIFeatureLevel::SM6)
		&& RHISupportsWaveOperations(ShaderPlatform)
		&& RHISupportsRayTracing(ShaderPlatform);
}

static TAutoConsoleVariable<int32> CVarForwardShadingForceSkyBoxBlending(
	TEXT("r.ForwardShading.ForceSkyLightCubemapBlending"),
	0,
	TEXT("Forces in shader sky box blending for forward shading. More expenssive but higher quality. This is similar to having Blend Sky Light Cubemaps = true on all forward materials."),
	ECVF_ReadOnly | ECVF_RenderThreadSafe);

bool ForwardShadingForcesSkyLightCubemapBlending(const FStaticShaderPlatform Platform)
{
	static FShaderPlatformCachedIniValue<int32> CVar(TEXT("r.ForwardShading.ForceSkyLightCubemapBlending"));
	return CVar.Get(Platform) > 0;
}

bool SupportsSkyLightCaptureAffectsHeightFogInScattering(const FStaticShaderPlatform Platform)
{
	static FShaderPlatformCachedIniValue<int32> CVar(TEXT("r.SupportsSkyLightCaptureAffectsHeightFogInScattering"));
	return CVar.Get(Platform) > 0;
}

bool HardwareVariableRateShadingSupportedByPlatform(EShaderPlatform ShaderPlatform)
{
	const static auto SupportVRSCVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.VRS.Support"));
	if (SupportVRSCVar && SupportVRSCVar->GetValueOnAnyThread() != 0)
	{
		return FDataDrivenShaderPlatformInfo::GetSupportsVariableRateShading(ShaderPlatform);
	}

	return false;
}

static TAutoConsoleVariable<int32> CVarSupportExpFogMatchesVolumetricFog(
	TEXT("r.SupportExpFogMatchesVolumetricFog"),
	0,
	TEXT("When enabled, the height fog scattering/ambient/emissive/phase will match the volumetric fog better."),
	ECVF_ReadOnly | ECVF_RenderThreadSafe);

bool DoesProjectSupportExpFogMatchesVolumetricFog()
{
	return CVarSupportExpFogMatchesVolumetricFog.GetValueOnAnyThread() > 0;
}
