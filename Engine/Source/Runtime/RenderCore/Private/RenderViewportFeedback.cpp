// Copyright Epic Games, Inc. All Rights Reserved.

#include "RenderViewportFeedback.h"
#include "Misc/ScopeLock.h"

#if WITH_EDITOR

namespace UE::RenderViewportFeedback
{

FReceiver::FReceiver()
: InternalData()
, Mutex()
{ }

FRenderViewportFeedback FReceiver::Data() const
{
	UE::TScopeLock Lock(Mutex);
	return InternalData;
}

TSharedPtr<FCollector> FReceiver::MakeCollector()
{
	return MakeShared<FCollector>(*this);
}

FCollector::FCollector(FReceiver& InReceiver)
: Receiver(InReceiver.AsWeak())
, InternalData()
{ }

void FCollector::EndFrameRenderThread()
{
	TSharedPtr<FReceiver> Feedback = Receiver.Pin();
	if (Feedback)
	{
		UE::TScopeLock Lock(Feedback->Mutex);
		Feedback->InternalData = MoveTemp(InternalData);
	}
}

};

#endif //WITH_EDITOR
