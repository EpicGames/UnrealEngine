// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	RenderingThread.cpp: Rendering thread implementation.
=============================================================================*/

#include "RenderingThread.h"
#include "HAL/Runnable.h"
#include "HAL/RunnableThread.h"
#include "HAL/ExceptionHandling.h" // IWYU pragma: keep
#include "HAL/PlatformApplicationMisc.h"
#include "Misc/App.h"
#include "Misc/CommandLine.h"
#include "Misc/OutputDeviceRedirector.h"
#include "Misc/CoreStats.h"
#include "Misc/TimeGuard.h"
#include "Misc/CoreDelegates.h"
#include "Misc/ScopeLock.h"
#include "RHI.h"
#include "RenderCommandStateStream.h"
#include "RenderCore.h"
#include "RenderCommandFence.h"
#include "RenderDeferredCleanup.h"
#include "TickableObjectRenderThread.h"
#include "Stats/StatsSystem.h"
#include "Stats/StatsData.h"
#include "HAL/ThreadHeartBeat.h"
#include "RenderResource.h"
#include "RHIUtilities.h"
#include "Misc/ScopeLock.h"
#include "HAL/LowLevelMemTracker.h"
#include "ProfilingDebugging/MiscTrace.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "Async/TaskTrace.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "HAL/ThreadManager.h"
#include "ProfilingDebugging/CountersTrace.h"
#include "Async/ParallelFor.h"

//
// Globals
//

FCoreRenderDelegates::FOnFlushRenderingCommandsStart FCoreRenderDelegates::OnFlushRenderingCommandsStart;
FCoreRenderDelegates::FOnFlushRenderingCommandsEnd FCoreRenderDelegates::OnFlushRenderingCommandsEnd;

UE_TRACE_CHANNEL_DEFINE(RenderCommandsChannel, "Render thread command execution and synchronization.");

RENDERCORE_API bool GIsThreadedRendering = false;
RENDERCORE_API bool GUseThreadedRendering = false;
RENDERCORE_API TOptional<bool> GPendingUseThreadedRendering;

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	RENDERCORE_API TAtomic<bool> GMainThreadBlockedOnRenderThread(false);
#endif // #if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)

static FRunnable* GRenderingThreadRunnable = nullptr;

/** If the rendering thread has been terminated by an unhandled exception, this contains the error message. */
FString GRenderingThreadError;

/**
 * Polled by the game thread to detect crashes in the rendering thread.
 * If the rendering thread crashes, it sets this variable to false.
 */
volatile bool GIsRenderingThreadHealthy = true;


/**
 * Maximum rate the rendering thread will tick tickables when idle (in Hz)
 */
float GRenderingThreadMaxIdleTickFrequency = 40.f;

/**
 * RT Task Graph polling.
 */

extern CORE_API bool GRenderThreadPollingOn;
extern CORE_API int32 GRenderThreadPollPeriodMs;

static void OnRenderThreadPollPeriodMsChanged(IConsoleVariable* Var)
{
	const int32 DesiredRTPollPeriod = Var->GetInt();

	GRenderThreadPollingOn = (DesiredRTPollPeriod >= 0);
	ENQUEUE_RENDER_COMMAND(WakeupCommand)([DesiredRTPollPeriod](FRHICommandListImmediate&)
	{
		GRenderThreadPollPeriodMs = DesiredRTPollPeriod;
	});
}

static FAutoConsoleVariable CVarRenderThreadPollPeriodMs(
	TEXT("TaskGraph.RenderThreadPollPeriodMs"),
	1,
	TEXT("Render thread polling period in milliseconds. If value < 0, task graph tasks explicitly wake up RT, otherwise RT polls for tasks."),
	FConsoleVariableDelegate::CreateStatic(&OnRenderThreadPollPeriodMsChanged)
);

bool GRenderCommandFenceBundling = true;
FAutoConsoleVariableRef CVarRenderCommandFenceBundling(
	TEXT("r.RenderCommandFenceBundling"),
	GRenderCommandFenceBundling,
	TEXT("Controls whether render command fences are allowed to be batched.\n")
	TEXT(" 0: disabled;\n")
	TEXT(" 1: enabled (default);\n"),
	ECVF_Default);

inline ERenderCommandPipeMode GetValidatedRenderCommandPipeMode(int32 CVarValue)
{
	ERenderCommandPipeMode Mode = ERenderCommandPipeMode::None;

	switch (CVarValue)
	{
	case 1:
		Mode = ERenderCommandPipeMode::RenderThread;
		break;
	case 2:
		Mode = ERenderCommandPipeMode::All;
		break;
	}

	const bool bAllowThreading = !GRHICommandList.Bypass() && FApp::ShouldUseThreadingForPerformance() && GIsThreadedRendering;

	if (Mode == ERenderCommandPipeMode::All && !bAllowThreading)
	{
		Mode = ERenderCommandPipeMode::RenderThread;
	}

	if (!FApp::CanEverRender() || IsMobilePlatform(GMaxRHIShaderPlatform))
	{
		Mode = ERenderCommandPipeMode::None;
	}

	return Mode;
}

ERenderCommandPipeMode GRenderCommandPipeMode = ERenderCommandPipeMode::None;
FAutoConsoleVariable CVarRenderCommandPipeMode(
	TEXT("r.RenderCommandPipeMode"),
	2,
	TEXT("Controls behavior of the main render thread command pipe.")
	TEXT(" 0: Render commands are launched individually as tasks;\n")
	TEXT(" 1: Render commands are enqueued into a render command pipe for the render thread only.;\n")
	TEXT(" 2: Render commands are enqueued into a render command pipe for all declared pipes.;\n"),
	FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* Variable)
	{
		UE::RenderCommandPipe::StopRecording();
		GRenderCommandPipeMode = GetValidatedRenderCommandPipeMode(Variable->GetInt());
	}));

/** How many cycles the renderthread used (excluding idle time). It's set once per frame in FViewport::Draw. */
uint32 GRenderThreadTime = 0;
/** How many cycles of wait time renderthread used. It's set once per frame in FViewport::Draw. */
uint32 GRenderThreadWaitTime = 0;
/** How many cycles the rhithread used (excluding idle time). */
uint32 GRHIThreadTime = 0;
/** How many cycles the renderthread used, including dependent wait time. */
uint32 GRenderThreadTimeCriticalPath = 0;

/** The RHI thread runnable object. */
class FRHIThread : private FRunnable
{
	FRunnableThread* Thread = nullptr;

public:
	static inline ERHIThreadMode TargetMode = ERHIThreadMode::DedicatedThread;

	FRHIThread()
	{
		check(IsInGameThread());

		UE::Trace::ThreadGroupBegin(TEXT("Render"));

		Thread = FRunnableThread::Create(
			this, 
			TEXT("RHIThread"), 
			512 * 1024,
			FPlatformAffinity::GetRHIThreadPriority(), 
			FPlatformAffinity::GetRHIThreadMask(), 
			FPlatformAffinity::GetRHIThreadFlags()
		);
		check(Thread);

		UE::Trace::ThreadGroupEnd();
	}

	~FRHIThread()
	{
		check(IsInGameThread());

		// Signal the task graph to make the RHI thread exit, and wait for it.
		TGraphTask<FReturnGraphTask>::CreateTask(nullptr, ENamedThreads::GameThread).ConstructAndDispatchWhenReady(ENamedThreads::RHIThread);
		Thread->WaitForCompletion();

		delete Thread;
	}

	virtual uint32 Run() override
	{
		LLM_SCOPE(ELLMTag::RHIMisc);

#if CSV_PROFILER_STATS
		FCsvProfiler::Get()->SetRHIThreadId(FPlatformTLS::GetCurrentThreadId());
#endif
		{
			FTaskTagScope Scope(ETaskTag::ERhiThread);

			FMemory::SetupTLSCachesOnCurrentThread();
			{
				FScopedRHIThreadOwnership ThreadOwnershipScope(true);
			
				FTaskGraphInterface::Get().AttachToThread(ENamedThreads::RHIThread);
				FTaskGraphInterface::Get().ProcessThreadUntilRequestReturn(ENamedThreads::RHIThread);
			}
			FMemory::ClearAndDisableTLSCachesOnCurrentThread();
		}

#if CSV_PROFILER_STATS
		FCsvProfiler::Get()->SetRHIThreadId(0);
#endif

		return 0;
	}
} static *GRHIThread = nullptr;

/** The rendering thread main loop */
void RenderingThreadMain( FEvent* TaskGraphBoundSyncEvent )
{
	LLM_SCOPE(ELLMTag::RenderingThreadMemory);

	ENamedThreads::Type RenderThread = ENamedThreads::Type(ENamedThreads::ActualRenderingThread);

	ENamedThreads::SetRenderThread(RenderThread);
	ENamedThreads::SetRenderThread_Local(ENamedThreads::Type(ENamedThreads::ActualRenderingThread_Local));

	FTaskGraphInterface::Get().AttachToThread(RenderThread);
	FPlatformMisc::MemoryBarrier();

	// Inform main thread that the render thread has been attached to the taskgraph and is ready to receive tasks
	if( TaskGraphBoundSyncEvent != NULL )
	{
		TaskGraphBoundSyncEvent->Trigger();
	}

#if STATS
	if (FThreadStats::WillEverCollectData())
	{
		FTaskTagScope Scope(ETaskTag::ERenderingThread);
		FThreadStats::ExplicitFlush(); // flush the stats and set update the scope so we don't flush again until a frame update, this helps prevent fragmentation
	}
#endif

	FCoreDelegates::PostRenderingThreadCreated.Broadcast();
	check(GIsThreadedRendering);
	{
		FTaskTagScope TaskTagScope(ETaskTag::ERenderingThread);

		// Acquire rendering context ownership on the current thread, unless using an RHI thread, which will be the real owner
		FScopedRHIThreadOwnership ThreadOwnershipScope(!IsRunningRHIInSeparateThread());

		FTaskGraphInterface::Get().ProcessThreadUntilRequestReturn(RenderThread);
	}
	FPlatformMisc::MemoryBarrier();
	check(!GIsThreadedRendering);
	FCoreDelegates::PreRenderingThreadDestroyed.Broadcast();
	
#if STATS
	if (FThreadStats::WillEverCollectData())
	{
		FThreadStats::ExplicitFlush(); // Another explicit flush to clean up the ScopeCount established above for any stats lingering since the last frame
	}
#endif
	
	ENamedThreads::SetRenderThread(ENamedThreads::GameThread);
	ENamedThreads::SetRenderThread_Local(ENamedThreads::GameThread_Local);
	FPlatformMisc::MemoryBarrier();
}

/**
 * Advances stats for the rendering thread.
 */
static void AdvanceRenderingThreadStats(int64 StatsFrame, int32 DisableChangeTagStartFrame)
{
#if STATS
	int64 Frame = StatsFrame;
	if (!FThreadStats::IsCollectingData() || DisableChangeTagStartFrame != FThreadStats::PrimaryDisableChangeTag())
	{
		Frame = -StatsFrame; // mark this as a bad frame
	}
	FThreadStats::AddMessage(FStatConstants::AdvanceFrame.GetEncodedName(), EStatOperation::AdvanceFrameEventRenderThread, Frame);

	// Propagate the stats frame value down to the end-of-pipe thread.
	UE::Stats::FStats::StatsFrameRT = Frame;

	if (IsInActualRenderingThread())
	{
		FThreadStats::ExplicitFlush();
	}
#endif
}

/**
 * Advances stats for the rendering thread. Called from the game thread.
 */
void AdvanceRenderingThreadStatsGT( bool bDiscardCallstack, int64 StatsFrame, int32 DisableChangeTagStartFrame )
{
	ENQUEUE_RENDER_COMMAND(RenderingThreadTickCommand)(
		[StatsFrame, DisableChangeTagStartFrame](FRHICommandList& RHICmdList)
		{
			AdvanceRenderingThreadStats(StatsFrame, DisableChangeTagStartFrame);
		}
	);
	if( bDiscardCallstack )
	{
		// we need to flush the rendering thread here, otherwise it can get behind and then the stats will get behind.
		FlushRenderingCommands();
	}
}

/** The rendering thread runnable object. */
class FRenderingThread : public FRunnable
{
public:
	/** 
	 * Sync event to make sure that render thread is bound to the task graph before main thread queues work against it.
	 */
	FEvent* TaskGraphBoundSyncEvent;

	FRenderingThread()
	{
		TaskGraphBoundSyncEvent	= FPlatformProcess::GetSynchEventFromPool(true);
	}

	virtual ~FRenderingThread()
	{
		FPlatformProcess::ReturnSynchEventToPool(TaskGraphBoundSyncEvent);
		TaskGraphBoundSyncEvent = nullptr;
	}

	// FRunnable interface.
	virtual bool Init(void) override
	{ 
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		GRenderThreadId = FPlatformTLS::GetCurrentThreadId();
PRAGMA_ENABLE_DEPRECATION_WARNINGS
		FTaskTagScope::SetTagNone();
		return true; 
	}

	virtual void Exit(void) override
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		GRenderThreadId = 0;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

#if PLATFORM_WINDOWS && !PLATFORM_SEH_EXCEPTIONS_DISABLED
	static int32 FlushRHILogsAndReportCrash(Windows::LPEXCEPTION_POINTERS ExceptionInfo)
	{
		if (GDynamicRHI)
		{
			GDynamicRHI->FlushPendingLogs();
		}

		return ReportCrash(ExceptionInfo);
	}
#endif
	
	void SetupRenderThread()
	{
		FTaskTagScope Scope(ETaskTag::ERenderingThread);
		FPlatformProcess::SetupRenderThread();
	}


	virtual uint32 Run(void) override
	{
		FMemory::SetupTLSCachesOnCurrentThread();
		SetupRenderThread();

#if PLATFORM_WINDOWS
		bool bNoExceptionHandler = FParse::Param(FCommandLine::Get(), TEXT("noexceptionhandler"));
		if ( !bNoExceptionHandler && (!FPlatformMisc::IsDebuggerPresent() || GAlwaysReportCrash))
		{
#if !PLATFORM_SEH_EXCEPTIONS_DISABLED
			__try
#endif
			{
				RenderingThreadMain( TaskGraphBoundSyncEvent );
			}
#if !PLATFORM_SEH_EXCEPTIONS_DISABLED
			__except (FPlatformMisc::GetCrashHandlingType() == ECrashHandlingType::Default ?
							FlushRHILogsAndReportCrash(GetExceptionInformation()) : 
							EXCEPTION_CONTINUE_SEARCH)
			{
#if !NO_LOGGING
				// Dump the error and flush the log. This is the same logging behavior as FWindowsErrorOutputDevice::HandleError which is called in GuardedMain's caller's __except
				FDebug::LogFormattedMessageWithCallstack(LogWindows.GetCategoryName(), __FILE__, __LINE__, TEXT("=== Critical error: ==="), GErrorHist, ELogVerbosity::Error);
#endif
				GLog->Panic();

				GRenderingThreadError = GErrorHist;

				// Use a memory barrier to ensure that the game thread sees the write to GRenderingThreadError before
				// the write to GIsRenderingThreadHealthy.
				FPlatformMisc::MemoryBarrier();

				GIsRenderingThreadHealthy = false;
			}
#endif
		}
		else
#endif // PLATFORM_WINDOWS
		{
			RenderingThreadMain( TaskGraphBoundSyncEvent );
		}
		FMemory::ClearAndDisableTLSCachesOnCurrentThread();
		return 0;
	}
};

/**
 * If the rendering thread is in its idle loop (which ticks rendering tickables
 */
TAtomic<bool> GRunRenderingThreadHeartbeat;

FThreadSafeCounter OutstandingHeartbeats;

/** rendering tickables shouldn't be updated during a flush */
TAtomic<int32> GSuspendRenderingTickables;
struct FSuspendRenderingTickables
{
	FSuspendRenderingTickables()
	{
		++GSuspendRenderingTickables;
	}
	~FSuspendRenderingTickables()
	{
		--GSuspendRenderingTickables;
	}
};

/** The rendering thread heartbeat runnable object. */
class FRenderingThreadTickHeartbeat : public FRunnable
{
public:

	// FRunnable interface.
	virtual bool Init(void) 
	{
		GSuspendRenderingTickables = 0;
		OutstandingHeartbeats.Reset();
		return true; 
	}

	virtual void Exit(void) 
	{
	}

	virtual void Stop(void)
	{
	}

	virtual uint32 Run(void)
	{
		while(GRunRenderingThreadHeartbeat.Load(EMemoryOrder::Relaxed))
		{
			FPlatformProcess::Sleep(1.f/(4.0f * GRenderingThreadMaxIdleTickFrequency));
			if (OutstandingHeartbeats.GetValue() < 4)
			{
				OutstandingHeartbeats.Increment();
				ENQUEUE_RENDER_COMMAND(HeartbeatTickTickables)(
					[](FRHICommandListImmediate& RHICmdList)
					{
						OutstandingHeartbeats.Decrement();
						// make sure that rendering thread tickables get a chance to tick, even if the render thread is starving
						// but if GSuspendRenderingTickables is != 0 a flush is happening so don't tick during it
						if (!GSuspendRenderingTickables.Load(EMemoryOrder::Relaxed))
						{
							TickRenderingTickables(RHICmdList);
						}
					});
			}
		}
		return 0;
	}
};

FRunnableThread* GRenderingThreadHeartbeat = NULL;
FRunnable* GRenderingThreadRunnableHeartbeat = NULL;

// not done in the CVar system as we don't access to render thread specifics there
struct FConsoleRenderThreadPropagation : public IConsoleThreadPropagation
{
	virtual void OnCVarChange(int32& Dest, int32 NewValue)
	{
		int32* DestPtr = &Dest;
		ENQUEUE_RENDER_COMMAND(OnCVarChange1)(
			[DestPtr, NewValue](FRHICommandListImmediate& RHICmdList)
			{
				*DestPtr = NewValue;
			});
	}
	
	virtual void OnCVarChange(float& Dest, float NewValue)
	{
		float* DestPtr = &Dest;
		ENQUEUE_RENDER_COMMAND(OnCVarChange2)(
			[DestPtr, NewValue](FRHICommandListImmediate& RHICmdList)
			{
				*DestPtr = NewValue;
			});
	}

	virtual void OnCVarChange(bool& Dest, bool NewValue)
	{
		bool* DestPtr = &Dest;
		ENQUEUE_RENDER_COMMAND(OnCVarChange2)(
			[DestPtr, NewValue](FRHICommandListImmediate& RHICmdList)
			{
				*DestPtr = NewValue;
			});
	}
	
	virtual void OnCVarChange(FString& Dest, const FString& NewValue)
	{
		FString* DestPtr = &Dest;
		ENQUEUE_RENDER_COMMAND(OnCVarChange3)(
			[DestPtr, NewValue](FRHICommandListImmediate& RHICmdList)
			{
				*DestPtr = NewValue;
			});
	}

	virtual void OnCVarChange(FName& Dest, const FName& NewValue)
	{
		FName* DestPtr = &Dest;
		ENQUEUE_RENDER_COMMAND(OnCVarChange3)(
			[DestPtr, NewValue](FRHICommandListImmediate& RHICmdList)
			{
				*DestPtr = NewValue;
			});
	}

	static FConsoleRenderThreadPropagation& GetSingleton()
	{
		static FConsoleRenderThreadPropagation This;

		return This;
	}

};

static FString BuildRenderingThreadName( uint32 ThreadIndex )
{
	return FString::Printf( TEXT( "%s %u" ), *FName( NAME_RenderThread ).GetPlainNameString(), ThreadIndex );
}

static FRunnableThread* GRenderingThread = nullptr;

static void StartRenderingThread()
{
	check(IsInGameThread());

	// Do nothing if we're already in the right mode
	if (GIsThreadedRendering || !GUseThreadedRendering)
	{
		check(GIsThreadedRendering == GUseThreadedRendering);
		return;
	}

	check(!IsRHIThreadRunning() && !GIsRunningRHIInSeparateThread_InternalUseOnly && !GIsRunningRHIInDedicatedThread_InternalUseOnly && !GIsRunningRHIInTaskThread_InternalUseOnly);

	// Pause asset streaming to prevent rendercommands from being enqueued.
	SuspendTextureStreamingRenderTasks();

	// Flush GT since render commands issued by threads other than GT are sent to
	// the main queue of GT when RT is disabled. Without this flush, those commands
	// will run on GT after RT is enabled
	FlushRenderingCommands();

	GDynamicRHI->RHIReleaseThreadOwnership();

	switch (GRHISupportsRHIThread ? FRHIThread::TargetMode : ERHIThreadMode::None)
	{
	case ERHIThreadMode::DedicatedThread:
		GIsRunningRHIInSeparateThread_InternalUseOnly  = true;
		GIsRunningRHIInDedicatedThread_InternalUseOnly = true;
		GIsRunningRHIInTaskThread_InternalUseOnly      = false;

		// Start the dedicated RHI thread
		GRHIThread = new FRHIThread();
		break;

	case ERHIThreadMode::Tasks:
		GIsRunningRHIInSeparateThread_InternalUseOnly  = true;
		GIsRunningRHIInDedicatedThread_InternalUseOnly = false;
		GIsRunningRHIInTaskThread_InternalUseOnly      = true;
		break;

	default: checkNoEntry(); [[fallthrough]];
	case ERHIThreadMode::None:
		GIsRunningRHIInSeparateThread_InternalUseOnly  = false;
		GIsRunningRHIInDedicatedThread_InternalUseOnly = false;
		GIsRunningRHIInTaskThread_InternalUseOnly      = false;
		break;
	}

	// Turn on the threaded rendering flag.
	GIsThreadedRendering = true;

	// Create the rendering thread.
	GRenderingThreadRunnable = new FRenderingThread();

	static uint32 ThreadCount = 0;

	UE::Trace::ThreadGroupBegin(TEXT("Render"));
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	GRenderingThread =
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	FRunnableThread::Create(GRenderingThreadRunnable, *BuildRenderingThreadName(ThreadCount), 0, FPlatformAffinity::GetRenderingThreadPriority(), FPlatformAffinity::GetRenderingThreadMask(), FPlatformAffinity::GetRenderingThreadFlags());
	UE::Trace::ThreadGroupEnd();

	// Wait for render thread to have taskgraph bound before we dispatch any tasks for it.
	((FRenderingThread*)GRenderingThreadRunnable)->TaskGraphBoundSyncEvent->Wait();

	// register
	IConsoleManager::Get().RegisterThreadPropagation(0, &FConsoleRenderThreadPropagation::GetSingleton());

	ENQUEUE_RENDER_COMMAND(LatchBypass)([](FRHICommandListImmediate&)
	{
		GRHICommandList.LatchBypass();
	});

	// ensure the thread has actually started and is idling
	FRenderCommandFence Fence;
	Fence.BeginFence();
	Fence.Wait();

	GRenderCommandPipeMode = GetValidatedRenderCommandPipeMode(CVarRenderCommandPipeMode->GetInt());

	GRunRenderingThreadHeartbeat = true;
	// Create the rendering thread heartbeat
	GRenderingThreadRunnableHeartbeat = new FRenderingThreadTickHeartbeat();

	UE::Trace::ThreadGroupBegin(TEXT("Render"));
	GRenderingThreadHeartbeat = FRunnableThread::Create(GRenderingThreadRunnableHeartbeat, *FString::Printf(TEXT("RTHeartBeat %d"), ThreadCount), 80 * 1024, TPri_AboveNormal, FPlatformAffinity::GetRTHeartBeatMask());
	UE::Trace::ThreadGroupEnd();

	ThreadCount++;

	// Update can now resume.
	ResumeTextureStreamingRenderTasks();
}

static FStopRenderingThread GStopRenderingThreadDelegate;

FDelegateHandle RegisterStopRenderingThreadDelegate(const FStopRenderingThread::FDelegate& InDelegate)
{
	return GStopRenderingThreadDelegate.Add(InDelegate);
}

void UnregisterStopRenderingThreadDelegate(FDelegateHandle InDelegateHandle)
{
	GStopRenderingThreadDelegate.Remove(InDelegateHandle);
}

static void StopRenderingThread()
{
	// This function is not thread-safe. Ensure it is only called by the main game thread.
	check(IsInGameThread());

	if (!GIsThreadedRendering)
	{
		return;
	}

	// unregister
	IConsoleManager::Get().RegisterThreadPropagation();

	// stop the render thread heartbeat first
	if (GRunRenderingThreadHeartbeat)
	{
		GRunRenderingThreadHeartbeat = false;

		// Wait for the rendering thread heartbeat to return.
		GRenderingThreadHeartbeat->WaitForCompletion();

		delete GRenderingThreadHeartbeat;
		GRenderingThreadHeartbeat = nullptr;

		delete GRenderingThreadRunnableHeartbeat;
		GRenderingThreadRunnableHeartbeat = nullptr;
	}

	GStopRenderingThreadDelegate.Broadcast();

	// Get the list of objects which need to be cleaned up when the rendering thread is done with them.
	FPendingCleanupObjects* PendingCleanupObjects = GetPendingCleanupObjects();

	// Make sure we're not in the middle of streaming textures.
	SuspendTextureStreamingRenderTasks();

	// Wait for the rendering thread to finish executing all enqueued commands.
	FlushRenderingCommands();

	// Shutdown RHI thread
	delete GRHIThread;
	GRHIThread = nullptr;

	GIsRunningRHIInSeparateThread_InternalUseOnly  = false;
	GIsRunningRHIInDedicatedThread_InternalUseOnly = false;
	GIsRunningRHIInTaskThread_InternalUseOnly      = false;

	// Turn off the threaded rendering flag.
	GIsThreadedRendering = false;

	{
		FGraphEventRef QuitTask = TGraphTask<FReturnGraphTask>::CreateTask(nullptr, ENamedThreads::GameThread).ConstructAndDispatchWhenReady(ENamedThreads::GetRenderThread());

		// Busy wait while BP debugging, to avoid opportunistic execution of game thread tasks
		// If the game thread is already executing tasks, then we have no choice but to spin
		if (GIntraFrameDebuggingGameThread || FTaskGraphInterface::Get().IsThreadProcessingTasks(ENamedThreads::GameThread))
		{
			while ((QuitTask.GetReference() != nullptr) && !QuitTask->IsComplete())
			{
				FPlatformProcess::Sleep(0.0f);
			}
		}
		else
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_StopRenderingThread);
			FTaskGraphInterface::Get().WaitUntilTaskCompletes(QuitTask, ENamedThreads::GameThread_Local);
		}
	}

	// Wait for the rendering thread to return.
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	GRenderingThread->WaitForCompletion();

	// Destroy the rendering thread objects.
	delete GRenderingThread;
	GRenderingThread = nullptr;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	GDynamicRHI->RHIAcquireThreadOwnership();

	GRHICommandList.LatchBypass();

	delete GRenderingThreadRunnable;
	GRenderingThreadRunnable = nullptr;

	// Delete the pending cleanup objects which were in use by the rendering thread.
	delete PendingCleanupObjects;

	// Update can now resume with renderthread being the gamethread.
	ResumeTextureStreamingRenderTasks();

	check(!IsRHIThreadRunning());
}

RENDERCORE_API void LatchRenderThreadConfiguration()
{
	check(IsInGameThread());

	// Check for pending state changes from the "togglerenderingthread" and "r.RHIThread.Enable" commands.
	if ((GPendingUseThreadedRendering.IsSet() && GPendingUseThreadedRendering != GUseThreadedRendering) ||
		(GPendingRHIThreadMode.IsSet() && *GPendingRHIThreadMode != FRHIThread::TargetMode))
	{
		// Something changed. Stop and restart the rendering and RHI threads according to the new config.
		StopRenderingThread();

		if (GPendingUseThreadedRendering.IsSet())
		{
			GUseThreadedRendering = *GPendingUseThreadedRendering;
			GPendingUseThreadedRendering.Reset();
		}

		if (GPendingRHIThreadMode.IsSet())
		{
			FRHIThread::TargetMode = *GPendingRHIThreadMode;
			GPendingRHIThreadMode.Reset();
		}

		StartRenderingThread();
	}

	ENQUEUE_RENDER_COMMAND(LatchBypass)([](FRHICommandListImmediate&)
	{
		GRHICommandList.LatchBypass();
	});
}

RENDERCORE_API void InitRenderingThread()
{
	UE_CALL_ONCE([]()
	{
		if (FParse::Param(FCommandLine::Get(), TEXT("norhithread")))
		{
			FRHIThread::TargetMode = ERHIThreadMode::None;
		}

		SCOPED_BOOT_TIMING("StartRenderingThread");
		StartRenderingThread();
	});
}

RENDERCORE_API void ShutdownRenderingThread()
{
	UE_CALL_ONCE([]()
	{
		StopRenderingThread();
	});
}

void CheckRenderingThreadHealth()
{
	if(!GIsRenderingThreadHealthy)
	{
		GErrorHist[0] = 0;
		GIsCriticalError = false;
		UE_LOGF(LogRendererCore, Fatal,"Rendering thread exception:\r\n%ls",*GRenderingThreadError);
	}

	if (IsInGameThread())
	{
		if (!GIsCriticalError)
		{
			GLog->FlushThreadedLogs(EOutputDeviceRedirectorFlushOptions::Async);
		}
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		TGuardValue<TAtomic<bool>, bool> GuardMainThreadBlockedOnRenderThread(GMainThreadBlockedOnRenderThread,true);
#endif
		//QUICK_SCOPE_CYCLE_COUNTER(STAT_PumpMessages);
		FPlatformApplicationMisc::PumpMessages(false);
	}
}

bool IsRenderingThreadHealthy()
{
	return GIsRenderingThreadHealthy;
}

static struct FRenderCommandFenceBundlerState
{
	TOptional<UE::Tasks::FTaskEvent> Event;
	FRenderCommandPipeBitArray RenderCommandPipeBits;
	int32 RecursionDepth = 0;

} GRenderCommandFenceBundlerState; 

#define UE_RENDER_COMMAND_FENCE_BUNDLER_REGION TEXT("Render Command Fence Bundler")
#define UE_RENDER_COMMAND_PIPE_RECORD_REGION TEXT("Render Command Pipe Recording")
#define UE_RENDER_COMMAND_PIPE_SYNC_REGION TEXT("Render Command Pipe Synced")

#if UE_TRACE_ENABLED
#define UE_RENDER_COMMAND_BEGIN_REGION(Region) \
	if (RenderCommandsChannel) \
	{ \
		TRACE_BEGIN_REGION(Region) \
	}

#define UE_RENDER_COMMAND_END_REGION(Region) \
	if (RenderCommandsChannel) \
	{ \
		TRACE_END_REGION(Region) \
	}
#else
#define UE_RENDER_COMMAND_BEGIN_REGION(Region)
#define UE_RENDER_COMMAND_END_REGION(Region)
#endif

void StartRenderCommandFenceBundler()
{
	if (!GIsThreadedRendering || !GRenderCommandFenceBundling)
	{
		return;
	}

	check(IsInGameThread());
	check(!GRenderCommandFenceBundlerState.Event.IsSet() == !GRenderCommandFenceBundlerState.RecursionDepth);

	++GRenderCommandFenceBundlerState.RecursionDepth;

	if (GRenderCommandFenceBundlerState.RecursionDepth > 1)
	{
		return;
	}

	GRenderCommandFenceBundlerState.Event.Emplace(TEXT("RenderCommandFenceBundlerEvent"));

	// Stop render command pipes so that the bundled render command fence is serialized with other render commands.
	GRenderCommandFenceBundlerState.RenderCommandPipeBits = UE::RenderCommandPipe::StopRecording();

	StartBatchedRelease();

	UE_RENDER_COMMAND_BEGIN_REGION(UE_RENDER_COMMAND_FENCE_BUNDLER_REGION);
}

void FlushRenderCommandFenceBundler()
{
	if (GRenderCommandFenceBundlerState.Event)
	{
		EndBatchedRelease();

		ENQUEUE_RENDER_COMMAND(InsertFence)(
			[CompletionEvent = MoveTemp(*GRenderCommandFenceBundlerState.Event)](FRHICommandListBase&) mutable
		{
			CompletionEvent.Trigger();
		});

		GRenderCommandFenceBundlerState.Event.Emplace(TEXT("RenderCommandFenceBundlerEvent"));

		StartBatchedRelease();
	}
}

void StopRenderCommandFenceBundler()
{
	if (!GIsThreadedRendering || !GRenderCommandFenceBundlerState.Event)
	{
		return;
	}

	TOptional<UE::Tasks::FTaskEvent>& CompletionEvent = GRenderCommandFenceBundlerState.Event;

	check(CompletionEvent);
	check(!CompletionEvent->IsCompleted());
	check(GRenderCommandFenceBundlerState.RecursionDepth > 0);

	--GRenderCommandFenceBundlerState.RecursionDepth;

	if (GRenderCommandFenceBundlerState.RecursionDepth > 0)
	{
		return;
	}

	UE_RENDER_COMMAND_END_REGION(UE_RENDER_COMMAND_FENCE_BUNDLER_REGION);

	EndBatchedRelease();

	ENQUEUE_RENDER_COMMAND(InsertFence)(
		[CompletionEvent = MoveTemp(*CompletionEvent)](FRHICommandListBase&) mutable
	{
		CompletionEvent.Trigger();
	});

	CompletionEvent.Reset();

	// Restart render command pipes that were previously recording.
	UE::RenderCommandPipe::StartRecording(GRenderCommandFenceBundlerState.RenderCommandPipeBits);
	GRenderCommandFenceBundlerState.RenderCommandPipeBits.Empty();
}

std::atomic<int> GTimeoutSuspendCount;

void SuspendRenderThreadTimeout()
{
	++GTimeoutSuspendCount;
}

void ResumeRenderThreadTimeout()
{
	--GTimeoutSuspendCount;

	check(GTimeoutSuspendCount >= 0);
}

bool IsRenderThreadTimeoutSuspended()
{
	return GTimeoutSuspendCount > 0;
}

FRenderCommandFence::FRenderCommandFence() = default;
FRenderCommandFence::~FRenderCommandFence() = default;

void FRenderCommandFence::BeginFence(ESyncDepth SyncDepth)
{
	if (!GIsThreadedRendering)
	{
		return;
	}

	check(IsInGameThread());
	
	if (GRenderCommandFenceBundlerState.Event && SyncDepth == ESyncDepth::RenderThread)
	{
		// Case for game->render thread syncs when fence bundling is enabled. These are used
		// throughout the engine when resources are destroyed. The fence bundling is an optimization
		// to avoid the overhead of hundreds of individual fences.
		// We aren't syncing any deeper than the render thread, so just use the bundled fence event.
		CompletionTask = *GRenderCommandFenceBundlerState.Event;
		return;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(FRenderCommandFence::BeginFence);
	UE::Tasks::FTaskEvent Event{ UE_SOURCE_LOCATION };

	if (GRenderCommandFenceBundlerState.Event)
	{
		// Render command fences are bundled, but we're syncing deeper than the render thread.
		// Flush the fence bundler so we can insert an RHIThread (or deeper) fence in the right location.
		Event.AddPrerequisites(*GRenderCommandFenceBundlerState.Event);
		FlushRenderCommandFenceBundler();
	}

	if (GRenderCommandPipeMode == ERenderCommandPipeMode::All)
	{
		for (FRenderCommandPipe* Pipe : UE::RenderCommandPipe::GetPipes())
		{
			// Skip pipes that aren't recording or replaying any work.
			if (Pipe->IsRecording() && !Pipe->IsEmpty())
			{
				UE::Tasks::FTaskEvent PipeEvent { UE_SOURCE_LOCATION };
				Event.AddPrerequisites(PipeEvent);

				ENQUEUE_RENDER_COMMAND(BeginFence)(Pipe, [PipeEvent = MoveTemp(PipeEvent)](FRHICommandList&) mutable
				{
					PipeEvent.Trigger();
				});
			}
		}
	}

	ENQUEUE_RENDER_COMMAND(BeginFence)([Event, SyncDepth](FRHICommandListImmediate& RHICmdList) mutable
	{
		if (SyncDepth == ESyncDepth::Swapchain)
		{
			UE::Tasks::FTaskEvent SwapchainEvent{ UE_SOURCE_LOCATION };
			Event.AddPrerequisites(SwapchainEvent);

			RHICmdList.EnqueueLambda([SyncDepth, SwapchainEvent](FRHICommandListImmediate&) mutable
			{
				// This command runs *after* a present has happened, so the counter has already been incremented.
				// Subtracting 1 gives us the index of the frame that has *just* been presented.
				RHITriggerTaskEventOnFlip(GRHIPresentCounter - 1, SwapchainEvent);
			});

			RHICmdList.ImmediateFlush(EImmediateFlushType::DispatchToRHIThread);
		}
		else if (SyncDepth == ESyncDepth::RHIThread)
		{
			Event.AddPrerequisites(GRHICommandList.Submit({}, ERHISubmitFlags::SubmitToGPU));
		}

		TRACE_CPUPROFILER_EVENT_SCOPE(SyncTrigger_RenderThread);
		Event.Trigger();
	});

	CompletionTask = MoveTemp(Event);
}

bool FRenderCommandFence::IsFenceComplete() const
{
	if (!GIsThreadedRendering)
	{
		return true;
	}
	check(IsInGameThread() || IsInAsyncLoadingThread());
	CheckRenderingThreadHealth();
	if (CompletionTask.IsCompleted())
	{
		CompletionTask = {}; // this frees the handle for other uses, the NULL state is considered completed
		return true;
	}
	return false;
}

/** How many cycles the gamethread used (excluding idle time). It's set once per frame in FViewport::Draw. */
uint32 GGameThreadTime = 0;

/** How much idle time on the game thread. It's set once per frame in FViewport::Draw. */
uint32 GGameThreadWaitTime = 0;

/** How many cycles the gamethread used, including dependent wait time. */
uint32 GGameThreadTimeCriticalPath = 0;

/** How many cycles it took to swap buffers to present the frame. */
uint32 GSwapBufferTime = 0;

static int32 GTimeToBlockOnRenderFence = 1;
static FAutoConsoleVariableRef CVarTimeToBlockOnRenderFence(
	TEXT("g.TimeToBlockOnRenderFence"),
	GTimeToBlockOnRenderFence,
	TEXT("Number of milliseconds the game thread should block when waiting on a render thread fence.")
	);


static int32 GTimeoutForBlockOnRenderFence = 120000;
static FAutoConsoleVariableRef CVarTimeoutForBlockOnRenderFence(
	TEXT("g.TimeoutForBlockOnRenderFence"),
	GTimeoutForBlockOnRenderFence,
	TEXT("Number of milliseconds the game thread should wait before failing when waiting on a render thread fence.")
);

static void HandleRenderTaskHang(uint32 ThreadThatHung, double HangDuration)
{
	// Get the name of the hung thread
	FString ThreadName = FThreadManager::GetThreadName(ThreadThatHung);
	if (ThreadName.IsEmpty())
	{
		ThreadName = FString::Printf(TEXT("unknown thread (%u)"), ThreadThatHung);
	}
	
#if !PLATFORM_WINDOWS || (PLATFORM_USE_MINIMAL_HANG_DETECTION && 1)
	UE_LOGF(LogRendererCore, Fatal, "GameThread timed out waiting for %ls after %.02f secs", *ThreadName, HangDuration);
#else
	// Capture the stack in the thread that hung
	static const int32 MaxStackFrames = 100;
	uint64 StackFrames[MaxStackFrames];
	int32 NumStackFrames = FPlatformStackWalk::CaptureThreadStackBackTrace(ThreadThatHung, StackFrames, MaxStackFrames);

	// Convert the stack trace to text
	TArray<FString> StackLines;
	for (int32 Idx = 0; Idx < NumStackFrames; Idx++)
	{
		ANSICHAR Buffer[1024];
		Buffer[0] = '\0';
		FPlatformStackWalk::ProgramCounterToHumanReadableString(Idx, StackFrames[Idx], Buffer, sizeof(Buffer));
		StackLines.Add(Buffer);
	}
	
	// Dump the callstack and the thread name to log
	FString StackTrimmed;
	UE_LOGF(LogRendererCore, Error, "GameThread timed out waiting for %ls after %.02f seconds:", *ThreadName, HangDuration);
	for (int32 Idx = 0; Idx < StackLines.Num(); Idx++)
	{
		UE_LOGF(LogRendererCore, Error, "  %ls", *StackLines[Idx]);
		if (StackTrimmed.Len() < 512)
		{
			StackTrimmed += TEXT("  ");
			StackTrimmed += StackLines[Idx];
			StackTrimmed += LINE_TERMINATOR;
		}
	}

	const FString ErrorMessage = FString::Printf(TEXT("GameThread timed out waiting for %s after %.02f seconds:%s%s%sCheck log for full callstack."),
		*ThreadName, HangDuration, LINE_TERMINATOR, *StackTrimmed, LINE_TERMINATOR);
	
	GLog->Panic();
	FCoreDelegates::OnHandleSystemHang.Broadcast();
	ReportHang(*ErrorMessage, StackFrames, NumStackFrames, ThreadThatHung);
	if (FApp::CanEverRender())
	{
		FPlatformMisc::MessageBoxExt(EAppMsgType::Ok,
			*NSLOCTEXT("MessageDialog", "ReportHangError_Body", "The application has hung and will now close. We apologize for the inconvenience.").ToString(),
			*NSLOCTEXT("MessageDialog", "ReportHangError_Title", "Application Hang Detected").ToString());
	}
	
	GIsCriticalError = true;
	FPlatformMisc::RequestExit(true, TEXT("GameThreadWaitForTask"));
#endif
}

/**
 * Block the game thread waiting for a task to finish on the rendering thread.
 */
static void GameThreadWaitForTask(const UE::Tasks::FTask& Task, bool bEmptyGameThreadTasks = false)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(GameThreadWaitForTask);
	SCOPE_TIME_GUARD(TEXT("GameThreadWaitForTask"));

	check(IsInGameThread());
	check(Task.IsValid());

	if (!Task.IsCompleted())
	{
		SCOPE_CYCLE_COUNTER(STAT_GameIdleTime);
		{
			static int32 NumRecursiveCalls = 0;
		
			// Check for recursion. It's not completely safe but because we pump messages while 
			// blocked it is expected.
			NumRecursiveCalls++;
			if (NumRecursiveCalls > 1)
			{
				UE_LOGF(LogRendererCore,Warning,"FlushRenderingCommands called recursively! %d calls on the stack.", NumRecursiveCalls);
			}
			if (NumRecursiveCalls > 1 || FTaskGraphInterface::Get().IsThreadProcessingTasks(ENamedThreads::GameThread))
			{
				bEmptyGameThreadTasks = false; // we don't do this on recursive calls or if we are at a blueprint breakpoint
			}

			// Check rendering thread health needs to be called from time to
			// time in order to pump messages, otherwise the RHI may block
			// on vsync causing a deadlock. Also we should make sure the
			// rendering thread hasn't crashed :)
			bool bDone;
			uint32 WaitTime = FMath::Clamp<uint32>(GTimeToBlockOnRenderFence, 0, 33);

			// Use a clamped clock to prevent taking into account time spent suspended.
			FThreadHeartBeatClock RenderThreadTimeoutClock((4 * WaitTime) / 1000.0);
			const double StartTime = RenderThreadTimeoutClock.Seconds();
			const double EndTime = StartTime + (GTimeoutForBlockOnRenderFence / 1000.0);

			bool bRenderThreadEnsured = FDebug::IsEnsuring();

			static bool bDisabled = FParse::Param(FCommandLine::Get(), TEXT("nothreadtimeout"));

			// Creating the wait task manually is a workaround for the problem of FTast::Wait creating
			// a separate wait task and event object on each call. It's a problem because we may call
			// Wait it in the loop below many times during long frame syncs (e.g. when using GPU profilers)
			// which would create thousands of such objects and run out of system resources.
			FSharedEventRef CompletionEvent;

			UE::Tasks::Launch(
				TEXT("Waiting Task (FrameSync)"),
				[CompletionEvent] { CompletionEvent->Trigger(); },
				Task,
				LowLevelTasks::ETaskPriority::Default,
				UE::Tasks::EExtendedTaskPriority::Inline,
				UE::Tasks::ETaskFlags::None
			);

			do
			{
				CheckRenderingThreadHealth();
				if (bEmptyGameThreadTasks)
				{
					// process gamethread tasks if there are any
					FTaskGraphInterface::Get().ProcessThreadUntilIdle(ENamedThreads::GameThread);
				}
				bDone = CompletionEvent->Wait(FTimespan::FromMilliseconds(WaitTime));

				RenderThreadTimeoutClock.Tick();

				const bool bOverdue = RenderThreadTimeoutClock.Seconds() >= EndTime && FThreadHeartBeat::Get().IsBeating();

				// track whether the thread ensured, if so don't do timeout checks
				bRenderThreadEnsured |= FDebug::IsEnsuring();

#if !WITH_EDITOR
#if !PLATFORM_IOS && !PLATFORM_MAC // @todo Metal: Timeout isn't long enough...
				// editor threads can block for quite a while... 
				if (!bDone && !bRenderThreadEnsured)
				{
					if (bOverdue && !bDisabled && !IsRenderThreadTimeoutSuspended() && !FPlatformMisc::IsDebuggerPresent())
					{
						double HangDuration = RenderThreadTimeoutClock.Seconds() - StartTime;
						// TODO: Walk the wait chain instead of explicitly setting the render thread as the hung thread id
						PRAGMA_DISABLE_DEPRECATION_WARNINGS
						uint32 ThreadThatHung = GRenderThreadId;
						PRAGMA_ENABLE_DEPRECATION_WARNINGS
						HandleRenderTaskHang(ThreadThatHung, HangDuration);
					}
				}
#endif
#endif
			}
			while (!bDone);

			NumRecursiveCalls--;
		}
	}
}

/**
 * Waits for pending fence commands to retire.
 */
void FRenderCommandFence::Wait(bool bProcessGameThreadTasks) const
{
	if (!IsFenceComplete())
	{
		FRenderCommandList::FFlushScope FlushScope;
		FlushRenderCommandFenceBundler();
		GameThreadWaitForTask(CompletionTask, bProcessGameThreadTasks);
		CompletionTask = {}; // release the internal memory as soon as it's not needed anymore
	}
}

/**
 * Waits for the rendering thread to finish executing all pending rendering commands.  Should only be used from the game thread.
 */
void FlushRenderingCommands()
{
	if (!GIsRHIInitialized)
	{
		return;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(FlushRenderingCommands);
	FRenderCommandList::FFlushScope FlushScope;
	FCoreRenderDelegates::OnFlushRenderingCommandsStart.Broadcast();
	FSuspendRenderingTickables SuspendRenderingTickables;

	// Need to flush GT because render commands from threads other than GT are sent to
	// the main queue of GT when RT is disabled
	if (!GIsThreadedRendering
		&& !FTaskGraphInterface::Get().IsThreadProcessingTasks(ENamedThreads::GameThread)
		&& !FTaskGraphInterface::Get().IsThreadProcessingTasks(ENamedThreads::GameThread_Local))
	{
		FTaskGraphInterface::Get().ProcessThreadUntilIdle(ENamedThreads::GameThread);
		FTaskGraphInterface::Get().ProcessThreadUntilIdle(ENamedThreads::GameThread_Local);
	}

	UE::RenderCommandPipe::StopRecording();

	ENQUEUE_RENDER_COMMAND(FlushPendingDeleteRHIResourcesCmd)([](FRHICommandListImmediate& RHICmdList)
	{
		RHICmdList.ImmediateFlush(EImmediateFlushType::FlushRHIThreadFlushResources);
		//double flush to flush out the deferred deletions queued into the ImmediateCmdList
		RHICmdList.ImmediateFlush(EImmediateFlushType::FlushRHIThread);
	});

	// Find the objects which may be cleaned up once the rendering thread command queue has been flushed.
	FPendingCleanupObjects* PendingCleanupObjects = GetPendingCleanupObjects();

	// Issue a fence command to the rendering thread and wait for it to complete.
	// Use the frame end sync here, so that it cleans up outstanding graph events, which is necessary on engine shutdown.
	FFrameEndSync::Sync(FFrameEndSync::EFlushMode::Threads);

	// Delete the objects which were enqueued for deferred cleanup before the command queue flush.
	delete PendingCleanupObjects;

	FCoreRenderDelegates::OnFlushRenderingCommandsEnd.Broadcast();
}

FRHICommandListImmediate& GetImmediateCommandList_ForRenderCommand()
{
	return FRHICommandListExecutor::GetImmediateCommandList();
}

static bool bEnablePendingCleanupObjectsCommandBatching = true;
static FAutoConsoleVariableRef CVarEnablePendingCleanupObjectsCommandBatching(
	TEXT("g.bEnablePendingCleanupObjectsCommandBatching"),
	bEnablePendingCleanupObjectsCommandBatching,
	TEXT("Enable batching PendingCleanupObjects destruction.")
);

#if WITH_EDITOR || IS_PROGRAM

// mainly concerned about the cooker here, but anyway, the editor can run without a frame for a very long time (hours) and we do not have enough lock free links. 

/** The set of deferred cleanup objects which are pending cleanup. */
TArray<FDeferredCleanupInterface*> PendingCleanupObjectsList;
FCriticalSection PendingCleanupObjectsListLock;

FPendingCleanupObjects::FPendingCleanupObjects()
{
	check(IsInGameThread());
	{
		FScopeLock Lock(&PendingCleanupObjectsListLock);
		Exchange(CleanupArray, PendingCleanupObjectsList);
	}
}

FPendingCleanupObjects::~FPendingCleanupObjects()
{
	if (CleanupArray.Num())
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FPendingCleanupObjects_Destruct);

		const bool bBatchingEnabled = bEnablePendingCleanupObjectsCommandBatching;
		if (bBatchingEnabled)
		{
			StartRenderCommandFenceBundler();
		}
		for (int32 ObjectIndex = 0; ObjectIndex < CleanupArray.Num(); ObjectIndex++)
		{
			delete CleanupArray[ObjectIndex];
		}
		if (bBatchingEnabled)
		{
			StopRenderCommandFenceBundler();
		}
	}
}

void BeginCleanup(FDeferredCleanupInterface* CleanupObject)
{
	{
		FScopeLock Lock(&PendingCleanupObjectsListLock);
		PendingCleanupObjectsList.Add(CleanupObject);
	}
}

#else

/** The set of deferred cleanup objects which are pending cleanup. */
static TLockFreePointerListUnordered<FDeferredCleanupInterface, PLATFORM_CACHE_LINE_SIZE>	PendingCleanupObjectsList;

FPendingCleanupObjects::FPendingCleanupObjects()
{
	check(IsInGameThread());
	PendingCleanupObjectsList.PopAll(CleanupArray);
}

FPendingCleanupObjects::~FPendingCleanupObjects()
{
	if (CleanupArray.Num())
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FPendingCleanupObjects_Destruct);

		const bool bBatchingEnabled = bEnablePendingCleanupObjectsCommandBatching;
		if (bBatchingEnabled)
		{
			StartRenderCommandFenceBundler();
		}
		for (int32 ObjectIndex = 0; ObjectIndex < CleanupArray.Num(); ObjectIndex++)
		{
			delete CleanupArray[ObjectIndex];
		}
		if (bBatchingEnabled)
		{
			StopRenderCommandFenceBundler();
		}
	}
}

void BeginCleanup(FDeferredCleanupInterface* CleanupObject)
{
	PendingCleanupObjectsList.Push(CleanupObject);
}

#endif

FPendingCleanupObjects* GetPendingCleanupObjects()
{
	return new FPendingCleanupObjects;
}

static void HandleRHIThreadEnableChanged(const TArray<FString>& Args)
{
	check(IsInGameThread());
	switch (Args.Num() == 1 ? FCString::Atoi(*Args[0]) : -1)
	{
	case 0:
		GPendingRHIThreadMode = ERHIThreadMode::None;
		UE_LOGF(LogConsoleResponse, Display, "RHI thread will be disabled.")
		break;

	case 1:
		GPendingRHIThreadMode = ERHIThreadMode::DedicatedThread;
		UE_LOGF(LogConsoleResponse, Display, "RHI thread will be enabled (dedicated thread).")
		break;

	case 2:
		GPendingRHIThreadMode = ERHIThreadMode::Tasks;
		UE_LOGF(LogConsoleResponse, Display, "RHI thread will be enabled (task threads).")
		break;

	default:
		UE_LOGF(LogConsoleResponse, Display, "Usage: r.RHIThread.Enable 0=off,  1=dedicated thread,  2=task threads; Currently %d", IsRunningRHIInSeparateThread() ? (IsRunningRHIInDedicatedThread() ? 1 : 2) : 0);
		break;
	}
}

static FAutoConsoleCommand CVarRHIThreadEnable(
	TEXT("r.RHIThread.Enable"),
	TEXT("Enables/disabled the RHI Thread and determine if the RHI work runs on a dedicated thread or not.\n"),	
	FConsoleCommandWithArgsDelegate::CreateStatic(&HandleRHIThreadEnableChanged)
	);

//////////////////////////////////////////////////////////////////////////

namespace UE::RenderCommandPipe
{

bool FCommandList::Enqueue(FRenderCommandFunctionVariant&& Function, const FRenderCommandTag& Tag)
{
	return Enqueue(AllocNoDestruct<FExecuteFunctionCommand>(MoveTemp(Function), Tag));
}

bool FCommandList::Enqueue(FCommandList* CommandList)
{
	return Enqueue(AllocNoDestruct<FExecuteCommandListCommand>(CommandList));
}

bool FCommandList::Enqueue(FCommand* Command)
{
#if DO_CHECK
	check(!bClosed);
#endif

	bool bWasEmpty = IsEmpty();
	if (bWasEmpty)
	{
		Commands.Head = Commands.Tail = Command;
	}
	else
	{
		Commands.Tail->Next = Command;
		Commands.Tail       = Command;
	}
	Commands.Num++;
	return bWasEmpty;
}

void FCommandList::Release()
{
	if (!IsEmpty())
	{
		ensureMsgf(false, TEXT("UE::RenderCommandPipe::FCommandList was released without consuming commands. You must call ConsumeCommands first."));

		for (FCommand* Command = Commands.Head; Command; Command = Command->Next)
		{
			Command->~FCommand();
		}
	}
	Commands = {};
}

} //! UE::RenderCommandPipe

//////////////////////////////////////////////////////////////////////////

static void ExecuteCommand_NoMarker(FRHICommandList* RHICmdList, const FRenderCommandFunctionVariant& Function)
{
	switch (Function.GetIndex())
	{
	case FRenderCommandFunctionVariant::IndexOfType<TUniqueFunction<void(FRHICommandListImmediate&)>>():
		Function.Get<TUniqueFunction<void(FRHICommandListImmediate&)>>()(RHICmdList->GetAsImmediate());
		break;

	case FRenderCommandFunctionVariant::IndexOfType<TUniqueFunction<void(FRHICommandList&)>>():
		Function.Get<TUniqueFunction<void(FRHICommandList&)>>()(*RHICmdList);
		break;

	case FRenderCommandFunctionVariant::IndexOfType<TUniqueFunction<void()>>():
		Function.Get<TUniqueFunction<void()>>()();
		break;

	default: checkNoEntry();
	}
}

static void ExecuteCommand(FRHICommandList* RHICmdList, const FRenderCommandFunctionVariant& Function, const FRenderCommandTag& Tag)
{
	if (Tag.GetStatId().IsValidStat())
	{
		TRACE_CPUPROFILER_EVENT_SCOPE_USE_ON_CHANNEL(Tag.GetSpecId(), Tag.GetName(), EventScope, RenderCommandsChannel, !GCycleStatsShouldEmitNamedEvents);
		FScopeCycleCounter Scope(Tag.GetStatId(), true);
		ExecuteCommand_NoMarker(RHICmdList, Function);
	}
	else
	{
		ExecuteCommand_NoMarker(RHICmdList, Function);
	}
}

//////////////////////////////////////////////////////////////////////////

thread_local FRenderCommandList* FRenderCommandList::InstanceTLS = nullptr;

FRenderCommandList* FRenderCommandList::GetInstanceTLS()
{
	return InstanceTLS;
}

FRenderCommandList* FRenderCommandList::SetInstanceTLS(FRenderCommandList* CommandList)
{
	FRenderCommandList* PreviousCommandList = InstanceTLS;
	InstanceTLS = CommandList;
	return PreviousCommandList;
}

//////////////////////////////////////////////////////////////////////////

FRenderCommandList::FRecordScope::FRecordScope(FRenderCommandList* InCommandList, EStopRecordingAction InStopAction)
	: CommandList(InCommandList)
	, StopAction(InStopAction)
{
#if DO_CHECK
	if (CommandList)
	{
		CommandList->NumRecordScopeRefs++;
		checkf(CommandList->NumRecordScopeRefs != 0, TEXT("FRecordScope is either being nested recursively on the same command list or is recording from multiple threads."));
		check(CommandList->bRecording);
	}
#endif

	PreviousCommandList = SetInstanceTLS(InCommandList);
}

FRenderCommandList::FRecordScope::~FRecordScope()
{
	SetInstanceTLS(PreviousCommandList);

	if (CommandList)
	{
	#if DO_CHECK
		CommandList->NumRecordScopeRefs--;
	#endif

		if (StopAction == EStopRecordingAction::Close)
		{
			CommandList->Close();
		}
		else if (StopAction == EStopRecordingAction::Submit)
		{
			CommandList->Close();
			FRenderCommandDispatcher::Submit(CommandList);
		}
	}
}

FRenderCommandList::FFlushScope::FFlushScope()
{
	CommandList = SetInstanceTLS(nullptr);

	if (CommandList)
	{
		CommandList->Flush();
	}
}

FRenderCommandList::FFlushScope::~FFlushScope()
{
	if (CommandList)
	{
		SetInstanceTLS(CommandList);
	}
}

//////////////////////////////////////////////////////////////////////////

FRenderCommandList::FParallelForContext::FParallelForContext(FRenderCommandList* InRootCommandList, int32 NumTasks, int32 BatchSize, EParallelForFlags Flags)
	: FParallelForContext(InRootCommandList, ParallelForImpl::GetNumberOfThreadTasks(NumTasks, BatchSize, Flags))
{}

FRenderCommandList::FParallelForContext::FParallelForContext(FRenderCommandList* InRootCommandList, int32 NumContexts)
	: RootCommandList(InRootCommandList)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FRenderCommandList::FParallelForContext::Init);

	if (!RootCommandList)
	{
		RootCommandList = FRenderCommandList::Create(ERenderCommandListFlags::CloseOnSubmit);
		bSubmitRootCommandList = true;
	}

	TaskCommandLists.Reserve(NumContexts);

	for (int32 Index = 0; Index < NumContexts - 1; ++Index)
	{
		TaskCommandLists.Emplace(FRenderCommandList::Create(ERenderCommandListFlags::CloseOnSubmit));
	}

	TaskCommandLists.Emplace(RootCommandList);
}

void FRenderCommandList::FParallelForContext::Submit()
{
	if (RootCommandList)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FRenderCommandList::FParallelForContext::Submit);

		for (int32 Index = 0; Index < TaskCommandLists.Num() - 1; ++Index)
		{
			FRenderCommandDispatcher::Submit(TaskCommandLists[Index], RootCommandList);
		}

		if (bSubmitRootCommandList)
		{
			FRenderCommandDispatcher::Submit(RootCommandList);
		}

		RootCommandList = nullptr;
		TaskCommandLists.Empty();
	}
}

//////////////////////////////////////////////////////////////////////////

FRenderCommandList::FRenderCommandList(ERenderCommandListFlags InFlags, EPageSize PageSize)
	: Allocator(PageSize)
	, Flags(InFlags)
{
	if (!EnumHasAnyFlags(Flags, ERenderCommandListFlags::CloseOnSubmit))
	{
		DispatchTaskEvent.Emplace(UE_SOURCE_LOCATION);
		Init();
	}
}

FRenderCommandList::~FRenderCommandList()
{
	FRenderCommandList* Child = Children.Head;
	while (Child)
	{
		FRenderCommandList* NextChild = Child->NextSibling;
		delete Child;
		Child = NextChild;
	}
	Children = {};

	if (DispatchTaskEvent)
	{
		DispatchTaskEvent->Trigger();
	}
}

void FRenderCommandList::Flush()
{
	if (bSubmitted || !bInitialized || !EnumHasAnyFlags(Flags, ERenderCommandListFlags::CloseOnSubmit))
	{
		return;
	}

	FRenderCommandList* FlushCommandList = FRenderCommandList::Create(ERenderCommandListFlags::CloseOnSubmit);
	FlushCommandList->Allocator = MoveTemp(Allocator);
	FlushCommandList->CommandLists = MoveTemp(CommandLists);
	FlushCommandList->DispatchTaskEvent = MoveTemp(DispatchTaskEvent);
	FlushCommandList->bInitialized = true;
	FlushCommandList->Children = Children;

	FRenderCommandDispatcher::Submit(FlushCommandList);

	bInitialized = false;
	Children = {};
}

void FRenderCommandList::Init()
{
	check(!bInitialized);
	bInitialized = true;
	const int32 NumCommandLists = UE::RenderCommandPipe::GetPipes().Num() + 1;
	CommandLists.Reserve(NumCommandLists);
	for (int32 Index = 0; Index < NumCommandLists; ++Index)
	{
		CommandLists.Emplace(Allocator);
	}
}

void FRenderCommandList::Close()
{
#if DO_CHECK
	checkf(NumRecordScopeRefs == 0, TEXT("Close called on command list while FRecordScope is active!"));
	checkf(bRecording, TEXT("Close has already been called on this command list."));
#endif

	bRecording = false;

	for (UE::RenderCommandPipe::FCommandList& CommandList : CommandLists)
	{
		CommandList.Close();
	}

	if (DispatchTaskEvent)
	{
		DispatchTaskEvent->Trigger();
	}
}

void FRenderCommandList::Submit(FRenderCommandList* InParent)
{
	const bool bCloseOnSubmit = EnumHasAnyFlags(Flags, ERenderCommandListFlags::CloseOnSubmit);

	if (bCloseOnSubmit)
	{
		if (bRecording)
		{
			Close();
		}

		if (!bInitialized)
		{
			delete this;
			return;
		}
	}

	checkf(!bSubmitted, TEXT("FRenderCommandList::Submit cannot be called multiple times."));
	bSubmitted = true;

	Parent = InParent;

	if (!Parent)
	{
		Parent = GetInstanceTLS();
	}

	TConstArrayView<FRenderCommandPipe*> Pipes = UE::RenderCommandPipe::GetPipes();

	// Submit into parent.
	if (Parent)
	{
		bool bSkippedAllLists = true;

		for (int32 Index = 0; Index < Pipes.Num(); ++Index)
		{
			FRenderCommandPipe* Pipe = Pipes[Index];
			UE::RenderCommandPipe::FCommandList& CommandList = CommandLists[Index];
			const bool bSkipEmptyList = bCloseOnSubmit && CommandList.IsEmpty();

			if (!bSkipEmptyList)
			{
				Parent->Get(Pipe).Enqueue(&CommandList);
				bSkippedAllLists = false;
			}
		}

		UE::RenderCommandPipe::FCommandList& CommandList = CommandLists.Last();
		const bool bSkipEmptyList = bCloseOnSubmit && CommandList.IsEmpty();

		if (!bSkipEmptyList)
		{
			Parent->GetRenderThread().Enqueue(&CommandList);
			bSkippedAllLists = false;
		}

		if (bSkippedAllLists)
		{
			delete this;
			return;
		}

		check(Parent->bInitialized);

		// Parent takes ownership of the child command list for deletion purposes.
		if (Parent->Children.Tail)
		{
			Parent->Children.Tail->NextSibling = this;
			Parent->Children.Tail              = this;
		}
		else
		{
			Parent->Children.Head = Parent->Children.Tail = this;
		}

		// If have a valid recording task event, either we are still recording or one of our children is, so we have to propagate the event up to the parent.
		if (DispatchTaskEvent)
		{
			if (!Parent->DispatchTaskEvent)
			{
				check(!Parent->bSubmitted && Parent->bRecording);
				Parent->DispatchTaskEvent.Emplace(UE_SOURCE_LOCATION);
			}

			Parent->DispatchTaskEvent->AddPrerequisites(*DispatchTaskEvent);
		}
	}
	// Submit into command pipe.
	else
	{
		// Start by setting the maximum amount of refs. This has to happen first to avoid race with release on pipe threads.
		const int32 MaxNumRefs = CommandLists.Num();
		NumPipeRefs.fetch_add(MaxNumRefs, std::memory_order_relaxed);

		int32 NumEnqueues = 0;
		int32 NumPipeEnqueueFailed = 0;

		for (int32 Index = 0; Index < Pipes.Num(); ++Index)
		{
			FRenderCommandPipe* Pipe = Pipes[Index];
			UE::RenderCommandPipe::FCommandList& CommandList = CommandLists[Index];
			const bool bSkipEmptyList = bCloseOnSubmit && CommandList.IsEmpty();

			if (!bSkipEmptyList)
			{
				if (Pipes[Index]->Enqueue(this))
				{
					NumEnqueues++;
				}
				else
				{
					if (PipeEnqueueFailedBits.IsEmpty())
					{
						PipeEnqueueFailedBits.Init(false, Pipes.Num());
					}
					PipeEnqueueFailedBits[Index] = true;
					NumPipeEnqueueFailed++;
				}
			}
		}

		UE::RenderCommandPipe::FCommandList& CommandList = CommandLists.Last();
		const bool bSkipEmptyList = bCloseOnSubmit && CommandList.IsEmpty();

		if (NumPipeEnqueueFailed > 0 || !bSkipEmptyList)
		{
			FRenderThreadCommandPipe::Enqueue(this);
			NumEnqueues++;
		}

		const int32 NumRefsSkipped = MaxNumRefs - NumEnqueues;
		ReleasePipeRefs(NumRefsSkipped);
	}
}

//////////////////////////////////////////////////////////////////////////

FRenderThreadCommandPipe FRenderThreadCommandPipe::Instance;

DECLARE_RENDER_COMMAND_TAG(FRenderCommandTag_ExecuteCommandLists, ExecuteCommandLists);

namespace UE::RenderCommandPipe
{
	static const FRenderCommandTag& GetExecuteCommandListTag()
	{
		DECLARE_RENDER_COMMAND_TAG(FTag, ExecuteCommandList);
		return FTag::Get();
	}
}

void FRenderThreadCommandPipe::EnqueueAndLaunch(FRenderCommandList* CommandList)
{
	EnqueueAndLaunch([this, CommandList](FRHICommandListImmediate&)
	{
		ExecuteCommands(CommandList);

	}, UE::RenderCommandPipe::GetExecuteCommandListTag());
}

void FRenderThreadCommandPipe::EnqueueAndLaunch(TUniqueFunction<void(FRHICommandListImmediate&)>&& Function, const FRenderCommandTag& Tag)
{
#if WITH_STATE_STREAM
	if (RenderCommandStateStream_AddCommand(Function, Tag))
		return;
#endif

	Mutex.Lock();
	const bool bWasEmpty = Context->CommandList.Enqueue(FRenderCommandFunctionVariant(TInPlaceType<TUniqueFunction<void(FRHICommandListImmediate&)>>(), MoveTemp(Function)), Tag);
	Mutex.Unlock();

	if (bWasEmpty)
	{
		TGraphTask<TFunctionGraphTaskImpl<void(), ESubsequentsMode::FireAndForget>>::CreateTask().ConstructAndDispatchWhenReady([this, ContextToConsume = Context] () mutable
		{
			Mutex.Lock();
			const bool bDeleteAfterExecute = ContextToConsume->bDeleteAfterExecute;
			FContext ContextToExecute(MoveTemp(*ContextToConsume));
			Mutex.Unlock();

			ContextToExecute.CommandList.Close();
			ExecuteCommands(ContextToExecute.CommandList);

			if (bDeleteAfterExecute)
			{
				delete ContextToConsume;
			}

		}, TStatId(), ENamedThreads::GetRenderThread());
	}
}

void FRenderThreadCommandPipe::ExecuteCommands(UE::RenderCommandPipe::FCommandList& CommandList)
{
	CommandList.ConsumeCommands([&RHICmdList = GetImmediateCommandList_ForRenderCommand()] (FRenderCommandFunctionVariant&& Function, const FRenderCommandTag& Tag)
	{
		ExecuteCommand(&RHICmdList, Function, Tag);
	});
}

void FRenderThreadCommandPipe::ExecuteCommands(FRenderCommandList* CommandList)
{
	// Wait for recording of commands to be complete prior to replay.
	if (const UE::Tasks::FTask* DispatchTask = CommandList->TryGetDispatchTask())
	{
		DispatchTask->Wait();
	}

	// Execute commands for pipes that failed to enqueue due to not being in a recording state.
	if (!CommandList->PipeEnqueueFailedBits.IsEmpty())
	{
		for (TConstSetBitIterator<FRenderCommandPipeBitArrayAllocator> BitIt(CommandList->PipeEnqueueFailedBits); BitIt; ++BitIt)
		{
			ExecuteCommands(CommandList->Get(BitIt.GetIndex()));
		}
	}

	ExecuteCommands(CommandList->GetRenderThread());
	CommandList->ReleasePipeRef();
}

//////////////////////////////////////////////////////////////////////////

class FRenderCommandPipeRegistry
{
public:
	static TLinkedList<FRenderCommandPipe*>*& GetGlobalList()
	{
		static TLinkedList<FRenderCommandPipe*>* GlobalList = nullptr;
		return GlobalList;
	}

	void Initialize()
	{
		AllPipes.Reset();

		for (TLinkedList<FRenderCommandPipe*>::TIterator PipeIt(GetGlobalList()); PipeIt; PipeIt.Next())
		{
			FRenderCommandPipe* Pipe = *PipeIt;
			Pipe->SetEnabled(Pipe->ConsoleVariable->GetBool());
			Pipe->Index = AllPipes.Num();

			AllPipes.Emplace(*PipeIt);
		}
	}

	void StartRecording()
	{
		if (GRenderCommandPipeMode != ERenderCommandPipeMode::All || !GIsThreadedRendering)
		{
			return;
		}

		FRenderCommandPipeBitArray PipeBits;
		PipeBits.Init(true, AllPipes.Num());
		StartRecording(PipeBits);
	}

	void StartRecording(const FRenderCommandPipeBitArray& PipeBits)
	{
		if (GRenderCommandPipeMode != ERenderCommandPipeMode::All || !GIsThreadedRendering || PipeBits.IsEmpty())
		{
			return;
		}

		SCOPED_NAMED_EVENT(FRenderCommandPipe_StartRecording, FColor::Magenta);

		check(PipeBits.Num() == AllPipes.Num());

		UE::TScopeLock Lock(Mutex);

		bool bAnyPipesToStartRecording = false;

		for (FRenderCommandPipeSetBitIterator BitIt(PipeBits); BitIt; ++BitIt)
		{
			FRenderCommandPipe* Pipe = AllPipes[BitIt.GetIndex()];

			if (Pipe->bEnabled && !Pipe->bRecording)
			{
				bAnyPipesToStartRecording = true;
				break;
			}
		}

		if (!bAnyPipesToStartRecording)
		{
			return;
		}

		UE_RENDER_COMMAND_BEGIN_REGION(UE_RENDER_COMMAND_PIPE_RECORD_REGION);

		UE::Tasks::FTaskEvent TaskEvent{ UE_SOURCE_LOCATION };
		int32 NumPipesToStartRecording = 0;

		FRenderCommandPipeBitArray PipesToStartRecordingBits;
		PipesToStartRecordingBits.Init(false, PipeBits.Num());

		for (FRenderCommandPipeSetBitIterator BitIt(PipeBits); BitIt; ++BitIt)
		{
			FRenderCommandPipe* Pipe = AllPipes[BitIt.GetIndex()];

			if (Pipe->bEnabled && !Pipe->bRecording)
			{
				Pipe->bRecording = true;
				NumPipesToStartRecording++;
				PipesToStartRecordingBits[BitIt.GetIndex()] = true;

				UE::TScopeLock PipeLock(Pipe->Mutex);
				Pipe->RecordTask = TaskEvent;
			}
		}

		NumPipesRecording += NumPipesToStartRecording;

		ENQUEUE_RENDER_COMMAND(RenderCommandPipe_Start)([this, TaskEvent = MoveTemp(TaskEvent), PipesToStartRecordingBits, NumPipesToStartRecording](FRHICommandListImmediate&) mutable
		{
			RHIResourceLifetimeAddRef(NumPipesToStartRecording);
			NumPipesReplaying += NumPipesToStartRecording;
			TaskEvent.Trigger();

			for (FRenderCommandPipeSetBitIterator BitIt(PipesToStartRecordingBits); BitIt; ++BitIt)
			{
				AllPipes[BitIt.GetIndex()]->bReplaying = true;
			}
		});
	}

	FRenderCommandPipeBitArray StopRecording()
	{
		UE::TScopeLock Lock(Mutex);
		if (!NumPipesRecording)
		{
			return {};
		}

		FRenderCommandPipeBitArray PipeBits;
		PipeBits.Init(false, AllPipes.Num());

		for (int32 PipeIndex = 0; PipeIndex < AllPipes.Num(); ++PipeIndex)
		{
			if (FRenderCommandPipe* Pipe = AllPipes[PipeIndex]; Pipe->bRecording)
			{
				PipeBits[PipeIndex] = true;
			}
		}

		StopRecording(PipeBits);
		return PipeBits;
	}

	FRenderCommandPipeBitArray StopRecording(TConstArrayView<FRenderCommandPipe*> Pipes)
	{
		if (Pipes.IsEmpty())
		{
			return {};
		}

		UE::TScopeLock Lock(Mutex);
		if (!NumPipesRecording)
		{
			return {};
		}

		bool bAnyPipesToStopRecording = false;
		FRenderCommandPipeBitArray PipeBits;
		PipeBits.Init(false, AllPipes.Num());

		for (FRenderCommandPipe* Pipe : Pipes)
		{
			if (Pipe->bRecording)
			{
				PipeBits[Pipe->Index] = true;
				bAnyPipesToStopRecording = true;
			}
		}

		if (!bAnyPipesToStopRecording)
		{
			return {};
		}

		StopRecording(PipeBits);
		return PipeBits;
	}

	TConstArrayView<FRenderCommandPipe*> GetPipes() const
	{
		return AllPipes;
	}

	bool IsRecording() const
	{
		ensureMsgf(!FTaskTagScope::IsCurrentTag(ETaskTag::EParallelRenderingThread) && !FTaskTagScope::IsCurrentTag(ETaskTag::ERenderingThread),
			TEXT("IsRecording() is not valid from the render thread timeline."));

		return NumPipesRecording > 0;
	}

	bool IsReplaying() const
	{
		ensure(IsInParallelRenderingThread());
		return NumPipesReplaying > 0;
	}

private:
	void StopRecording(const FRenderCommandPipeBitArray& PipeBits)
	{
		SCOPED_NAMED_EVENT(FRenderCommandPipe_StopRecording, FColor::Magenta);

		UE::Tasks::FTaskEvent TaskEvent{ UE_SOURCE_LOCATION };
		uint32 NumPipesToStopRecording = 0;

		for (FRenderCommandPipeSetBitIterator BitIt(PipeBits); BitIt; ++BitIt)
		{
			FRenderCommandPipe* Pipe = AllPipes[BitIt.GetIndex()];
			check(Pipe->bRecording);
			Pipe->bRecording = false;
			NumPipesToStopRecording++;

			Pipe->Mutex.Lock();

			Pipe->ResetContext();
			TaskEvent.AddPrerequisites(Pipe->RecordTask);
			Pipe->RecordTask = {};
		}

		NumPipesRecording -= NumPipesToStopRecording;
		TaskEvent.Trigger();

		ENQUEUE_RENDER_COMMAND(RenderCommandPipe_Stop)([this, PipeBits, TaskEvent = MoveTemp(TaskEvent), NumPipesToStopRecording](FRHICommandListImmediate& RHICmdList)
		{
			TArray<FRHICommandListImmediate::FQueuedCommandList, FConcurrentLinearArrayAllocator> QueuedCommandLists;
			QueuedCommandLists.Reserve(NumPipesToStopRecording);
			TaskEvent.Wait();

			for (FRenderCommandPipeSetBitIterator BitIt(PipeBits); BitIt; ++BitIt)
			{
				FRenderCommandPipe* Pipe = AllPipes[BitIt.GetIndex()];

				if (Pipe->RHICmdList)
				{
					Pipe->RHICmdList->FinishRecording();
					QueuedCommandLists.Emplace(Pipe->RHICmdList);
					Pipe->RHICmdList = nullptr;
				}

				Pipe->bReplaying = false;
			}

			NumPipesReplaying -= NumPipesToStopRecording;

			RHICmdList.QueueAsyncCommandListSubmit(QueuedCommandLists);
			RHIResourceLifetimeReleaseRef(RHICmdList, NumPipesToStopRecording);
		});

		// Wait to unlock the mutex until the sync command has been submitted to the render thread. This avoids
		// race conditions where a command meant for a specific pipe might be inserted to the render thread pipe
		// prior to the actual wait command.
		for (FRenderCommandPipeSetBitIterator BitIt(PipeBits); BitIt; ++BitIt)
		{
			AllPipes[BitIt.GetIndex()]->Mutex.Unlock();
		}

		UE_RENDER_COMMAND_END_REGION(UE_RENDER_COMMAND_PIPE_RECORD_REGION);
	}

	UE::FMutex Mutex;
	TArray<FRenderCommandPipe*> AllPipes;
	uint32 NumPipesRecording = 0;
	uint32 NumPipesReplaying = 0;
};

static FRenderCommandPipeRegistry GRenderCommandPipeRegistry;

inline bool HasBitsSet(const FRenderCommandPipeBitArray& Bits)
{
	for (FRenderCommandPipeBitArray::FConstWordIterator It(Bits); It; ++It)
	{
		if (It.GetWord() != 0)
		{
			return true;
		}
	}
	return false;
}

namespace UE::RenderCommandPipe
{
	static thread_local FRenderCommandPipe* ReplayingPipe = nullptr;
	static FStopRecordingDelegate StopRecordingDelegate;

	void Initialize()
	{
		GRenderCommandPipeRegistry.Initialize();
	}

	bool IsRecording()
	{
		return GRenderCommandPipeRegistry.IsRecording();
	}

	bool IsReplaying()
	{
		return GRenderCommandPipeRegistry.IsReplaying();
	}

	bool IsReplaying(const FRenderCommandPipe& Pipe)
	{
		return ReplayingPipe == &Pipe;
	}

	void StartRecording()
	{
		GRenderCommandPipeRegistry.StartRecording();
	}

	void StartRecording(const FRenderCommandPipeBitArray& PipeBits)
	{
		GRenderCommandPipeRegistry.StartRecording(PipeBits);
	}

	FRenderCommandPipeBitArray StopRecording()
	{
		FRenderCommandPipeBitArray PipeBits = GRenderCommandPipeRegistry.StopRecording();
		GetStopRecordingDelegate().Broadcast(PipeBits);
		return PipeBits;
	}

	FRenderCommandPipeBitArray StopRecording(TConstArrayView<FRenderCommandPipe*> Pipes)
	{
		FRenderCommandPipeBitArray PipeBits = GRenderCommandPipeRegistry.StopRecording(Pipes);
		GetStopRecordingDelegate().Broadcast(PipeBits);
		return PipeBits;
	}

	TConstArrayView<FRenderCommandPipe*> GetPipes()
	{
		return GRenderCommandPipeRegistry.GetPipes();
	}

	FStopRecordingDelegate& GetStopRecordingDelegate()
	{
		return StopRecordingDelegate;
	}

	FSyncScope::FSyncScope()
	{
		PipeBits = StopRecording();

#if UE_TRACE_ENABLED
		if (HasBitsSet(PipeBits))
		{
			UE_RENDER_COMMAND_BEGIN_REGION(UE_RENDER_COMMAND_PIPE_SYNC_REGION);
		}
#endif
	}

	FSyncScope::FSyncScope(TConstArrayView<FRenderCommandPipe*> Pipes)
	{
		PipeBits = StopRecording(Pipes);

#if UE_TRACE_ENABLED
		if (HasBitsSet(PipeBits))
		{
			UE_RENDER_COMMAND_BEGIN_REGION(UE_RENDER_COMMAND_PIPE_SYNC_REGION);
		}
#endif
	}

	FSyncScope::~FSyncScope()
	{
#if UE_TRACE_ENABLED
		if (HasBitsSet(PipeBits))
		{
			UE_RENDER_COMMAND_END_REGION(UE_RENDER_COMMAND_PIPE_SYNC_REGION);
		}
#endif

		StartRecording(PipeBits);
	}
}

//////////////////////////////////////////////////////////////////////////

FRenderCommandPipe::FRenderCommandPipe(const TCHAR* InName, ERenderCommandPipeFlags Flags, const TCHAR* CVarName, const TCHAR* CVarDescription)
	: Name(InName)
	, GlobalListLink(this)
	, ConsoleVariable(CVarName, !EnumHasAnyFlags(Flags, ERenderCommandPipeFlags::Disabled), CVarDescription, FConsoleVariableDelegate::CreateLambda([this](IConsoleVariable* Variable)
	{
		SetEnabled(Variable->GetBool());
	}))
{
#if !UE_SERVER
	GlobalListLink.LinkHead(FRenderCommandPipeRegistry::GetGlobalList());
#endif
}

void FRenderCommandPipe::ExecuteCommand(FRenderCommandFunctionVariant&& FunctionVariant, const FRenderCommandTag& Tag)
{
	if (!RHICmdList && FunctionVariant.IsType<TUniqueFunction<void(FRHICommandList&)>>())
	{
		RHICmdList = new FRHICommandList();
		RHICmdList->SwitchPipeline(ERHIPipeline::Graphics);
	}

	::ExecuteCommand(RHICmdList, FunctionVariant, Tag);
}

void FRenderCommandPipe::EnqueueAndLaunch(FRenderCommandList* CommandList)
{
	NumInFlightCommandLists.fetch_add(1, std::memory_order_relaxed);

	auto ExecuteLambda = [this, CommandList]
	{
		TRACE_CPUPROFILER_EVENT_SCOPE_ON_CHANNEL_STR("RenderCommandPipe ReplayCommands", RenderCommandsChannel);
		SCOPED_NAMED_EVENT_TCHAR(Name, FColor::Magenta);
		FTaskTagScope Scope(ETaskTag::EParallelRenderingThread);

		ExecuteCommands(CommandList->Get(this));
		NumInFlightCommandLists.fetch_sub(1, std::memory_order_relaxed);
		CommandList->ReleasePipeRef();
	};

	if (const UE::Tasks::FTask* DispatchTask = CommandList->TryGetDispatchTask())
	{
		ResetContext();

		RecordTask = UE::Tasks::Launch(Name, MoveTemp(ExecuteLambda), MakeArrayView<UE::Tasks::FTask>({ RecordTask, *DispatchTask }));
	}
	else
	{
		EnqueueAndLaunch(MoveTemp(ExecuteLambda), UE::RenderCommandPipe::GetExecuteCommandListTag());
	}
}

void FRenderCommandPipe::EnqueueAndLaunch(FRenderCommandFunctionVariant&& FunctionVariant, const FRenderCommandTag& Tag)
{
	ensureMsgf(!UE::RenderCommandPipe::ReplayingPipe, TEXT("Attempting to launch render command to render command pipe %s from another pipe %s"), Name, UE::RenderCommandPipe::ReplayingPipe->Name);

	bool bWasEmpty = Context->CommandList.Enqueue(MoveTemp(FunctionVariant), Tag);
	NumInFlightCommands.fetch_add(1, std::memory_order_relaxed);

	if (bWasEmpty)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE_ON_CHANNEL_STR("RenderCommandPipe LaunchTask", RenderCommandsChannel);

		RecordTask = UE::Tasks::Launch(Name, [this, ContextToConsume = Context]
		{
			TRACE_CPUPROFILER_EVENT_SCOPE_ON_CHANNEL_STR("RenderCommandPipe ReplayCommands", RenderCommandsChannel)
			SCOPED_NAMED_EVENT_TCHAR(Name, FColor::Magenta);
			FTaskTagScope Scope(ETaskTag::EParallelRenderingThread);

			Mutex.Lock();
			const bool bDeleteAfterExecute = ContextToConsume->bDeleteAfterExecute;
			FContext ContextToExecute(MoveTemp(*ContextToConsume));
			Mutex.Unlock();

			ContextToExecute.CommandList.Close();
			ExecuteCommands(ContextToExecute.CommandList);

			const int32 NumCommands = ContextToExecute.CommandList.NumCommands();
			const int32 LocalNumInFlightCommands = NumInFlightCommands.fetch_sub(NumCommands, std::memory_order_release) - NumCommands;
			check(LocalNumInFlightCommands >= 0);

			if (bDeleteAfterExecute)
			{
				delete ContextToConsume;
			}

		}, RecordTask);
	}
}

void FRenderCommandPipe::ExecuteCommands(UE::RenderCommandPipe::FCommandList& CommandList)
{
	FRenderCommandPipe* const PreviousReplayingPipe = UE::RenderCommandPipe::ReplayingPipe;
	UE::RenderCommandPipe::ReplayingPipe = this;

	int32 NumCommands = CommandList.NumCommands();

	CommandList.ConsumeCommands([this](FRenderCommandFunctionVariant&& FunctionVariant, const FRenderCommandTag& Tag)
	{
		ExecuteCommand(MoveTemp(FunctionVariant), Tag);
	});

	UE::RenderCommandPipe::ReplayingPipe = PreviousReplayingPipe;
}

//////////////////////////////////////////////////////////////////////////

TAutoConsoleVariable<int32> CVarAllowOneFrameThreadLag(
	TEXT("r.OneFrameThreadLag"),
	1,
	TEXT("Whether to allow the rendering thread to lag one frame behind the game thread (0: disabled, otherwise enabled)")
);

TAutoConsoleVariable<int32> CVarGTSyncType(
	TEXT("r.GTSyncType"),
	0,
	TEXT("Determines how the game thread syncs with the render thread, RHI thread and GPU.\n")
	TEXT("Syncing to the GPU swap chain flip allows for lower frame latency.\n")
	TEXT(" <= 0 - Sync the game thread with the N-1 render thread frame. Then sync with the N-m RHI thread frame where m is (2 + (-r.GTSyncType)) (i.e. negative values increase the amount of RHI thread overlap) (default = 0).\n")
	TEXT("    1 - Sync the game thread with the N-1 RHI thread frame.\n")
	TEXT("    2 - Sync the game thread with the GPU swap chain flip (only on supported platforms).\n"),
	ECVF_Default
);

DECLARE_CYCLE_STAT(TEXT("Frame Sync Time"), STAT_FrameSyncTime, STATGROUP_RenderThreadProcessing);

namespace FFrameEndSync
{
	using ESyncDepth = FRenderCommandFence::ESyncDepth;

	struct FRenderThreadFence
	{
		// Legacy game code assumes the game thread will never get further than 1 frame ahead of the render thread.
		// This fence is used to sync the game thread with the N-1 render thread frame.
		FRenderCommandFence Fence;

		FRenderThreadFence()
		{
			Fence.BeginFence(ESyncDepth::RenderThread);
		}

		~FRenderThreadFence()
		{
			Fence.Wait(true);
		}
	};
	TArray<FRenderThreadFence, TInlineAllocator<2>> RenderThreadFences;

	// Additional fences to await. These sync with either the RHI thread or swapchain,
	// and are used to prevent the game thread running too far ahead of presented frames.
	TArray<FRenderCommandFence, TInlineAllocator<3>> PipelineFences;

	void Sync(EFlushMode FlushMode)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(Sync_RenderingThread);
		static bool bRecursive = false;
		TGuardValue<bool> RecursionGuard(bRecursive, true);

		if (RecursionGuard.GetOriginalValue())
		{
			// This is a recursive call to FFrameEndSync::Sync(). Use a standard render fence and do a full sync.
			FRenderCommandFence Fence;
			Fence.BeginFence();
			Fence.Wait();
			return;
		}

		bool bFullSync = FlushMode == EFlushMode::Threads;

		// The "r.OneFrameThreadLag" cvar forces a full sync, meaning the game thread will
		// not start work until all the rendering work for the previous frame has completed.
		bFullSync |= CVarAllowOneFrameThreadLag.GetValueOnAnyThread() <= 0;

		SCOPE_CYCLE_COUNTER(STAT_FrameSyncTime);

		check(IsInGameThread());

		// Always sync with the render thread (either current frame, or N-1 frame)
		RenderThreadFences.Emplace();
		while (RenderThreadFences.Num() > (bFullSync ? 0 : 1))
		{
			RenderThreadFences.RemoveAt(0);
		}

		// Insert an additional fence based on how we want to sync with the RHI thread / swapchain
		ESyncDepth SyncDepth;
		int32 NumFramesOverlap;

		int32 const GTSyncType = CVarGTSyncType.GetValueOnAnyThread();

		if (bFullSync)
		{
			SyncDepth = (GTSyncType >= 2 && FlushMode != EFlushMode::Threads)
				? ESyncDepth::Swapchain
				: ESyncDepth::RHIThread;

			NumFramesOverlap = 0;
		}
		else if (GTSyncType >= 2)
		{
			SyncDepth = ESyncDepth::Swapchain;
			NumFramesOverlap = 1;
		}
		else if (GTSyncType == 1)
		{
			SyncDepth = ESyncDepth::RHIThread;
			NumFramesOverlap = 1;
		}
		else
		{
			check(GTSyncType <= 0);

			// Modes <= 0 allows N frames of overlap with the RHI thread.
			SyncDepth = ESyncDepth::RHIThread;
			NumFramesOverlap = 2 + (-GTSyncType);
		}

		if (SyncDepth == ESyncDepth::Swapchain)
		{
			// Swapchain sync mode does not work when vsync is disabled. Fallback to RHI thread sync in that case.
			static auto CVarVsync = IConsoleManager::Get().FindConsoleVariable(TEXT("r.VSync"));
			check(CVarVsync != nullptr);

			if (CVarVsync->GetInt() == 0)
			{
				SyncDepth = ESyncDepth::RHIThread;
			}
		}

		PipelineFences.Emplace_GetRef().BeginFence(SyncDepth);

		// Don't process game thread tasks when flushing all threads. This can result in strange behavior where the game thread
		// is flushing the render thread and then gets pre-empted by another task that has an implicit dependency on the one
		// being processed.
		if (FlushMode == EFlushMode::EndFrame && !FTaskGraphInterface::Get().IsThreadProcessingTasks(ENamedThreads::GameThread))
		{
			// need to process gamethread tasks at least once a frame no matter what
			FTaskGraphInterface::Get().ProcessThreadUntilIdle(ENamedThreads::GameThread);
		}

		while (PipelineFences.Num() > NumFramesOverlap)
		{
			PipelineFences[0].Wait(true);
			PipelineFences.RemoveAt(0);
		}
	}
}
