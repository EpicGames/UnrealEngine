// Copyright Epic Games, Inc. All Rights Reserved.

#include "ResolveShader.h"
#include "ShaderCompilerCore.h"
#include "ShaderParameterUtils.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "StereoRenderUtils.h"

IMPLEMENT_SHADER_TYPE(, FResolveDepthPS, TEXT("/Engine/Private/ResolvePixelShader.usf"), TEXT("MainDepth"), SF_Pixel);
IMPLEMENT_SHADER_TYPE(, FResolveDepth2XPS, TEXT("/Engine/Private/ResolvePixelShader.usf"), TEXT("MainDepth"), SF_Pixel);
IMPLEMENT_SHADER_TYPE(, FResolveDepth4XPS, TEXT("/Engine/Private/ResolvePixelShader.usf"), TEXT("MainDepth"), SF_Pixel);
IMPLEMENT_SHADER_TYPE(, FResolveDepth8XPS, TEXT("/Engine/Private/ResolvePixelShader.usf"), TEXT("MainDepth"), SF_Pixel);
IMPLEMENT_SHADER_TYPE(, FResolveDepthArray2XPS, TEXT("/Engine/Private/ResolvePixelShader.usf"), TEXT("MainDepth"), SF_Pixel);
IMPLEMENT_SHADER_TYPE(, FResolveDepthArray4XPS, TEXT("/Engine/Private/ResolvePixelShader.usf"), TEXT("MainDepth"), SF_Pixel);
IMPLEMENT_SHADER_TYPE(, FResolveDepthArray8XPS, TEXT("/Engine/Private/ResolvePixelShader.usf"), TEXT("MainDepth"), SF_Pixel);
IMPLEMENT_SHADER_TYPE(, FResolveSingleSamplePS, TEXT("/Engine/Private/ResolvePixelShader.usf"), TEXT("MainSingleSample"), SF_Pixel);
IMPLEMENT_SHADER_TYPE(, FResolveVS, TEXT("/Engine/Private/ResolveVertexShader.usf"), TEXT("Main"), SF_Vertex);
IMPLEMENT_SHADER_TYPE(, FResolveArrayVS, TEXT("/Engine/Private/ResolveVertexShader.usf"), TEXT("Main"), SF_Vertex);

// FResolveDepthPS

FResolveDepthPS::FResolveDepthPS() = default;
FResolveDepthPS::FResolveDepthPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
	: FGlobalShader(Initializer)
{
	UnresolvedSurface.Bind(Initializer.ParameterMap, TEXT("UnresolvedSurface"), SPF_Mandatory);
}

void FResolveDepthPS::ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
{
	FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);

	if (FDataDrivenShaderPlatformInfo::GetRequiresBindfulUtilityShaders(Parameters.Platform))
	{
		OutEnvironment.CompilerFlags.Add(CFLAG_ForceBindful);
	}
}

bool FResolveDepthPS::ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
{
	return !IsOpenGLPlatform(Parameters.Platform);
}

void FResolveDepthPS::SetParameters(FRHIBatchedShaderParameters& BatchedParameters, FParameter)
{
}

// FResolveDepth2XPS

FResolveDepth2XPS::FResolveDepth2XPS() = default;
FResolveDepth2XPS::FResolveDepth2XPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
	: FResolveDepthPS(Initializer)
{
}

void FResolveDepth2XPS::ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
{
	FResolveDepthPS::ModifyCompilationEnvironment(Parameters, OutEnvironment);
	OutEnvironment.SetDefine(TEXT("DEPTH_RESOLVE_NUM_SAMPLES"), 2);
}

// FResolveDepth4XPS

FResolveDepth4XPS::FResolveDepth4XPS() = default;
FResolveDepth4XPS::FResolveDepth4XPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
	: FResolveDepthPS(Initializer)
{
}

void FResolveDepth4XPS::ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
{
	FResolveDepthPS::ModifyCompilationEnvironment(Parameters, OutEnvironment);
	OutEnvironment.SetDefine(TEXT("DEPTH_RESOLVE_NUM_SAMPLES"), 4);
}

// FResolveDepth8XPS

FResolveDepth8XPS::FResolveDepth8XPS() = default;
FResolveDepth8XPS::FResolveDepth8XPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
	: FResolveDepthPS(Initializer)
{
}

bool FResolveDepth8XPS::ShouldCache(EShaderPlatform Platform)
{
	return GetMaxSupportedFeatureLevel(Platform) >= ERHIFeatureLevel::SM5;
}

void FResolveDepth8XPS::ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
{
	FResolveDepthPS::ModifyCompilationEnvironment(Parameters, OutEnvironment);
	OutEnvironment.SetDefine(TEXT("DEPTH_RESOLVE_NUM_SAMPLES"), 8);
}


static bool IsMobileMultiViewEnabled(EShaderPlatform ShaderPlatform)
{
	UE::StereoRenderUtils::FStereoShaderAspects Aspects(ShaderPlatform);
	return Aspects.IsMobileMultiViewEnabled();
}

// FResolveDepthArray2XPS

FResolveDepthArray2XPS::FResolveDepthArray2XPS() = default;
FResolveDepthArray2XPS::FResolveDepthArray2XPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
	: FResolveDepthPS(Initializer)
{
}

bool FResolveDepthArray2XPS::ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
{
	return IsMobileMultiViewEnabled(Parameters.Platform);
}

void FResolveDepthArray2XPS::ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
{
	FResolveDepthPS::ModifyCompilationEnvironment(Parameters, OutEnvironment);
	OutEnvironment.SetDefine(TEXT("DEPTH_RESOLVE_TEXTUREARRAY"), 1);
	OutEnvironment.SetDefine(TEXT("DEPTH_RESOLVE_NUM_SAMPLES"), 2);
}

// FResolveDepthArray4XPS

FResolveDepthArray4XPS::FResolveDepthArray4XPS() = default;
FResolveDepthArray4XPS::FResolveDepthArray4XPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
	: FResolveDepthPS(Initializer)
{
}

bool FResolveDepthArray4XPS::ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
{
	return IsMobileMultiViewEnabled(Parameters.Platform);
}

void FResolveDepthArray4XPS::ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
{
	FResolveDepthPS::ModifyCompilationEnvironment(Parameters, OutEnvironment);
	OutEnvironment.SetDefine(TEXT("DEPTH_RESOLVE_TEXTUREARRAY"), 1);
	OutEnvironment.SetDefine(TEXT("DEPTH_RESOLVE_NUM_SAMPLES"), 4);
}

// FResolveDepthArray8XPS

FResolveDepthArray8XPS::FResolveDepthArray8XPS() = default;
FResolveDepthArray8XPS::FResolveDepthArray8XPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
	: FResolveDepthPS(Initializer)
{
}

bool FResolveDepthArray8XPS::ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
{
	return IsMobileMultiViewEnabled(Parameters.Platform);
}

void FResolveDepthArray8XPS::ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
{
	FResolveDepthPS::ModifyCompilationEnvironment(Parameters, OutEnvironment);
	OutEnvironment.SetDefine(TEXT("DEPTH_RESOLVE_TEXTUREARRAY"), 1);
	OutEnvironment.SetDefine(TEXT("DEPTH_RESOLVE_NUM_SAMPLES"), 8);
}

// FResolveSingleSamplePS

FResolveSingleSamplePS::FResolveSingleSamplePS() = default;
FResolveSingleSamplePS::FResolveSingleSamplePS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
	: FGlobalShader(Initializer)
{
	UnresolvedSurface.Bind(Initializer.ParameterMap, TEXT("UnresolvedSurface"), SPF_Mandatory);
	SingleSampleIndex.Bind(Initializer.ParameterMap, TEXT("SingleSampleIndex"), SPF_Mandatory);
}

bool FResolveSingleSamplePS::ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
{
	return FDataDrivenShaderPlatformInfo::GetIsLanguageD3D(Parameters.Platform);
}

void FResolveSingleSamplePS::SetParameters(FRHIBatchedShaderParameters& BatchedParameters, uint32 SingleSampleIndexValue)
{
	SetShaderValue(BatchedParameters, SingleSampleIndex, SingleSampleIndexValue);
}

// FResolveVS

FResolveVS::FResolveVS() = default;
FResolveVS::FResolveVS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
	: FGlobalShader(Initializer)
{
	PositionMinMax.Bind(Initializer.ParameterMap, TEXT("PositionMinMax"), SPF_Mandatory);
	UVMinMax.Bind(Initializer.ParameterMap, TEXT("UVMinMax"), SPF_Mandatory);
}

bool FResolveVS::ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
{
	return true;
}

void FResolveVS::SetParameters(FRHIBatchedShaderParameters& BatchedParameters, const FResolveRect& SrcBounds, const FResolveRect& DstBounds, uint32 DstSurfaceWidth, uint32 DstSurfaceHeight)
{
	// Generate the vertices used to copy from the source surface to the destination surface.
	const float MinU = (float)SrcBounds.X1;
	const float MinV = (float)SrcBounds.Y1;
	const float MaxU = (float)SrcBounds.X2;
	const float MaxV = (float)SrcBounds.Y2;
	const float MinX = -1.f + DstBounds.X1 / ((float)DstSurfaceWidth * 0.5f);
	const float MinY = +1.f - DstBounds.Y1 / ((float)DstSurfaceHeight * 0.5f);
	const float MaxX = -1.f + DstBounds.X2 / ((float)DstSurfaceWidth * 0.5f);
	const float MaxY = +1.f - DstBounds.Y2 / ((float)DstSurfaceHeight * 0.5f);

	SetShaderValue(BatchedParameters, PositionMinMax, FVector4f(MinX, MinY, MaxX, MaxY));
	SetShaderValue(BatchedParameters, UVMinMax, FVector4f(MinU, MinV, MaxU, MaxV));
}

// FResolveArrayVS

FResolveArrayVS::FResolveArrayVS() = default;
FResolveArrayVS::FResolveArrayVS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
	: FResolveVS(Initializer)
{
}

bool FResolveArrayVS::ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
{
	if (FResolveVS::ShouldCompilePermutation(Parameters))
	{
		UE::StereoRenderUtils::FStereoShaderAspects Aspects(Parameters.Platform);
		return Aspects.IsMobileMultiViewEnabled();
	}
	return false;
}

void FResolveArrayVS::ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
{
	FResolveVS::ModifyCompilationEnvironment(Parameters, OutEnvironment);
	OutEnvironment.SetDefine(TEXT("DEPTH_RESOLVE_TEXTUREARRAY"), 1);
}

void CreateResolveShaders()
{
	ForceInitGlobalShaderType<FResolveDepthPS>();
	ForceInitGlobalShaderType<FResolveDepth2XPS>();
	ForceInitGlobalShaderType<FResolveDepth4XPS>();
	ForceInitGlobalShaderType<FResolveDepth8XPS>();
	ForceInitGlobalShaderType<FResolveDepthArray2XPS>();
	ForceInitGlobalShaderType<FResolveDepthArray4XPS>();
	ForceInitGlobalShaderType<FResolveDepthArray8XPS>();
	ForceInitGlobalShaderType<FResolveSingleSamplePS>();

	ForceInitGlobalShaderType<FResolveVS>();
	ForceInitGlobalShaderType<FResolveArrayVS>();
}
