// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	Shader.cpp: Shader implementation.
=============================================================================*/

#include "Shader.h"
#include "Misc/CoreMisc.h"
#include "Misc/StringBuilder.h"
#include "VertexFactory.h"
#include "ProfilingDebugging/DiagnosticTable.h"
#include "Interfaces/ITargetPlatform.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#include "Interfaces/IShaderFormat.h"
#include "Internationalization/Regex.h"
#include "IOS/IOSPlatformProperties.h"
#include "Serialization/ShaderKeyGenerator.h"
#include "ShaderCodeLibrary.h"
#include "ShaderCore.h"
#include "ShaderCompilerCore.h"
#include "RenderUtils.h"
#include "StereoRenderUtils.h"
#include "Mac/MacPlatformProperties.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/ScopeLock.h"
#include "UObject/RenderingObjectVersion.h"
#include "ProfilingDebugging/LoadTimeTracker.h"
#include "Misc/LargeWorldRenderPosition.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "ShaderPlatformCachedIniValue.h"
#include "ShaderPlatformConfig.h"
#include "ColorManagement/ColorSpace.h"

#if WITH_EDITORONLY_DATA
#include "Interfaces/IShaderFormat.h"
#endif

#if WITH_EDITOR
#include "Serialization/CompactBinary.h"
#include "Serialization/CompactBinaryWriter.h"
#endif

#if RHI_RAYTRACING
#include "RayTracingPayloadType.h"
#endif

DEFINE_LOG_CATEGORY(LogShaders);

IMPLEMENT_TYPE_LAYOUT(FShader);
IMPLEMENT_TYPE_LAYOUT(FShaderParameterBindings);
IMPLEMENT_TYPE_LAYOUT(FShaderMapContent);
IMPLEMENT_TYPE_LAYOUT(FShaderTypeDependency);
IMPLEMENT_TYPE_LAYOUT(FShaderPipeline);
IMPLEMENT_TYPE_LAYOUT(FShaderUniformBufferParameterInfo);
IMPLEMENT_TYPE_LAYOUT(FShaderResourceParameterInfo);
IMPLEMENT_TYPE_LAYOUT(FShaderLooseParameterInfo);
IMPLEMENT_TYPE_LAYOUT(FShaderLooseParameterBufferInfo);
IMPLEMENT_TYPE_LAYOUT(FShaderParameterMapInfo);

void Freeze::IntrinsicToString(const TIndexedPtr<FShaderType>& Object, const FTypeLayoutDesc& TypeDesc, const FPlatformTypeLayoutParameters& LayoutParams, FMemoryToStringContext& OutContext)
{
	const FShaderType* Type = Object.Get(OutContext.TryGetPrevPointerTable());
	if (Type)
	{
		OutContext.String->Appendf(TEXT("%s\n"), Type->GetName());
	}
	else
	{
		OutContext.AppendNullptr();
	}
}

void Freeze::IntrinsicToString(const TIndexedPtr<FVertexFactoryType>& Object, const FTypeLayoutDesc& TypeDesc, const FPlatformTypeLayoutParameters& LayoutParams, FMemoryToStringContext& OutContext)
{
	const FVertexFactoryType* Type = Object.Get(OutContext.TryGetPrevPointerTable());
	if (Type)
	{
		OutContext.String->Appendf(TEXT("%s\n"), Type->GetName());
	}
	else
	{
		OutContext.AppendNullptr();
	}
}

IMPLEMENT_EXPORTED_INTRINSIC_TYPE_LAYOUT(TIndexedPtr<FShaderType>);
IMPLEMENT_EXPORTED_INTRINSIC_TYPE_LAYOUT(TIndexedPtr<FVertexFactoryType>);

static TAutoConsoleVariable<int32> CVarUsePipelines(
	TEXT("r.ShaderPipelines"),
	1,
	TEXT("Enable using Shader pipelines."));

static TAutoConsoleVariable<int32> CVarRemoveUnusedInterpolators(
	TEXT("r.Shaders.RemoveUnusedInterpolators"),
	0,
	TEXT("Enables removing unused interpolators mode when compiling shader pipelines.\n")
	TEXT(" 0: Disable (default)\n")
	TEXT(" 1: Enable removing unused"),
	ECVF_ReadOnly
);

static TAutoConsoleVariable<int32> CVarSkipShaderCompression(
	TEXT("r.Shaders.SkipCompression"),
	0,
	TEXT("Skips shader compression after compiling. Shader compression time can be quite significant when using debug shaders. This CVar is only valid in non-shipping/test builds."),
	ECVF_ReadOnly | ECVF_Cheat
	);

static TAutoConsoleVariable<int32> CVarAllowCompilingThroughWorkers(
	TEXT("r.Shaders.AllowCompilingThroughWorkers"),
	1,
	TEXT("Allows shader compilation through external ShaderCompileWorker processes.\n")
	TEXT("1 - (Default) Allows external shader compiler workers\n") 
	TEXT("0 - Disallows external shader compiler workers. Will run shader compilation in proc of UE process."),
	ECVF_ReadOnly
	);

static TAutoConsoleVariable<int32> CVarShadersForceDXC(
	TEXT("r.Shaders.ForceDXC"),
	1,
	TEXT("Forces DirectX Shader Compiler (DXC) to be used for all shaders instead of HLSLcc if supported.\n")
	TEXT(" 1: Force new compiler for all shaders (default)\n")
	TEXT(" 0: Disable"),
	ECVF_ReadOnly);

static TLinkedList<FShaderType*>*			GShaderTypeList = nullptr;
static TLinkedList<FShaderPipelineType*>*	GShaderPipelineList = nullptr;

static FSHAHash ShaderSourceDefaultHash; //will only be read (never written) for the cooking case

/**
 * Find the shader pipeline type with the given name.
 * @return NULL if no type matched.
 */
inline const FShaderPipelineType* FindShaderPipelineType(FName TypeName)
{
	for (TLinkedList<FShaderPipelineType*>::TIterator ShaderPipelineTypeIt(FShaderPipelineType::GetTypeList()); ShaderPipelineTypeIt; ShaderPipelineTypeIt.Next())
	{
		if (ShaderPipelineTypeIt->GetFName() == TypeName)
		{
			return *ShaderPipelineTypeIt;
		}
	}
	return nullptr;
}


/**
 * Serializes a reference to a shader pipeline type.
 */
FArchive& operator<<(FArchive& Ar, const FShaderPipelineType*& TypeRef)
{
	if (Ar.IsSaving())
	{
		FName TypeName = TypeRef ? FName(TypeRef->Name) : NAME_None;
		Ar << TypeName;
	}
	else if (Ar.IsLoading())
	{
		FName TypeName = NAME_None;
		Ar << TypeName;
		TypeRef = FindShaderPipelineType(TypeName);
	}
	return Ar;
}


void FShaderParameterMap::VerifyBindingsAreComplete(const TCHAR* ShaderTypeName, FShaderTarget Target, const FVertexFactoryType* InVertexFactoryType) const
{
#if WITH_EDITORONLY_DATA
	// Only people working on shaders (and therefore have LogShaders unsuppressed) will want to see these errors
	if (UE_LOG_ACTIVE(LogShaders, Warning))
	{
		const TCHAR* VertexFactoryName = InVertexFactoryType ? InVertexFactoryType->GetName() : TEXT("?");

		bool bBindingsComplete = true;
		FString UnBoundParameters;
		for (TMap<FString,FParameterAllocation>::TConstIterator ParameterIt(ParameterMap);ParameterIt;++ParameterIt)
		{
			const FString& ParamName = ParameterIt.Key();
			const FParameterAllocation& ParamValue = ParameterIt.Value();
			if(!ParamValue.bBound)
			{
				// Only valid parameters should be in the shader map
				checkSlow(ParamValue.Size > 0);
				bBindingsComplete = bBindingsComplete && ParamValue.bBound;
				UnBoundParameters += FString(TEXT("		Parameter ")) + ParamName + TEXT(" not bound!\n");
			}
		}
		
		if (!bBindingsComplete)
		{
			FString ErrorMessage = FString("Found unbound parameters being used in shadertype ") + ShaderTypeName + TEXT(" (VertexFactory: ") + VertexFactoryName + TEXT(")\n") + UnBoundParameters;

			// We use a non-Slate message box to avoid problem where we haven't compiled the shaders for Slate.
			FPlatformMisc::MessageBoxExt(EAppMsgType::Ok, *ErrorMessage, TEXT("Error"));
		}
	}
#endif // WITH_EDITORONLY_DATA
}

static TArray<FShaderType*>& GetSortedShaderTypes(FShaderType::EShaderTypeForDynamicCast Type)
{
	static TArray<FShaderType*>* SortedTypesArray = new TArray<FShaderType*>[(uint32)FShaderType::EShaderTypeForDynamicCast::NumShaderTypes];
	return SortedTypesArray[(uint32)Type];
}

namespace {

uint32 RegisteredRayTracingPayloads = 0;
uint32 RayTracingPayloadSizes[32] = {};
TRaytracingPayloadSizeFunction RayTracingPayloadSizeFunctions[32] = {};

bool IsRayTracingPayloadRegistered(ERayTracingPayloadType PayloadType)
{
	// make sure all bits are on in the registered bitmask
	return (static_cast<uint32>(PayloadType) & RegisteredRayTracingPayloads) == static_cast<uint32>(PayloadType);
}

} // anonymous namespace


FShaderType::FShaderType(
	EShaderTypeForDynamicCast InShaderTypeForDynamicCast,
	FTypeLayoutDesc& InTypeLayout,
	const TCHAR* InName,
	const TCHAR* InSourceFilename,
	const TCHAR* InFunctionName,
	uint32 InFrequency,
	int32 InTotalPermutationCount,
	int32 InTotalSpecializationCount,
	ConstructSerializedType InConstructSerializedRef,
	ConstructCompiledType InConstructCompiledRef,
	ShouldCompilePermutationType InShouldCompilePermutationRef,
	ShouldPrecachePermutationType InShouldPrecachePermutationRef,
	GetUnspecializedIdType InGetUnspecializedIdRef,
	GetSpecializationValuesType InGetSpecializationValuesRef,
	GetRayTracingPayloadTypeType InGetRayTracingPayloadTypeRef,
	GetShaderBindingLayoutType InGetShaderBindingLayoutTypeRef,
#if WITH_EDITOR
	ModifyCompilationEnvironmentType InModifyCompilationEnvironmentRef,
	ValidateCompiledResultType InValidateCompiledResultRef,
	GetOverrideJobPriorityType InGetOverrideJobPriorityRef,
#endif // WITH_EDITOR
	uint32 InTypeSize,
	const FShaderParametersMetadata* InRootParametersMetadata
#if WITH_EDITOR
	, GetPermutationIdStringType InGetPermutationIdStringRef
#endif
)
	: ShaderTypeForDynamicCast(InShaderTypeForDynamicCast)
	, TypeLayout(&InTypeLayout)
	, Name(InName)
	, TypeName(InName)
	, HashedName(TypeName)
	, HashedSourceFilename(InSourceFilename)
	, SourceFilename(InSourceFilename)
	, FunctionName(InFunctionName)
	, Frequency(InFrequency)
	, TypeSize(InTypeSize)
	, TotalPermutationCount(InTotalPermutationCount)
	, TotalSpecializationCount(InTotalSpecializationCount)
	, ConstructSerializedRef(InConstructSerializedRef)
	, ConstructCompiledRef(InConstructCompiledRef)
	, ShouldCompilePermutationRef(InShouldCompilePermutationRef)
	, ShouldPrecachePermutationRef(InShouldPrecachePermutationRef)
	, GetUnspecializedIdRef(InGetUnspecializedIdRef)
	, GetSpecializationValuesRef(InGetSpecializationValuesRef)
	, GetRayTracingPayloadTypeRef(InGetRayTracingPayloadTypeRef)
	, GetShaderBindingLayoutTypeRef(InGetShaderBindingLayoutTypeRef)
#if WITH_EDITOR
	, ModifyCompilationEnvironmentRef(InModifyCompilationEnvironmentRef)
	, ValidateCompiledResultRef(InValidateCompiledResultRef)
	, GetOverrideJobPriorityRef(InGetOverrideJobPriorityRef)
	, GetPermutationIdStringRef(InGetPermutationIdStringRef)
#endif // WITH_EDITOR
	, RootParametersMetadata(InRootParametersMetadata)
	, GlobalListLink(this)
{
	FTypeLayoutDesc::Register(InTypeLayout);

	//make sure the name is shorter than the maximum serializable length
	check(FCString::Strlen(InName) < NAME_SIZE);

	// Make sure the format of the source file path is right.
	check(CheckVirtualShaderFilePath(InSourceFilename));

	// register this shader type
	GlobalListLink.LinkHead(GetTypeList());
	GetNameToTypeMap().Add(HashedName, this);

	TArray<FShaderType*>& SortedTypes = GetSortedShaderTypes(InShaderTypeForDynamicCast);
	const int32 SortedIndex = Algo::LowerBoundBy(SortedTypes, HashedName, [](const FShaderType* InType) { return InType->GetHashedName(); });
	SortedTypes.Insert(this, SortedIndex);
}

FShaderType::~FShaderType()
{
	GlobalListLink.Unlink();
	GetNameToTypeMap().Remove(HashedName);

	TArray<FShaderType*>& SortedTypes = GetSortedShaderTypes(ShaderTypeForDynamicCast);
	const int32 SortedIndex = Algo::BinarySearchBy(SortedTypes, HashedName, [](const FShaderType* InType) { return InType->GetHashedName(); });
	check(SortedIndex != INDEX_NONE);
	SortedTypes.RemoveAt(SortedIndex);
}

bool FShaderTypeRegistration::bShaderTypesInitialized = false;
static TArray<const FShaderTypeRegistration*>* GShaderTypeRegistrationInstances = nullptr;

FShaderTypeRegistration::FShaderTypeRegistration(TFunctionRef<FShaderType& ()> LazyShaderTypeAccessor)
	: LazyShaderTypeAccessor(LazyShaderTypeAccessor)
{
	checkf(!AreShaderTypesInitialized(), TEXT("Shader type was loaded too late, use ELoadingPhase::PostConfigInit on your module to cause it to load earlier. This shader will not be compiled or function.  The module and shader will appear in the callstack of this assert."));
	GetInstances().Add(this);
}

TArray<const FShaderTypeRegistration*>& FShaderTypeRegistration::GetInstances()
{
	if (GShaderTypeRegistrationInstances == nullptr)
	{
		GShaderTypeRegistrationInstances = new TArray<const FShaderTypeRegistration*>();
	}
	return *GShaderTypeRegistrationInstances;
}

void FShaderTypeRegistration::CommitAll()
{
	for (const auto& Instance : GetInstances())
	{
		FShaderType& ShaderType = Instance->LazyShaderTypeAccessor(); // constructs and registers type
	}
	GetInstances().Empty();
	bShaderTypesInitialized = true;
}

bool FShaderTypeRegistration::AreShaderTypesInitialized()
{
	return bShaderTypesInitialized;
}

TLinkedList<FShaderType*>*& FShaderType::GetTypeList()
{
	return GShaderTypeList;
}

FShaderType* FShaderType::GetShaderTypeByName(const TCHAR* Name)
{
	for(TLinkedList<FShaderType*>::TIterator It(GetTypeList()); It; It.Next())
	{
		FShaderType* Type = *It;

		if (FPlatformString::Strcmp(Name, Type->GetName()) == 0)
		{
			return Type;
		}
	}

	return nullptr;
}

TArray<const FShaderType*> FShaderType::GetShaderTypesByFilename(const TCHAR* InFilename, bool bSearchAsRegexFilter)
{
	if (bSearchAsRegexFilter)
	{
		const FRegexPattern RegexSearch(FString(InFilename).Replace(TEXT("*"), TEXT("(.)*")), ERegexPatternFlags::CaseInsensitive);
		return FShaderType::GetShaderTypesByFilenameFilter(
			[&RegexSearch](const TCHAR* ShaderTypeFilename) -> bool
			{
				return FRegexMatcher(RegexSearch, ShaderTypeFilename).FindNext();
			}
		);
	}
	else
	{
		return FShaderType::GetShaderTypesByFilenameFilter(
			[InFilename](const TCHAR* ShaderTypeFilename) -> bool
			{
				return FPlatformString::Strcmp(InFilename, ShaderTypeFilename) == 0;
			}
		);
	}
}

TArray<const FShaderType*> FShaderType::GetShaderTypesByFilenameFilter(const TFunction<bool(const TCHAR*)>& InFilenameFilter)
{
	TArray<const FShaderType*> OutShaders;
	for (TLinkedList<FShaderType*>::TIterator It(GetTypeList()); It; It.Next())
	{
		const FShaderType* Type = *It;
		if (InFilenameFilter(Type->GetShaderFilename()))
		{
			OutShaders.Add(Type);
		}
	}
	return OutShaders;
}

TMap<FHashedName, FShaderType*>& FShaderType::GetNameToTypeMap()
{
	static TMap<FHashedName, FShaderType*> ShaderNameToTypeMap;
	return ShaderNameToTypeMap;
}

const TArray<FShaderType*>& FShaderType::GetSortedTypes(EShaderTypeForDynamicCast Type)
{
	return GetSortedShaderTypes(Type);
}

FArchive& operator<<(FArchive& Ar,FShaderType*& Ref)
{
	if(Ar.IsSaving())
	{
		FName ShaderTypeName = Ref ? FName(Ref->Name) : NAME_None;
		Ar << ShaderTypeName;
	}
	else if(Ar.IsLoading())
	{
		FName ShaderTypeName = NAME_None;
		Ar << ShaderTypeName;
		
		Ref = NULL;

		if(ShaderTypeName != NAME_None)
		{
			// look for the shader type in the global name to type map
			FShaderType** ShaderType = FShaderType::GetNameToTypeMap().Find(ShaderTypeName);
			if (ShaderType)
			{
				// if we found it, use it
				Ref = *ShaderType;
			}
			else
			{
				UE_LOGF(LogShaders, Verbose, "ShaderType '%ls' dependency was not found.", *ShaderTypeName.ToString());
			}
		}
	}
	return Ar;
}

FShader* FShaderType::ConstructForDeserialization() const
{
	return (*ConstructSerializedRef)();
}

FShader* FShaderType::ConstructCompiled(const FShader::CompiledShaderInitializerType& Initializer) const
{
	return (*ConstructCompiledRef)(Initializer);
}

static bool ShouldCompileShaderFrequency(EShaderFrequency Frequency, EShaderPlatform ShaderPlatform)
{
	if (IsMobilePlatform(ShaderPlatform))
	{
		return Frequency == SF_Vertex || Frequency == SF_Pixel || Frequency == SF_Compute;
	}

	return true;
}

bool FShaderType::ShouldCompilePermutation(const FShaderPermutationParameters& Parameters) const
{
	return ShouldCompileShaderFrequency((EShaderFrequency)Frequency, Parameters.Platform) && (*ShouldCompilePermutationRef)(Parameters);
}

EShaderPermutationPrecacheRequest FShaderType::ShouldPrecachePermutation(const FShaderPermutationParameters& Parameters) const
{
	return ShouldCompileShaderFrequency((EShaderFrequency)Frequency, Parameters.Platform) ? (*ShouldPrecachePermutationRef)(Parameters) : EShaderPermutationPrecacheRequest::NotUsed;
}

bool FShaderType::IsSpecialization(const int32 PermutationId) const
{
	return PermutationId != (*GetUnspecializedIdRef)(PermutationId);
}

int32 FShaderType::GetUnspecializedId(const int32 PermutationId) const
{
	return (TotalSpecializationCount > 0) ? (*GetUnspecializedIdRef)(PermutationId) : PermutationId;
}

FSpecializationContainerType FShaderType::GetSpecializationValues(const int32 PermutationId) const
{
	return (*GetSpecializationValuesRef)(PermutationId);
}

const FShaderBindingLayout* FShaderType::GetShaderBindingLayout(const FShaderPermutationParameters& Parameters) const
{
	return (*GetShaderBindingLayoutTypeRef)(Parameters);
}


#if WITH_EDITOR

void FShaderType::ModifyCompilationEnvironment(const FShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment) const
{
	(*ModifyCompilationEnvironmentRef)(Parameters, OutEnvironment);
	
	OutEnvironment.ShaderBindingLayout = GetShaderBindingLayout(Parameters);
	if (OutEnvironment.ShaderBindingLayout)
	{
		// Store copy of RHI version of the shader binding layout in the environment so it can be serialized for the shader compiler workers
		OutEnvironment.RHIShaderBindingLayout = OutEnvironment.ShaderBindingLayout->RHILayout;
	}

	if (Frequency == SF_RayHitGroup)
	{
		// TODO: add a define for each of the 3 possible entry points?
		// See UE::ShaderCompilerCommon::ParseRayTracingEntryPoint for how to parse them
	}
	else
	{
		// define the function name as itself so one can use #ifdef to isolate the shader being compiled within a larger .usf file
		OutEnvironment.SetDefine(FunctionName, FunctionName);
	}

#if RHI_RAYTRACING
	ERayTracingPayloadType RayTracingPayloadType = GetRayTracingPayloadType(Parameters.PermutationId);
	switch (Frequency)
	{
		case SF_RayGen:
		{
			// Raygen shader can use any number of payloads, but must use at least one
			checkf(RayTracingPayloadType != ERayTracingPayloadType::None, TEXT("Raygen shader %s did not declare which payload type(s) it uses. Make sure you override GetRayTracingPayloadType()"), Name);
			break;
		}
		case SF_RayHitGroup:
		case SF_RayMiss:
		case SF_RayCallable:
		{
			// these shader types must know which payload type they are using
			checkf(RayTracingPayloadType != ERayTracingPayloadType::None, TEXT("Raytracing shader %s did not declare which payload type(s) it uses. Make sure you override GetRayTracingPayloadType()"), Name);
			checkf(FMath::CountBits(static_cast<uint32>(RayTracingPayloadType)) == 1, TEXT("Raytracing shader %s did not declare a unique payload type. Only one payload type is supported for this shader frequency."), Name);
			break;
		}
		default:
		{
			// not a raytracing shader, specifying a payload type would suggest some confusion has occured
			checkf(RayTracingPayloadType == ERayTracingPayloadType::None, TEXT("Non-Raytracing shader %s declared a payload type!"), Name);
			break;
		}
	}
	if (RayTracingPayloadType != ERayTracingPayloadType::None)
	{
		checkf(IsRayTracingPayloadRegistered(RayTracingPayloadType), TEXT("Raytracing shader %s is using a payload type (%u) which was never registered"), Name, EnumToUnderlyingType(RayTracingPayloadType));

		OutEnvironment.SetDefineAndCompileArgument(TEXT("RT_PAYLOAD_TYPE"), static_cast<uint32>(RayTracingPayloadType));
		OutEnvironment.SetDefineAndCompileArgument(TEXT("RT_PAYLOAD_MAX_SIZE"), GetRayTracingPayloadTypeMaxSize(RayTracingPayloadType));

		if (   (uint32(RayTracingPayloadType) & uint32(ERayTracingPayloadType::RayTracingMaterial))
			|| (uint32(RayTracingPayloadType) & uint32(ERayTracingPayloadType::GPULightmass))
			)
		{
			// If any payload requires a fully simplified material, we force fully simplified material all the way.
			// That is used to have material ray tracing shaders compressed to single slab.
			// Smaller payload means faster performance and for some tracing this will be enough, e.g. reflected materials, lightmass diffuse interactions.
			OutEnvironment.SetDefine(TEXT("SUBSTRATE_USE_FULLYSIMPLIFIED_MATERIAL"), 1);
		}
	}
#endif
}

bool FShaderType::ValidateCompiledResult(EShaderPlatform Platform, const FShaderParameterMap& ParameterMap, TArray<FString>& OutError) const
{
	return (*ValidateCompiledResultRef)(Platform, ParameterMap, OutError);
}

EShaderCompileJobPriority FShaderType::GetOverrideJobPriority() const
{
	return (*GetOverrideJobPriorityRef)();
}

void FShaderType::UpdateReferencedUniformBufferNames(const TMap<FString, TArray<const TCHAR*>>& ShaderFileToUniformBufferVariables)
{
	ReferencedUniformBuffers.Empty();
	GenerateReferencedUniformBuffers(SourceFilename, Name, ShaderFileToUniformBufferVariables, ReferencedUniformBuffers);
}

FString FShaderType::GetPermutationIdString(int32 PermutationId, bool bFullNames) const
{
	// This function is for diagnostics only, so ignore it if the shader type did not initialize its format string
	if (GetPermutationIdStringRef)
	{
		FString PermutationIdentifier;
		(*GetPermutationIdStringRef)(PermutationId, PermutationIdentifier, bFullNames);
		return PermutationIdentifier;
	}
	return FString();
}
#endif // WITH_EDITOR

ERayTracingPayloadType FShaderType::GetRayTracingPayloadType(const int32 PermutationId) const
{
#if RHI_RAYTRACING
	return (*GetRayTracingPayloadTypeRef)(PermutationId);
#else
	return static_cast<ERayTracingPayloadType>(0);
#endif
}

const FSHAHash& FShaderType::GetSourceHash(EShaderPlatform ShaderPlatform) const
{
	return GetShaderFileHash(GetShaderFilename(), ShaderPlatform);
}

void FShaderType::Initialize(const TMap<FString, TArray<const TCHAR*> >& ShaderFileToUniformBufferVariables)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FShaderType::Initialize);

#if WITH_EDITOR
	//#todo-rco: Need to call this only when Initializing from a Pipeline once it's removed from the global linked list
	if (!FPlatformProperties::RequiresCookedData())
	{
#if UE_BUILD_DEBUG
		TArray<FShaderType*> UniqueShaderTypes;
#endif
		TArray<UE::Tasks::TTask<void>> Tasks;
		Tasks.Reserve(FShaderType::GetNameToTypeMap().Num());

		for(TLinkedList<FShaderType*>::TIterator It(FShaderType::GetTypeList()); It; It.Next())
		{
			FShaderType* Type = *It;
#if UE_BUILD_DEBUG
			UniqueShaderTypes.Add(Type);
#endif
			Tasks.Emplace(
				UE::Tasks::Launch(
					TEXT("UpdateReferencedUniformBufferNames"),
					[Type, &ShaderFileToUniformBufferVariables]() mutable
					{
						Type->UpdateReferencedUniformBufferNames(ShaderFileToUniformBufferVariables);
					}
				)
			);
		}
		UE::Tasks::Wait(Tasks);
	
#if UE_BUILD_DEBUG
		// Check for duplicated shader type names
		UniqueShaderTypes.Sort([](const FShaderType& A, const FShaderType& B) { return (SIZE_T)&A < (SIZE_T)&B; });
		for (int32 Index = 1; Index < UniqueShaderTypes.Num(); ++Index)
		{
			checkf(UniqueShaderTypes[Index - 1] != UniqueShaderTypes[Index], TEXT("Duplicated FShader type name %s found, please rename one of them!"), UniqueShaderTypes[Index]->GetName());
		}
#endif
	}
#endif // WITH_EDITOR
}

int32 FShaderMapPointerTable::AddIndexedPointer(const FTypeLayoutDesc& TypeDesc, void* Ptr)
{
	int32 Index = INDEX_NONE;
	if (ShaderTypes.TryAddIndexedPtr(TypeDesc, Ptr, Index)) return Index;
	if (VFTypes.TryAddIndexedPtr(TypeDesc, Ptr, Index)) return Index;
	return Index;
}

void* FShaderMapPointerTable::GetIndexedPointer(const FTypeLayoutDesc& TypeDesc, uint32 i) const
{
	void* Ptr = nullptr;
	if (ShaderTypes.TryGetIndexedPtr(TypeDesc, i, Ptr)) return Ptr;
	if (VFTypes.TryGetIndexedPtr(TypeDesc, i, Ptr)) return Ptr;
	return Ptr;
}

void FShaderMapPointerTable::SaveToArchive(FArchive& Ar, const FPlatformTypeLayoutParameters& LayoutParams, const void* FrozenObject) const
{
	FPointerTableBase::SaveToArchive(Ar, LayoutParams, FrozenObject);

	int32 NumTypes = ShaderTypes.Num();
	int32 NumVFTypes = VFTypes.Num();

	Ar << NumTypes;
	Ar << NumVFTypes;

	for (int32 TypeIndex = 0; TypeIndex < NumTypes; ++TypeIndex)
	{
		const FShaderType* Type = ShaderTypes.GetIndexedPointer(TypeIndex);
		FHashedName TypeName = Type->GetHashedName();
		Ar << TypeName;
	}

	for (int32 VFTypeIndex = 0; VFTypeIndex < NumVFTypes; ++VFTypeIndex)
	{
		const FVertexFactoryType* VFType = VFTypes.GetIndexedPointer(VFTypeIndex);
		FHashedName TypeName = VFType->GetHashedName();
		Ar << TypeName;
	}
}

bool FShaderMapPointerTable::LoadFromArchive(FArchive& Ar, const FPlatformTypeLayoutParameters& LayoutParams, void* FrozenObject)
{
	SCOPED_LOADTIMER(FShaderMapPointerTable_LoadFromArchive);

	const bool bResult = FPointerTableBase::LoadFromArchive(Ar, LayoutParams, FrozenObject);

	int32 NumTypes = 0;
	int32 NumVFTypes = 0;
		
	Ar << NumTypes;
	Ar << NumVFTypes;

	ShaderTypes.Empty(NumTypes);
	for (int32 TypeIndex = 0; TypeIndex < NumTypes; ++TypeIndex)
	{
		FHashedName TypeName;
		Ar << TypeName;
		FShaderType* Type = FindShaderTypeByName(TypeName);
		ShaderTypes.LoadIndexedPointer(Type);
	}

	VFTypes.Empty(NumVFTypes);
	for (int32 VFTypeIndex = 0; VFTypeIndex < NumVFTypes; ++VFTypeIndex)
	{
		FHashedName TypeName;
		Ar << TypeName;
		FVertexFactoryType* VFType = FVertexFactoryType::GetVFByName(TypeName);
		VFTypes.LoadIndexedPointer(VFType);
	}

	return bResult;
}

FShaderCompiledShaderInitializerType::FShaderCompiledShaderInitializerType(
	const FShaderType* InType,
	const FShaderType::FParameters* InParameters,
	int32 InPermutationId,
	const FShaderCompilerOutput& CompilerOutput,
	uint64 InMaterialShaderMapHash,
	const FShaderPipelineType* InShaderPipeline,
	const FVertexFactoryType* InVertexFactoryType
	) 
	: Type(InType)
	, Parameters(InParameters)
	, Target(CompilerOutput.Target)
	, Code(CompilerOutput.ShaderCode.GetReadView())
	, ParameterMap(CompilerOutput.ParameterMap)
	, OutputHash(CompilerOutput.OutputHash)
	, MaterialShaderMapHash(InMaterialShaderMapHash)
	, ShaderPipeline(InShaderPipeline)
	, VertexFactoryType(InVertexFactoryType)
	, NumInstructions(CompilerOutput.NumInstructions)
	, NumTextureSamplers(CompilerOutput.NumTextureSamplers)
	, CodeSize(CompilerOutput.ShaderCode.GetShaderCodeSize())
	, PermutationId(InPermutationId)
	, ShaderStatistics(CompilerOutput.ShaderStatistics)
{
}

/** 
 * Used to construct a shader for deserialization.
 * This still needs to initialize members to safe values since FShaderType::GenerateSerializationHistory uses this constructor.
 */
FShader::FShader()
	// set to undefined (currently shared with SF_Vertex)
	: Target((EShaderFrequency)0, GShaderPlatformForFeatureLevel[GMaxRHIFeatureLevel])
	, ResourceIndex(INDEX_NONE)
#if WITH_EDITORONLY_DATA
	, NumInstructions(0u)
	, NumTextureSamplers(0u)
	, CodeSize(0u)
#endif // WITH_EDITORONLY_DATA
{
}

/**
 * Construct a shader from shader compiler output.
 */
FShader::FShader(const CompiledShaderInitializerType& Initializer)
	: Type(const_cast<FShaderType*>(Initializer.Type)) // TODO - remove const_cast, make TIndexedPtr work with 'const'
	, VFType(const_cast<FVertexFactoryType*>(Initializer.VertexFactoryType))
	, Target(Initializer.Target)
	, ResourceIndex(INDEX_NONE)
#if WITH_EDITORONLY_DATA
	, NumInstructions(Initializer.NumInstructions)
	, NumTextureSamplers(Initializer.NumTextureSamplers)
	, CodeSize(Initializer.CodeSize)
#endif // WITH_EDITORONLY_DATA
{
	checkSlow(Initializer.OutputHash != FShaderHash());

	// Only store a truncated hash to minimize memory overhead
	static_assert(sizeof(SortKey) <= sizeof(Initializer.OutputHash.Hash));
	FMemory::Memcpy(&SortKey, &Initializer.OutputHash.Hash, sizeof(SortKey));

#if WITH_EDITORONLY_DATA
	OutputHash = Initializer.OutputHash;

	// Store off the source hash that this shader was compiled with
	// This will be used as part of the shader key in order to identify when shader files have been changed and a recompile is needed
	SourceHash = Initializer.Type->GetSourceHash(Initializer.Target.GetPlatform());

	if (Initializer.VertexFactoryType)
	{
		// Store off the VF source hash that this shader was compiled with
		VFSourceHash = Initializer.VertexFactoryType->GetSourceHash(Initializer.Target.GetPlatform());
	}
#endif // WITH_EDITORONLY_DATA

	BuildParameterMapInfo(Initializer.ParameterMap.GetParameterMap());

	// Bind uniform buffer parameters automatically 
	for (TLinkedList<FShaderParametersMetadata*>::TIterator StructIt(FShaderParametersMetadata::GetStructList()); StructIt; StructIt.Next())
	{
		if (Initializer.ParameterMap.ContainsParameterAllocation(StructIt->GetShaderVariableName()))
		{
			UniformBufferParameterStructs.Add(StructIt->GetShaderVariableHashedName());
			FShaderUniformBufferParameter& Parameter = UniformBufferParameters.AddDefaulted_GetRef();
			Parameter.Bind(Initializer.ParameterMap, StructIt->GetShaderVariableName(), SPF_Mandatory);
		}
	}

	// Register the shader now that it is valid, so that it can be reused
	//Register(false);
}

FShader::~FShader()
{
}

void FShader::Finalize(const FShaderMapResourceCode* Code)
{
	const FShaderHash& Hash = GetOutputHash();
	const int32 NewResourceIndex = Code->FindShaderIndex(Hash);
	checkf(NewResourceIndex != INDEX_NONE, TEXT("Missing shader code %s"), *Hash.ToString());
	ResourceIndex = NewResourceIndex;
}

template<class TType>
static void CityHashArray(uint64& Hash, const TMemoryImageArray<TType>& Array)
{
	const int32 ArrayNum = Array.Num();
	CityHash64WithSeed((const char*)&ArrayNum, sizeof(ArrayNum), Hash);
	CityHash64WithSeed((const char*)Array.GetData(), Array.Num() * sizeof(TType), Hash);
}

void FShader::BuildParameterMapInfo(const TMap<FString, FParameterAllocation>& ParameterMap)
{
	uint32 UniformCount = 0;
	uint32 SamplerCount = 0;
	uint32 SRVCount = 0;

	for (TMap<FString, FParameterAllocation>::TConstIterator ParameterIt(ParameterMap); ParameterIt; ++ParameterIt)
	{
		const FParameterAllocation& ParamValue = ParameterIt.Value();

		switch (ParamValue.Type)
		{
		case EShaderParameterType::UniformBuffer:
			UniformCount++;
			break;
		case EShaderParameterType::BindlessSampler:
		case EShaderParameterType::Sampler:
			SamplerCount++;
			break;
		case EShaderParameterType::BindlessSRV:
		case EShaderParameterType::SRV:
			SRVCount++;
			break;
		}
	}

	ParameterMapInfo.UniformBuffers.Empty(UniformCount);
	ParameterMapInfo.TextureSamplers.Empty(SamplerCount);
	ParameterMapInfo.SRVs.Empty(SRVCount);

	auto GetResourceParameterMap = [this](EShaderParameterType ParameterType) -> TMemoryImageArray<FShaderResourceParameterInfo>*
	{
		switch (ParameterType)
		{
		case EShaderParameterType::Sampler:
			return &ParameterMapInfo.TextureSamplers;
		case EShaderParameterType::SRV:
			return &ParameterMapInfo.SRVs;
		case EShaderParameterType::BindlessSRV:
			return &ParameterMapInfo.SRVs;
		case EShaderParameterType::BindlessSampler:
			return &ParameterMapInfo.TextureSamplers;
		default:
			return nullptr;
		}
	};

	for (TMap<FString, FParameterAllocation>::TConstIterator ParameterIt(ParameterMap); ParameterIt; ++ParameterIt)
	{
		const FParameterAllocation& ParamValue = ParameterIt.Value();

		if (ParamValue.Type == EShaderParameterType::LooseData)
		{
			bool bAddedToExistingBuffer = false;

			for (int32 LooseParameterBufferIndex = 0; LooseParameterBufferIndex < ParameterMapInfo.LooseParameterBuffers.Num(); LooseParameterBufferIndex++)
			{
				FShaderLooseParameterBufferInfo& LooseParameterBufferInfo = ParameterMapInfo.LooseParameterBuffers[LooseParameterBufferIndex];

				if (LooseParameterBufferInfo.BaseIndex == ParamValue.BufferIndex)
				{
					LooseParameterBufferInfo.Parameters.Emplace(ParamValue.BaseIndex, ParamValue.Size);
					LooseParameterBufferInfo.Size += ParamValue.Size;
					bAddedToExistingBuffer = true;
				}
			}

			if (!bAddedToExistingBuffer)
			{
				FShaderLooseParameterBufferInfo NewParameterBufferInfo(ParamValue.BufferIndex, ParamValue.Size);

				NewParameterBufferInfo.Parameters.Emplace(ParamValue.BaseIndex, ParamValue.Size);

				ParameterMapInfo.LooseParameterBuffers.Add(NewParameterBufferInfo);
			}
		}
		else if (ParamValue.Type == EShaderParameterType::UniformBuffer)
		{
			ParameterMapInfo.UniformBuffers.Emplace(ParamValue.BufferIndex);
		}
		else if (TMemoryImageArray<FShaderResourceParameterInfo>* ParameterInfoArray = GetResourceParameterMap(ParamValue.Type))
		{
			ParameterInfoArray->Emplace(ParamValue.BaseIndex, ParamValue.BufferIndex, ParamValue.Type);
		}
	}

	for (FShaderLooseParameterBufferInfo& Info : ParameterMapInfo.LooseParameterBuffers)
	{
		Info.Parameters.Sort();
	}
	ParameterMapInfo.LooseParameterBuffers.Sort();
	ParameterMapInfo.UniformBuffers.Sort();
	ParameterMapInfo.TextureSamplers.Sort();
	ParameterMapInfo.SRVs.Sort();

	uint64 Hash = 0;

	{
		const auto CityHashValue = [&](auto Value)
		{
			CityHash64WithSeed((const char*)&Value, sizeof(Value), Hash);
		};

		for (FShaderLooseParameterBufferInfo& Info : ParameterMapInfo.LooseParameterBuffers)
		{
			CityHashValue(Info.BaseIndex);
			CityHashValue(Info.Size);
			CityHashArray(Hash, Info.Parameters);
		}
		CityHashArray(Hash, ParameterMapInfo.UniformBuffers);
		CityHashArray(Hash, ParameterMapInfo.TextureSamplers);
		CityHashArray(Hash, ParameterMapInfo.SRVs);
	}

	ParameterMapInfo.Hash = Hash;
}

const FShaderHash& FShader::GetOutputHash() const
{
#if WITH_EDITORONLY_DATA
	return OutputHash;
#else
	static const FShaderHash DefaultOutputHash;
	return DefaultOutputHash;
#endif
}

const FSHAHash& FShader::GetHash() const 
{
#if WITH_EDITORONLY_DATA
	return SourceHash;
#else
	return ShaderSourceDefaultHash;
#endif
}

const FSHAHash& FShader::GetVertexFactoryHash() const
{
#if WITH_EDITORONLY_DATA
	return VFSourceHash;
#else
	return ShaderSourceDefaultHash;
#endif
}

const FTypeLayoutDesc& GetTypeLayoutDesc(const FPointerTableBase* PtrTable, const FShader& Shader)
{
	const FShaderType* Type = Shader.GetType(PtrTable);
	checkf(Type, TEXT("FShaderType is missing"));
	return Type->GetLayout();
}

const FShaderParametersMetadata* FShader::FindAutomaticallyBoundUniformBufferStruct(int32 BaseIndex) const
{
	for (int32 i = 0; i < UniformBufferParameters.Num(); i++)
	{
		if (UniformBufferParameters[i].GetBaseIndex() == BaseIndex)
		{
			FShaderParametersMetadata** Parameters = FShaderParametersMetadata::GetNameStructMap().Find(UniformBufferParameterStructs[i]);
			return Parameters ? *Parameters : nullptr;
		}
	}

	return nullptr;
}

void FShader::DumpDebugInfo(const FShaderMapPointerTable& InPtrTable)
{
	FVertexFactoryType* VertexFactoryType = GetVertexFactoryType(InPtrTable);

	UE_LOGF(LogConsoleResponse, Display, "      FShader  :Frequency %ls", GetShaderFrequencyString(GetFrequency()));
	UE_LOGF(LogConsoleResponse, Display, "               :Target %ls", *LegacyShaderPlatformToShaderFormat(GetShaderPlatform()).ToString());
	UE_LOGF(LogConsoleResponse, Display, "               :VFType %ls", VertexFactoryType ? VertexFactoryType->GetName() : TEXT("null"));
	UE_LOGF(LogConsoleResponse, Display, "               :Type %ls", GetType(InPtrTable)->GetName());
	UE_LOGF(LogConsoleResponse, Display, "               :SourceHash %ls", *GetHash().ToString());
	UE_LOGF(LogConsoleResponse, Display, "               :VFSourceHash %ls", *GetVertexFactoryHash().ToString());
	UE_LOGF(LogConsoleResponse, Display, "               :OutputHash %ls", *GetOutputHash().ToString());
}

#if WITH_EDITOR
void FShader::SaveShaderStableKeys(const FShaderMapPointerTable& InPtrTable, EShaderPlatform TargetShaderPlatform, int32 PermutationId, const FStableShaderKeyAndValue& InSaveKeyVal)
{
	if ((TargetShaderPlatform == EShaderPlatform::SP_NumPlatforms || GetShaderPlatform() == TargetShaderPlatform) 
		&& FShaderLibraryCooker::NeedsShaderStableKeys(TargetShaderPlatform))
	{
		FShaderType* ShaderType = GetType(InPtrTable);
		FVertexFactoryType* VertexFactoryType = GetVertexFactoryType(InPtrTable);

		FStableShaderKeyAndValue SaveKeyVal(InSaveKeyVal);
		SaveKeyVal.TargetFrequency = FName(GetShaderFrequencyString(GetFrequency()));
		SaveKeyVal.TargetPlatform = LegacyShaderPlatformToShaderFormat(GetShaderPlatform());
		SaveKeyVal.VFType = FName(VertexFactoryType ? VertexFactoryType->GetName() : TEXT("null"));
		SaveKeyVal.PermutationId = FName(*FString::Printf(TEXT("Perm_%d"), PermutationId));
		SaveKeyVal.OutputHash = GetOutputHash();
		if (ShaderType)
		{
			ShaderType->GetShaderStableKeyParts(SaveKeyVal);
		}
		FShaderLibraryCooker::AddShaderStableKeyValue(GetShaderPlatform(), SaveKeyVal);
	}
}
#endif // WITH_EDITOR

bool FShaderPipelineType::bInitialized = false;

static TArray<FShaderPipelineType*>& GetSortedShaderPipelineTypes(FShaderType::EShaderTypeForDynamicCast Type)
{
	static TArray<FShaderPipelineType*> SortedTypes[(uint32)FShaderType::EShaderTypeForDynamicCast::NumShaderTypes];
	return SortedTypes[(uint32)Type];
}

FShaderPipelineType::FShaderPipelineType(
	const TCHAR* InName,
	const FShaderType* InVertexOrMeshShader,
	const FShaderType* InGeometryOrAmplificationShader,
	const FShaderType* InPixelShader,
	bool bInIsMeshPipeline,
	bool bInShouldOptimizeUnusedOutputs) :
	Name(InName),
	TypeName(Name),
	HashedName(TypeName),
	HashedPrimaryShaderFilename(InVertexOrMeshShader->GetShaderFilename()),
	GlobalListLink(this),
	bShouldOptimizeUnusedOutputs(bInShouldOptimizeUnusedOutputs)
{
	checkf(Name && *Name, TEXT("Shader Pipeline Type requires a valid Name!"));

	checkf(InVertexOrMeshShader, TEXT("A Shader Pipeline always requires a Vertex or Mesh Shader"));

	//make sure the name is shorter than the maximum serializable length
	check(FCString::Strlen(InName) < NAME_SIZE);

	FMemory::Memzero(AllStages);

	if (InPixelShader)
	{
		check(InPixelShader->GetTypeForDynamicCast() == InVertexOrMeshShader->GetTypeForDynamicCast());
		Stages.Add(InPixelShader);
		AllStages[SF_Pixel] = InPixelShader;
	}

	if (InGeometryOrAmplificationShader)
	{
		check(InGeometryOrAmplificationShader->GetTypeForDynamicCast() == InVertexOrMeshShader->GetTypeForDynamicCast());
		Stages.Add(InGeometryOrAmplificationShader);
		AllStages[bInIsMeshPipeline ? SF_Amplification : SF_Geometry] = InGeometryOrAmplificationShader;
	}
	Stages.Add(InVertexOrMeshShader);
	AllStages[bInIsMeshPipeline ? SF_Mesh : SF_Vertex] = InVertexOrMeshShader;

	for (uint32 FrequencyIndex = 0; FrequencyIndex < SF_NumStandardFrequencies; ++FrequencyIndex)
	{
		if (const FShaderType* ShaderType = AllStages[FrequencyIndex])
		{
			checkf(ShaderType->GetPermutationCount() == 1, TEXT("Shader '%s' has multiple shader permutations. Shader pipelines only support a single permutation."), ShaderType->GetName())
		}
	}

	static uint32 TypeHashCounter = 0;
	++TypeHashCounter;
	HashIndex = TypeHashCounter;

	GlobalListLink.LinkHead(GetTypeList());
	GetNameToTypeMap().Add(HashedName, this);

	TArray<FShaderPipelineType*>& SortedTypes = GetSortedShaderPipelineTypes(InVertexOrMeshShader->GetTypeForDynamicCast());
	const int32 SortedIndex = Algo::LowerBoundBy(SortedTypes, HashedName, [](const FShaderPipelineType* InType) { return InType->GetHashedName(); });
	SortedTypes.Insert(this, SortedIndex);

	// This will trigger if an IMPLEMENT_SHADER_TYPE was in a module not loaded before InitializeShaderTypes
	// Shader types need to be implemented in modules that are loaded before that
	checkf(!bInitialized, TEXT("Shader Pipeline was loaded after Engine init, use ELoadingPhase::PostConfigInit on your module to cause it to load earlier."));
}

FShaderPipelineType::~FShaderPipelineType()
{
	GetNameToTypeMap().Remove(HashedName);
	GlobalListLink.Unlink();

	TArray<FShaderPipelineType*>& SortedTypes = GetSortedShaderPipelineTypes(AllStages[HasMeshShader() ? SF_Mesh : SF_Vertex]->GetTypeForDynamicCast());
	const int32 SortedIndex = Algo::BinarySearchBy(SortedTypes, HashedName, [](const FShaderPipelineType* InType) { return InType->GetHashedName(); });
	check(SortedIndex != INDEX_NONE);
	SortedTypes.RemoveAt(SortedIndex);
}

TMap<FHashedName, FShaderPipelineType*>& FShaderPipelineType::GetNameToTypeMap()
{
	static TMap<FHashedName, FShaderPipelineType*> GShaderPipelineNameToTypeMap;
	return GShaderPipelineNameToTypeMap;
}

TLinkedList<FShaderPipelineType*>*& FShaderPipelineType::GetTypeList()
{
	return GShaderPipelineList;
}

const TArray<FShaderPipelineType*>& FShaderPipelineType::GetSortedTypes(FShaderType::EShaderTypeForDynamicCast Type)
{
	return GetSortedShaderPipelineTypes(Type);
}

TArray<const FShaderPipelineType*> FShaderPipelineType::GetShaderPipelineTypesByFilename(const TCHAR* InFilename, bool bSearchAsRegexFilter)
{
	if (bSearchAsRegexFilter)
	{
		const FRegexPattern RegexSearch(FString(InFilename).Replace(TEXT("*"), TEXT("(.)*")), ERegexPatternFlags::CaseInsensitive);
		return FShaderPipelineType::GetShaderPipelineTypesByFilenameFilter(
			[&RegexSearch](const TCHAR* ShaderTypeFilename) -> bool
			{
				return FRegexMatcher(RegexSearch, ShaderTypeFilename).FindNext();
			}
		);
	}
	else
	{
		return FShaderPipelineType::GetShaderPipelineTypesByFilenameFilter(
			[InFilename](const TCHAR* ShaderTypeFilename) -> bool
			{
				return FPlatformString::Strcmp(InFilename, ShaderTypeFilename) == 0;
			}
		);
	}
}

TArray<const FShaderPipelineType*> FShaderPipelineType::GetShaderPipelineTypesByFilenameFilter(const TFunction<bool(const TCHAR*)>& InFilenameFilter)
{
	TArray<const FShaderPipelineType*> PipelineTypes;
	for (TLinkedList<FShaderPipelineType*>::TIterator It(FShaderPipelineType::GetTypeList()); It; It.Next())
	{
		auto* PipelineType = *It;
		for (auto* ShaderType : PipelineType->Stages)
		{
			if (InFilenameFilter(ShaderType->GetShaderFilename()))
			{
				PipelineTypes.AddUnique(PipelineType);
				break;
			}
		}
	}
	return PipelineTypes;
}

void FShaderPipelineType::Initialize()
{
	check(!bInitialized);

	TSet<FName> UsedNames;

#if UE_BUILD_DEBUG
	TArray<const FShaderPipelineType*> UniqueShaderPipelineTypes;
#endif
	for (TLinkedList<FShaderPipelineType*>::TIterator It(FShaderPipelineType::GetTypeList()); It; It.Next())
	{
		const auto* PipelineType = *It;

#if UE_BUILD_DEBUG
		UniqueShaderPipelineTypes.Add(PipelineType);
#endif

		// Validate stages
		for (int32 Index = 0; Index < SF_NumFrequencies; ++Index)
		{
			check(!PipelineType->AllStages[Index] || PipelineType->AllStages[Index]->GetFrequency() == (EShaderFrequency)Index);
		}

		auto& Stages = PipelineType->GetStages();

		// #todo-rco: Do we allow mix/match of global/mesh/material stages?
		// Check all shaders are the same type, start from the top-most stage
		const FGlobalShaderType* GlobalType = Stages[0]->GetGlobalShaderType();
		const FMeshMaterialShaderType* MeshType = Stages[0]->GetMeshMaterialShaderType();
		const FMaterialShaderType* MateriallType = Stages[0]->GetMaterialShaderType();
		for (int32 Index = 1; Index < Stages.Num(); ++Index)
		{
			if (GlobalType)
			{
				checkf(Stages[Index]->GetGlobalShaderType(), TEXT("Invalid combination of Shader types on Pipeline %s"), PipelineType->Name);
			}
			else if (MeshType)
			{
				checkf(Stages[Index]->GetMeshMaterialShaderType(), TEXT("Invalid combination of Shader types on Pipeline %s"), PipelineType->Name);
			}
			else if (MateriallType)
			{
				checkf(Stages[Index]->GetMaterialShaderType(), TEXT("Invalid combination of Shader types on Pipeline %s"), PipelineType->Name);
			}
		}

		FName PipelineName = PipelineType->GetFName();
		checkf(!UsedNames.Contains(PipelineName), TEXT("Two Pipelines with the same name %s found!"), PipelineType->Name);
		UsedNames.Add(PipelineName);
	}

#if UE_BUILD_DEBUG
	// Check for duplicated shader pipeline type names
	UniqueShaderPipelineTypes.Sort([](const FShaderPipelineType& A, const FShaderPipelineType& B) { return (SIZE_T)&A < (SIZE_T)&B; });
	for (int32 Index = 1; Index < UniqueShaderPipelineTypes.Num(); ++Index)
	{
		checkf(UniqueShaderPipelineTypes[Index - 1] != UniqueShaderPipelineTypes[Index], TEXT("Duplicated FShaderPipeline type name %s found, please rename one of them!"), UniqueShaderPipelineTypes[Index]->GetName());
	}
#endif

	bInitialized = true;
}

const FShaderPipelineType* FShaderPipelineType::GetShaderPipelineTypeByName(const FHashedName& Name)
{
	FShaderPipelineType** FoundType = GetNameToTypeMap().Find(Name);
	return FoundType ? *FoundType : nullptr;
}

bool FShaderPipelineType::ShouldOptimizeUnusedOutputs(EShaderPlatform Platform) const
{
	return bShouldOptimizeUnusedOutputs && RHISupportsShaderPipelines(Platform);
}

const FSHAHash& FShaderPipelineType::GetSourceHash(EShaderPlatform ShaderPlatform) const
{
	TArray<FString> Filenames;
	for (const FShaderType* ShaderType : Stages)
	{
		Filenames.Add(ShaderType->GetShaderFilename());
	}
	return GetShaderFilesHash(Filenames, ShaderPlatform);
}

bool FShaderPipelineType::ShouldCompilePermutation(const FShaderPermutationParameters& Parameters) const
{
	for (const FShaderType* ShaderType : Stages)
	{
		if (!ShaderType->ShouldCompilePermutation(Parameters))
		{
			return false;
		}
	}
	return true;
}

EShaderPermutationPrecacheRequest FShaderPipelineType::ShouldPrecachePermutation(const FShaderPermutationParameters& Parameters) const
{
	EShaderPermutationPrecacheRequest Result = EShaderPermutationPrecacheRequest::NotUsed;
	for (const FShaderType* ShaderType : Stages)
	{
		EShaderPermutationPrecacheRequest ShaderTypeRequest = ShaderType->ShouldPrecachePermutation(Parameters);
		if (ShaderTypeRequest == EShaderPermutationPrecacheRequest::Precached)
		{
			return ShaderTypeRequest;
		}
		else if (ShaderTypeRequest == EShaderPermutationPrecacheRequest::NotPrecached)
		{
			Result = ShaderTypeRequest;
		}
	}
	return Result;
}

void FShaderTypeDependency::RefreshCachedSourceHash(EShaderPlatform ShaderPlatform)
{
	const FShaderType*const* ShaderTypePtr = FShaderType::GetNameToTypeMap().Find(ShaderTypeName);
	const FShaderType* ShaderType = ShaderTypePtr ? *ShaderTypePtr : nullptr;
	if (!ShaderType)
	{
		SourceHash = FSHAHash();
		return;
	}
	SourceHash = ShaderType->GetSourceHash(ShaderPlatform);
}

void FShaderPipelineTypeDependency::RefreshCachedSourceHash(EShaderPlatform ShaderPlatform)
{
	const FShaderPipelineType* ShaderPipelineType =
		FShaderPipelineType::GetShaderPipelineTypeByName(ShaderPipelineTypeName);
	if (!ShaderPipelineType)
	{
		StagesSourceHash = FSHAHash();
		return;
	}
	StagesSourceHash = ShaderPipelineType->GetSourceHash(ShaderPlatform);
}

#if WITH_EDITOR

void FShaderTypeDependency::Save(FCbWriter& Writer) const
{
	Writer.BeginArray();
	Writer << ShaderTypeName;
	Writer << SourceHash;
	Writer << PermutationId;
	Writer.EndArray();
}

bool FShaderTypeDependency::TryLoad(FCbFieldView Field)
{
	*this = FShaderTypeDependency();
	FCbFieldViewIterator ElementField(Field.CreateViewIterator());
	if (!LoadFromCompactBinary(ElementField++, ShaderTypeName))
	{
		return false;
	}
	if (!LoadFromCompactBinary(ElementField++, SourceHash))
	{
		return false;
	}
	PermutationId = ElementField.AsInt32();
	if ((ElementField++).HasError())
	{
		return false;
	}
	return true;
}

bool LoadFromCompactBinary(FCbFieldView Field, FShaderTypeDependency& OutValue)
{
	return OutValue.TryLoad(Field);
}

void FShaderPipelineTypeDependency::Save(FCbWriter& Writer) const
{
	Writer.BeginArray();
	Writer << ShaderPipelineTypeName;
	Writer << StagesSourceHash;
	Writer.EndArray();
}

bool FShaderPipelineTypeDependency::TryLoad(FCbFieldView Field)
{
	*this = FShaderPipelineTypeDependency();
	FCbFieldViewIterator ElementField(Field.CreateViewIterator());
	if (!LoadFromCompactBinary(ElementField++, ShaderPipelineTypeName))
	{
		return false;
	}
	if (!LoadFromCompactBinary(ElementField++, StagesSourceHash))
	{
		return false;
	}
	return true;
}

bool LoadFromCompactBinary(FCbFieldView Field, FShaderPipelineTypeDependency& OutValue)
{
	return OutValue.TryLoad(Field);
}
#endif // WITH_EDITOR

void FShaderPipeline::AddShader(FShader* Shader, int32 PermutationId)
{
	const EShaderFrequency Frequency = Shader->GetFrequency();
	check(Shaders[Frequency].IsNull());
	Shaders[Frequency] = Shader;
	PermutationIds[Frequency] = PermutationId;
}

FShader* FShaderPipeline::FindOrAddShader(FShader* Shader, int32 PermutationId)
{
	const EShaderFrequency Frequency = Shader->GetFrequency();
	FShader* PrevShader = Shaders[Frequency];
	if (PrevShader && PermutationIds[Frequency] == PermutationId)
	{
		DeleteObjectFromLayout(Shader);
		return PrevShader;
	}

	Shaders[Frequency].SafeDelete();
	Shaders[Frequency] = Shader;
	PermutationIds[Frequency] = PermutationId;
	return Shader;
}

FShaderPipeline::~FShaderPipeline()
{
	// Manually set references to nullptr, helps debugging
	for (uint32 i = 0u; i < SF_NumGraphicsFrequencies; ++i)
	{
		Shaders[i] = nullptr;
	}
}

void FShaderPipeline::Validate(const FShaderPipelineType* InPipelineType) const
{
	check(InPipelineType->GetHashedName() == TypeName);
	for (const FShaderType* Stage : InPipelineType->GetStages())
	{
		const FShader* Shader = GetShader(Stage->GetFrequency());
		check(Shader);
		check(Shader->GetTypeUnfrozen() == Stage);
	}
}

void FShaderPipeline::Finalize(const FShaderMapResourceCode* Code)
{
	for (uint32 i = 0u; i < SF_NumGraphicsFrequencies; ++i)
	{
		if (Shaders[i])
		{
			Shaders[i]->Finalize(Code);
		}
	}
}


#if WITH_EDITOR
void FShaderPipeline::SaveShaderStableKeys(const FShaderMapPointerTable& InPtrTable, EShaderPlatform TargetShaderPlatform, const struct FStableShaderKeyAndValue& InSaveKeyVal) const
{
	// the higher level code can pass SP_NumPlatforms, in which case play it safe and use a platform that we know can remove inteprolators
	const EShaderPlatform ShaderPlatformThatSupportsRemovingInterpolators = SP_PCD3D_SM5;
	checkf(RHISupportsShaderPipelines(ShaderPlatformThatSupportsRemovingInterpolators), TEXT("We assumed that shader platform %d supports shaderpipelines while it doesn't"), static_cast<int32>(ShaderPlatformThatSupportsRemovingInterpolators));

	FShaderPipelineType** FoundPipelineType = FShaderPipelineType::GetNameToTypeMap().Find(TypeName);
	check(FoundPipelineType);
	FShaderPipelineType* PipelineType = *FoundPipelineType;

	bool bCanHaveUniqueShaders = (TargetShaderPlatform != SP_NumPlatforms) ? PipelineType->ShouldOptimizeUnusedOutputs(TargetShaderPlatform) : PipelineType->ShouldOptimizeUnusedOutputs(ShaderPlatformThatSupportsRemovingInterpolators);
	if (bCanHaveUniqueShaders)
	{
		FStableShaderKeyAndValue SaveKeyVal(InSaveKeyVal);
		SaveKeyVal.SetPipelineHash(this); // could use PipelineType->GetSourceHash(), but each pipeline instance even of the same type can have unique shaders

		for (uint32 Frequency = 0u; Frequency < SF_NumGraphicsFrequencies; ++Frequency)
		{
			FShader* Shader = Shaders[Frequency];
			if (Shader)
			{
				Shader->SaveShaderStableKeys(InPtrTable, TargetShaderPlatform, PermutationIds[Frequency], SaveKeyVal);
			}
		}
	}
}
#endif // WITH_EDITOR

void DumpShaderStats(EShaderPlatform Platform, EShaderFrequency Frequency)
{
#if ALLOW_DEBUG_FILES
	FDiagnosticTableViewer ShaderTypeViewer(*FDiagnosticTableViewer::GetUniqueTemporaryFilePath(TEXT("ShaderStats")));

	// Iterate over all shader types and log stats.
	int32 TotalShaderCount		= 0;
	int32 TotalTypeCount		= 0;
	int32 TotalInstructionCount	= 0;
	int32 TotalSize				= 0;
	int32 TotalPipelineCount	= 0;
	float TotalSizePerType		= 0;

	// Write a row of headings for the table's columns.
	ShaderTypeViewer.AddColumn(TEXT("Type"));
	ShaderTypeViewer.AddColumn(TEXT("Instances"));
	ShaderTypeViewer.AddColumn(TEXT("Average instructions"));
	ShaderTypeViewer.AddColumn(TEXT("Size"));
	ShaderTypeViewer.AddColumn(TEXT("AvgSizePerInstance"));
	ShaderTypeViewer.AddColumn(TEXT("Pipelines"));
	ShaderTypeViewer.AddColumn(TEXT("Shared Pipelines"));
	ShaderTypeViewer.CycleRow();

	for( TLinkedList<FShaderType*>::TIterator It(FShaderType::GetTypeList()); It; It.Next() )
	{
		const FShaderType* Type = *It;
		if (Type->GetNumShaders())
		{
			// Calculate the average instruction count and total size of instances of this shader type.
			float AverageNumInstructions	= 0.0f;
			int32 NumInitializedInstructions	= 0;
			int32 Size						= 0;
			int32 NumShaders					= 0;
			int32 NumPipelines = 0;
			int32 NumSharedPipelines = 0;
#if 0
			for (TMap<FShaderId,FShader*>::TConstIterator ShaderIt(Type->ShaderIdMap);ShaderIt;++ShaderIt)
			{
				const FShader* Shader = ShaderIt.Value();
				// Skip shaders that don't match frequency.
				if( Shader->GetTarget().Frequency != Frequency && Frequency != SF_NumFrequencies )
				{
					continue;
				}
				// Skip shaders that don't match platform.
				if( Shader->GetTarget().Platform != Platform && Platform != SP_NumPlatforms )
				{
					continue;
				}

				NumInitializedInstructions += Shader->GetNumInstructions();
				Size += Shader->GetCode().Num();
				NumShaders++;
			}
			AverageNumInstructions = (float)NumInitializedInstructions / (float)Type->GetNumShaders();
#endif
			
			for (TLinkedList<FShaderPipelineType*>::TConstIterator PipelineIt(FShaderPipelineType::GetTypeList()); PipelineIt; PipelineIt.Next())
			{
				const FShaderPipelineType* PipelineType = *PipelineIt;
				bool bFound = false;
				if (Frequency == SF_NumFrequencies)
				{
					if (PipelineType->GetShader(Type->GetFrequency()) == Type)
					{
						++NumPipelines;
						bFound = true;
					}
				}
				else
				{
					if (PipelineType->GetShader(Frequency) == Type)
					{
						++NumPipelines;
						bFound = true;
					}
				}

				if (!PipelineType->ShouldOptimizeUnusedOutputs(Platform) && bFound)
				{
					++NumSharedPipelines;
				}
			}

			// Only add rows if there is a matching shader.
			if( NumShaders )
			{
				// Write a row for the shader type.
				ShaderTypeViewer.AddColumn(Type->GetName());
				ShaderTypeViewer.AddColumn(TEXT("%u"),NumShaders);
				ShaderTypeViewer.AddColumn(TEXT("%.1f"),AverageNumInstructions);
				ShaderTypeViewer.AddColumn(TEXT("%u"),Size);
				ShaderTypeViewer.AddColumn(TEXT("%.1f"),Size / (float)NumShaders);
				ShaderTypeViewer.AddColumn(TEXT("%d"), NumPipelines);
				ShaderTypeViewer.AddColumn(TEXT("%d"), NumSharedPipelines);
				ShaderTypeViewer.CycleRow();

				TotalShaderCount += NumShaders;
				TotalPipelineCount += NumPipelines;
				TotalInstructionCount += NumInitializedInstructions;
				TotalTypeCount++;
				TotalSize += Size;
				TotalSizePerType += Size / (float)NumShaders;
			}
		}
	}

	// go through non shared pipelines

	// Write a total row.
	ShaderTypeViewer.AddColumn(TEXT("Total"));
	ShaderTypeViewer.AddColumn(TEXT("%u"),TotalShaderCount);
	ShaderTypeViewer.AddColumn(TEXT("%u"),TotalInstructionCount);
	ShaderTypeViewer.AddColumn(TEXT("%u"),TotalSize);
	ShaderTypeViewer.AddColumn(TEXT("0"));
	ShaderTypeViewer.AddColumn(TEXT("%u"), TotalPipelineCount);
	ShaderTypeViewer.AddColumn(TEXT("-"));
	ShaderTypeViewer.CycleRow();

	// Write an average row.
	ShaderTypeViewer.AddColumn(TEXT("Average"));
	ShaderTypeViewer.AddColumn(TEXT("%.1f"),TotalTypeCount   ? (TotalShaderCount / (float)TotalTypeCount)        : 0.0f);
	ShaderTypeViewer.AddColumn(TEXT("%.1f"),TotalShaderCount ? ((float)TotalInstructionCount / TotalShaderCount) : 0.0f);
	ShaderTypeViewer.AddColumn(TEXT("%.1f"),TotalShaderCount ? (TotalSize / (float)TotalShaderCount)             : 0.0f);
	ShaderTypeViewer.AddColumn(TEXT("%.1f"),TotalTypeCount   ? (TotalSizePerType / TotalTypeCount)               : 0.0f);
	ShaderTypeViewer.AddColumn(TEXT("-"));
	ShaderTypeViewer.AddColumn(TEXT("-"));
	ShaderTypeViewer.CycleRow();
#endif
}

void DumpShaderPipelineStats(EShaderPlatform Platform)
{
#if ALLOW_DEBUG_FILES
	FDiagnosticTableViewer ShaderTypeViewer(*FDiagnosticTableViewer::GetUniqueTemporaryFilePath(TEXT("ShaderPipelineStats")));

	int32 TotalNumPipelines = 0;
	int32 TotalSize = 0;
	float TotalSizePerType = 0;

	// Write a row of headings for the table's columns.
	ShaderTypeViewer.AddColumn(TEXT("Type"));
	ShaderTypeViewer.AddColumn(TEXT("Shared/Unique"));

	// Exclude compute
	for (int32 Index = 0; Index < SF_NumFrequencies - 1; ++Index)
	{
		ShaderTypeViewer.AddColumn(GetShaderFrequencyString((EShaderFrequency)Index));
	}
	ShaderTypeViewer.CycleRow();

	int32 TotalTypeCount = 0;
	for (TLinkedList<FShaderPipelineType*>::TIterator It(FShaderPipelineType::GetTypeList()); It; It.Next())
	{
		const FShaderPipelineType* Type = *It;

		// Write a row for the shader type.
		ShaderTypeViewer.AddColumn(Type->GetName());
		ShaderTypeViewer.AddColumn(Type->ShouldOptimizeUnusedOutputs(Platform) ? TEXT("U") : TEXT("S"));

		for (int32 Index = 0; Index < SF_NumFrequencies - 1; ++Index)
		{
			const FShaderType* ShaderType = Type->GetShader((EShaderFrequency)Index);
			ShaderTypeViewer.AddColumn(ShaderType ? ShaderType->GetName() : TEXT(""));
		}

		ShaderTypeViewer.CycleRow();
	}
#endif
}

FShaderType* FindShaderTypeByName(const FHashedName& ShaderTypeName)
{
	FShaderType** FoundShader = FShaderType::GetNameToTypeMap().Find(ShaderTypeName);
	if (FoundShader)
	{
		return *FoundShader;
	}

	return nullptr;
}

void DispatchComputeShader(
	FRHIComputeCommandList& RHICmdList,
	FShader* Shader,
	uint32 ThreadGroupCountX,
	uint32 ThreadGroupCountY,
	uint32 ThreadGroupCountZ)
{
	RHICmdList.DispatchComputeShader(ThreadGroupCountX, ThreadGroupCountY, ThreadGroupCountZ);
}

void DispatchIndirectComputeShader(
	FRHIComputeCommandList& RHICmdList,
	FShader* Shader,
	FRHIBuffer* ArgumentBuffer,
	uint32 ArgumentOffset)
{
	RHICmdList.DispatchIndirectComputeShader(ArgumentBuffer, ArgumentOffset);
}

bool IsDxcEnabledForPlatform(EShaderPlatform Platform, bool bHlslVersion2021)
{
	// Check the generic console variable first (if DXC is supported)
	if (FDataDrivenShaderPlatformInfo::GetSupportsDxc(Platform))
	{
		static FShaderPlatformCachedIniValue<bool> ShaderForceDXC(TEXT("r.Shaders.ForceDXC"));
		if (bHlslVersion2021 || (ShaderForceDXC.Get(Platform) != 0))
		{
			return true;
		}
	}
	// Check backend specific console variables next
	if (IsD3DPlatform(Platform) && IsPCPlatform(Platform))
	{
		// D3D backend supports a precompile step for HLSL2021 which is separate from ForceDXC option
		static const IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.D3D.ForceDXC"));
		return ((CVar && CVar->GetInt() != 0));
	}
	// Hlslcc has been removed for Metal, Vulkan, and OpenGL backends. There is only DXC now.
	if (IsMetalPlatform(Platform) || IsVulkanPlatform(Platform) || IsOpenGLPlatform(Platform))
	{
		return true;
	}
	return false;
}

bool IsUsingEmulatedUniformBuffers(EShaderPlatform Platform)
{
	if (IsOpenGLPlatform(Platform))
	{
		// DXC only supports emulated uniform buffers on GLES
		return true;
	}
	return false;
}

void ShaderMapAppendKeyString(EShaderPlatform Platform, FString& KeyString)
{
	FShaderKeyGenerator KeyGen(KeyString);
	ShaderMapAppendKey(Platform, KeyGen);
}

void ShaderMapAppendKey(EShaderPlatform Platform, FShaderKeyGenerator& KeyGen)
{
	const FName ShaderFormatName = LegacyShaderPlatformToShaderFormat(Platform);

	for (const FAutoConsoleObject* ConsoleObject : FAutoConsoleObject::AccessGeneralShaderChangeCvars())
	{
		FString ConsoleObjectName = IConsoleManager::Get().FindConsoleObjectName(ConsoleObject->AsVariable());
		KeyGen.AppendSeparator();
		KeyGen.Append(ConsoleObjectName);
		KeyGen.AppendSeparator();
		KeyGen.Append(ConsoleObject->AsVariable()->GetString());
	}
	if (IsMobilePlatform(Platform))
	{
		for (const FAutoConsoleObject* ConsoleObject : FAutoConsoleObject::AccessMobileShaderChangeCvars())
		{
			FString ConsoleObjectName = IConsoleManager::Get().FindConsoleObjectName(ConsoleObject->AsVariable());
			KeyGen.AppendSeparator();
			KeyGen.Append(ConsoleObjectName);
			KeyGen.AppendSeparator();
			KeyGen.Append(ConsoleObject->AsVariable()->GetString());
		}
	}
	else if (IsConsolePlatform(Platform))
	{
		for (const FAutoConsoleObject* ConsoleObject : FAutoConsoleObject::AccessDesktopShaderChangeCvars())
		{
			FString ConsoleObjectName = IConsoleManager::Get().FindConsoleObjectName(ConsoleObject->AsVariable());
			KeyGen.AppendSeparator();
			KeyGen.Append(ConsoleObjectName);
			KeyGen.AppendSeparator();
			KeyGen.Append(ConsoleObject->AsVariable()->GetString());
		}
	}

	// Globals that should cause all shaders to recompile when changed must be appended to the key here
	// Key should be kept as short as possible while being somewhat human readable for debugging

	{
		static const auto CVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("Compat.UseDXT5NormalMaps"));
		KeyGen.AppendSeparator();
		KeyGen.Append((CVar && CVar->GetValueOnAnyThread() != 0) ? TEXT("DXTN") : TEXT("BC5N"));
	}

	{
		static const auto CVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.ClearCoatNormal"));
		KeyGen.AppendSeparator();
		KeyGen.Append((CVar && CVar->GetValueOnAnyThread() != 0 && (!IsMobilePlatform(Platform) || !IsMobileDeferredShadingEnabled(Platform) || MobileUsesExtendedGBuffer(Platform))) ? TEXT("CCBN") : TEXT("NoCCBN"));
	}

	{
		static const auto CVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.IrisNormal"));
		KeyGen.AppendSeparator();
		KeyGen.Append((CVar && CVar->GetValueOnAnyThread() != 0) ? TEXT("Iris") : TEXT("NoIris"));
	}

	{
		static const auto CVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.CompileShadersForDevelopment"));
		KeyGen.AppendSeparator();
		KeyGen.Append((CVar && CVar->GetValueOnAnyThread() != 0) ? TEXT("DEV") : TEXT("NoDEV"));
	}

	{
		static const auto CVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.Material.UseShaderCompilationParameters"));
		KeyGen.AppendSeparator();
		KeyGen.Append((CVar && CVar->GetValueOnAnyThread() != 0) ? TEXT("CSP") : TEXT("NoCSP"));
	}

	{
		const bool bValue = IsStaticLightingAllowed();
		KeyGen.AppendSeparator();
		KeyGen.Append(bValue ? TEXT("SL") : TEXT("NoSL"));
	}


	{
		static const auto CVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.MaterialEditor.LWCTruncateMode"));
		const int32 LWCTruncateValue = CVar ? CVar->GetValueOnAnyThread() : 0;
		if (LWCTruncateValue == 1)
		{
			KeyGen.AppendSeparator();
			KeyGen.Append(TEXT("LWC1"));
		}
		else if (LWCTruncateValue == 2)
		{
			KeyGen.AppendSeparator();
			KeyGen.Append(TEXT("LWC2"));
		}
	}

	{
		static const auto CVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.VelocityOutputPass"));
		const int32 VelocityOutputPassValue = CVar ? CVar->GetValueOnAnyThread() : 0;
		if (VelocityOutputPassValue == 1)
		{
			KeyGen.AppendSeparator();
			KeyGen.Append(TEXT("GV"));
		}
		else
		{
			KeyGen.AppendSeparator();
			KeyGen.Append(TEXT("VOP"));
			KeyGen.AppendSeparator();
			KeyGen.Append(VelocityOutputPassValue);
		}
	}

	{
		const UE::StereoRenderUtils::FStereoShaderAspects Aspects(Platform);

		if (Aspects.IsInstancedStereoEnabled())
		{
			KeyGen.AppendSeparator();
			KeyGen.Append(TEXT("VRIS"));
		}

		if (Aspects.IsMobileMultiViewEnabled())
		{
			KeyGen.AppendSeparator();
			KeyGen.Append(TEXT("MMVIEW"));
		}
	}

	{
		if (IsUsingSelectiveBasePassOutputs(Platform))
		{
			KeyGen.AppendSeparator();
			KeyGen.Append(TEXT("SO"));
		}
	}

	{
		// PreExposure is always used
		KeyGen.AppendSeparator();
		KeyGen.Append(TEXT("PreExp"));
	}

	{
		KeyGen.AppendSeparator();
		KeyGen.Append(IsUsingDBuffers(Platform) ? TEXT("DBuf") : TEXT("NoDBuf"));
	}

	{
		static IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.AllowGlobalClipPlane"));
		if (CVar && CVar->GetInt() != 0)
		{
			KeyGen.AppendSeparator();
			KeyGen.Append(TEXT("ClipP"));
		}
	}

	{
		// Extra data (names, etc)
		if (ShouldEnableExtraShaderData(ShaderFormatName))
		{
			KeyGen.AppendSeparator();
			KeyGen.Append(TEXT("ExtraData"));
		}
		// Symbols and/or SymbolsInfo and version if symbols serialization changes
		if (ShouldGenerateShaderSymbols(ShaderFormatName))
		{
			KeyGen.AppendSeparator();
			KeyGen.Append(TEXT("Symbols"));
		}
		if (ShouldGenerateShaderSymbolsInfo(ShaderFormatName))
		{
			KeyGen.AppendSeparator();
			KeyGen.Append(TEXT("SymbolsInfo"));
		}
		// Are symbols based on source or results
		if (ShouldAllowUniqueShaderSymbols(ShaderFormatName))
		{
			KeyGen.AppendSeparator();
			KeyGen.Append(TEXT("FullDbg"));
		}
	}

	if (!ShouldOptimizeShaders(ShaderFormatName))
	{
		KeyGen.AppendSeparator();
		KeyGen.Append(TEXT("NoOpt"));
	}
	
	{
		// Always default to fast math unless specified
		static const auto CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Shaders.FastMath"));
		if ((CVar && CVar->GetInt() == 0))
		{
			KeyGen.AppendSeparator();
			KeyGen.Append(TEXT("NoFastMath"));
		}
	}

	{
		static FShaderPlatformCachedIniValue<int32> CVarWarningsAsErrorsPerPlatform(TEXT("r.Shaders.WarningsAsErrors"));
		if (const int32 Level = CVarWarningsAsErrorsPerPlatform.Get(Platform); Level != 0)
		{
			KeyGen.AppendSeparator();
			KeyGen.Append(TEXT("WX"));
			KeyGen.Append(Level);
		}
	}
	
	{
		static const auto CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Shaders.CheckLevel"));
		// Note: Since 1 is the default, we don't modify the hash for this case, so as to not force a rebuild, and to keep the hash shorter.
		if (CVar && (CVar->GetInt() == 0 || CVar->GetInt() == 2))
		{
			KeyGen.AppendSeparator();
			KeyGen.Append(TEXT("C"));
			KeyGen.Append(CVar->GetInt());
		}
	}
	
	{
		static const auto CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Shaders.FlowControlMode"));
		if (CVar)
		{
			switch(CVar->GetInt())
			{
				case 2:
					KeyGen.AppendSeparator();
					KeyGen.Append(TEXT("AvoidFlow"));
					break;
				case 1:
					KeyGen.AppendSeparator();
					KeyGen.Append(TEXT("PreferFlow"));
					break;
				case 0:
				default:
					break;
			}
		}
	}

	if (!AllowPixelDepthOffset(Platform))
	{
		KeyGen.AppendSeparator();
		KeyGen.Append(TEXT("NoPDO"));
	}

	if (!AllowPerPixelShadingModels(Platform))
	{
		KeyGen.AppendSeparator();
		KeyGen.Append(TEXT("NoPPSM"));
	}
	
	if (UseRemoveUnsedInterpolators(Platform) && !IsOpenGLPlatform(Platform))
	{
		KeyGen.AppendSeparator();
		KeyGen.Append(TEXT("UnInt"));
	}

	if (ForwardShadingForcesSkyLightCubemapBlending(Platform))
	{
		KeyGen.AppendSeparator();
		KeyGen.Append(TEXT("FwdSkyBlnd"));
	}

	if (SupportsSkyLightCaptureAffectsHeightFogInScattering(Platform))
	{
		KeyGen.AppendSeparator();
		KeyGen.Append(TEXT("SkyCapFog"));
	}

	if (IsMobilePlatform(Platform))
	{
		{
			KeyGen.AppendSeparator();
			KeyGen.Append(IsMobileHDR() ? TEXT("HDR") : TEXT("LDR"));
		}
		
		{
			static IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Mobile.DisableVertexFog"));
			if (CVar && CVar->GetInt() != 0)
			{
				KeyGen.AppendSeparator();
				KeyGen.Append(TEXT("NoVFog"));
			}
		}

		{
			static IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Mobile.EnableClusteredDeferredShadingPermutations"));
			if (CVar)
			{
				KeyGen.AppendSeparator();
				KeyGen.Append(TEXT("Clust"));
				KeyGen.Append(CVar->GetInt());
			}
		}
		
		{
			static IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Mobile.FloatPrecisionMode"));
			if (CVar && CVar->GetInt() > 0)
			{
				KeyGen.AppendSeparator();
				KeyGen.Append(TEXT("highp"));
				KeyGen.Append(CVar->GetInt());
			}
		}

		{
			if (FReadOnlyCVARCache::MobileAllowDitheredLODTransition(Platform))
			{
				KeyGen.AppendSeparator();
				KeyGen.Append(TEXT("DLODT"));
			}
		}
		
		if (IsUsingEmulatedUniformBuffers(Platform))
		{
			KeyGen.AppendSeparator();
			KeyGen.Append(TEXT("NoUB"));
		}

		{
			const bool bMobileMovableSpotlightShadowsEnabled = IsMobileMovableSpotlightShadowsEnabled(Platform);
			if (bMobileMovableSpotlightShadowsEnabled)
			{
				KeyGen.Append(TEXT("S"));
			}
		}
		
		{
			static IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Mobile.UseHWsRGBEncoding"));
			if (CVar && CVar->GetInt() != 0)
			{
				KeyGen.AppendSeparator();
				KeyGen.Append(TEXT("HWsRGB"));
			}
		}
		
		{
			// make it per shader platform ?
			static IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Mobile.SupportGPUScene"));
			if (CVar && CVar->GetInt() != 0)
			{
				KeyGen.AppendSeparator();
				KeyGen.Append(TEXT("MobGPUSc"));
			}
		}

		{
			bool bIsMobileDeferredShading = IsMobileDeferredShadingEnabled(Platform);

			if (bIsMobileDeferredShading)
			{
				KeyGen.AppendSeparator();
				KeyGen.Append(MobileUsesExtendedGBuffer(Platform) ? TEXT("MobDShEx") : TEXT("MobDSh"));
			}
			else
			{
				static IConsoleVariable* MobileForwardEnableClusteredReflectionsCVAR = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Mobile.Forward.EnableClusteredReflections"));
				if (MobileForwardEnableClusteredReflectionsCVAR && MobileForwardEnableClusteredReflectionsCVAR->GetInt() != 0)
				{
					KeyGen.AppendSeparator();
					KeyGen.Append(TEXT("MobFCR"));
				}

				if (AreMobileScreenSpaceReflectionsEnabled(Platform))
				{
					KeyGen.AppendSeparator();
					KeyGen.Append(TEXT("MobSSR"));
				}
			}
		}

		{
			static IConsoleVariable* MobileGTAOPreIntegratedTextureTypeCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Mobile.GTAOPreIntegratedTextureType"));
			int32 GTAOPreIntegratedTextureType = MobileGTAOPreIntegratedTextureTypeCVar ? MobileGTAOPreIntegratedTextureTypeCVar->GetInt() : 0;
			if (IsMobileHDR())
			{
				KeyGen.AppendSeparator();
				KeyGen.Append(GTAOPreIntegratedTextureType);
			}
		}

		if (IsMobileDistanceFieldEnabled(Platform))
		{
			KeyGen.AppendSeparator();
			KeyGen.Append(TEXT("MobSDF"));
		}

		{
			static FShaderPlatformCachedIniValue<bool> EnableCullBeforeFetchIniValue(TEXT("r.CullBeforeFetch"));
			if (EnableCullBeforeFetchIniValue.Get(Platform) == 1)
			{
				KeyGen.AppendSeparator();
				KeyGen.Append(TEXT("CBF"));
			}
			static FShaderPlatformCachedIniValue<bool> EnableWarpCullingIniValue(TEXT("r.WarpCulling"));
			if (EnableWarpCullingIniValue.Get(Platform) == 1)
			{
				KeyGen.AppendSeparator();
				KeyGen.Append(TEXT("WC"));
			}
		}

		if (MobileUsesFullDepthPrepass(Platform))
		{
			KeyGen.AppendSeparator();
			KeyGen.Append(TEXT("MobFDP"));
		}

		if (MobileAllowFramebufferFetch(Platform) == false)
		{
			KeyGen.AppendSeparator();
			KeyGen.Append(TEXT("NoFBF"));
		}

		if (MobileRequiresSceneDepthAux(Platform))
		{
			KeyGen.AppendSeparator();
			KeyGen.Append(TEXT("DAux"));
		}
	}
	else
	{
		if (IsUsingEmulatedUniformBuffers(Platform))
		{
			KeyGen.AppendSeparator();
			KeyGen.Append(TEXT("NoUB"));
		}
	}

	if (RenderRectLightsAsSpotLights(GetMaxSupportedFeatureLevel(Platform)))
	{
		KeyGen.AppendSeparator();
		KeyGen.Append(TEXT("R2S"));
	}
	
	uint32 PlatformShadingModelsMask = GetPlatformShadingModelsMask(Platform);
	if (PlatformShadingModelsMask != 0xFFFFFFFF)
	{
		KeyGen.Append(TEXT("SMM"));
		KeyGen.AppendSeparator();
		KeyGen.AppendHex(PlatformShadingModelsMask);
	}

	const IShaderFormat* ShaderFormat = GetTargetPlatformManagerRef().FindShaderFormat(ShaderFormatName);
	if (ShaderFormat)
	{
		FString ShaderFormatExtraData;
		ShaderFormat->AppendToKeyString(ShaderFormatExtraData);
		KeyGen.Append(ShaderFormatExtraData);
	}

	ITargetPlatform* TargetPlatform = GetTargetPlatformManagerRef().FindTargetPlatformWithSupport(TEXT("ShaderFormat"), ShaderFormatName);

	uint32 SupportedHardwareMask = TargetPlatform ? TargetPlatform->GetSupportedHardwareMask() : 0;

	if (SupportedHardwareMask != 0)
	{
		KeyGen.AppendSeparator();
		KeyGen.Append(TEXT("SHM"));
		KeyGen.AppendSeparator();
		KeyGen.AppendHex(SupportedHardwareMask);
	}

	// Encode the Metal standard into the shader compile options so that they recompile if the settings change.
	if (IsMetalPlatform(Platform))
	{
		{
			static const auto CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Shaders.ZeroInitialise"));
			if (CVar && CVar->GetInt() != 0)
			{
				KeyGen.AppendSeparator();
				KeyGen.Append(TEXT("ZeroInit"));
			}
		}
		{
			static const auto CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Shaders.BoundsChecking"));
			if (CVar && CVar->GetInt() != 0)
			{
				KeyGen.AppendSeparator();
				KeyGen.Append(TEXT("BoundsChecking"));
			}
		}
		{
			if (RHISupportsManualVertexFetch(Platform))
			{
				KeyGen.AppendSeparator();
				KeyGen.Append(TEXT("MVF"));
				KeyGen.AppendSeparator();
			}
		}
		
		uint32 ShaderVersion = RHIGetMetalShaderLanguageVersion(Platform);
		KeyGen.AppendSeparator();
		KeyGen.Append(TEXT("MTLSTD"));
		KeyGen.Append(ShaderVersion);
		KeyGen.AppendSeparator();

		bool bAllowFastIntrinsics = false;
		bool bEnableMathOptimisations = true;
		bool bForceFloats = false;
		bool bMetalOptimizeBySize = false;
		bool bAllowsRuntimeShaderCompiling = false;
		bool bStripIndividualMetalLibs = false;
		
		const bool bIsMac = IsPCPlatform(Platform);
		const TCHAR* RuntimeSettingsClassName = bIsMac ? FMacPlatformProperties<false, false, false>::GetRuntimeSettingsClassName() : FIOSPlatformProperties::GetRuntimeSettingsClassName();

		GConfig->GetBool(RuntimeSettingsClassName, TEXT("UseFastIntrinsics"), bAllowFastIntrinsics, GEngineIni);
		GConfig->GetBool(RuntimeSettingsClassName, TEXT("EnableMathOptimisations"), bEnableMathOptimisations, GEngineIni);
		GConfig->GetBool(RuntimeSettingsClassName, TEXT("ForceFloats"), bForceFloats, GEngineIni);
		GConfig->GetBool(RuntimeSettingsClassName, TEXT("MetalOptimizeBySize"), bMetalOptimizeBySize, GEngineIni);
		GConfig->GetBool(RuntimeSettingsClassName, TEXT("AllowsRuntimeShaderCompiling"), bAllowsRuntimeShaderCompiling, GEngineIni);
		GConfig->GetBool(RuntimeSettingsClassName, TEXT("StripIndividualMetalLibs"), bStripIndividualMetalLibs, GEngineIni);
		
		if (bIsMac)
		{
			// No half precision support on MacOS at the moment
			bForceFloats = true;
		}
		
		if (bAllowFastIntrinsics)
		{
			KeyGen.AppendSeparator();
			KeyGen.Append(TEXT("MTLSL_FastIntrin"));
		}
		
		// Same as console-variable above, but that's global and this is per-platform, per-project
		if (!bEnableMathOptimisations)
		{
			KeyGen.AppendSeparator();
			KeyGen.Append(TEXT("NoFastMath"));
		}
		
		if (bForceFloats)
		{
			KeyGen.AppendSeparator();
			KeyGen.Append(TEXT("FP32"));
		}
		
		if(bMetalOptimizeBySize)
        {
			KeyGen.AppendSeparator();
			KeyGen.Append(TEXT("Os"));
        }
		
		if(bAllowsRuntimeShaderCompiling)
		{
			KeyGen.AppendSeparator();
			KeyGen.Append(TEXT("AllowsRuntimeShaderCompiling"));
		}
		
		if(bStripIndividualMetalLibs)
		{
			KeyGen.AppendSeparator();
			KeyGen.Append(TEXT("StripIndividualMetalLibs"));
		}
        
		KeyGen.AppendSeparator();
		KeyGen.Append(TEXT("IAB"));
		
		// Shaders built for archiving - for Metal that requires compiling the code in a different way so that we can strip it later
		bool bArchive = false;
		GConfig->GetBool(TEXT("/Script/UnrealEd.ProjectPackagingSettings"), TEXT("bSharedMaterialNativeLibraries"), bArchive, GGameIni);
		if (bArchive)
		{
			KeyGen.AppendSeparator();
			KeyGen.Append(TEXT("ARCHIVE"));
		}
	}

	if (Platform == SP_VULKAN_ES3_1_ANDROID || Platform == SP_VULKAN_SM5_ANDROID)
	{
		bool bStripReflect = true;
		GConfig->GetBool(TEXT("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings"), TEXT("bStripShaderReflection"), bStripReflect, GEngineIni);
		if (!bStripReflect)
		{
			KeyGen.AppendSeparator();
			KeyGen.Append(TEXT("NoStripReflect"));
		}
	}

	// Is DXC shader compiler enabled for this platform?
	KeyGen.AppendSeparator();
	KeyGen.Append(IsDxcEnabledForPlatform(Platform) ? TEXT("DXC1") : TEXT("DXC0"));
	
	if (IsStencilForLODDitherEnabled(Platform))
	{
		KeyGen.AppendSeparator();
		KeyGen.Append(TEXT("SD"));
	}

	{
		bool bForwardShading = false;
		if (TargetPlatform)
		{
			// if there is a specific target platform that matches our shader platform, use that to drive forward shading
			bForwardShading = TargetPlatform->UsesForwardShading();
		}
		else
		{
			// shader platform doesn't match a specific target platform, use cvar setting for forward shading
			static IConsoleVariable* CVarForwardShadingLocal = IConsoleManager::Get().FindConsoleVariable(TEXT("r.ForwardShading"));
			bForwardShading = CVarForwardShadingLocal ? (CVarForwardShadingLocal->GetInt() != 0) : false;
		}

		if (bForwardShading)
		{
			KeyGen.AppendSeparator();
			KeyGen.Append(TEXT("FS"));
		}
	}

	{
		static const auto CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Deferred.SupportPrimitiveAlphaHoldout"));
		const bool bDeferredSupportPrimitiveAlphaHoldout = CVar ? CVar->GetBool() : false;

		if (bDeferredSupportPrimitiveAlphaHoldout)
		{
			KeyGen.AppendSeparator();
			KeyGen.Append(TEXT("PAH"));
		}
	}

	if (TargetPlatform && 
		TargetPlatform->SupportsFeature(ETargetPlatformFeatures::NormalmapLAEncodingMode))
	{
		KeyGen.AppendSeparator();
		KeyGen.Append(TEXT("NLA"));
	}

	{
		static const auto CVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.VertexFoggingForOpaque"));
		bool bVertexFoggingForOpaque = CVar && CVar->GetValueOnAnyThread() > 0;
		if (TargetPlatform)
		{
			const int32 PlatformHeightFogMode = TargetPlatform->GetHeightFogModeForOpaque();
			if (PlatformHeightFogMode == 1)
			{
				bVertexFoggingForOpaque = false;
			}
			else if (PlatformHeightFogMode == 2)
			{
				bVertexFoggingForOpaque = true;
			}
		}
		if (bVertexFoggingForOpaque)
		{
			KeyGen.AppendSeparator();
			KeyGen.Append(TEXT("VFO"));
		}
	}

	bool bSupportLocalFogVolumes = false;
	{
		static IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.SupportLocalFogVolumes"));
		bSupportLocalFogVolumes = CVar && CVar->GetInt() > 0;
		if (bSupportLocalFogVolumes)
		{
			KeyGen.AppendSeparator();
			KeyGen.Append(TEXT("LFV"));
		}
	}

	{
		if (DoesProjectSupportLumenRayTracedTranslucentRefraction())
		{
			KeyGen.AppendSeparator();
			KeyGen.Append(TEXT("LTRRT"));
		}
	}

	{
		static IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.LocalFogVolume.ApplyOnTranslucent"));
		const bool bLocalFogVolumesApplyOnTranclucent = CVar && CVar->GetInt() > 0;
		if (bSupportLocalFogVolumes && bLocalFogVolumesApplyOnTranclucent)
		{
			KeyGen.AppendSeparator();
			KeyGen.Append(TEXT("LFVTRA"));
		}
	}

	{
		static const auto CVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.SupportSkyAtmosphere"));
		if (CVar && CVar->GetValueOnAnyThread() > 0)
		{
			KeyGen.AppendSeparator();
			KeyGen.Append(TEXT("SKYATM"));

			static const auto CVarHeightFog = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.SupportsSkyAtmosphereAffectsHeightFog"));
			if (CVarHeightFog && CVarHeightFog->GetValueOnAnyThread() > 0)
			{
				KeyGen.AppendSeparator();
				KeyGen.Append(TEXT("SKYHF"));
			}
		}
	}

	{
		static const auto CVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.VolumetricCloud.Support"));
		if (CVar && CVar->GetValueOnAnyThread() > 0)
		{
			KeyGen.AppendSeparator();
			KeyGen.Append(TEXT("VCLOUD"));
		}
	}

	{
		if (DoesProjectSupportExpFogMatchesVolumetricFog())
		{
			KeyGen.AppendSeparator();
			KeyGen.Append(TEXT("EXPVFOG"));
		}
	}

	{
		static const auto CVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.ClusteredDeferredShading.EnableForProject"));
		if (CVar && CVar->GetValueOnAnyThread() > 0)
		{
			KeyGen.AppendSeparator();
			KeyGen.Append(TEXT("CLUDEFF"));
		}
	}

	const bool bNeedsSeparateMainDirLightTexture = IsWaterSeparateMainDirLightEnabled(Platform);
	if (bNeedsSeparateMainDirLightTexture)
	{
		KeyGen.AppendSeparator();
		KeyGen.Append(TEXT("SLWSMDLT"));
	}

	{
		static const auto CVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.SupportCloudShadowOnForwardLitTranslucent"));
		if (CVar && CVar->GetValueOnAnyThread() > 0)
		{
			KeyGen.AppendSeparator();
			KeyGen.Append(TEXT("CLDTRANS"));
		}
	}

	{
		const bool bTranslucentUsesLightRectLights = GetTranslucentUsesLightRectLights();
		if (bTranslucentUsesLightRectLights)
		{
			KeyGen.AppendSeparator();
			KeyGen.Append(TEXT("RECTTRANS"));
		}
	}

	{
		const bool bTranslucentUsesShadowedLocalLights = GetTranslucentUsesShadowedLocalLights();
		if (bTranslucentUsesShadowedLocalLights)
		{
			KeyGen.AppendSeparator();
			KeyGen.Append(TEXT("SHALOLIT"));
		}
	}

	{
		const bool bTranslucentUsesLightIESProfiles = GetTranslucentUsesLightIESProfiles();
		if (bTranslucentUsesLightIESProfiles)
		{
			KeyGen.AppendSeparator();
			KeyGen.Append(TEXT("IESTRANS"));
		}
	}

	{
		static const auto CVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.Shadow.Virtual.TranslucentQuality"));
		if (CVar && CVar->GetValueOnAnyThread() > 0)
		{
			KeyGen.AppendSeparator();
			KeyGen.Append(TEXT("VSMTRANSQUALITY"));
		}
	}

	if (GetHairStrandsUsesTriangleStrips())
	{
		KeyGen.AppendSeparator();
		KeyGen.Append(TEXT("STRDSTRIP"));
	}

	if (Substrate::IsSubstrateEnabled())
	{
		{
			KeyGen.AppendSeparator();
			KeyGen.Append(TEXT("SUBSTRATE"));
		}

		{
			KeyGen.AppendSeparator();
			KeyGen.Append(TEXT("SUBGBFMT"));
			KeyGen.Append(Substrate::IsSubstrateBlendableGBufferEnabled(Platform));
		}

		{
			KeyGen.AppendSeparator();
			KeyGen.Append(TEXT("BUDGET"));
			KeyGen.Append(Substrate::GetBytePerPixel(Platform));
		}

		{
			KeyGen.AppendSeparator();
			KeyGen.Append(TEXT("CLOSURE"));
			// We append all closure count to the key from low to epic material quality levels for the current platfom.
			KeyGen.Append(Substrate::GetClosurePerPixel(Platform, 0));
			KeyGen.Append(Substrate::GetClosurePerPixel(Platform, 1));
			KeyGen.Append(Substrate::GetClosurePerPixel(Platform, 2));
			KeyGen.Append(Substrate::GetClosurePerPixel(Platform, 3));
		}

		if (Substrate::IsDBufferPassEnabled(Platform))
		{
			KeyGen.AppendSeparator();
			KeyGen.Append(TEXT("DBUFFERPASS"));
		}

		if (Substrate::IsHiddenMaterialAssetConversionEnabled())
		{
			KeyGen.AppendSeparator();
			KeyGen.Append(TEXT("HIDDENCONV"));
		}

		if (Substrate::IsOpaqueRoughRefractionEnabled(Platform))
		{
			KeyGen.AppendSeparator();
			KeyGen.Append(TEXT("ROUGHDIFF"));
		}

		if (Substrate::GetNormalQuality() > 0)
		{
			KeyGen.AppendSeparator();
			KeyGen.Append(TEXT("STRTNRMQ"));
		}

		if (Substrate::IsAdvancedVisualizationEnabled(Platform))
		{
			KeyGen.AppendSeparator();
			KeyGen.Append(TEXT("ADVDEBUG"));
		}

		{
			KeyGen.AppendSeparator();
			KeyGen.Append(TEXT("STSHQL"));
			KeyGen.Append(Substrate::GetShadingQuality(Platform));
		}

		if (Substrate::IsGlintEnabled(Platform))
		{
			KeyGen.AppendSeparator();
			KeyGen.Append(TEXT("STRTGLT"));
		}

		if (Substrate::IsSpecularProfileEnabled(Platform))
		{
			KeyGen.AppendSeparator();
			KeyGen.Append(TEXT("STRTSP"));
		}

		{
			static IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Substrate.Experimental.ToonReflectionQuantizationEnabled"));
			if (CVar && CVar->GetInt() > 0)
			{
				KeyGen.AppendSeparator();
				KeyGen.Append(TEXT("EXPTOONRQ"));
			}
		}

		{
			static IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Substrate.Experimental.ToonUnifiedDiffuse"));
			if (CVar && CVar->GetInt() > 0)
			{
				KeyGen.AppendSeparator();
				KeyGen.Append(TEXT("EXPTOONUD"));
			}
		}
	}

	{
		static const auto CVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.Material.RoughDiffuse"));
		const bool bSubstrateRough = Substrate::IsSubstrateEnabled() && Substrate::IsRoughDiffuseEnabled(Platform);
		const bool bMaterialtRough = CVar && CVar->GetValueOnAnyThread() > 0;
		if (bSubstrateRough || bMaterialtRough)
		{
			KeyGen.AppendSeparator();
			KeyGen.Append(TEXT("MATRDIFF"));
		}
	}

	{
		int32 LightFunctionAtlasFormat = GetLightFunctionAtlasFormat();
		if (LightFunctionAtlasFormat > 0)
		{
			KeyGen.AppendSeparator();
			KeyGen.Append(TEXT("LFAC"));
			KeyGen.Append(static_cast<uint32>(LightFunctionAtlasFormat));
		}

		bool bSingleLayerWaterUsesLightFunctionAtlas = GetSingleLayerWaterUsesLightFunctionAtlas();
		if (bSingleLayerWaterUsesLightFunctionAtlas)
		{
			KeyGen.AppendSeparator();
			KeyGen.Append(TEXT("SLWLFA"));
		}

		bool bTranslucentUsesLightFunctionAtlas = GetTranslucentUsesLightFunctionAtlas();
		if (bTranslucentUsesLightFunctionAtlas)
		{
			KeyGen.AppendSeparator();
			KeyGen.Append(TEXT("FWDLFA"));
		}

	}

	{
		static const auto CVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.Material.EnergyConservation"));
		if (CVar && CVar->GetValueOnAnyThread() > 0)
		{
			KeyGen.AppendSeparator();
			KeyGen.Append(TEXT("MATENERGY"));
		}
	}

	{
		int32 MaterialShadingPathOverride = GetMaterialShadingPathNodeOverride(Platform);
		if (MaterialShadingPathOverride != -1)
		{
			KeyGen.AppendSeparator();
			switch (ERHIShadingPath::Type(MaterialShadingPathOverride))
			{
				case ERHIShadingPath::Deferred:
					KeyGen.Append(TEXT("MATSPD"));
					break;
				case ERHIShadingPath::Forward:
					KeyGen.Append(TEXT("MATSPF"));
					break;
				case ERHIShadingPath::Mobile:
					KeyGen.Append(TEXT("MATSPM"));
					break;
			}
		}

		if (GetMaterialSimulateMobileForLowQuality(Platform))
		{
			KeyGen.AppendSeparator();
			KeyGen.Append(TEXT("MATSIMMOB"));
		}
	}

	{
		if (MaskedInEarlyPass(Platform))
		{
			KeyGen.AppendSeparator();
			KeyGen.Append(TEXT("EZPMM"));
		}
	}
	
	{
		static const auto CVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.GPUSkin.Limit2BoneInfluences"));
		if (CVar && CVar->GetValueOnAnyThread() != 0)
		{
			KeyGen.AppendSeparator();
			KeyGen.Append(TEXT("2bi"));
		}
	}
	{
		if(UseGPUScene(Platform))
		{
			KeyGen.AppendSeparator();
			KeyGen.Append(TEXT("gs1"));
		}
		else
		{
			KeyGen.AppendSeparator();
			KeyGen.Append(TEXT("gs0"));
		}
	}

	if (FDataDrivenShaderPlatformInfo::GetSupportSceneDataCompressedTransforms(Platform))
	{
		KeyGen.AppendSeparator();
		KeyGen.Append(TEXT("sdct"));
	}

	{
		static const auto CVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.GBufferDiffuseSampleOcclusion"));
		if (CVar && CVar->GetValueOnAnyThread() != 0)
		{
			KeyGen.AppendSeparator();
			KeyGen.Append(TEXT("GDSO"));
		}
	}

	{
		const bool VTLightmaps = UseVirtualTextureLightmap(Platform, TargetPlatform);
		const bool VTTextures = UseVirtualTexturing(TargetPlatform);

		static const auto CVarVTAnisotropic = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.VT.AnisotropicFiltering"));
		int32 VTFiltering = CVarVTAnisotropic && CVarVTAnisotropic->GetValueOnAnyThread() != 0 ? 1 : 0;

		if (IsMobilePlatform(Platform) && VTTextures)
		{
			static FShaderPlatformCachedIniValue<bool> CVarVTMobileManualTrilinearFiltering(TEXT("r.VT.Mobile.ManualTrilinearFiltering"));
			VTFiltering += (CVarVTMobileManualTrilinearFiltering.Get(Platform) ? 2 : 0);
		}

		KeyGen.AppendSeparator();
		KeyGen.Append(TEXT("VT"));
		KeyGen.AppendDebugText(TEXT("-"));
		KeyGen.Append(VTLightmaps);
		KeyGen.AppendDebugText(TEXT("-"));
		KeyGen.Append(VTTextures);
		KeyGen.AppendDebugText(TEXT("-"));
		KeyGen.Append(VTTextures); // Field repeated to maintain KeyGen layout and avoid unwanted global invalidation.
		KeyGen.AppendDebugText(TEXT("-"));
		KeyGen.Append(VTFiltering);
	}

	{
		const UE::Color::FColorSpace& WCS = UE::Color::FColorSpace::GetWorking();
		if (!WCS.IsSRGB())
		{
			// The working color space is uniquely defined by its chromaticities (as loaded from renderer settings).
			uint32 WCSHash = 0;
			WCSHash ^= GetTypeHash(WCS.GetRedChromaticity());
			WCSHash ^= GetTypeHash(WCS.GetGreenChromaticity());
			WCSHash ^= GetTypeHash(WCS.GetBlueChromaticity());
			WCSHash ^= GetTypeHash(WCS.GetWhiteChromaticity());
			KeyGen.AppendSeparator();
			KeyGen.Append(TEXT("WCS"));
			KeyGen.AppendDebugText(TEXT("-"));
			KeyGen.Append(WCSHash);
		}

		static const auto CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.LegacyLuminanceFactors"));
		if (CVar && CVar->GetInt() != 0)
		{
			KeyGen.AppendSeparator();
			KeyGen.Append(TEXT("LLF"));
		}
	}

	{
		static const auto CVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.Shaders.RemoveDeadCode"));
		if (CVar && CVar->GetValueOnAnyThread() != 0)
		{
			KeyGen.AppendSeparator();
			KeyGen.Append(TEXT("MIN"));
		}
	}

	{
		static const auto CVar = IConsoleManager::Get().FindTConsoleVariableDataBool(TEXT("r.ShaderCompiler.PreprocessedJobCache"));
		if (CVar && CVar->GetValueOnAnyThread())
		{
			KeyGen.AppendSeparator();
			KeyGen.Append(TEXT("PJC"));
		}
	}

	if (RHISupportsShaderRootConstants(Platform))
	{
		KeyGen.AppendSeparator();
		KeyGen.Append(TEXT("SHRC"));
	}

	if (RHISupportsShaderBundleDispatch(Platform))
	{
		KeyGen.AppendSeparator();
		KeyGen.Append(TEXT("SHBD"));
	}

	if (RHISupportsRenderTargetWriteMask(Platform))
	{
		KeyGen.AppendSeparator();
		KeyGen.Append(TEXT("RTWM"));
	}

	if (FDataDrivenShaderPlatformInfo::GetSupportsPerPixelDBufferMask(Platform))
	{
		KeyGen.AppendSeparator();
		KeyGen.Append(TEXT("PPDBM"));
	}

	if (FDataDrivenShaderPlatformInfo::GetSupportsDistanceFields(Platform))
	{
		KeyGen.AppendSeparator();
		KeyGen.Append(TEXT("DF"));
	}

	if (RHISupportsMeshShadersTier0(Platform))
	{
		KeyGen.AppendSeparator();
		KeyGen.Append(TEXT("MS_T0"));
	}

	if (RHISupportsMeshShadersTier1(Platform))
	{
		KeyGen.AppendSeparator();
		KeyGen.Append(TEXT("MS_T1"));
	}

	if (FDataDrivenShaderPlatformInfo::GetSupportsBindless(Platform))
	{
		const ERHIBindlessConfiguration BindlessConfig = FShaderPlatformConfig::GetBindlessConfiguration(Platform);
		if (BindlessConfig != ERHIBindlessConfiguration::Disabled)
		{
			KeyGen.AppendSeparator();
			KeyGen.Append("BNDLS");

			switch (BindlessConfig)
			{
			case ERHIBindlessConfiguration::RayTracing:
				KeyGen.Append("RT");
				break;
			case ERHIBindlessConfiguration::Minimal:
				KeyGen.Append("MIN");
				break;
			case ERHIBindlessConfiguration::All:
				KeyGen.Append("ALL");
				break;
			}
		}
	}

	const ERHIStaticShaderBindingLayoutSupport StaticShaderBindingLayoutSupport = FDataDrivenShaderPlatformInfo::GetStaticShaderBindingLayoutSupport(Platform);
	if (StaticShaderBindingLayoutSupport != ERHIStaticShaderBindingLayoutSupport::Unsupported)
	{
		KeyGen.AppendSeparator();
		KeyGen.Append(StaticShaderBindingLayoutSupport == ERHIStaticShaderBindingLayoutSupport::RayTracingOnly ?
		              TEXT("SSBL-RT") : TEXT("SSBL"));
	}

	if (ShouldCompileRayTracingShadersForProject(Platform))
	{
		static FShaderPlatformCachedIniValue<int32> CVarCompileRayTracingMaterialCHS(TEXT("r.RayTracing.CompileMaterialCHS"));
		static FShaderPlatformCachedIniValue<int32> CVarCompileRayTracingMaterialAHS(TEXT("r.RayTracing.CompileMaterialAHS"));

		static const auto CVarTextureLod = IConsoleManager::Get().FindConsoleVariable(TEXT("r.RayTracing.UseTextureLod"));

		KeyGen.AppendSeparator();
		KeyGen.Append(TEXT("RAY"));
		KeyGen.AppendDebugText(TEXT("-CHS"));
		KeyGen.AppendBoolInt(CVarCompileRayTracingMaterialCHS.Get(Platform) != 0);
		KeyGen.AppendDebugText(TEXT("AHS"));
		KeyGen.AppendBoolInt(CVarCompileRayTracingMaterialAHS.Get(Platform) != 0);
		KeyGen.AppendDebugText(TEXT("LOD"));
		KeyGen.AppendBoolInt(CVarTextureLod && CVarTextureLod->GetBool());
	}

	if (DoesPlatformSupportHeterogeneousVolumes(Platform))
	{
		static const auto ShadowCVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.HeterogeneousVolumes.Shadows"));
		if (ShadowCVar && ShadowCVar->GetValueOnAnyThread() != 0)
		{
			KeyGen.AppendSeparator();
			KeyGen.Append(TEXT("HVSHADOW"));
		}

		static const auto CompTranslucencyCVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.Translucency.HeterogeneousVolumes"));
		if (CompTranslucencyCVar && CompTranslucencyCVar->GetValueOnAnyThread() != 0)
		{
			KeyGen.AppendSeparator();
			KeyGen.Append(TEXT("HVCOMPTRANSL"));
		}
	}

	if (ForceSimpleSkyDiffuse(Platform))
	{
		KeyGen.AppendSeparator();
		KeyGen.Append(TEXT("SSD"));
	}

	if (VelocityEncodeDepth(Platform))
	{
		KeyGen.AppendSeparator();
		KeyGen.Append(TEXT("VED"));
	}

	if (VelocitySupportsTemporalResponsiveness(Platform))
	{
		KeyGen.AppendSeparator();
		KeyGen.Append(TEXT("VTR"));
	}
	
	if (VelocitySupportsPixelShaderMotionVectorWorldOffset(Platform))
	{
		KeyGen.AppendSeparator();
		KeyGen.Append(TEXT("VPSMVWO"));
	}

	{
		const bool bSupportsAnisotropicMaterials = FDataDrivenShaderPlatformInfo::GetSupportsAnisotropicMaterials(Platform);
		KeyGen.AppendSeparator();
		KeyGen.Append(TEXT("Aniso"));
		KeyGen.AppendDebugText(TEXT("-"));
		KeyGen.AppendBoolInt(bSupportsAnisotropicMaterials);
	}

	{
		// add shader compression format
		KeyGen.AppendSeparator();
		KeyGen.Append(TEXT("Compr"));
		FName CompressionFormat = GetShaderCompressionFormat();
		KeyGen.Append(CompressionFormat);
		if (CompressionFormat == NAME_Oodle)
		{
			FOodleDataCompression::ECompressor OodleCompressor;
			FOodleDataCompression::ECompressionLevel OodleLevel;
			GetShaderCompressionOodleSettings(OodleCompressor, OodleLevel);
			KeyGen.AppendSeparator();
			KeyGen.Append(TEXT("Compr"));
			KeyGen.Append(static_cast<int32>(OodleCompressor));
			KeyGen.AppendSeparator();
			KeyGen.AppendDebugText(TEXT("Lev"));
			KeyGen.Append(static_cast<int32>(OodleLevel));
		}
	}

	{
		// add whether or not non-pipelined shader types are included
		KeyGen.AppendSeparator();
		KeyGen.Append(TEXT("ExclNonPipSh-"));
		KeyGen.AppendBoolInt(ExcludeNonPipelinedShaderTypes(Platform));
	}

	KeyGen.AppendSeparator();
	KeyGen.Append(TEXT("LWC-"));
	KeyGen.Append(FMath::FloorToInt(FLargeWorldRenderScalar::GetTileSize()));

	uint64 ShaderPlatformPropertiesHash = FDataDrivenShaderPlatformInfo::GetShaderPlatformPropertiesHash(Platform);
	KeyGen.AppendSeparator();
	KeyGen.Append(ShaderPlatformPropertiesHash);

	if (IsSingleLayerWaterDepthPrepassEnabled(Platform, GetMaxSupportedFeatureLevel(Platform)))
	{
		KeyGen.AppendSeparator();
		KeyGen.Append(TEXT("SLWDP"));
	}

	if (IsGPUSkinPassThroughSupported(Platform))
	{
		KeyGen.AppendSeparator();
		KeyGen.Append(TEXT("SKPassThrough1"));
	}
	else
	{
		KeyGen.AppendSeparator();
		KeyGen.Append(TEXT("SKPassThrough0"));
	}

	if (DoesRuntimeSupportNanite(Platform, false, true))
	{
		KeyGen.AppendSeparator();
		KeyGen.Append(TEXT("Nanite-"));
		KeyGen.AppendDebugText(TEXT("Spline"));
		KeyGen.Append(NaniteSplineMeshesSupported() ? 1 : 0);
		KeyGen.AppendDebugText(TEXT("Skinned"));
		KeyGen.Append(NaniteSkinnedMeshesSupported() ? 1 : 0);
		KeyGen.AppendDebugText(TEXT("Assemblies"));
		KeyGen.Append(NaniteAssembliesSupported() ? 1 : 0);
		KeyGen.AppendDebugText(TEXT("Voxels"));
		KeyGen.Append(NaniteVoxelsSupported() ? 1 : 0);
		KeyGen.AppendDebugText(TEXT("Curves"));
		KeyGen.Append(NaniteCurvesSupported() ? 1 : 0);
		KeyGen.AppendDebugText(TEXT("Translucency"));
		KeyGen.Append(NaniteTranslucencySupported() ? 1 : 0);
	}
}

static EShaderPermutationFlags GAdditionalShaderPermutationFlags = EShaderPermutationFlags::None;

void SetAdditionalShaderPermutationFlags(EShaderPermutationFlags AdditionalFlags)
{
	check(GAdditionalShaderPermutationFlags == EShaderPermutationFlags::None);
	GAdditionalShaderPermutationFlags = AdditionalFlags;
}

EShaderPermutationFlags GetShaderPermutationFlags(const FPlatformTypeLayoutParameters& LayoutParams)
{
	EShaderPermutationFlags Result = GAdditionalShaderPermutationFlags;
	if (LayoutParams.WithEditorOnly())
	{
		Result |= EShaderPermutationFlags::HasEditorOnlyData;
	}
	return Result;
}

void RegisterRayTracingPayloadType(ERayTracingPayloadType PayloadType, uint32 PayloadSize, TRaytracingPayloadSizeFunction PayloadSizeFunction)
{
	// Make sure we haven't registered this payload type yet
	uint32 PayloadTypeInt = static_cast<uint32>(PayloadType);
	checkf(FMath::CountBits(PayloadTypeInt) == 1, TEXT("PayloadType should have only 1 bit set -- got %u"), PayloadTypeInt);
	checkf(!IsRayTracingPayloadRegistered(PayloadType), TEXT("Payload type %u has already been registered"), PayloadTypeInt);
	int32 PayloadIndex = FPlatformMath::CountTrailingZeros(PayloadTypeInt);
	RayTracingPayloadSizeFunctions[PayloadIndex] = PayloadSizeFunction;
	RayTracingPayloadSizes[PayloadIndex] = PayloadSizeFunction ? 0u : PayloadSize;
	RegisteredRayTracingPayloads |= PayloadTypeInt;
}

uint32 GetRayTracingPayloadTypeMaxSize(ERayTracingPayloadType PayloadType)
{
	// Compute the largest payload size among all set bits
	uint32 Result = 0;
	checkf(IsRayTracingPayloadRegistered(PayloadType), TEXT("Payload type %u has not been registered"), uint32{EnumToUnderlyingType(PayloadType)});
	for (uint32 PayloadTypeInt = static_cast<uint32>(PayloadType); PayloadTypeInt;)
	{
		const int32 PayloadIndex = FPlatformMath::CountTrailingZeros(PayloadTypeInt);
		if (RayTracingPayloadSizeFunctions[PayloadIndex] != nullptr)
		{
			Result = FMath::Max(Result, RayTracingPayloadSizeFunctions[PayloadIndex]());
		}
		else
		{
			Result = FMath::Max(Result, RayTracingPayloadSizes[PayloadIndex]);
		}

		// remove bit we just processed
		PayloadTypeInt &= ~(1u << PayloadIndex);
	}
	return Result;
}
