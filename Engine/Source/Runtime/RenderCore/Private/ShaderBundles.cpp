// Copyright Epic Games, Inc. All Rights Reserved.

#include "ShaderBundles.h"

#include "ShaderCompilerCore.h"
#include "DataDrivenShaderPlatformInfo.h"

static TAutoConsoleVariable<int32> CVarShaderBundleMaxSize(
	TEXT("r.ShaderBundle.MaxSize"),
	8192,
	TEXT("Maximum number of items in a work graph shader bundle."),
	ECVF_RenderThreadSafe | ECVF_ReadOnly);


IMPLEMENT_GLOBAL_SHADER(FDispatchShaderBundleCS, "/Engine/Private/ShaderBundleDispatch.usf", "DispatchShaderBundleEntry", SF_Compute);

bool FDispatchShaderBundleCS::ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
{
	return RHISupportsShaderBundleDispatch(Parameters.Platform);
}

void FDispatchShaderBundleCS::ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
{
	if (FDataDrivenShaderPlatformInfo::GetRequiresBindfulUtilityShaders(Parameters.Platform))
	{
		OutEnvironment.CompilerFlags.Add(CFLAG_ForceBindful);
	}
	OutEnvironment.CompilerFlags.Add(CFLAG_RootConstants);
	OutEnvironment.CompilerFlags.Add(CFLAG_SupportsMinimalBindless);

	OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZEX"), ThreadGroupSizeX);
	OutEnvironment.SetDefine(TEXT("USE_SHADER_ROOT_CONSTANTS"), RHISupportsShaderRootConstants(Parameters.Platform) ? 1 : 0);
	OutEnvironment.SetDefine(TEXT("BUNDLE_MODE_CS"), int32(ERHIShaderBundleMode::CS));
	OutEnvironment.SetDefine(TEXT("BUNDLE_MODE_MSPS"), int32(ERHIShaderBundleMode::MSPS));
	OutEnvironment.SetDefine(TEXT("BUNDLE_MODE_VSPS"), int32(ERHIShaderBundleMode::VSPS));
	FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
}


IMPLEMENT_GLOBAL_SHADER(FDispatchShaderBundleWorkGraph, "/Engine/Private/ShaderBundleWorkGraphDispatch.usf", "WorkGraphMainCS", SF_WorkGraphComputeNode);

bool FDispatchShaderBundleWorkGraph::ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
{
	return RHISupportsWorkGraphs(Parameters.Platform);
}

void FDispatchShaderBundleWorkGraph::ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
{
	OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZEX"), ThreadGroupSizeX);
	OutEnvironment.SetDefine(TEXT("MAX_DISPATCHGRID_SIZEX"), (CVarShaderBundleMaxSize.GetValueOnAnyThread() + ThreadGroupSizeX - 1) / ThreadGroupSizeX);
	FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
}

int32 FDispatchShaderBundleWorkGraph::GetMaxShaderBundleSize()
{
	static int32 ShaderBundleMaxSize = CVarShaderBundleMaxSize.GetValueOnAnyThread();
	return ShaderBundleMaxSize; 
}

FDispatchShaderBundleWorkGraph::FEntryNodeRecord FDispatchShaderBundleWorkGraph::MakeInputRecord(uint32 RecordCount, uint32 ArgOffset, uint32 ArgStride, uint32 ArgsBindlessHandle)
{
	return FEntryNodeRecord{ (RecordCount + ThreadGroupSizeX - 1) / ThreadGroupSizeX, RecordCount, FUintVector(ArgOffset, ArgStride, ArgsBindlessHandle) };
}
