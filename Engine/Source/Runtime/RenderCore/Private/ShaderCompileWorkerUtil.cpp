// Copyright Epic Games, Inc. All Rights Reserved.

#include "ShaderCompileWorkerUtil.h"

#include "Containers/AnsiString.h"
#include "Containers/Map.h"
#include "Interfaces/IShaderFormat.h"
#include "Interfaces/IShaderFormatModule.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#include "Logging/StructuredLog.h"
#include "Misc/Compression.h"
#include "Misc/FileHelper.h"
#include "Misc/MessageDialog.h"
#include "Misc/OutputDeviceRedirector.h"
#include "Misc/StringBuilder.h"
#include "Modules/ModuleManager.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"
#include "RHIShaderFormatDefinitions.inl"
#include "Serialization/MemoryReader.h"
#include "Serialization/MemoryWriter.h"
#include "ShaderDiagnostics.h"

static TAutoConsoleVariable<bool> CVarShadersPropagateLocalWorkerOOMs(
	TEXT("r.Shaders.PropagateLocalWorkerOOMs"),
	false,
	TEXT("When set, out-of-memory conditions in a local shader compile worker will be treated as regular out-of-memory conditions and propagated to the main process.\n")
	TEXT("This is useful when running in environment with hard memory limits, where it does not matter which process in particular caused us to violate the memory limit."),
	ECVF_Default);

static void ModalErrorOrLog(const FString& Title, const FString& Text, int64 CurrentFilePos = 0, int64 ExpectedFileSize = 0, bool bIsErrorFatal = true)
{
	static FThreadSafeBool bModalReported;

	FString BadFile;
	if (CurrentFilePos > ExpectedFileSize)
	{
		// Corrupt file
		BadFile = FString::Printf(TEXT(" (Truncated or corrupt output file! Current file pos %lld, file size %lld)"), CurrentFilePos, ExpectedFileSize);
	}

	if (bIsErrorFatal)
	{
		// Ensure errors are logged before exiting
		GLog->Panic();

		if (FPlatformProperties::SupportsWindowedMode() && !FApp::IsUnattended())
		{
			if (!bModalReported.AtomicSet(true))
			{
				UE_LOGF(LogShaders, Error, "%ls\n%ls", *Text, *BadFile);

				// Show dialog box with error message and request exit
				FMessageDialog::Open(EAppMsgType::Ok, FText::FromString(Text), FText::FromString(Title));
				constexpr bool bForceExit = true;
				FPlatformMisc::RequestExit(bForceExit, TEXT("ShaderCompiler.ModalErrorOrLog"));
			}
			else
			{
				// Another thread already opened a dialog box and requests exit
				FPlatformProcess::SleepInfinite();
			}
		}
		else
		{
			UE_LOGF(LogShaders, Fatal, "%ls\n%ls\n%ls", *Title, *Text, *BadFile);
		}
	}
	else
	{
		UE_LOGF(LogShaders, Error, "%ls\n%ls\n%ls", *Title, *Text, *BadFile);
	}
}

static TMap<FName, uint32> GetFormatVersionMap()
{
	TMap<FName, uint32> FormatVersionMap;

	const TArray<const class IShaderFormat*>& ShaderFormats = GetTargetPlatformManagerRef().GetShaderFormats();
	check(ShaderFormats.Num());
	for (int32 Index = 0; Index < ShaderFormats.Num(); Index++)
	{
		TArray<FName> OutFormats;
		ShaderFormats[Index]->GetSupportedFormats(OutFormats);
		check(OutFormats.Num());
		for (int32 InnerIndex = 0; InnerIndex < OutFormats.Num(); InnerIndex++)
		{
			uint32 Version = ShaderFormats[Index]->GetVersion(OutFormats[InnerIndex]);
			FormatVersionMap.Add(OutFormats[InnerIndex], Version);
		}
	}

	return FormatVersionMap;
}

static const TCHAR* GetCompileJobSuccessText(FShaderCompileJob* SingleJob)
{
	if (SingleJob)
	{
		return SingleJob->Output.bSucceeded ? TEXT("Succeeded") : TEXT("Failed");
	}
	return TEXT("");
}

void FShaderCompileWorkerUtil::LogQueuedCompileJobs(const TArray<FShaderCommonCompileJobPtr>& QueuedJobs, int32 NumProcessedJobs)
{
	if (NumProcessedJobs == -1)
	{
		UE_LOGF(LogShaders, Error, "SCW %d Queued Jobs, Unknown number of processed jobs!", QueuedJobs.Num());
	}
	else
	{
		UE_LOGF(LogShaders, Error, "SCW %d Queued Jobs, Finished %d single jobs", QueuedJobs.Num(), NumProcessedJobs);
	}

	for (int32 Index = 0; Index < QueuedJobs.Num(); ++Index)
	{
		if (FShaderCompileJob* SingleJob = QueuedJobs[Index]->GetSingleShaderJob())
		{
			UE_LOGF(LogShaders, Error, "Job %d [Single] %ls: %ls", Index, GetCompileJobSuccessText(SingleJob), *GetSingleJobCompilationDump(SingleJob));
		}
		else
		{
			FShaderPipelineCompileJob* PipelineJob = QueuedJobs[Index]->GetShaderPipelineJob();
			UE_LOGF(LogShaders, Error, "Job %d: Pipeline %ls ", Index, PipelineJob->Key.ShaderPipeline->GetName());
			for (int32 JobIndex = 0; JobIndex < PipelineJob->StageJobs.Num(); ++JobIndex)
			{
				FShaderCompileJob* StageJob = PipelineJob->StageJobs[JobIndex]->GetSingleShaderJob();
				UE_LOGF(LogShaders, Error, "PipelineJob %d %ls: %ls", JobIndex, GetCompileJobSuccessText(StageJob), *GetSingleJobCompilationDump(StageJob));
			}
		}
	}

	// Force a log flush so we can track the crash before the cooker potentially crashes before the output shows up
	GLog->Flush();
}

// Make functions so the crash reporter can disambiguate the actual error because of the different callstacks
namespace ShaderCompileWorkerError
{
	void HandleGeneralCrash(const TCHAR* ExceptionInfo, const TCHAR* Callstack)
	{
		ModalErrorOrLog(TEXT("ShaderCompileWorker crashed"), *FString::Printf(TEXT("Exception:\n%s\n\nCallstack:\n%s"), ExceptionInfo, Callstack));
	}

	void HandleBadShaderFormatVersion(const TCHAR* Data)
	{
		ModalErrorOrLog(TEXT("ShaderCompileWorker failed"), Data);
	}

	void HandleBadInputVersion(const TCHAR* Data)
	{
		ModalErrorOrLog(TEXT("ShaderCompileWorker failed"), Data);
	}

	void HandleBadSingleJobHeader(const TCHAR* Data)
	{
		ModalErrorOrLog(TEXT("ShaderCompileWorker failed"), Data);
	}

	void HandleBadPipelineJobHeader(const TCHAR* Data)
	{
		ModalErrorOrLog(TEXT("ShaderCompileWorker failed"), Data);
	}

	void HandleCantDeleteInputFile(const TCHAR* Data)
	{
		ModalErrorOrLog(TEXT("ShaderCompileWorker failed"), Data);
	}

	void HandleCantSaveOutputFile(const TCHAR* Data)
	{
		ModalErrorOrLog(TEXT("ShaderCompileWorker failed"), Data);
	}

	void HandleNoTargetShaderFormatsFound(const TCHAR* Data)
	{
		ModalErrorOrLog(TEXT("ShaderCompileWorker failed"), Data);
	}

	void HandleCantCompileForSpecificFormat(const TCHAR* Data)
	{
		ModalErrorOrLog(TEXT("ShaderCompileWorker failed"), Data);
	}

	void HandleOutputFileEmpty(const TCHAR* Filename)
	{
		ModalErrorOrLog(TEXT("ShaderCompileWorker failed"), FString::Printf(TEXT("Output file %s size is 0. Are you out of disk space?"), Filename));
	}

	void HandleOutputFileCorrupted(const TCHAR* Filename, int64 ExpectedSize, int64 ActualSize)
	{
		ModalErrorOrLog(TEXT("ShaderCompileWorker failed"), FString::Printf(TEXT("Output file corrupted (expected %I64d bytes, but only got %I64d): %s"), ExpectedSize, ActualSize, Filename));
	}

	void HandleCrashInsidePlatformCompiler(const TCHAR* Data)
	{
		// If the crash originates from a platform compiler, the error code must have been reported and we don't have to assume a corrupted output file.
		// In that case, don't crash the cooker with a fatal error, just report the error so the cooker can dump debug info.
		constexpr bool bIsErrorFatal = false;
		ModalErrorOrLog(TEXT("ShaderCompileWorker failed"), FString::Printf(TEXT("Crash inside the platform compiler:\n%s"), Data), 0, 0, bIsErrorFatal);
	}

	void HandleBadInputFile(const TCHAR* Data)
	{
		ModalErrorOrLog(TEXT("ShaderCompileWorker failed"), FString::Printf(TEXT("Bad-input-file exception:\n%s"), Data));
	}

	void HandleMissingVirtualInputFile(const TCHAR* Data)
	{
		ModalErrorOrLog(TEXT("ShaderCompileWorker failed"), FString::Printf(TEXT("Missing-virtual-input-file exception:\n%s"), Data));
	}

	bool HandleOutOfMemory(const TCHAR* ExceptionInfo, const TCHAR* Hostname, const FPlatformMemoryStats& MemoryStats, const TArray<FShaderCommonCompileJobPtr>& QueuedJobs, bool bWillRetry)
	{
		constexpr int64 Gibibyte = 1024 * 1024 * 1024;
		const FString ErrorReport = FString::Printf(
			TEXT("ShaderCompileWorker failed with out-of-memory (OOM) exception on machine \"%s\" (%s); MemoryStats:")
			TEXT("\n\tAvailablePhysical %llu (%.2f GiB)")
			TEXT("\n\t AvailableVirtual %llu (%.2f GiB)")
			TEXT("\n\t     UsedPhysical %llu (%.2f GiB)")
			TEXT("\n\t PeakUsedPhysical %llu (%.2f GiB)")
			TEXT("\n\t      UsedVirtual %llu (%.2f GiB)")
			TEXT("\n\t  PeakUsedVirtual %llu (%.2f GiB)"),
			Hostname,
			(ExceptionInfo[0] == TEXT('\0') ? TEXT("No exception information") : ExceptionInfo),
			MemoryStats.AvailablePhysical, double(MemoryStats.AvailablePhysical) / Gibibyte,
			MemoryStats.AvailableVirtual, double(MemoryStats.AvailableVirtual) / Gibibyte,
			MemoryStats.UsedPhysical, double(MemoryStats.UsedPhysical) / Gibibyte,
			MemoryStats.PeakUsedPhysical, double(MemoryStats.PeakUsedPhysical) / Gibibyte,
			MemoryStats.UsedVirtual, double(MemoryStats.UsedVirtual) / Gibibyte,
			MemoryStats.PeakUsedVirtual, double(MemoryStats.PeakUsedVirtual) / Gibibyte
		);

		if (bWillRetry)
		{
			// assume caller will retry the failed jobs rather than aborting
			return true;
		}
		else
		{
			if (CVarShadersPropagateLocalWorkerOOMs.GetValueOnAnyThread())
			{
				FPlatformMemory::OnOutOfMemory(0, 64);
			}
			ModalErrorOrLog(TEXT("ShaderCompileWorker failed"), ErrorReport);
			return false;
		}
	}
}


// Disable optimization for this crash handler to get full access to the entire stack frame when debugging a crash dump
UE_DISABLE_OPTIMIZATION_SHIP
bool FShaderCompileWorkerUtil::HandleWorkerCrash(
	const TArray<FShaderCommonCompileJobPtr>& QueuedJobs, 
	FArchive& OutputFile, 
	int32 OutputVersion, 
	int64 FileSize, 
	FSCWErrorCode::ECode ErrorCode, 
	int32 NumProcessedJobs, 
	int32 CallstackLength, 
	int32 ExceptionInfoLength, 
	int32 HostnameLength,
	bool bWillRetry)
{
	TArray<TCHAR> Callstack;
	Callstack.AddUninitialized(CallstackLength + 1);
	OutputFile.Serialize(Callstack.GetData(), CallstackLength * sizeof(TCHAR));
	Callstack[CallstackLength] = TEXT('\0');

	TArray<TCHAR> ExceptionInfo;
	ExceptionInfo.AddUninitialized(ExceptionInfoLength + 1);
	OutputFile.Serialize(ExceptionInfo.GetData(), ExceptionInfoLength * sizeof(TCHAR));
	ExceptionInfo[ExceptionInfoLength] = TEXT('\0');

	TArray<TCHAR> Hostname;
	Hostname.AddUninitialized(HostnameLength + 1);
	OutputFile.Serialize(Hostname.GetData(), HostnameLength * sizeof(TCHAR));
	Hostname[HostnameLength] = TEXT('\0');

	// Read available and used physical memory from worker machine on OOM error
	FPlatformMemoryStats MemoryStats;
	if (ErrorCode == FSCWErrorCode::OutOfMemory)
	{
		OutputFile
			<< MemoryStats.AvailablePhysical
			<< MemoryStats.AvailableVirtual
			<< MemoryStats.UsedPhysical
			<< MemoryStats.PeakUsedPhysical
			<< MemoryStats.UsedVirtual
			<< MemoryStats.PeakUsedVirtual
			;
	}

	// Store primary job information onto stack to make it part of a crash dump
	static const int32 MaxNumCharsForSourcePaths = 8192;
	int32 JobInputSourcePathsLength = 0;
	ANSICHAR JobInputSourcePaths[MaxNumCharsForSourcePaths];
	JobInputSourcePaths[0] = 0;

	auto WriteInputSourcePathOntoStack = [&JobInputSourcePathsLength, &JobInputSourcePaths](const ANSICHAR* InputSourcePath)
	{
		if (InputSourcePath != nullptr && JobInputSourcePathsLength + 3 < MaxNumCharsForSourcePaths)
		{
			// Copy input source path into stack buffer
			int32 InputSourcePathLength = FMath::Min(FCStringAnsi::Strlen(InputSourcePath), (MaxNumCharsForSourcePaths - JobInputSourcePathsLength - 2));
			FMemory::Memcpy(JobInputSourcePaths + JobInputSourcePathsLength, InputSourcePath, InputSourcePathLength);

			// Write newline character and put NUL character at the end
			JobInputSourcePathsLength += InputSourcePathLength;
			JobInputSourcePaths[JobInputSourcePathsLength] = TEXT('\n');
			++JobInputSourcePathsLength;
			JobInputSourcePaths[JobInputSourcePathsLength] = 0;
		}
	};

	auto StoreInputDebugInfo = [&WriteInputSourcePathOntoStack, &JobInputSourcePathsLength, &JobInputSourcePaths](const FShaderCompilerInput& Input)
	{
		FString DebugInfo = FString::Printf(TEXT("%s:%s"), *Input.VirtualSourceFilePath, *Input.EntryPointName);
		WriteInputSourcePathOntoStack(TCHAR_TO_UTF8(*DebugInfo));
	};

	for (auto CommonJob : QueuedJobs)
	{
		if (FShaderCompileJob* SingleJob = CommonJob->GetSingleShaderJob())
		{
			StoreInputDebugInfo(SingleJob->Input);
		}
		else if (FShaderPipelineCompileJob* PipelineJob = CommonJob->GetShaderPipelineJob())
		{
			for (int32 Job = 0; Job < PipelineJob->StageJobs.Num(); ++Job)
			{
				if (FShaderCompileJob* SingleStageJob = PipelineJob->StageJobs[Job])
				{
					StoreInputDebugInfo(SingleStageJob->Input);
				}
			}
		}
	}

	// One entry per error code as we want to have different callstacks for crash reporter...
	switch (ErrorCode)
	{
	default:
	case FSCWErrorCode::GeneralCrash:
		LogQueuedCompileJobs(QueuedJobs, NumProcessedJobs);
		ShaderCompileWorkerError::HandleGeneralCrash(ExceptionInfo.GetData(), Callstack.GetData());
		break;
	case FSCWErrorCode::BadShaderFormatVersion:
		ShaderCompileWorkerError::HandleBadShaderFormatVersion(ExceptionInfo.GetData());
		break;
	case FSCWErrorCode::BadInputVersion:
		ShaderCompileWorkerError::HandleBadInputVersion(ExceptionInfo.GetData());
		break;
	case FSCWErrorCode::BadSingleJobHeader:
		ShaderCompileWorkerError::HandleBadSingleJobHeader(ExceptionInfo.GetData());
		break;
	case FSCWErrorCode::BadPipelineJobHeader:
		ShaderCompileWorkerError::HandleBadPipelineJobHeader(ExceptionInfo.GetData());
		break;
	case FSCWErrorCode::CantDeleteInputFile:
		ShaderCompileWorkerError::HandleCantDeleteInputFile(ExceptionInfo.GetData());
		break;
	case FSCWErrorCode::CantSaveOutputFile:
		ShaderCompileWorkerError::HandleCantSaveOutputFile(ExceptionInfo.GetData());
		break;
	case FSCWErrorCode::NoTargetShaderFormatsFound:
		ShaderCompileWorkerError::HandleNoTargetShaderFormatsFound(ExceptionInfo.GetData());
		break;
	case FSCWErrorCode::CantCompileForSpecificFormat:
		ShaderCompileWorkerError::HandleCantCompileForSpecificFormat(ExceptionInfo.GetData());
		break;
	case FSCWErrorCode::CrashInsidePlatformCompiler:
		LogQueuedCompileJobs(QueuedJobs, NumProcessedJobs);
		ShaderCompileWorkerError::HandleCrashInsidePlatformCompiler(ExceptionInfo.GetData());
		break;
	case FSCWErrorCode::BadInputFile:
		ShaderCompileWorkerError::HandleBadInputFile(ExceptionInfo.GetData());
		break;
	case FSCWErrorCode::MissingVirtualInputFile:
		ShaderCompileWorkerError::HandleMissingVirtualInputFile(ExceptionInfo.GetData());
		break;
	case FSCWErrorCode::OutOfMemory:
		return ShaderCompileWorkerError::HandleOutOfMemory(ExceptionInfo.GetData(), Hostname.GetData(), MemoryStats, QueuedJobs, bWillRetry);
	case FSCWErrorCode::Success:
		// Can't get here...
		return true;
	}
	return false;
}
UE_ENABLE_OPTIMIZATION_SHIP


static void SplitJobsByType(const TArray<FShaderCommonCompileJobPtr>& QueuedJobs, TArray<FShaderCompileJob*>& OutQueuedSingleJobs, TArray<FShaderPipelineCompileJob*>& OutQueuedPipelineJobs)
{
	for (int32 Index = 0; Index < QueuedJobs.Num(); ++Index)
	{
		FShaderCommonCompileJobPtr CommonJob = QueuedJobs[Index];
		if (FShaderCompileJob* SingleJob = CommonJob->GetSingleShaderJob())
		{
			OutQueuedSingleJobs.Add(SingleJob);

		}
		else if (FShaderPipelineCompileJob* PipelineJob = CommonJob->GetShaderPipelineJob())
		{
			OutQueuedPipelineJobs.Add(PipelineJob);
		}
		else
		{
			checkf(0, TEXT("FShaderCommonCompileJob::Type=%d is not a valid type for a shader compile job"), (int32)CommonJob->Type);
		}
	}
}

// Serialize Queued Job information
bool FShaderCompileWorkerUtil::WriteTasks(const TArray<FShaderCommonCompileJobPtr>& QueuedJobs, FArchive& InTransferFile, EWriteTasksFlags Flags, TArray<FMemoryView>* OutCompressedSource)
{
	int32 InputVersion = ShaderCompileWorkerInputVersion;
	InTransferFile << InputVersion;

	TArray<uint8> UncompressedArray;
	FMemoryWriter TransferMemory(UncompressedArray);
	bool bCompressTaskFile = EnumHasAnyFlags(Flags, FShaderCompileWorkerUtil::EWriteTasksFlags::CompressTaskFile);
	FArchive& TransferFile = bCompressTaskFile ? TransferMemory : InTransferFile;
	if (!bCompressTaskFile)
	{
		// still write NAME_None as string
		FString FormatNone = FName(NAME_None).ToString();
		TransferFile << FormatNone;
	}

	static TMap<FName, uint32> FormatVersionMap = GetFormatVersionMap();

	TransferFile << FormatVersionMap;

	TArray<FShaderCompileJob*> QueuedSingleJobs;
	TArray<FShaderPipelineCompileJob*> QueuedPipelineJobs;
	SplitJobsByType(QueuedJobs, QueuedSingleJobs, QueuedPipelineJobs);

	TArray<TRefCountPtr<FSharedShaderCompilerEnvironment>> SharedEnvironments;
	TArray<const FShaderParametersMetadata*> RequestShaderParameterStructures;

	// gather shared environments and parameter structures, these tend to be shared between jobs
	{

		for (int32 JobIndex = 0; JobIndex < QueuedSingleJobs.Num(); JobIndex++)
		{
			FShaderCompileJob* Job = QueuedSingleJobs[JobIndex];
			Job->Input.GatherSharedInputs(SharedEnvironments, RequestShaderParameterStructures);
			if (OutCompressedSource)
			{
				OutCompressedSource->Add(Job->PreprocessOutput.GetCompressedSource());
				if (Job->SecondaryPreprocessOutput.IsValid())
				{
					OutCompressedSource->Add(Job->SecondaryPreprocessOutput->GetCompressedSource());
				}
			}
		}

		for (int32 JobIndex = 0; JobIndex < QueuedPipelineJobs.Num(); JobIndex++)
		{
			auto* PipelineJob = QueuedPipelineJobs[JobIndex];
			int32 NumStageJobs = PipelineJob->StageJobs.Num();

			for (int32 Index = 0; Index < NumStageJobs; Index++)
			{
				FShaderCompileJob* Job = PipelineJob->StageJobs[Index];
				Job->Input.GatherSharedInputs(SharedEnvironments, RequestShaderParameterStructures);
				if (OutCompressedSource)
				{
					OutCompressedSource->Add(Job->PreprocessOutput.GetCompressedSource());
					if (Job->SecondaryPreprocessOutput.IsValid())
					{
						OutCompressedSource->Add(Job->SecondaryPreprocessOutput->GetCompressedSource());
					}
				}
			}
		}

		int32 NumSharedEnvironments = SharedEnvironments.Num();
		TransferFile << NumSharedEnvironments;

		for (int32 EnvironmentIndex = 0; EnvironmentIndex < SharedEnvironments.Num(); EnvironmentIndex++)
		{
			SharedEnvironments[EnvironmentIndex]->SerializeCompilationDependencies(TransferFile);
		}
	}

	// Write shader parameter structures
	TArray<const FShaderParametersMetadata*> AllShaderParameterStructures;
	{
		// List all dependencies.
		for (int32 StructId = 0; StructId < RequestShaderParameterStructures.Num(); StructId++)
		{
			RequestShaderParameterStructures[StructId]->IterateStructureMetadataDependencies(
				[&](const FShaderParametersMetadata* Struct)
			{
				AllShaderParameterStructures.AddUnique(Struct);
			});
		}

		// Write all shader parameter structure.
		int32 NumParameterStructures = AllShaderParameterStructures.Num();
		TransferFile << NumParameterStructures;
		for (const FShaderParametersMetadata* Struct : AllShaderParameterStructures)
		{
			FString LayoutName = Struct->GetLayout().GetDebugName();
			FString StructTypeName = Struct->GetStructTypeName();
			FString ShaderVariableName = Struct->GetShaderVariableName();
			uint8 UseCase = uint8(Struct->GetUseCase());
			FString StructFileName = FString(ANSI_TO_TCHAR(Struct->GetFileName()));
			int32 StructFileLine = Struct->GetFileLine();
			uint32 Size = Struct->GetSize();
			int32 MemberCount = Struct->GetMembers().Num();

			static_assert(sizeof(UseCase) == sizeof(FShaderParametersMetadata::EUseCase), "Cast failure.");

			TransferFile << LayoutName;
			TransferFile << StructTypeName;
			TransferFile << ShaderVariableName;
			TransferFile << UseCase;
			TransferFile << StructFileName;
			TransferFile << StructFileLine;
			TransferFile << Size;
			TransferFile << MemberCount;

			for (const FShaderParametersMetadata::FMember& Member : Struct->GetMembers())
			{
				FString Name = Member.GetName();
				FString ShaderType = Member.GetShaderType();
				int32 FileLine = Member.GetFileLine();
				uint32 Offset = Member.GetOffset();
				uint8 BaseType = uint8(Member.GetBaseType());
				uint8 PrecisionModifier = uint8(Member.GetPrecision());
				uint32 NumRows = Member.GetNumRows();
				uint32 NumColumns = Member.GetNumColumns();
				uint32 NumElements = Member.GetNumElements();
				int32 StructMetadataIndex = INDEX_NONE;
				if (Member.GetStructMetadata())
				{
					StructMetadataIndex = AllShaderParameterStructures.Find(Member.GetStructMetadata());
					check(StructMetadataIndex != INDEX_NONE);
				}

				static_assert(sizeof(BaseType) == sizeof(EUniformBufferBaseType), "Cast failure.");
				static_assert(sizeof(PrecisionModifier) == sizeof(EShaderPrecisionModifier::Type), "Cast failure.");

				TransferFile << Name;
				TransferFile << ShaderType;
				TransferFile << FileLine;
				TransferFile << Offset;
				TransferFile << BaseType;
				TransferFile << PrecisionModifier;
				TransferFile << NumRows;
				TransferFile << NumColumns;
				TransferFile << NumElements;
				TransferFile << StructMetadataIndex;
			}
		}
	}

	bool bSkipSource = EnumHasAnyFlags(Flags, FShaderCompileWorkerUtil::EWriteTasksFlags::SkipSource);
	
	// Write individual shader jobs
	{
		int32 SingleJobHeader = ShaderCompileWorkerSingleJobHeader;
		TransferFile << SingleJobHeader;

		int32 NumBatches = QueuedSingleJobs.Num();
		TransferFile << NumBatches;

		// Serialize all the batched jobs
		for (int32 JobIndex = 0; JobIndex < QueuedSingleJobs.Num(); JobIndex++)
		{
			if (bSkipSource)
			{
				QueuedSingleJobs[JobIndex]->SerializeWorkerInputNoSource(TransferFile);
			}
			else
			{
				QueuedSingleJobs[JobIndex]->SerializeWorkerInput(TransferFile);
			}
			QueuedSingleJobs[JobIndex]->Input.SerializeSharedInputs(TransferFile, SharedEnvironments, AllShaderParameterStructures);
		}
	}

	// Write shader pipeline jobs
	{
		int32 PipelineJobHeader = ShaderCompileWorkerPipelineJobHeader;
		TransferFile << PipelineJobHeader;

		int32 NumBatches = QueuedPipelineJobs.Num();
		TransferFile << NumBatches;
		for (int32 JobIndex = 0; JobIndex < QueuedPipelineJobs.Num(); JobIndex++)
		{
			auto* PipelineJob = QueuedPipelineJobs[JobIndex];
			FString PipelineName = PipelineJob->Key.ShaderPipeline->GetName();
			TransferFile << PipelineName;
			int32 NumStageJobs = PipelineJob->StageJobs.Num();
			TransferFile << NumStageJobs;
			for (int32 Index = 0; Index < NumStageJobs; Index++)
			{
				if (bSkipSource)
				{
					PipelineJob->StageJobs[Index]->SerializeWorkerInputNoSource(TransferFile);
				}
				else
				{
					PipelineJob->StageJobs[Index]->SerializeWorkerInput(TransferFile);
				}
				PipelineJob->StageJobs[Index]->Input.SerializeSharedInputs(TransferFile, SharedEnvironments, AllShaderParameterStructures);
			}
		}
	}

	if (bCompressTaskFile)
	{
		TransferFile.Close();

		FName CompressionFormatToUse = NAME_Oodle;

		FString FormatName = CompressionFormatToUse.ToString();
		InTransferFile << FormatName;

		// serialize uncompressed data size
		int32 UncompressedDataSize = UncompressedArray.Num();
		checkf(UncompressedDataSize != 0, TEXT("Did not write any data to the task file for the compression."));
		InTransferFile << UncompressedDataSize;

		// not using SerializeCompressed because it splits into smaller chunks
		int32 CompressedSizeBound = FCompression::CompressMemoryBound(CompressionFormatToUse, static_cast<int32>(UncompressedDataSize));
		TArray<uint8> CompressedBuffer;
		CompressedBuffer.SetNumUninitialized(CompressedSizeBound);

		int32 ActualCompressedSize = CompressedSizeBound;
		bool bSucceeded = FCompression::CompressMemory(CompressionFormatToUse, CompressedBuffer.GetData(), ActualCompressedSize, UncompressedArray.GetData(), UncompressedDataSize, COMPRESS_BiasSpeed);
		checkf(ActualCompressedSize <= CompressedSizeBound, TEXT("Compressed size was larger than the bound - we stomped the memory."));
		CompressedBuffer.SetNum(ActualCompressedSize, EAllowShrinking::No);

		InTransferFile << CompressedBuffer;
	}

	return InTransferFile.Close();
}

const TCHAR* DebugWorkerInputFileName = TEXT("DebugCompile.in");
const TCHAR* DebugWorkerOutputFileName = TEXT("DebugCompile.out");
const TCHAR* DebugCompileArgsFileName = TEXT("DebugCompileArgs.txt");

static void WriteShaderCompileWorkerDebugCommandLine(FShaderCommonCompileJob& Job, const FString& JobDebugInfoPath, const FString& InputFilePath, FShaderDebugDataContext& Ctx)
{

	TStringBuilder<512> JobArgsPath;
	FPathViews::Append(JobArgsPath, JobDebugInfoPath, TEXT("DebugCompileArgs.txt"));

	TStringBuilder<512> CmdLine;
	CmdLine << TEXT("\"") << JobDebugInfoPath << TEXT("\"");
	CmdLine << TEXT(" 0 \"DebugCompile\" "); // parent PID (not meaningful for debug compile mode) followed by window title

	// output path to the single generated input file for the root job. this will be written in the first stage folder for pipeline jobs,
	// so make the path relative to the working directory for the current stage. 
	
	// note that we pass the path of the compile args txt file to all invocations of FPaths::MakePathRelativeTo in this function 
	// because it doesn't properly handle normalized paths when the path points to a directory (lack of a trailing / causes an internal 
	// call to FPaths::GetPath to strip the last folder)
	FString InputFilePathRelative = InputFilePath;
	FPaths::MakePathRelativeTo(InputFilePathRelative, JobArgsPath.ToString());
	CmdLine << InputFilePathRelative << TEXT(" ") << DebugWorkerOutputFileName;

	CmdLine << " -DebugSourceFiles=";
	TArray<FString> RelativeSourcePaths;
	RelativeSourcePaths.Reserve(Ctx.DebugSourceFiles.Num());
	for (const TPair<EShaderFrequency, FString>& SourceFilePair : Ctx.DebugSourceFiles) 
	{
		FString SourceFile = SourceFilePair.Value; // note: intentional copy of path here since MakePathRelativeTo modifies in-place
		// as above this may refer to multiple source files for different stages of a pipeline job
		// so make all paths relative to the working directory for this specific job.
		FPaths::MakePathRelativeTo(SourceFile, JobArgsPath.ToString());
		RelativeSourcePaths.Add(SourceFile);
	}
	CmdLine.JoinQuoted(RelativeSourcePaths, TEXT(","), TEXT("\""));
	CmdLine << " -TimeToLive=0.0f -KeepInput"; // pass zero TTL and KeepInput to make SCW process the job and exit without deleting the input

	FFileHelper::SaveStringToFile(CmdLine.ToString(), JobArgsPath.ToString());
}

void FShaderCompileWorkerUtil::DumpDebugCompileInput(FShaderCommonCompileJob& Job, FShaderDebugDataContext& Ctx)
{
	FString CreatedInput;
	Job.ForEachSingleShaderJob([&CreatedInput, &Job, &Ctx](FShaderCompileJob& SingleJob)
		{
			const FString& JobDebugPath = SingleJob.Input.DumpDebugInfoPath;
			FString JobInput = JobDebugPath / DebugWorkerInputFileName;
			if (CreatedInput.IsEmpty())
			{
				TArray<FShaderCommonCompileJobPtr> SingleJobArray;
				// export the .in file for just the "root" job; this is either a single job in which case this lambda will only be called once
				// (and Job == SingleJob), or it's a pipeline job and we want to export a single input file for all jobs and reference it for each stage directory
				SingleJobArray.Add(&Job);
				CreatedInput = JobInput;
				FArchive* DebugWorkerInputFileWriter = IFileManager::Get().CreateFileWriter(*CreatedInput, FILEWRITE_NoFail);
				WriteTasks(
					SingleJobArray,
					*DebugWorkerInputFileWriter,
					// Always compress the debug input files; they are rather large so this saves some disk space
					EWriteTasksFlags::CompressTaskFile |
					// Do not include source code in the debug files; this will be read from the debug usf to maintain readability and save disk space
					EWriteTasksFlags::SkipSource);
				DebugWorkerInputFileWriter->Close();
				delete DebugWorkerInputFileWriter;
			}

			// Always write out the DebugCompileArgs.txt for every stage; this will always run the full pipeline compile for pipeline jobs,
			// but is just a workflow improvement (so you can navigate to the debug folder for any particular problematic stage and run the full job
			// without having to know which stage folder contains the input file).
			WriteShaderCompileWorkerDebugCommandLine(SingleJob, JobDebugPath, CreatedInput, Ctx);
		});
}

static void ReadSingleJob(FShaderCompileJob* CurrentJob, FArchive& WorkerOutputFileReader)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ReadSingleJob)

	// Deserialize the shader compilation output.
	CurrentJob->SerializeWorkerOutput(WorkerOutputFileReader);

	// The job should already have a non-zero output hash
	checkf(CurrentJob->Output.OutputHash != FShaderHash() || !CurrentJob->bSucceeded, TEXT("OutputHash for a successful job was not set in the shader compile worker!"));
}

// Helper struct to provide consistent error report with detailed information about corrupted ShaderCompileWorker output file.
struct FSCWOutputFileContext
{
	FArchive& OutputFile;
	int64 FileSize = 0;

	FSCWOutputFileContext(FArchive& OutputFile) :
		OutputFile(OutputFile)
	{
	}

	template <typename... Types>
	void ModalErrorOrLog(UE::Core::TCheckedFormatString<FString::FmtCharType, Types...> Format, Types... Args)
	{
		FString Text = FString::Printf(Format, Args...);
		Text = FString::Printf(TEXT("File path: \"%s\"\n%s\nForgot to build ShaderCompileWorker or delete invalidated DerivedDataCache?"), *OutputFile.GetArchiveName(), *Text);
		const TCHAR* Title = TEXT("Corrupted ShaderCompileWorker output file");
		if (FileSize > 0)
		{
			::ModalErrorOrLog(Title, Text, OutputFile.Tell(), FileSize);
		}
		else
		{
			::ModalErrorOrLog(Title, Text, 0, 0);
		}
	}
};

// Process results from Worker Process.
// Returns false if reading the tasks failed but we were able to recover from handing a crash report. In this case, all jobs must be submitted/processed again.
FSCWErrorCode::ECode FShaderCompileWorkerUtil::ReadTasks(const TArray<FShaderCommonCompileJobPtr>& QueuedJobs, FArchive& OutputFile, FShaderCompileWorkerDiagnostics* OutWorkerDiagnostics, EReadTasksFlags Flags)
{
	FSCWOutputFileContext OutputFileContext(OutputFile);

	if (OutputFile.TotalSize() == 0)
	{
		ShaderCompileWorkerError::HandleOutputFileEmpty(*OutputFile.GetArchiveName());
	}

	int32 OutputVersion = ShaderCompileWorkerOutputVersion;
	OutputFile << OutputVersion;

	if (ShaderCompileWorkerOutputVersion != OutputVersion)
	{
		OutputFileContext.ModalErrorOrLog(TEXT("Expecting output version %d, got %d instead!"), ShaderCompileWorkerOutputVersion, OutputVersion);
	}

	OutputFile << OutputFileContext.FileSize;

	// Check for corrupted output file
	if (OutputFileContext.FileSize > OutputFile.TotalSize())
	{
		ShaderCompileWorkerError::HandleOutputFileCorrupted(*OutputFile.GetArchiveName(), OutputFileContext.FileSize, OutputFile.TotalSize());
	}

	FShaderCompileWorkerDiagnostics WorkerDiagnostics;
	OutputFile << WorkerDiagnostics;

	if (OutWorkerDiagnostics)
	{
		*OutWorkerDiagnostics = WorkerDiagnostics;
	}

	int32 NumProcessedJobs = 0;
	OutputFile << NumProcessedJobs;

	int32 CallstackLength = 0;
	OutputFile << CallstackLength;

	int32 ExceptionInfoLength = 0;
	OutputFile << ExceptionInfoLength;

	int32 HostnameLength = 0;
	OutputFile << HostnameLength;

	bool bWillRetry = EnumHasAnyFlags(Flags, EReadTasksFlags::WillRetry);

	if (WorkerDiagnostics.ErrorCode != FSCWErrorCode::Success)
	{
		// If worker crashed in a way we were able to recover from, return and expect the compile jobs to be reissued already
		if (HandleWorkerCrash(QueuedJobs, OutputFile, OutputVersion, OutputFileContext.FileSize, (FSCWErrorCode::ECode)WorkerDiagnostics.ErrorCode, NumProcessedJobs, CallstackLength, ExceptionInfoLength, HostnameLength, bWillRetry))
		{
			FSCWErrorCode::Reset();
			return (FSCWErrorCode::ECode)WorkerDiagnostics.ErrorCode;
		}
	}

	TArray<FShaderCompileJob*> QueuedSingleJobs;
	TArray<FShaderPipelineCompileJob*> QueuedPipelineJobs;
	SplitJobsByType(QueuedJobs, QueuedSingleJobs, QueuedPipelineJobs);

	// Read single jobs
	{
		int32 SingleJobHeader = -1;
		OutputFile << SingleJobHeader;
		if (SingleJobHeader != ShaderCompileWorkerSingleJobHeader)
		{
			OutputFileContext.ModalErrorOrLog(TEXT("Expecting single job header ID 0x%08X, got 0x%08X instead!"), ShaderCompileWorkerSingleJobHeader, SingleJobHeader);
		}

		int32 NumJobs;
		OutputFile << NumJobs;
		if (NumJobs != QueuedSingleJobs.Num())
		{
			OutputFileContext.ModalErrorOrLog(TEXT("Expecting %d single %s, got %d instead!"), QueuedSingleJobs.Num(), (QueuedSingleJobs.Num() == 1 ? TEXT("job") : TEXT("jobs")), NumJobs);
		}
		else
		{
			for (int32 JobIndex = 0; JobIndex < NumJobs; JobIndex++)
			{
				auto* CurrentJob = QueuedSingleJobs[JobIndex];
				ReadSingleJob(CurrentJob, OutputFile);
			}
		}
	}

	// Pipeline jobs
	{
		int32 PipelineJobHeader = -1;
		OutputFile << PipelineJobHeader;
		if (PipelineJobHeader != ShaderCompileWorkerPipelineJobHeader)
		{
			OutputFileContext.ModalErrorOrLog(TEXT("Expecting pipeline jobs header ID 0x%08X, got 0x%08X instead!"), ShaderCompileWorkerPipelineJobHeader, PipelineJobHeader);
		}

		int32 NumJobs;
		OutputFile << NumJobs;
		if (NumJobs != QueuedPipelineJobs.Num())
		{
			OutputFileContext.ModalErrorOrLog(TEXT("Expecting %d pipeline %s, got %d instead!"), QueuedPipelineJobs.Num(), (QueuedPipelineJobs.Num() == 1 ? TEXT("job") : TEXT("jobs")), NumJobs);
		}
		else
		{
			for (int32 JobIndex = 0; JobIndex < NumJobs; JobIndex++)
			{
				FShaderPipelineCompileJob* CurrentJob = QueuedPipelineJobs[JobIndex];

				FString PipelineName;
				OutputFile << PipelineName;
				bool bSucceeded = false;
				OutputFile << bSucceeded;
				CurrentJob->bSucceeded = bSucceeded;
				if (PipelineName != CurrentJob->Key.ShaderPipeline->GetName())
				{
					OutputFileContext.ModalErrorOrLog(TEXT("Expecting pipeline job \"%s\", got \"%s\" instead!"), CurrentJob->Key.ShaderPipeline->GetName(), *PipelineName);
				}

				int32 NumStageJobs = -1;
				OutputFile << NumStageJobs;

				if (NumStageJobs != CurrentJob->StageJobs.Num())
				{
					OutputFileContext.ModalErrorOrLog(TEXT("Expecting %d stage pipeline %s, got %d instead!"), CurrentJob->StageJobs.Num(), (CurrentJob->StageJobs.Num() == 1 ? TEXT("job") : TEXT("jobs")), NumStageJobs);
				}
				else
				{
					for (int32 Index = 0; Index < NumStageJobs; Index++)
					{
						FShaderCompileJob* SingleJob = CurrentJob->StageJobs[Index];
						ReadSingleJob(SingleJob, OutputFile);
					}
				}
			}
		}
	}
	
	return FSCWErrorCode::Success;
}

const TArray<const IShaderFormat*>& FShaderCompileWorkerUtil::GetShaderFormats()
{
	static const TArray<const IShaderFormat*> Formats = []
	{
		TArray<const IShaderFormat*> Output;
		TArray<FName> Modules;
		FModuleManager::Get().FindModules(SHADERFORMAT_MODULE_WILDCARD, Modules);
		for (FName Module : Modules)
		{
			if (IShaderFormat* Format = FModuleManager::LoadModuleChecked<IShaderFormatModule>(Module).GetShaderFormat())
			{
				Output.Add(Format);
			}
		}
		return Output;
	}();
	return Formats;
}

// Like GetFormatVersionMap above but does not rely on TargetPlatformManager.
[[nodiscard]] static const TMap<FName, uint32>& GetShaderFormatVersionMap()
{
	static const TMap<FName, uint32> Versions = []
	{
		TMap<FName, uint32> Output;
		for (const IShaderFormat* ShaderFormat : FShaderCompileWorkerUtil::GetShaderFormats())
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(ShaderFormat);
			TArray<FName> Formats;
			ShaderFormat->GetSupportedFormats(Formats);
			for (FName Format : Formats)
			{
				UE_LOGFMT(LogShaders, Display, "Available Shader Format {Format}", Format);
				uint32 Version = ShaderFormat->GetVersion(Format);
				Output.Add(Format, Version);
			}
		}
		return Output;
	}();
	return Versions;
}

[[nodiscard]] static bool VerifyFormatVersions(const TMap<FName, uint32>& FormatVersionMap, TMap<FName, uint32>& ReceivedFormatVersionMap)
{
	for (const TTuple<FName, uint32>& Pair : ReceivedFormatVersionMap)
	{
		if (const uint32* Expected = FormatVersionMap.Find(Pair.Key))
		{
			if (Pair.Value != *Expected)
			{
				FSCWErrorCode::Report(FSCWErrorCode::BadShaderFormatVersion, FString::Printf(TEXT("Mismatched shader version for format %s: Found version %u but expected %u; did you forget to build ShaderCompilerWorker?"), *Pair.Key.ToString(), Pair.Value, *Expected));
				return false;
			}
		}
	}
	return true;
}

void FShaderCompileWorkerUtil::ProcessInputFromArchive(
	FArchive* InputFilePtr,
	const FString& DumpDebugInfoPathOverride,
	TConstArrayView<FString> DebugSourceFiles,
	TFunctionRef<TArray<uint8> (int32)> LoadVirtualSourceFile,
	int32& OutNumProcessedJobs,
	TArray<FShaderCompileJob>& OutSingleJobs,
	TArray<FShaderPipelineCompileJob>& OutPipelineJobs,
	TArray<FString>& OutPipelineNames)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ProcessInputFromArchive);

	int32 InputVersion;
	*InputFilePtr << InputVersion;
	if (ShaderCompileWorkerInputVersion != InputVersion)
	{
		FSCWErrorCode::Report(FSCWErrorCode::BadInputVersion, FString::Printf(TEXT("Exiting due to ShaderCompilerWorker expecting input version %d, got %d instead! Did you forget to build ShaderCompilerWorker?"), ShaderCompileWorkerInputVersion, InputVersion));
		return;
	}

	FString CompressionFormatString;
	*InputFilePtr << CompressionFormatString;
	FName CompressionFormat(*CompressionFormatString);

	const bool bWasCompressed = !CompressionFormat.IsNone();

	TArray<uint8> UncompressedData;
	if (bWasCompressed)
	{
		int32 UncompressedDataSize = 0;
		*InputFilePtr << UncompressedDataSize;

		if (UncompressedDataSize == 0)
		{
			FSCWErrorCode::Report(FSCWErrorCode::BadInputFile, TEXT("Exiting due to bad input file to ShaderCompilerWorker (uncompressed size is 0)! Did you forget to build ShaderCompilerWorker?"));
			return;
		}

		UncompressedData.SetNumUninitialized(UncompressedDataSize);
		TArray<uint8> CompressedData;
		*InputFilePtr << CompressedData;
		if (!FCompression::UncompressMemory(CompressionFormat, UncompressedData.GetData(), UncompressedDataSize, CompressedData.GetData(), CompressedData.Num()))
		{
			FSCWErrorCode::Report(FSCWErrorCode::BadInputFile, FString::Printf(TEXT("Exiting due to bad input file to ShaderCompilerWorker (cannot uncompress with the format %s)! Did you forget to build ShaderCompilerWorker?"), *CompressionFormatString));
			return;
		}
	}
	FMemoryReader InputMemory(UncompressedData);
	FArchive& InputFile = bWasCompressed ? InputMemory : *InputFilePtr;

	TMap<FName, uint32> ReceivedFormatVersionMap;
	InputFile << ReceivedFormatVersionMap;

	const TMap<FName, uint32>& FormatVersionMap = GetShaderFormatVersionMap();

	const bool bIsDebugCompile = !DebugSourceFiles.IsEmpty();

	// When running a debug compile we are more permissive w.r.t. format versions.
	// Instead of requiring all versions match, we check only the version for the debug compilation that we will execute.
	if (!bIsDebugCompile && !VerifyFormatVersions(FormatVersionMap, ReceivedFormatVersionMap))
	{
		return;
	}

	// Initialize shader hash cache before reading any includes.
	InitializeShaderHashCache();

	// Array of string used as const TCHAR* during compilation process.
	TArray<TUniquePtr<FString>> AllocatedStrings;
	auto DeserializeConstTCHAR = [&AllocatedStrings](FArchive& Archive)
	{
		FString Name;
		Archive << Name;

		const TCHAR* CharName = nullptr;
		if (Name.Len() != 0)
		{
			if (AllocatedStrings.GetSlack() == 0)
			{
				AllocatedStrings.Reserve(AllocatedStrings.Num() + 1024);
			}
			CharName = **AllocatedStrings.Add_GetRef(MakeUnique<FString>(MoveTemp(Name)));
		}
		return CharName;
	};

	// Array of string used as const ANSICHAR* during compilation process.
	TArray<TUniquePtr<FAnsiString>> AllocatedAnsiStrings;
	auto DeserializeConstANSICHAR = [&AllocatedAnsiStrings](FArchive& Archive)
	{
		FString Name;
		Archive << Name;

		const ANSICHAR* CharName = nullptr;
		if (Name.Len() != 0)
		{
			if (AllocatedAnsiStrings.GetSlack() == 0)
			{
				AllocatedAnsiStrings.Reserve(AllocatedAnsiStrings.Num() + 1024);
			}
			CharName = **AllocatedAnsiStrings.Add_GetRef(MakeUnique<FAnsiString>(MoveTemp(Name)));
		}
		return CharName;
	};

	// Shared environments
	TArray<FShaderCompilerEnvironment> SharedEnvironments;
	{
		int32 NumSharedEnvironments = 0;
		InputFile << NumSharedEnvironments;
		SharedEnvironments.Empty(NumSharedEnvironments);
		SharedEnvironments.AddDefaulted(NumSharedEnvironments);

		for (int32 EnvironmentIndex = 0; EnvironmentIndex < NumSharedEnvironments; EnvironmentIndex++)
		{
			SharedEnvironments[EnvironmentIndex].SerializeCompilationDependencies(InputFile);
		}
	}

	// All the shader parameter structures
	// Note: this is a bit more complicated, purposefully to avoid switch const TCHAR* to FString in runtime FShaderParametersMetadata.
	TArray<TUniquePtr<FShaderParametersMetadata>> ParameterStructures;
	{
		int32 NumParameterStructures = 0;
		InputFile << NumParameterStructures;
		ParameterStructures.Reserve(NumParameterStructures);

		for (int32 StructIndex = 0; StructIndex < NumParameterStructures; StructIndex++)
		{
			const TCHAR* LayoutName;
			const TCHAR* StructTypeName;
			const TCHAR* ShaderVariableName;
			FShaderParametersMetadata::EUseCase UseCase;
			const ANSICHAR* StructFileName;
			int32 StructFileLine;
			uint32 Size;
			int32 MemberCount;

			LayoutName = DeserializeConstTCHAR(InputFile);
			StructTypeName = DeserializeConstTCHAR(InputFile);
			ShaderVariableName = DeserializeConstTCHAR(InputFile);
			InputFile << UseCase;
			StructFileName = DeserializeConstANSICHAR(InputFile);
			InputFile << StructFileLine;
			InputFile << Size;
			InputFile << MemberCount;

			TArray<FShaderParametersMetadata::FMember> Members;
			Members.Reserve(MemberCount);

			for (int32 MemberIndex = 0; MemberIndex < MemberCount; MemberIndex++)
			{
				const TCHAR* Name;
				const TCHAR* ShaderType;
				int32 FileLine;
				uint32 Offset;
				uint8 BaseType;
				uint8 PrecisionModifier;
				uint32 NumRows;
				uint32 NumColumns;
				uint32 NumElements;
				int32 StructMetadataIndex;

				static_assert(sizeof(BaseType) == sizeof(EUniformBufferBaseType), "Cast failure.");
				static_assert(sizeof(PrecisionModifier) == sizeof(EShaderPrecisionModifier::Type), "Cast failure.");

				Name = DeserializeConstTCHAR(InputFile);
				ShaderType = DeserializeConstTCHAR(InputFile);
				InputFile << FileLine;
				InputFile << Offset;
				InputFile << BaseType;
				InputFile << PrecisionModifier;
				InputFile << NumRows;
				InputFile << NumColumns;
				InputFile << NumElements;
				InputFile << StructMetadataIndex;

				if (ShaderType == nullptr)
				{
					ShaderType = TEXT("");
				}

				const FShaderParametersMetadata* StructMetadata = nullptr;
				if (StructMetadataIndex != INDEX_NONE)
				{
					StructMetadata = ParameterStructures[StructMetadataIndex].Get();
				}

				FShaderParametersMetadata::FMember Member(
					Name,
					ShaderType,
					FileLine,
					Offset,
					EUniformBufferBaseType(BaseType),
					EShaderPrecisionModifier::Type(PrecisionModifier),
					NumRows,
					NumColumns,
					NumElements,
					StructMetadata);
				Members.Add(Member);
			}

			ParameterStructures.Add(MakeUnique<FShaderParametersMetadata>(
				UseCase,
				EUniformBufferBindingFlags::Shader,
				/* InLayoutName = */ LayoutName,
				/* InStructTypeName = */ StructTypeName,
				/* InShaderVariableName = */ ShaderVariableName,
				/* InStaticSlotName = */ nullptr,
				StructFileName,
				StructFileLine,
				Size,
				Members,
				/* bForceCompleteInitialization = */ true));
		}
	}

	OutNumProcessedJobs = 0;

	auto SetupDebugCompile = [&FormatVersionMap, &ReceivedFormatVersionMap, &DumpDebugInfoPathOverride](FShaderCompileJob& Job, const TCHAR* SourcePath, const TCHAR* SecondarySourcePath = nullptr)
	{
		Job.Input.DebugInfoFlags |= EShaderDebugInfoFlags::CompileFromDebugUSF;

		checkf(FormatVersionMap.FindRef(Job.Input.ShaderFormat, 0) == ReceivedFormatVersionMap.FindRef(Job.Input.ShaderFormat, 0),
			TEXT("Format mismatch for shader format %s; ensure SDK/tools for this platform match the build where this debug compile input was generated.\n")
			TEXT("This check is skippable, with the caveat that this debug compilation may fail for unexpected reasons and/or results may be different."),
			*Job.Input.ShaderFormat.ToString());

		if (!DumpDebugInfoPathOverride.IsEmpty())
		{
			Job.Input.DumpDebugInfoPath = DumpDebugInfoPathOverride;
		}

		auto LoadDebugSource = [](const TCHAR* Path, FShaderPreprocessOutput& Output)
		{
			if (FString DebugSource; FFileHelper::LoadFileToString(DebugSource, Path))
			{
				// Strip comments from source when loading from a debug source file. Some backends don't handle the comments that the debug dump inserts properly.
				FAnsiString Stripped;
				// Since we are directly assigning the stripped result to an FShaderSource object which automatically appends SIMD patting as part of the Set function,
				// we don't need to also add padding during the convert-and-strip operation.
				ShaderConvertAndStripComments(DebugSource, Stripped.GetCharArray(), EConvertAndStripFlags::NoSimdPadding);
				Output.EditSource() = MoveTemp(Stripped);
			}
		};

		LoadDebugSource(SourcePath, Job.PreprocessOutput);

		if (SecondarySourcePath && Job.SecondaryPreprocessOutput)
		{
			LoadDebugSource(SecondarySourcePath, *Job.SecondaryPreprocessOutput);
		}
	};

	auto TryLoadVirtualSourceFile = [LoadVirtualSourceFile, VirtualSourceIndex = 0](FShaderPreprocessOutput* PreprocessOutput) mutable
	{
		if (PreprocessOutput)
		{
			// The virtual file mechanism is used by some build controllers to avoid duplicating source data between the
			// input file and job object. Use of the virtual file mechanism is indicated by the absence of compressed source data.
			if (FShaderSource& Source = PreprocessOutput->EditSource(); Source.GetCompressed().IsEmpty())
			{
				TArray<uint8> Compressed = LoadVirtualSourceFile(VirtualSourceIndex);
				if (Compressed.IsEmpty())
				{
					FSCWErrorCode::Report(FSCWErrorCode::MissingVirtualInputFile, FString::Printf(TEXT("Failed to load virtual source file at index %d."), VirtualSourceIndex));
					return false;
				}
				Source.EditCompressed() = MoveTemp(Compressed);
				++VirtualSourceIndex;
			}
		}
		return true;
	};

	// Individual jobs
	{
		int32 SingleJobHeader = ShaderCompileWorkerSingleJobHeader;
		InputFile << SingleJobHeader;
		if (ShaderCompileWorkerSingleJobHeader != SingleJobHeader)
		{
			FSCWErrorCode::Report(FSCWErrorCode::BadSingleJobHeader, FString::Printf(TEXT("Exiting due to ShaderCompilerWorker expecting job header %d, got %d instead! Did you forget to build ShaderCompilerWorker?"), ShaderCompileWorkerSingleJobHeader, SingleJobHeader));
			return;
		}

		int32 NumSingleJobs = 0;
		InputFile << NumSingleJobs;

		FlushShaderFileCache();

		OutSingleJobs.Reserve(NumSingleJobs);
		
		for (int32 BatchIndex = 0; BatchIndex < NumSingleJobs; BatchIndex++)
		{
			FShaderCompileJob& Job = OutSingleJobs.AddDefaulted_GetRef();
			// Deserialize the job's inputs.
			Job.SerializeWorkerInput(InputFile);
			if (bIsDebugCompile)
			{
				bool bHasSecondary = Job.SecondaryPreprocessOutput.IsValid();
				// this should only be used in debug compile mode; in this case we expect either a single job or a single pipeline job
				if (bHasSecondary)
				{
					// jobs with secondary compiles should have two source files
					check(DebugSourceFiles.Num() == 2 && NumSingleJobs == 1);
					SetupDebugCompile(Job, *DebugSourceFiles[0], *DebugSourceFiles[1]);
				}
				else
				{
					// normal jobs should just have a single source file
					check(DebugSourceFiles.Num() == 1 && NumSingleJobs == 1);
					SetupDebugCompile(Job, *DebugSourceFiles[0]);
				}
			}
			else
			{
				if (!TryLoadVirtualSourceFile(&Job.PreprocessOutput) ||
					!TryLoadVirtualSourceFile(Job.SecondaryPreprocessOutput.Get()))
				{
					return;
				}
			}

			Job.Input.DeserializeSharedInputs(InputFile, SharedEnvironments, ParameterStructures);

			// Process the job.
			CompileShader(GetShaderFormats(), Job, &OutNumProcessedJobs);
			if (bIsDebugCompile)
			{
				// if we're in debug compile mode and we have a single job, skip the pipeline job section below so
				// we can check that in either debug compile case we have the correct number of debug source files.
				return;
			}
		}
	}

	// Shader pipeline jobs
	{
		int32 PipelineJobHeader = ShaderCompileWorkerPipelineJobHeader;
		InputFile << PipelineJobHeader;
		if (ShaderCompileWorkerPipelineJobHeader != PipelineJobHeader)
		{
			FSCWErrorCode::Report(FSCWErrorCode::BadPipelineJobHeader, FString::Printf(TEXT("Exiting due to ShaderCompilerWorker expecting pipeline job header %d, got %d instead! Did you forget to build ShaderCompilerWorker?"), ShaderCompileWorkerSingleJobHeader, PipelineJobHeader));
			return;
		}

		int32 NumPipelines = 0;
		InputFile << NumPipelines;

		OutPipelineNames.Reserve(NumPipelines);
		OutPipelineJobs.Reserve(NumPipelines);

		// ensure that we are only compiling a single pipeline if debug source files were specified
		check(!bIsDebugCompile || NumPipelines == 1);

		for (int32 Index = 0; Index < NumPipelines; ++Index)
		{
			FString& PipelineName = OutPipelineNames.AddDefaulted_GetRef();
			InputFile << PipelineName;

			int32 NumStages = 0;
			InputFile << NumStages;
			FShaderPipelineCompileJob& PipelineJob = OutPipelineJobs.Emplace_GetRef(NumStages);

			int32 DebugSourceFileIndex = 0, DebugSourceFileCount = DebugSourceFiles.Num();
			for (int32 StageIndex = 0; StageIndex < NumStages; ++StageIndex)
			{
				// Deserialize the job's inputs.
				FShaderCompileJob* Job = PipelineJob.StageJobs[StageIndex]->GetSingleShaderJob();
				Job->SerializeWorkerInput(InputFile);
				if (bIsDebugCompile)
				{
					const TCHAR* DebugSourceFile = DebugSourceFileIndex < DebugSourceFileCount ? *DebugSourceFiles[DebugSourceFileIndex++] : nullptr;
					const TCHAR* DebugSecondarySourceFile = (Job->SecondaryPreprocessOutput.IsValid() && (DebugSourceFileIndex < DebugSourceFileCount)) ? *DebugSourceFiles[DebugSourceFileIndex++] : nullptr;
					SetupDebugCompile(*Job, DebugSourceFile, DebugSecondarySourceFile);
				}
				else
				{
					if (!TryLoadVirtualSourceFile(&Job->PreprocessOutput) ||
						!TryLoadVirtualSourceFile(Job->SecondaryPreprocessOutput.Get()))
					{
						return;
					}
				}
				Job->Input.DeserializeSharedInputs(InputFile, SharedEnvironments, ParameterStructures);

				// SCW doesn't run DDPI, GShaderHasCache Initialize is run at start with no knowledge of the CustomPlatforms
				// CustomPlatforms are known when we parse the WorkerInput so we populate the Directory here
				if (IsCustomPlatform((EShaderPlatform)Job->Input.Target.Platform))
				{
					const EShaderPlatform ShaderPlatform = ShaderFormatNameToShaderPlatform(Job->Input.ShaderFormat);
					UpdateIncludeDirectoryForPreviewPlatform((EShaderPlatform)Job->Input.Target.Platform, ShaderPlatform);
				}
			}

			CompileShaderPipeline(GetShaderFormats(), &PipelineJob, &OutNumProcessedJobs);
		}
	}
}

void FShaderCompileWorkerUtil::UpdateFileSize(FArchive& OutputFile, int64 FileSizePosition)
{
	int64 Current = OutputFile.Tell();
	OutputFile.Seek(FileSizePosition);
	OutputFile << Current;
	OutputFile.Seek(Current);
}

int64 FShaderCompileWorkerUtil::WriteOutputFileHeader(
	FArchive& OutputFile,
	const FString& HostName,
	FShaderCompileWorkerDiagnostics& WorkerDiagnostics,
	int32 NumProcessedJobs,
	int32 CallstackLength,
	const TCHAR* Callstack,
	int32 ExceptionInfoLength,
	const TCHAR* ExceptionInfo)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(WriteOutputFileHeader);

	int64 FileSizePosition = 0;
	int32 OutputVersion = ShaderCompileWorkerOutputVersion;
	OutputFile << OutputVersion;

	int64 FileSize = 0;
	// Get the position of the Size value to be patched in as the shader progresses
	FileSizePosition = OutputFile.Tell();
	OutputFile << FileSize;

	OutputFile << WorkerDiagnostics;

	OutputFile << NumProcessedJobs;

	// Note: Can't use FStrings here as SEH can't be used with destructors
	OutputFile << CallstackLength;

	OutputFile << ExceptionInfoLength;

	int32 HostNameLength = HostName.Len();
	OutputFile << HostNameLength;

	if (WorkerDiagnostics.ErrorCode != FSCWErrorCode::Success)
	{
		if (CallstackLength > 0)
		{
			OutputFile.Serialize((void*)Callstack, CallstackLength * sizeof(TCHAR));
		}

		if (ExceptionInfoLength > 0)
		{
			OutputFile.Serialize((void*)ExceptionInfo, ExceptionInfoLength * sizeof(TCHAR));
		}

		if (HostNameLength > 0)
		{
			OutputFile.Serialize((void*)*HostName, HostNameLength * sizeof(TCHAR));
		}

		// Store available and used physical memory of host machine on OOM error
		if (WorkerDiagnostics.ErrorCode == FSCWErrorCode::OutOfMemory)
		{
			FPlatformMemoryStats MemoryStats = FPlatformMemory::GetStats();

			OutputFile
				<< MemoryStats.AvailablePhysical
				<< MemoryStats.AvailableVirtual
				<< MemoryStats.UsedPhysical
				<< MemoryStats.PeakUsedPhysical
				<< MemoryStats.UsedVirtual
				<< MemoryStats.PeakUsedVirtual
				;
		}
	}

	FShaderCompileWorkerUtil::UpdateFileSize(OutputFile, FileSizePosition);
	return FileSizePosition;
}

void FShaderCompileWorkerUtil::WriteToOutputArchive(
	FArchive& OutputFile,
	const FString& HostName,
	FShaderCompileWorkerDiagnostics& WorkerDiagnostics,
	int32 NumProcessedJobs,
	TArray<FShaderCompileJob>& SingleJobs,
	TArray<FShaderPipelineCompileJob>& PipelineJobs,
	TArray<FString>& PipelineNames)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(WriteToOutputArchive);

	const int64 FileSizePosition = FShaderCompileWorkerUtil::WriteOutputFileHeader(OutputFile, HostName, WorkerDiagnostics, NumProcessedJobs, 0, nullptr, FSCWErrorCode::GetInfo().Len(), *FSCWErrorCode::GetInfo());

	{
		int32 SingleJobHeader = ShaderCompileWorkerSingleJobHeader;
		OutputFile << SingleJobHeader;

		int32 NumBatches = SingleJobs.Num();
		OutputFile << NumBatches;

		for (int32 JobIndex = 0; JobIndex < SingleJobs.Num(); JobIndex++)
		{
			SingleJobs[JobIndex].SerializeWorkerOutput(OutputFile);
			FShaderCompileWorkerUtil::UpdateFileSize(OutputFile, FileSizePosition);
		}
	}

	{
		int32 PipelineJobHeader = ShaderCompileWorkerPipelineJobHeader;
		OutputFile << PipelineJobHeader;
		int32 NumBatches = PipelineJobs.Num();
		OutputFile << NumBatches;

		for (int32 JobIndex = 0; JobIndex < PipelineJobs.Num(); JobIndex++)
		{
			auto& PipelineJob = PipelineJobs[JobIndex];
			OutputFile << PipelineNames[JobIndex];
			bool bSucceeded = (bool)PipelineJob.bSucceeded;
			OutputFile << bSucceeded;
			int32 NumStageJobs = PipelineJob.StageJobs.Num();
			OutputFile << NumStageJobs;

			for (int32 Index = 0; Index < NumStageJobs; ++Index)
			{
				PipelineJob.StageJobs[Index]->SerializeWorkerOutput(OutputFile);
				FShaderCompileWorkerUtil::UpdateFileSize(OutputFile, FileSizePosition);
			}
		}
	}
}
