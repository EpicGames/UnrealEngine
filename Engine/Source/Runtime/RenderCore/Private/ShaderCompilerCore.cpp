// Copyright Epic Games, Inc. All Rights Reserved.

#include "ShaderCompilerCore.h"

const FGuid UE_SHADER_CACHE_VERSION{"B90637EA-0482-48F0-8D5D-58D3C7F71810"};
