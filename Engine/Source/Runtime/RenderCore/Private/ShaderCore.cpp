// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ShaderCore.cpp: Shader core module implementation.
=============================================================================*/

#include "ShaderCore.h"
#include "Algo/Find.h"
#include "Async/ParallelFor.h"
#include "Compression/OodleDataCompression.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformStackWalk.h"
#include "Interfaces/IShaderFormat.h"
#include "Interfaces/IShaderFormatModule.h"
#include "Interfaces/ITargetPlatform.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#include "Math/BigInt.h"
#include "Misc/CommandLine.h"
#include "Misc/FileHelper.h"
#include "Misc/Compression.h"
#include "Misc/CoreMisc.h"
#include "Misc/Paths.h"
#include "Misc/PathViews.h"
#include "Misc/ScopeLock.h"
#include "Misc/ScopeRWLock.h"
#include "Misc/ScopedSlowTask.h"
#include "Misc/StringBuilder.h"
#include "Containers/StripedMap.h"
#include "Modules/ModuleManager.h"
#include "RHIShaderFormatDefinitions.inl"
#include "Serialization/MemoryHasher.h"
#include "Serialization/MemoryWriter.h"
#include "Serialization/ShaderKeyGenerator.h"
#include "Shader.h"
#include "ShaderSerialization.h"
#include "ShaderCompilerCore.h"
#include "ShaderCompilerDefinitions.h"
#include "ShaderCompilerJobTypes.h"
#include "ShaderCompileWorkerUtil.h"
#include "ShaderDiagnostics.h"
#include "ShaderPlatformConfig.h"
#include "Stats/StatsMisc.h"
#include "String/Find.h"
#include "Tasks/Task.h"
#include "Templates/UniquePtr.h"
#include "VertexFactory.h"
#if WITH_EDITOR
#include "Misc/ConfigCacheIni.h"
#include "Misc/StringBuilder.h"
#endif

#if PLATFORM_WINDOWS
#include "Windows/AllowWindowsPlatformTypes.h"
#	include <winnt.h>
#include "Windows/HideWindowsPlatformTypes.h"
#endif

namespace FShaderStatTagNames
{
	extern RENDERCORE_API const FName AnalysisArtifactsName = "ShaderStatTags.AnalysisArtifacts";
}

bool operator==(const FShaderStatVariant& LHS, const FShaderStatVariant& RHS)
{
	if (LHS.IsType<bool>() && RHS.IsType<bool>())
	{
		return LHS.Get<bool>() == RHS.Get<bool>();
	}
	else if (LHS.IsType<float>() && RHS.IsType<float>())
	{
		return LHS.Get<float>() == RHS.Get<float>();
	}
	else if (LHS.IsType<int32>() && RHS.IsType<int32>())
	{
		return LHS.Get<int32>() == RHS.Get<int32>();
	}
	else if (LHS.IsType<uint32>() && RHS.IsType<uint32>())
	{
		return LHS.Get<uint32>() == RHS.Get<uint32>();
	}
	else if (LHS.IsType<FString>() && RHS.IsType<FString>())
	{
		return LHS.Get<FString>() == RHS.Get<FString>();
	}

	return false;
}

bool operator<(const FShaderStatVariant& LHS, const FShaderStatVariant& RHS)
{
	if (LHS.IsType<bool>() && RHS.IsType<bool>())
	{
		return LHS.Get<bool>() < RHS.Get<bool>();
	}
	else if (LHS.IsType<float>() && RHS.IsType<float>()) 
	{
		return LHS.Get<float>() < RHS.Get<float>();
	}
	else if (LHS.IsType<int32>() && RHS.IsType<int32>())
	{
		return LHS.Get<int32>() < RHS.Get<int32>();
	}
	else if (LHS.IsType<uint32>() && RHS.IsType<uint32>())
	{
		return LHS.Get<uint32>() < RHS.Get<uint32>();
	}
	else if (LHS.IsType<FString>() && RHS.IsType<FString>())
	{
		return LHS.Get<FString>() < RHS.Get<FString>();
	}

	return false;
}

FStringBuilderBase& operator<<(FStringBuilderBase& Builder, const FShaderStatVariant& Variant)
{
	if (Variant.IsType<bool>())
	{
		Builder << Variant.Get<bool>();
	}
	else if (Variant.IsType<float>())
	{
		Builder << Variant.Get<float>();
	}
	else if (Variant.IsType<int32>())
	{
		Builder << Variant.Get<int32>();
	}
	else if (Variant.IsType<uint32>())
	{
		Builder << Variant.Get<uint32>();
	}
	else if (Variant.IsType<FString>())
	{
		Builder << Variant.Get<FString>();
	}
	return Builder;
}

void FGenericShaderStat::StreamArchive(FArchive& Ar)
{
	FGenericShaderStat& Stat = *this;
	if (Ar.IsSaving())
	{
		FString StatNameString = Stat.StatName.ToString();
		Ar << StatNameString;

		FString TagNameString = Stat.TagName.ToString();
		Ar << TagNameString;
	}
	else if (Ar.IsLoading())
	{
		FString StatNameString;
		Ar << StatNameString;
		Stat.StatName = FName(*StatNameString);

		FString TagNameString;
		Ar << TagNameString;
		Stat.TagName = FName(*TagNameString);
	}
	else
	{
		Ar << Stat.StatName;
		Ar << Stat.TagName;
	}

	Ar << Stat.Value;
	Ar << Stat.Flags;
}

bool FGenericShaderStat::operator==(const FGenericShaderStat& RHS) const
{
	return (StatName == RHS.StatName) && (Value == RHS.Value) && (Flags == RHS.Flags) && (TagName == RHS.TagName);
}

bool FGenericShaderStat::operator<(const FGenericShaderStat& RHS) const
{
	int32 NameComp = StatName.Compare(RHS.StatName);
	if (NameComp != 0)
	{
		return NameComp < 0;
	}
	
	if (!(Value == RHS.Value))
	{
		return Value < RHS.Value;
	}
	
	if (Flags != RHS.Flags)
	{
		return Flags < RHS.Flags;
	}

	return TagName.Compare(RHS.TagName) < 0;
}

FStringBuilderBase& operator<<(FStringBuilderBase& Builder, const FGenericShaderStat& Stat)
{
	Builder << Stat.StatName << TEXT("=") << Stat.Value;
	if (!Stat.TagName.IsNone())
	{
		Builder.Appendf(TEXT(" (Tag=%s)"), *Stat.TagName.ToString());
	}
	if (Stat.Flags == FGenericShaderStat::EFlags::Hidden)
	{
		Builder << TEXT(" (hidden)");
	}
	return Builder;
}

static TAutoConsoleVariable<bool> CVarDumpDebugInfoForCacheHits(
	TEXT("r.ShaderCompiler.DumpDebugInfoForCacheHits"),
	true,
	TEXT("If true, debug info (via IShaderFormat::OutputDebugData) will be output for all jobs including duplicates and cache/DDC hits. If false, only jobs that actually executed compilation will dump debug info."),
	ECVF_Default);

static FString GBreakOnPreprocessJob;
static FAutoConsoleVariableRef CVarBreakOnPreprocessJob(
	TEXT("r.ShaderCompiler.BreakOnPreprocessJob"),
	GBreakOnPreprocessJob,
	TEXT("If specified, triggers a breakpoint when preprocessing a job whose name matches the given string (case-insensitive, substrings are supported)"));

static TAutoConsoleVariable<FString> CVarShaderOverrideDebugDir(
	TEXT("r.OverrideShaderDebugDir"),
	"",
	TEXT("Override output location of shader debug files\n")
	TEXT("Empty: use default location Saved\\ShaderDebugInfo.\n"),
	ECVF_ReadOnly);

static TAutoConsoleVariable<bool> CVarDisambiguateShaderDebugDir(
	TEXT("r.DisambiguateShaderDebugDir"),
	false,
	TEXT("If true, appends a folder containing the full project path with directory separators/drive qualifiers replaced with _ to the root debug info folder.\n")
	TEXT("Intended for use in conjunction with r.OverrideShaderDebugDir to avoid shaderdebuginfo output clashing across workspaces/projects."),
	ECVF_ReadOnly);

void UpdateShaderDevelopmentMode()
{
	// Keep LogShaders verbosity in sync with r.ShaderDevelopmentMode
	// r.ShaderDevelopmentMode==1 results in all LogShaders log messages being displayed.
	// if r.ShaderDevelopmentMode isn't set, we leave the category alone (it defaults to Error, but we can be overriding it to something higher)
	bool bLogShadersUnsuppressed = UE_LOG_ACTIVE(LogShaders, Log);
	bool bDesiredLogShadersUnsuppressed = IsShaderDevelopmentModeEnabled();

	if (bLogShadersUnsuppressed != bDesiredLogShadersUnsuppressed)
	{
		if (bDesiredLogShadersUnsuppressed)
		{
			UE_SET_LOG_VERBOSITY(LogShaders, Log);
		}
	}
}

//
// Shader stats
//

DEFINE_STAT(STAT_ShaderCompiling_NiagaraShaders);
DEFINE_STAT(STAT_ShaderCompiling_NumTotalNiagaraShaders);

DEFINE_STAT(STAT_ShaderCompiling_OpenColorIOShaders);
DEFINE_STAT(STAT_ShaderCompiling_NumTotalOpenColorIOShaders);

DEFINE_STAT(STAT_ShaderCompiling_MaterialShaders);
DEFINE_STAT(STAT_ShaderCompiling_GlobalShaders);
DEFINE_STAT(STAT_ShaderCompiling_RHI);
DEFINE_STAT(STAT_ShaderCompiling_HashingShaderFiles);
DEFINE_STAT(STAT_ShaderCompiling_LoadingShaderFiles);
DEFINE_STAT(STAT_ShaderCompiling_HLSLTranslation);
DEFINE_STAT(STAT_ShaderCompiling_DDCLoading);
DEFINE_STAT(STAT_ShaderCompiling_MaterialLoading);
DEFINE_STAT(STAT_ShaderCompiling_MaterialCompiling);

DEFINE_STAT(STAT_ShaderCompiling_NumTotalMaterialShaders);
DEFINE_STAT(STAT_ShaderCompiling_NumSpecialMaterialShaders);
DEFINE_STAT(STAT_ShaderCompiling_NumParticleMaterialShaders);
DEFINE_STAT(STAT_ShaderCompiling_NumSkinnedMaterialShaders);
DEFINE_STAT(STAT_ShaderCompiling_NumLitMaterialShaders);
DEFINE_STAT(STAT_ShaderCompiling_NumUnlitMaterialShaders);
DEFINE_STAT(STAT_ShaderCompiling_NumTransparentMaterialShaders);
DEFINE_STAT(STAT_ShaderCompiling_NumOpaqueMaterialShaders);
DEFINE_STAT(STAT_ShaderCompiling_NumMaskedMaterialShaders);


DEFINE_STAT(STAT_Shaders_NumShadersLoaded);
DEFINE_STAT(STAT_Shaders_NumShadersCreated);
DEFINE_STAT(STAT_Shaders_NumShaderMaps);
DEFINE_STAT(STAT_Shaders_NumShaderMapsUsedForRendering);
DEFINE_STAT(STAT_Shaders_RTShaderLoadTime);
DEFINE_STAT(STAT_Shaders_ShaderMemory);
DEFINE_STAT(STAT_Shaders_ShaderResourceMemory);
DEFINE_STAT(STAT_Shaders_ShaderPreloadMemory);

CSV_DEFINE_CATEGORY(Shaders, (!UE_BUILD_SHIPPING));

/**
 * Singleton initial set of defines added when constructing a defines structure with bIncludeInitialDefines==true.  The
 * advantage of using preset defines is that the index of the initial define can be cached in the FShaderCompilerDefineNameCache
 * class, allowing direct lookup by index, bypassing the hash table.  This optimization is applied to system level defines used
 * by every shader (ones referenced by FShaderCompileUtilities::ApplyDerivedDefines).
 */
FShaderCompilerDefinitions* FShaderCompilerDefinitions::GInitialDefines = nullptr;

FShaderCompilerDefinitions::FShaderCompilerDefinitions(bool bIncludeInitialDefines)
	: InitialDefineCount(0), ValueCount(0)
{
	if (bIncludeInitialDefines && GInitialDefines)
	{
		*this = *GInitialDefines;
	}
	else
	{
		Pairs.Reserve(16);
		ValueTypes.Reserve(16);
	}
}

FShaderCompilerDefinitions::FShaderCompilerDefinitions(const FShaderCompilerDefinitions&) = default;

void FShaderCompilerDefinitions::InitializeInitialDefines(const FShaderCompilerDefinitions& InDefines)
{
	check(GInitialDefines == nullptr);
	GInitialDefines = new FShaderCompilerDefinitions;
	*GInitialDefines = InDefines;
	GInitialDefines->InitialDefineCount = InDefines.Pairs.Num();
}

struct FShaderFileCacheEntry
{
	FShaderSharedStringPtr Source;
	FShaderSharedAnsiStringPtr StrippedSource;				// Source with comments stripped out, and converted to ANSICHAR
	FShaderPreprocessDependenciesShared Dependencies;		// Stripped source with include dependencies, all in one shareable struct
	
	bool IsEmpty() const
	{
		return !Source.IsValid() && !StrippedSource.IsValid() && !Dependencies.IsValid();
	}
};

// Apply lock striping as we're mostly reader lock bound.
// Use prime number for the number of buckets for best distribution using modulo.
TStripedMap<31, FString, FShaderFileCacheEntry> GShaderFileCache;

// Enables transient missing shader file caching to avoid write lock contention on parallel loading.
class FMissingShaderFileCacheGuard
{
public:
	FMissingShaderFileCacheGuard()
	{
		++EnabledCount;
	}
	
	~FMissingShaderFileCacheGuard()
	{
		if (--EnabledCount == 0)
		{
			// Clean up empty entries which were cached for parallelization performance and turn further caching off
			// allowing for shaders that are dynamically generated to be loaded.
			GShaderFileCache.RemoveIf([](const TPair<FString, FShaderFileCacheEntry>& Entry)
			{
				return Entry.Value.IsEmpty();
			});
		}
	}

	static bool IsEnabled()
	{
		return EnabledCount > 0;
	}
	
private:
	static int EnabledCount;
};

int FMissingShaderFileCacheGuard::EnabledCount = 0;

class FShaderHashCache
{
public:
	FShaderHashCache()
		: bInitialized(false)
	{
	}

	void Initialize()
	{
		const FString EmptyDirectory;
		for (auto& Platform : Platforms)
		{
			Platform.IncludeDirectory = EmptyDirectory;
			Platform.ShaderHashCache.Reset();
		}


		TArray<FName> Modules;
		FModuleManager::Get().FindModules(SHADERFORMAT_MODULE_WILDCARD, Modules);

		if (!Modules.Num())
		{
			UE_LOGF(LogShaders, Error, "No target shader formats found!");
		}

		TArray<FName> SupportedFormats;

		for (int32 ModuleIndex = 0; ModuleIndex < Modules.Num(); ++ModuleIndex)
		{
			IShaderFormat* ShaderFormat = FModuleManager::LoadModuleChecked<IShaderFormatModule>(Modules[ModuleIndex]).GetShaderFormat();
			if (ShaderFormat)
			{
				FString IncludeDirectory = ShaderFormat->GetPlatformIncludeDirectory();
				if (!IncludeDirectory.IsEmpty())
				{
					IncludeDirectory = "/" + IncludeDirectory + "/";
				}

				SupportedFormats.Reset();
				ShaderFormat->GetSupportedFormats(SupportedFormats);

				for (int32 FormatIndex = 0; FormatIndex < SupportedFormats.Num(); ++FormatIndex)
				{
					const EShaderPlatform ShaderPlatform = ShaderFormatNameToShaderPlatform(SupportedFormats[FormatIndex]);
					if (ShaderPlatform != SP_NumPlatforms)
					{
						Platforms[ShaderPlatform].IncludeDirectory = IncludeDirectory;
					}
				}
			}
		}

#if WITH_EDITOR
		for (int i = 0; i < EShaderPlatform::SP_NumPlatforms; ++i)
		{
			EShaderPlatform ShaderPlatform = EShaderPlatform(i);
			if (FDataDrivenShaderPlatformInfo::IsValid(ShaderPlatform) && FDataDrivenShaderPlatformInfo::GetIsPreviewPlatform(ShaderPlatform))
			{
				const EShaderPlatform CompilingPlatform = ShaderFormatNameToShaderPlatform(FDataDrivenShaderPlatformInfo::GetShaderFormat(ShaderPlatform));
				if (CompilingPlatform != SP_NumPlatforms)
				{
					UpdateIncludeDirectoryForPreviewPlatform(ShaderPlatform, CompilingPlatform);
				}
			}
		}
#endif

		bInitialized = true;
	}

	void UpdateIncludeDirectoryForPreviewPlatform(EShaderPlatform PreviewShaderPlatform, EShaderPlatform ParentShaderPlatform)
	{
		Platforms[PreviewShaderPlatform].IncludeDirectory = Platforms[ParentShaderPlatform].IncludeDirectory;
	}

	const FSHAHash* FindHash(EShaderPlatform ShaderPlatform, const FString& VirtualFilePath) const
	{
		check(ShaderPlatform < UE_ARRAY_COUNT(Platforms));
		checkf(bInitialized, TEXT("GShaderHashCache::Initialize needs to be called before GShaderHashCache::FindHash."));

		return Platforms[ShaderPlatform].ShaderHashCache.Find(VirtualFilePath);
	}

	FSHAHash& AddHash(EShaderPlatform ShaderPlatform, const FString& VirtualFilePath)
	{
		check(ShaderPlatform < UE_ARRAY_COUNT(Platforms));
		checkf(bInitialized, TEXT("GShaderHashCache::Initialize needs to be called before GShaderHashCache::AddHash."));

		return Platforms[ShaderPlatform].ShaderHashCache.Add(VirtualFilePath, FSHAHash());
	}

	void RemoveHash(EShaderPlatform ShaderPlatform, const FString& VirtualFilePath)
	{
		Platforms[ShaderPlatform].ShaderHashCache.Remove(VirtualFilePath);
	}

	static bool IsPlatformInclude(const FString& VirtualFilePath)
	{
		return (VirtualFilePath.StartsWith(TEXT("/Engine/Private/Platform/"))
			|| VirtualFilePath.StartsWith(TEXT("/Engine/Public/Platform/"))
			|| VirtualFilePath.StartsWith(TEXT("/Platform/")));
	}

	bool ShouldIgnoreInclude(const FString& VirtualFilePath, EShaderPlatform ShaderPlatform) const
	{
		// Ignore only platform specific files, which won't be used by the target platform.
		if (IsPlatformInclude(VirtualFilePath))
		{
			const FString& PlatformIncludeDirectory = GetPlatformIncludeDirectory(ShaderPlatform);
			if (PlatformIncludeDirectory.IsEmpty() || !(VirtualFilePath.Contains(PlatformIncludeDirectory)))
			{
				return true;
			}
		}

		return false;
	}

	void Empty()
	{
		for (auto& Platform : Platforms)
		{
			Platform.ShaderHashCache.Reset();
		}
	}

	const FString& GetPlatformIncludeDirectory(EShaderPlatform ShaderPlatform) const
	{
		check(ShaderPlatform < EShaderPlatform::SP_NumPlatforms);
		checkf(bInitialized, TEXT("GShaderHashCache::Initialize needs to be called before GShaderHashCache::GetPlatformIncludeDirectory."));

		return Platforms[ShaderPlatform].IncludeDirectory;
	}

private:

	struct FPlatform
	{
		/** Folder with platform specific shader files. */
		FString IncludeDirectory;

		/** The shader file hash cache, used to minimize loading and hashing shader files; it includes also hashes for multiple filenames
		by making the key the concatenated list of filenames.
		*/
		TMap<FString, FSHAHash> ShaderHashCache;
	};

	FPlatform Platforms[EShaderPlatform::SP_NumPlatforms];
	bool bInitialized;	
};


static FSCWErrorCode::ECode GSCWErrorCode = FSCWErrorCode::NotSet;
static FString GSCWErrorCodeInfo;

void FSCWErrorCode::Report(ECode Code, const FStringView& Info)
{
	GSCWErrorCode = Code;
	GSCWErrorCodeInfo = Info;
}

void FSCWErrorCode::Reset()
{
	GSCWErrorCode = FSCWErrorCode::NotSet;
	GSCWErrorCodeInfo.Empty();
}

FSCWErrorCode::ECode FSCWErrorCode::Get()
{
	return GSCWErrorCode;
}

const FString& FSCWErrorCode::GetInfo()
{
	return GSCWErrorCodeInfo;
}

bool FSCWErrorCode::IsSet()
{
	return GSCWErrorCode != FSCWErrorCode::NotSet;
}

/** Protects GShaderHashCache from simultaneous modification by multiple threads. Note that it can cover more than one method of the class, e.g. a block of code doing Find() then Add() can be guarded */
FRWLock GShaderHashAccessRWLock;

FShaderHashCache GShaderHashCache;

/** Global map of virtual file path to physical file paths */
static TMap<FString, FString> GShaderSourceDirectoryMappings;
static TArray<FString> GShaderSourceSharedVirtualDirectories = { TEXT("/Engine/Shared/") };

static TAutoConsoleVariable<int32> CVarForceDebugViewModes(
	TEXT("r.ForceDebugViewModes"),
	0,
	TEXT("0: Setting has no effect.\n")
	TEXT("1: Forces debug view modes to be available, even on cooked builds.")
	TEXT("2: Forces debug view modes to be unavailable, even on editor builds.  Removes many shader permutations for faster shader iteration."),
	ECVF_RenderThreadSafe | ECVF_ReadOnly);

/** Returns true if debug viewmodes are allowed for the current platform. */
bool AllowDebugViewmodes()
{	
	int32 ForceDebugViewValue = CVarForceDebugViewModes.GetValueOnAnyThread();

	// To use debug viewmodes on consoles, r.ForceDebugViewModes must be set to 1 in ConsoleVariables.ini
	// And EngineDebugMaterials must be in the StartupPackages for the target platform.

	// Force enabled: r.ForceDebugViewModes 1
	const bool bForceEnable = ForceDebugViewValue == 1;
	if (bForceEnable)
	{
		return true;
	}

	// Force disabled: r.ForceDebugViewModes 2
	const bool bForceDisable = ForceDebugViewValue == 2;
	if (bForceDisable)
	{
		return false;
	}

	// Disable when running a commandlet without -AllowCommandletRendering
	if (IsRunningCommandlet() && !IsAllowCommandletRendering())
	{
		return false;
	}

	// Disable if we require cooked data
	if (FPlatformProperties::RequiresCookedData())
	{
		return false;
	}

	return true;
}

/** Returns true if debug viewmodes are allowed for the current platform. */
bool AllowDebugViewmodes(EShaderPlatform Platform)
{
#if WITH_EDITOR
	const int32 ForceDebugViewValue = CVarForceDebugViewModes.GetValueOnAnyThread();
	bool bForceEnable = ForceDebugViewValue == 1;
	bool bForceDisable = ForceDebugViewValue == 2;

	#if 0
		// We can't distinguish between editor and non-editor targets solely from EShaderPlatform.
		// RequiresCookedData() was always returning true for Windows in UE 4, and false in UE 5, for Windows
		TStringBuilder<64> PlatformName;
		ShaderPlatformToPlatformName(Platform).ToString(PlatformName);
		ITargetPlatform* TargetPlatform = GetTargetPlatformManager()->FindTargetPlatform(PlatformName);
		return (!bForceDisable) && (bForceEnable || !TargetPlatform || !TargetPlatform->RequiresCookedData());
	#else
		// Always include debug shaders for Windows targets until we have a way to distinguish
		return (!bForceDisable) && (bForceEnable || IsPCPlatform(Platform));
	#endif

#else
	return AllowDebugViewmodes();
#endif
}

struct FShaderSettingHelper
{
	const IConsoleVariable* const SettingCVar;
#if WITH_EDITOR
	const TCHAR* const SettingSection;
	const TCHAR* const SettingSectionBuildMachine;
	const TCHAR* const SettingName;
#endif

	FShaderSettingHelper() = delete;
	FShaderSettingHelper(
		const TCHAR* InSettingSection,
		const TCHAR* InSettingSectionBuildMachine,
		const TCHAR* InSettingName,
		const IConsoleVariable* InSettingCVar)
		: SettingCVar(InSettingCVar)
#if WITH_EDITOR
		, SettingSection(InSettingSection)
		, SettingSectionBuildMachine(InSettingSectionBuildMachine)
		, SettingName(InSettingName)
#endif
	{
	}

#if WITH_EDITOR
	static FConfigCacheIni* GetPlatformConfigForFormat(const FName ShaderFormat)
	{
		ITargetPlatform* TargetPlatform = GetTargetPlatformManager()->FindTargetPlatformWithSupport(TEXT("ShaderFormat"), ShaderFormat);
		return TargetPlatform ? TargetPlatform->GetConfigSystem() : nullptr;
	}
#endif

	bool GetBoolForPlatform(FName ShaderFormat) const
	{
		bool bEnabled = false;

		// First check the global cvar
		if (SettingCVar && SettingCVar->GetInt())
		{
			bEnabled = true;
		}
#if WITH_EDITOR
		// Then check the per platform settings.
		else if (FConfigCacheIni* PlatformConfig = GetPlatformConfigForFormat(ShaderFormat))
		{
			bool bFoundConfig = false;
			TStringBuilder<128> ShaderFormatStr;
			ShaderFormat.ToString(ShaderFormatStr);
			// first check for a shaderformat-specific value for the setting
			bFoundConfig = PlatformConfig->GetValue(ShaderFormatStr.ToString(), SettingName, bEnabled, GEngineIni);

			// if not found fall back to the per-platform value for compatibility with existing configs
			// (from either of the configured settings sections, i.e. [ShaderCompiler] or [ShaderCompiler_BuildMachine])
			if (!bFoundConfig && GIsBuildMachine && SettingSectionBuildMachine)
			{
				bFoundConfig = PlatformConfig->GetValue(SettingSectionBuildMachine, SettingName, bEnabled, GEngineIni);
			}
			if (!bFoundConfig)
			{
				PlatformConfig->GetValue(SettingSection, SettingName, bEnabled, GEngineIni);
			}
		}
#endif

		return bEnabled;
	}

	int32 GetIntForPlatform(FName ShaderFormat) const
	{
		int32 Value = false;

		// First check the global cvar
		if (SettingCVar && SettingCVar->GetInt())
		{
			Value = SettingCVar->GetInt();
		}
#if WITH_EDITOR
		// Then check the per platform settings.
		else if (FConfigCacheIni* PlatformConfig = GetPlatformConfigForFormat(ShaderFormat))
		{
			// Override with a build machine specific setting, if present.
			bool bFoundConfig = false;
			if (GIsBuildMachine && SettingSectionBuildMachine)
			{
				bFoundConfig = PlatformConfig->GetValue(SettingSectionBuildMachine, SettingName, Value, GEngineIni);
			}
			if (!bFoundConfig)
			{
				PlatformConfig->GetValue(SettingSection, SettingName, Value, GEngineIni);
			}
		}
#endif

		return Value;
	}

	bool GetStringForPlatform(FString& OutputString, FName ShaderFormat) const
	{
		// First check the global cvar
		if (SettingCVar)
		{
			OutputString = SettingCVar->GetString();
		}

#if WITH_EDITOR
		if (OutputString.IsEmpty())
		{
			if (FConfigCacheIni* PlatformConfig = GetPlatformConfigForFormat(ShaderFormat))
			{
				// Override with a build machine specific setting, if present.
				if (GIsBuildMachine && SettingSectionBuildMachine)
				{
					OutputString = PlatformConfig->GetStr(SettingSectionBuildMachine, SettingName, GEngineIni);
				}
				if (OutputString.IsEmpty())
				{
					OutputString = PlatformConfig->GetStr(SettingSection, SettingName, GEngineIni);
				}
			}
		}
#endif

		return !OutputString.IsEmpty();
	}
};

struct FShaderSymbolSettingHelper : public FShaderSettingHelper
{
public:
	FShaderSymbolSettingHelper(const TCHAR* InSettingName, bool bPlatformOnly = false)
		: FShaderSettingHelper(TEXT("ShaderCompiler"), TEXT("ShaderCompiler_BuildMachine"), InSettingName, !bPlatformOnly ? IConsoleManager::Get().FindConsoleVariable(InSettingName) : nullptr)
	{
		check(SettingCVar || bPlatformOnly);
	}

	bool IsEnabled(FName ShaderFormat) const
	{
		return GetBoolForPlatform(ShaderFormat);
	}

	bool GetString(FString& OutString, FName ShaderFormat) const
	{
		return GetStringForPlatform(OutString, ShaderFormat);
	}
};

ERHIBindlessConfiguration UE::ShaderCompiler::GetBindlessConfiguration(EShaderPlatform ShaderPlatform)
{
	return FShaderPlatformConfig::GetBindlessConfiguration(ShaderPlatform);
}

// deprecated
ERHIBindlessConfiguration UE::ShaderCompiler::GetBindlessResourcesConfiguration(FName ShaderFormat)
{
	return FShaderPlatformConfig::GetBindlessConfiguration(ShaderFormatToLegacyShaderPlatform(ShaderFormat));
}

// deprecated
ERHIBindlessConfiguration UE::ShaderCompiler::GetBindlessSamplersConfiguration(FName ShaderFormat)
{
	return FShaderPlatformConfig::GetBindlessConfiguration(ShaderFormatToLegacyShaderPlatform(ShaderFormat));
}

static bool CheckCombinedCompilerFlags(const FShaderCompilerInput& Input, ECompilerFlags InFlags)
{
	return Input.Environment.CompilerFlags.Contains(InFlags) || (Input.SharedEnvironment.IsValid() && Input.SharedEnvironment->CompilerFlags.Contains(InFlags));
}

bool UE::ShaderCompiler::ShouldCompileWithBindlessEnabled(EShaderPlatform ShaderPlatform, const FShaderCompilerInput& Input)
{
	if (!FDataDrivenShaderPlatformInfo::GetSupportsBindless(Input.Target.GetPlatform()) || Input.Environment.CompilerFlags.Contains(CFLAG_ForceBindful))
	{
		return false;
	}

	// If inline raytracing is enabled and bindless is required, make sure bindless is enabled for raytracing at least
	if (Input.Environment.CompilerFlags.Contains(CFLAG_InlineRayTracing) && FDataDrivenShaderPlatformInfo::GetRequiresBindlessForInlineRayTracing(Input.Target.GetPlatform()))
	{
		check(FShaderPlatformConfig::GetBindlessConfiguration(ShaderPlatform) != ERHIBindlessConfiguration::Disabled);
		return true;
	}

	const ERHIBindlessConfiguration BindlessConfig = FShaderPlatformConfig::GetBindlessConfiguration(ShaderPlatform);

	switch (BindlessConfig)
	{
	case ERHIBindlessConfiguration::All:
		return true;

	case ERHIBindlessConfiguration::Minimal:
		if (CheckCombinedCompilerFlags(Input, CFLAG_SupportsMinimalBindless))
		{
			return true;
		}
		[[fallthrough]];

	case ERHIBindlessConfiguration::RayTracing:
		if (IsRayTracingShaderFrequency(Input.Target.GetFrequency()))
		{
			return true;
		}
		[[fallthrough]];

	default:
	case ERHIBindlessConfiguration::Disabled:
		return false;
	}
}


bool ShouldGenerateShaderSymbols(FName ShaderFormat)
{
	static const FShaderSymbolSettingHelper Symbols(TEXT("r.Shaders.Symbols"));
	static const FShaderSymbolSettingHelper GenerateSymbols(TEXT("r.Shaders.GenerateSymbols"), true);
	return Symbols.IsEnabled(ShaderFormat) || GenerateSymbols.IsEnabled(ShaderFormat);
}

bool ShouldGenerateShaderSymbolsInfo(FName ShaderFormat)
{
	static const FShaderSymbolSettingHelper SymbolsInfo(TEXT("r.Shaders.SymbolsInfo"));
	return SymbolsInfo.IsEnabled(ShaderFormat);
}

bool ShouldWriteShaderSymbols(FName ShaderFormat)
{
	static const FShaderSymbolSettingHelper Symbols(TEXT("r.Shaders.Symbols"));
	static const FShaderSymbolSettingHelper WriteSymbols(TEXT("r.Shaders.WriteSymbols"), true);
	return Symbols.IsEnabled(ShaderFormat) || WriteSymbols.IsEnabled(ShaderFormat);
}

bool ShouldAllowUniqueShaderSymbols(FName ShaderFormat)
{
	static const FShaderSymbolSettingHelper AllowUniqueSymbols(TEXT("r.Shaders.AllowUniqueSymbols"));
	return AllowUniqueSymbols.IsEnabled(ShaderFormat);
}

const FString& GetBuildMachineArtifactBasePath()
{
	// no need for thread safety here; this will be hit on a single thread during shader compiling manager init long before it's called anywhere else
	static FString ArtifactBasePath;
	if (ArtifactBasePath.IsEmpty())
	{
		ArtifactBasePath = FPaths::Combine(*FPaths::EngineDir(), TEXT("Programs"), TEXT("AutomationTool"), TEXT("Saved"), TEXT("Logs"));
	}
	return ArtifactBasePath;
}


const FString& GetShaderDebugInfoPath()
{
	static FString DebugInfoPath;
	// no need for thread safety here; this will be hit on a single thread during shader compiling manager init long before it's called anywhere else
	if (DebugInfoPath.IsEmpty())
	{
		// Build machines should dump to the AutomationTool/Saved/Logs directory and they will upload as build artifacts via the AutomationTool.
		const FString BaseDebugInfoPath = GIsBuildMachine ? GetBuildMachineArtifactBasePath() : FPaths::ProjectSavedDir();

		FString AbsoluteDebugInfoDirectory = IFileManager::Get().ConvertToAbsolutePathForExternalAppForWrite(*(BaseDebugInfoPath / TEXT("ShaderDebugInfo")));
		const FString OverrideShaderDebugDir = CVarShaderOverrideDebugDir.GetValueOnAnyThread();
		if (!OverrideShaderDebugDir.IsEmpty())
		{
			AbsoluteDebugInfoDirectory = OverrideShaderDebugDir;
		}

		if (CVarDisambiguateShaderDebugDir.GetValueOnAnyThread())
		{
			FString AppendFolder = FPaths::ConvertRelativePathToFull(FPaths::ProjectDir());
			FPaths::NormalizeDirectoryName(AppendFolder);
			AppendFolder.ReplaceInline(TEXT(":/"), TEXT("_"));
			AppendFolder.ReplaceCharInline('/', '_');
			AbsoluteDebugInfoDirectory /= AppendFolder;
		}

		FPaths::NormalizeDirectoryName(AbsoluteDebugInfoDirectory);
		DebugInfoPath = AbsoluteDebugInfoDirectory;
	}
	return DebugInfoPath;
}

bool GetShaderSymbolPathOverride(FString& OutPathOverride, FName ShaderFormat)
{
	static const FShaderSymbolSettingHelper SymbolPathOverride(TEXT("r.Shaders.SymbolPathOverride"));
	if (SymbolPathOverride.GetString(OutPathOverride, ShaderFormat))
	{
		if (!OutPathOverride.IsEmpty())
		{
			// Allow the user to specify the location of the per-platform string.
			OutPathOverride = OutPathOverride.Replace(TEXT("{Platform}"), *ShaderFormat.ToString(), ESearchCase::IgnoreCase);
			// Allow the user to specify the location of the per-project string.
			OutPathOverride = OutPathOverride.Replace(TEXT("{ProjectDir}"), *FPaths::ProjectDir(), ESearchCase::IgnoreCase);
			// Allow the user to specify the location of the per-project saved folder string.
			OutPathOverride = OutPathOverride.Replace(TEXT("{ProjectSavedDir}"), *FPaths::ProjectSavedDir(), ESearchCase::IgnoreCase);
			// Allow the user to specify the configured shader debug info folder.
			OutPathOverride = OutPathOverride.Replace(TEXT("{ShaderDebugInfoDir}"), *GetShaderDebugInfoPath(), ESearchCase::IgnoreCase);
		}
		return !OutPathOverride.IsEmpty();
	}
	return false;
}

bool GetShaderFileNameOverride(FString& OutFileNameOverride, const TCHAR* Cvar, FName ShaderFormat, FName PlatformName)
{
	FString OutFileName;
	const FShaderSymbolSettingHelper SymbolFileNameOverride(Cvar);
	if (SymbolFileNameOverride.GetString(OutFileName, ShaderFormat))
	{
		if (!OutFileName.IsEmpty())
		{
			// Allow the user to specify the location of the per-platform string.
			OutFileName = OutFileName.Replace(TEXT("{Platform}"), *PlatformName.ToString(), ESearchCase::IgnoreCase);

			// Allow adding name of the DLC if any to differentiate cooks, and one or more optional formatting dashes
			FString DLCName;
			FParse::Value(FCommandLine::Get(), TEXT("-dlcname="), DLCName);
			OutFileName = OutFileName.Replace(TEXT("{?DLC-}"), DLCName.IsEmpty() ? TEXT("") : TEXT("-"), ESearchCase::IgnoreCase);
			OutFileName = OutFileName.Replace(TEXT("{DLC}"), *DLCName, ESearchCase::IgnoreCase);

			// Sanitize for safety
			OutFileName.TrimStartAndEndInline();
			OutFileName = FPaths::MakeValidFileName(OutFileName, TEXT('_'));
		}
	}

	if (OutFileName.IsEmpty())
	{
		return false;
	}

	OutFileNameOverride = OutFileName;
	return true;
}

EWriteShaderSymbols GetWriteShaderSymbolsOptions(FName ShaderFormat)
{
	static const FShaderSymbolSettingHelper WriteSymbolsZip(TEXT("r.Shaders.WriteSymbols.Zip"));
	return static_cast<EWriteShaderSymbols>(WriteSymbolsZip.GetIntForPlatform(ShaderFormat));
}

bool ShouldEnableExtraShaderData(FName ShaderFormat)
{
	static const FShaderSymbolSettingHelper ExtraData(TEXT("r.Shaders.ExtraData"));
	return ExtraData.IsEnabled(ShaderFormat);
}

bool ShouldOptimizeShaders(FName ShaderFormat)
{
	static const FShaderSymbolSettingHelper Optimize(TEXT("r.Shaders.Optimize"));
	return Optimize.IsEnabled(ShaderFormat);
}

#ifndef UE_ALLOW_SHADER_COMPILING
// is shader compiling allowed at all? (set to 0 in a cooked editor .Target.cs if the target has no shaders available)
#define UE_ALLOW_SHADER_COMPILING 1
#endif

#ifndef UE_ALLOW_SHADER_COMPILING_BASED_ON_SHADER_DIRECTORY_EXISTENCE
// should ability to compile shaders be based on the presence of Engine/Shaders directory?
#define UE_ALLOW_SHADER_COMPILING_BASED_ON_SHADER_DIRECTORY_EXISTENCE 0
#endif

bool AllowShaderCompiling()
{
#if UE_ALLOW_SHADER_COMPILING_BASED_ON_SHADER_DIRECTORY_EXISTENCE
	static bool bShaderDirectoryExists = FPaths::DirectoryExists(FPaths::Combine(FPaths::EngineDir(), TEXT("Shaders"), TEXT("Public")));
	// if it doesn't exist, dont allow compiling. otherwise, check the other flags to see if those have disabled it
	if (!bShaderDirectoryExists)
	{
		return false;
	}
#endif

	static const bool bNoShaderCompile = FParse::Param(FCommandLine::Get(), TEXT("NoShaderCompile")) ||
		FParse::Param(FCommandLine::Get(), TEXT("PrecompiledShadersOnly"));

	return UE_ALLOW_SHADER_COMPILING && !bNoShaderCompile;
}

// note that when UE_ALLOW_SHADER_COMPILING is false, we still need to load the global shaders, so that is the difference in these two functions
bool AllowGlobalShaderLoad()
{
	static const bool bNoShaderCompile = FParse::Param(FCommandLine::Get(), TEXT("NoShaderCompile"));

	// Commandlets and dedicated servers don't load global shaders (the cook commandlet will load for the necessary target platform(s) later).
	return !bNoShaderCompile && !IsRunningDedicatedServer() && (!IsRunningCommandlet() || IsAllowCommandletRendering());

}

TOptional<FParameterAllocation> FShaderParameterMap::FindParameterAllocation(FStringView ParameterName) const
{
	if (const FParameterAllocation* Allocation = ParameterMap.FindByHash(GetTypeHash(ParameterName), ParameterName))
	{
		if (Allocation->bBound)
		{
			// Can detect copy-paste errors in binding parameters.  Need to fix all the false positives before enabling.
			//UE_LOGF(LogShaders, Warning, "Parameter %ls was bound multiple times. Code error?", ParameterName);
		}

		Allocation->bBound = true;

		return TOptional<FParameterAllocation>(*Allocation);
	}

	return TOptional<FParameterAllocation>();
}

TOptional<FParameterAllocation> FShaderParameterMap::FindAndRemoveParameterAllocation(FStringView ParameterName)
{
	FParameterAllocation Result;
	if (ParameterMap.RemoveAndCopyValueByHash(GetTypeHash(ParameterName), ParameterName, Result))
	{
		return TOptional<FParameterAllocation>(Result);
	}

	return TOptional<FParameterAllocation>();
}

bool FShaderParameterMap::FindParameterAllocation(FStringView ParameterName, uint16& OutBufferIndex, uint16& OutBaseIndex, uint16& OutSize) const
{
	if (TOptional<FParameterAllocation> Allocation = FindParameterAllocation(ParameterName))
	{
		OutBufferIndex = Allocation->BufferIndex;
		OutBaseIndex = Allocation->BaseIndex;
		OutSize = Allocation->Size;

		return true;
	}

	return false;
}

bool FShaderParameterMap::ContainsParameterAllocation(FStringView ParameterName) const
{
	return ParameterMap.FindByHash(GetTypeHash(ParameterName), ParameterName) != nullptr;
}

void FShaderParameterMap::AddParameterAllocation(FStringView ParameterName,uint16 BufferIndex,uint16 BaseIndex,uint16 Size,EShaderParameterType ParameterType)
{
	check(ParameterType < EShaderParameterType::Num);
	ParameterMap.Emplace(ParameterName, FParameterAllocation(BufferIndex, BaseIndex, Size, ParameterType));
}

void FShaderParameterMap::RemoveParameterAllocation(FStringView ParameterName)
{
	ParameterMap.RemoveByHash(GetTypeHash(ParameterName), ParameterName);
}

TArray<FStringView> FShaderParameterMap::GetAllParameterNamesOfType(EShaderParameterType InType) const
{
	TArray<FStringView> Result;
	for (const TMap<FString, FParameterAllocation>::ElementType& Parameter : ParameterMap)
	{
		if (Parameter.Value.Type == InType)
		{
			Result.Emplace(Parameter.Key);
		}
	}
	return Result;
}

uint32 FShaderParameterMap::CountParametersOfType(EShaderParameterType InType) const
{
	uint32 Result = 0;
	for (const TMap<FString, FParameterAllocation>::ElementType& Parameter : ParameterMap)
	{
		if (Parameter.Value.Type == InType)
		{
			Result++;
		}
	}
	return Result;
}

#if WITH_EDITOR

void FShaderBindingLayout::SetUniformBufferDeclarationAnsiPtr(const FShaderParametersMetadata* ShaderParametersMetadata, FThreadSafeSharedAnsiStringPtr UniformBufferDeclarationAnsi)
{
	FString UniformBufferName(ShaderParametersMetadata->GetShaderVariableName());
	check(!UniformBufferMap.Contains(UniformBufferName));
	UniformBufferMap.Add(UniformBufferName, UniformBufferDeclarationAnsi);
}

void FShaderBindingLayout::AddRequiredSymbols(TArray<FString>& RequiredSymbols) const
{
	// assume only bindless for now so only need to add the CBuffer declares as required symbols
	check(EnumHasAllFlags(RHILayout.GetFlags(), EShaderBindingLayoutFlags::BindlessResources | EShaderBindingLayoutFlags::BindlessSamplers));

	// Don't remove unused uniform buffers defined in the fixed shader binding layout because they are required to be declared for certain platforms
	for (auto Iter = UniformBufferMap.CreateConstIterator(); Iter; ++Iter)
	{
		RequiredSymbols.Add(Iter.Key());
	}
}

#endif // WITH_EDITOR

void FShaderResourceTableMap::Append(const FShaderResourceTableMap& Other)
{
	// Get the set of uniform buffers used by the target resource table map
	TSet<FString> UniformBufferNames;
	FStringView PreviousUniformBufferName;

	for (const FUniformResourceEntry& Resource : Resources)
	{
		// Cheaper to check if consecutive array elements are from the same uniform buffer (which is common) before adding to set,
		// which involves a more expensive hash lookup versus a string comparison.
		if (!PreviousUniformBufferName.Equals(Resource.GetUniformBufferName(), ESearchCase::CaseSensitive))
		{
			PreviousUniformBufferName = Resource.GetUniformBufferName();
			UniformBufferNames.Add(FString(PreviousUniformBufferName));
		}
	}

	// Then add any entries from "Other" that aren't from a uniform buffer we already include.
	PreviousUniformBufferName = FStringView();
	bool PreviousUniformBufferFound = false;
	for (const FUniformResourceEntry& OtherResource : Other.Resources)
	{
		if (!PreviousUniformBufferName.Equals(OtherResource.GetUniformBufferName(), ESearchCase::CaseSensitive))
		{
			PreviousUniformBufferName = OtherResource.GetUniformBufferName();
			PreviousUniformBufferFound = UniformBufferNames.Find(FString(PreviousUniformBufferName)) != nullptr;
		}

		if (!PreviousUniformBufferFound)
		{
			Resources.Add(OtherResource);
		}
	}
}

void FShaderResourceTableMap::FixupOnLoad(const TMap<FString, FUniformBufferEntry>& UniformBufferMap)
{
	// Need to fix up UniformBufferMemberName string pointers to point into the MemberNameBuffer storage in UniformBufferMap
	uint16 ResourceIndex = 0;
	for (const auto& Pair : UniformBufferMap)
	{
		const TArray<TCHAR>* MemberNameBuffer = Pair.Value.MemberNameBuffer.Get();
		if (MemberNameBuffer && MemberNameBuffer->Num())
		{
			const TCHAR* MemberNameCurrent = MemberNameBuffer->GetData();
			const TCHAR* MemberNameEnd = MemberNameCurrent + MemberNameBuffer->Num();

			for (; MemberNameCurrent < MemberNameEnd; MemberNameCurrent += FCString::Strlen(MemberNameCurrent) + 1)
			{
				Resources[ResourceIndex++].UniformBufferMemberName = MemberNameCurrent;
			}
		}
	}
}

FShaderCompilerEnvironment::FShaderCompilerEnvironment()
{
	// Enable initial defines in FShaderCompilerEnvironment to improve performance (helpful here, but not for defines declared in various shader compiler backends).
	const bool bIncludeInitialDefines = true;

	Definitions = MakePimpl<FShaderCompilerDefinitions, EPimplPtrMode::DeepCopy>(bIncludeInitialDefines);

	// Presize to reduce re-hashing while building shader jobs
	IncludeVirtualPathToContentsMap.Empty(15);
}

FShaderCompilerEnvironment::FShaderCompilerEnvironment(FMemoryHasherBlake3& Hasher) : Hasher(&Hasher)
{
}

void FShaderCompilerEnvironment::Merge(const FShaderCompilerEnvironment& Other)
{
	// Merge the include maps
	// Merge the values of any existing keys
	for (TMap<FString, FString>::TConstIterator It(Other.IncludeVirtualPathToContentsMap); It; ++It)
	{
		FString* ExistingContents = IncludeVirtualPathToContentsMap.Find(It.Key());

		if (ExistingContents)
		{
			ExistingContents->Append(It.Value());
		}
		else
		{
			IncludeVirtualPathToContentsMap.Add(It.Key(), It.Value());
		}
	}

	check(Other.IncludeVirtualPathToSharedContentsMap.Num() == 0);

	CompilerFlags.Append(Other.CompilerFlags);
	ResourceTableMap.Append(Other.ResourceTableMap);
	{
		// Append, but don't overwrite the value of existing elements, to preserve MemberNameBuffer which is pointed to by ResourceTableMap entries
		UniformBufferMap.Reserve(UniformBufferMap.Num() + Other.UniformBufferMap.Num());
		for (auto& Pair : Other.UniformBufferMap)
		{
			uint32 KeyHash = GetTypeHash(Pair.Key);
			if (!UniformBufferMap.ContainsByHash(KeyHash, Pair.Key))
			{
				UniformBufferMap.AddByHash(KeyHash, Pair.Key, Pair.Value);
			}
		}
	}
	checkf(Definitions.IsValid(), TEXT("Merge is not supported on FShaderCompilerEnvironment in hashing mode"));
	Definitions->Merge(*Other.Definitions);
	CompileArgs.Append(Other.CompileArgs);
	RenderTargetOutputFormatsMap.Append(Other.RenderTargetOutputFormatsMap);
	FullPrecisionInPS |= Other.FullPrecisionInPS;
}

FString FShaderCompilerEnvironment::GetDefinitionsAsCommentedCode() const
{
	checkf(Definitions.IsValid(), TEXT("GetDefinitionsAsCommentedCode is not supported on FShaderCompilerEnvironment in hashing mode"));
	TArray<FString> DefinesLines;
	DefinesLines.Reserve(Definitions->Num());
	for (FShaderCompilerDefinitions::FConstIterator DefineIt(*Definitions); DefineIt; ++DefineIt)
	{
		DefinesLines.Add(FString::Printf(TEXT("// #define %s %s\n"), DefineIt.Key(), DefineIt.Value()));
	}
	DefinesLines.Sort();

	FString Defines;
	for (const FString& DefineLine : DefinesLines)
	{
		Defines += DefineLine;
	}

	return MakeInjectedShaderCodeBlock(TEXT("DumpShaderDefinesAsCommentedCode"), Defines);
}

// Pass through functions to definitions
void FShaderCompilerEnvironment::SetDefine(const TCHAR* Name, const TCHAR* Value)
{
	if (Definitions.IsValid())
	{
		Definitions->SetDefine(Name, Value);
	}
	else
	{
		Hasher->Serialize(const_cast<TCHAR*>(Name), FCString::Strlen(Name));
		Hasher->Serialize(const_cast<TCHAR*>(Value), FCString::Strlen(Value));
	}
}

void FShaderCompilerEnvironment::SetDefine(const TCHAR* Name, const FString& Value)
{ 
	if (Definitions.IsValid())
	{
		Definitions->SetDefine(Name, Value);
	}
	else
	{
		Hasher->Serialize(const_cast<TCHAR*>(Name), FCString::Strlen(Name));
		*Hasher << const_cast<FString&>(Value);
	}
}

void FShaderCompilerEnvironment::SetDefine(const TCHAR* Name, uint32 Value)
{
	if (Definitions.IsValid())
	{
		Definitions->SetDefine(Name, Value);
	}
	else
	{
		Hasher->Serialize(const_cast<TCHAR*>(Name), FCString::Strlen(Name));
		*Hasher << Value;
	}
}
void FShaderCompilerEnvironment::SetDefine(const TCHAR* Name, int32 Value)
{
	if (Definitions.IsValid())
	{
		Definitions->SetDefine(Name, Value);
	}
	else
	{
		Hasher->Serialize(const_cast<TCHAR*>(Name), FCString::Strlen(Name));
		*Hasher << Value;
	}
}
void FShaderCompilerEnvironment::SetDefine(const TCHAR* Name, bool Value)
{
	if (Definitions.IsValid())
	{
		Definitions->SetDefine(Name, Value);
	}
	else
	{
		Hasher->Serialize(const_cast<TCHAR*>(Name), FCString::Strlen(Name));
		*Hasher << Value;
	}
}
void FShaderCompilerEnvironment::SetDefine(const TCHAR* Name, float Value)
{
	if (Definitions.IsValid())
	{
		Definitions->SetDefine(Name, Value);
	}
	else
	{
		Hasher->Serialize(const_cast<TCHAR*>(Name), FCString::Strlen(Name));
		*Hasher << Value;
	}
}

void FShaderCompilerEnvironment::SetDefine(FName Name, const TCHAR* Value)
{
	if (Definitions.IsValid())
	{
		Definitions->SetDefine(Name, Value);
	}
	else
	{
		SetDefine(*Name.ToString(), Value);
	}
}

void FShaderCompilerEnvironment::SetDefine(FName Name, const FString& Value)
{
	if (Definitions.IsValid())
	{
		Definitions->SetDefine(Name, Value);
	}
	else
	{
		SetDefine(*Name.ToString(), Value);
	}
}

void FShaderCompilerEnvironment::SetDefine(FName Name, uint32 Value)
{
	if (Definitions.IsValid())
	{
		Definitions->SetDefine(Name, Value);
	}
	else
	{
		SetDefine(*Name.ToString(), Value);
	}
}

void FShaderCompilerEnvironment::SetDefine(FName Name, int32 Value)
{
	if (Definitions.IsValid())
	{
		Definitions->SetDefine(Name, Value);
	}
	else
	{
		SetDefine(*Name.ToString(), Value);
	}
}

void FShaderCompilerEnvironment::SetDefine(FName Name, bool Value)
{
	if (Definitions.IsValid())
	{
		Definitions->SetDefine(Name, Value);
	}
	else
	{
		SetDefine(*Name.ToString(), Value);
	}
}

void FShaderCompilerEnvironment::SetDefine(FName Name, float Value)
{
	if (Definitions.IsValid())
	{
		Definitions->SetDefine(Name, Value);
	}
	else
	{
		SetDefine(*Name.ToString(), Value);
	}
}

void FShaderCompilerEnvironment::SetDefine(FShaderCompilerDefineNameCache& Name, const TCHAR* Value) 
{
	if (Definitions.IsValid())
	{
		Definitions->SetDefine(Name, Value);
	}
	else
	{
		SetDefine(*Name.ToString(), Value);
	}
}
void FShaderCompilerEnvironment::SetDefine(FShaderCompilerDefineNameCache& Name, const FString& Value)
{
	if (Definitions.IsValid())
	{
		Definitions->SetDefine(Name, Value);
	}
	else
	{
		SetDefine(*Name.ToString(), Value);
	}
}
void FShaderCompilerEnvironment::SetDefine(FShaderCompilerDefineNameCache& Name, uint32 Value)
{
	if (Definitions.IsValid())
	{
		Definitions->SetDefine(Name, Value);
	}
	else
	{
		SetDefine(*Name.ToString(), Value);
	}
}
void FShaderCompilerEnvironment::SetDefine(FShaderCompilerDefineNameCache& Name, int32 Value)
{
	if (Definitions.IsValid())
	{
		Definitions->SetDefine(Name, Value);
	}
	else
	{
		SetDefine(*Name.ToString(), Value);
	}
}
void FShaderCompilerEnvironment::SetDefine(FShaderCompilerDefineNameCache& Name, bool Value)
{
	if (Definitions.IsValid())
	{
		Definitions->SetDefine(Name, Value);
	}
	else
	{
		SetDefine(*Name.ToString(), Value);
	}
}
void FShaderCompilerEnvironment::SetDefine(FShaderCompilerDefineNameCache& Name, float Value)
{
	if (Definitions.IsValid())
	{
		Definitions->SetDefine(Name, Value);
	}
	else
	{
		SetDefine(*Name.ToString(), Value);
	}
}

bool FShaderCompilerEnvironment::ContainsDefinition(FName Name) const
{
	if (Definitions.IsValid())
	{
		return Definitions->Contains(Name);
	}

	// If we're in hashing mode only, always report "false" for contains definition.
	// This is only used by SetDefineIfUnset and as such will just have a minor impact of potential over-invalidation
	// from certain shader types which call the aforementioned function (i.e. they will re-set potentially already set
	// defines to a new value, generating a slightly different hash for the shader type). There are very few calls to this
	// at the time of writing and with per-shader DDC this will only serve to force reconstruction of the shadermap so it's 
	// not significant enough to worry about.
	return false;
}

/** This "core" serialization is also used for the hashing the compiler job (where files are handled differently). Should stay in sync with the ShaderCompileWorker. */
void FShaderCompilerEnvironment::SerializeEverythingButFiles(FArchive& Ar)
{
	// If we don't have a definitions object created then we're in hashing mode and the defines were already hashed on set.
	if (Definitions.IsValid())
	{
		Ar << *Definitions;
	}
	Ar << CompileArgs;
	Ar << CompilerFlags;
	Ar << RenderTargetOutputFormatsMap;
	Ar << ResourceTableMap.Resources;
	Ar << UniformBufferMap;
	Ar << RHIShaderBindingLayout;
	Ar << SpecializationConstantValues;
	Ar << FullPrecisionInPS;
	if (Ar.IsLoading())
	{
		ResourceTableMap.FixupOnLoad(UniformBufferMap);
	}
}

// Serializes the portions of the environment that are used as input to the backend compilation process (i.e. after all preprocessing)
void FShaderCompilerEnvironment::SerializeCompilationDependencies(FArchive& Ar)
{
	Ar << CompileArgs;
	Ar << CompilerFlags;
	Ar << RenderTargetOutputFormatsMap;
	Ar << ResourceTableMap.Resources;
	Ar << UniformBufferMap;
	Ar << RHIShaderBindingLayout;
	Ar << SpecializationConstantValues;
	Ar << FullPrecisionInPS;
	if (Ar.IsLoading())
	{
		ResourceTableMap.FixupOnLoad(UniformBufferMap);
	}
}

void FShaderCompilerOutput::GenerateOutputHash()
{
	FShaderHashBuilder HashState;

	TArrayView<const uint8> Code = ShaderCode.GetReadView();

	// we don't hash the optional attachments as they would prevent sharing (e.g. many materials share the same VS)
	uint32 ShaderCodeSize = ShaderCode.GetShaderCodeSize();

	// make sure we are not generating the hash on compressed data
	checkf(!ShaderCode.IsCompressed(), TEXT("Attempting to generate the output hash of a compressed code"));

	HashState.Update(Code.GetData(), ShaderCodeSize * Code.GetTypeSize());
	ParameterMap.UpdateHash(HashState);
	OutputHash = HashState.Finalize();
}

void FShaderCompilerOutput::CompressOutput(FName ShaderCompressionFormat, FOodleDataCompression::ECompressor OodleCompressor, FOodleDataCompression::ECompressionLevel OodleLevel)
{
	// make sure the hash has been generated
	checkf(OutputHash != FShaderHash(), TEXT("Output hash must be generated before compressing the shader code."));
	checkf(ShaderCompressionFormat != NAME_None, TEXT("Compression format should be valid"));
	ShaderCode.Compress(ShaderCompressionFormat, OodleCompressor, OodleLevel);
}

void FShaderCompilerOutput::SerializeShaderCodeValidation()
{
	if (ParametersStrideToValidate.Num() > 0 || ParametersSRVTypeToValidate.Num() > 0 ||
		ParametersUAVTypeToValidate.Num() > 0 || ParametersUBSizeToValidate.Num() > 0)
	{
		FShaderCodeValidationExtension ShaderCodeValidationExtension;

		ShaderCodeValidationExtension.ShaderCodeValidationStride.Append(ParametersStrideToValidate);
		ShaderCodeValidationExtension.ShaderCodeValidationStride.Sort([](const FShaderCodeValidationStride& lhs, const FShaderCodeValidationStride& rhs) -> bool { return lhs.BindPoint < rhs.BindPoint; });

		ShaderCodeValidationExtension.ShaderCodeValidationSRVType.Append(ParametersSRVTypeToValidate);
		ShaderCodeValidationExtension.ShaderCodeValidationSRVType.Sort([](const FShaderCodeValidationType& lhs, const FShaderCodeValidationType& rhs) -> bool { return lhs.BindPoint < rhs.BindPoint; });

		ShaderCodeValidationExtension.ShaderCodeValidationUAVType.Append(ParametersUAVTypeToValidate);
		ShaderCodeValidationExtension.ShaderCodeValidationUAVType.Sort([](const FShaderCodeValidationType& lhs, const FShaderCodeValidationType& rhs) -> bool { return lhs.BindPoint < rhs.BindPoint; });

		ShaderCodeValidationExtension.ShaderCodeValidationUBSize.Append(ParametersUBSizeToValidate);
		ShaderCodeValidationExtension.ShaderCodeValidationUBSize.Sort([](const FShaderCodeValidationUBSize& lhs, const FShaderCodeValidationUBSize& rhs) -> bool { return lhs.BindPoint < rhs.BindPoint; });

		TArray<uint8> WriterBytes;
		FMemoryWriter Writer(WriterBytes);
		Writer << ShaderCodeValidationExtension;

		ShaderCode.AddOptionalData(FShaderCodeValidationExtension::Key, WriterBytes.GetData(), WriterBytes.Num());
	}
}

void FShaderCompilerOutput::SerializeShaderDiagnosticData()
{
	if (ShaderDiagnosticDatas.Num() > 0)
	{
		FShaderDiagnosticExtension ShaderDiagnosticExtension;
		ShaderDiagnosticExtension.ShaderDiagnosticDatas = ShaderDiagnosticDatas;

		TArray<uint8> WriterBytes;
		FMemoryWriter Writer(WriterBytes);
		Writer << ShaderDiagnosticExtension;
		ShaderCode.AddOptionalData(FShaderDiagnosticExtension::Key, WriterBytes.GetData(), WriterBytes.Num());
	}
}

static void ReportVirtualShaderFilePathError(TArray<FShaderCompilerError>* CompileErrors, FString ErrorString)
{
	if (CompileErrors)
	{
		CompileErrors->Add(FShaderCompilerError(MoveTemp(ErrorString)));
	}
	else
	{
		UE_LOGF(LogShaders, Error, "%ls", *ErrorString);
	}
}


static bool Contains(FStringView View, FStringView Search)
{
	return UE::String::FindFirst(View, Search) != INDEX_NONE;
}

bool CheckVirtualShaderFilePath(FStringView VirtualFilePath, TArray<FShaderCompilerError>* CompileErrors /*= nullptr*/)
{
	bool bSuccess = true;

	if (!VirtualFilePath.StartsWith('/'))
	{
		FString Error = FString::Printf(TEXT("Virtual shader source file name \"%s\" should be absolute from the virtual root directory \"/\"."), *FString(VirtualFilePath));
		ReportVirtualShaderFilePathError(CompileErrors, Error);
		bSuccess = false;
	}

	if (Contains(VirtualFilePath, TEXTVIEW("..")))
	{
		FString Error = FString::Printf(TEXT("Virtual shader source file name \"%s\" should have relative directories (\"../\") collapsed."), *FString(VirtualFilePath));
		ReportVirtualShaderFilePathError(CompileErrors, Error);
		bSuccess = false;
	}

	if (Contains(VirtualFilePath, TEXTVIEW("\\")))
	{
		FString Error = FString::Printf(TEXT("Backslashes are not permitted in virtual shader source file name \"%s\""), *FString(VirtualFilePath));
		ReportVirtualShaderFilePathError(CompileErrors, Error);
		bSuccess = false;
	}

	const FStringView Extension = FPathViews::GetExtension(VirtualFilePath);
	const bool bIsSharedDirectory = GShaderSourceSharedVirtualDirectories.ContainsByPredicate([&VirtualFilePath](const FString& SharedDirectory) { return VirtualFilePath.StartsWith(SharedDirectory); });
	if (bIsSharedDirectory)
	{
		if ((Extension != TEXTVIEW("h")))
		{
			FString Error = FString::Printf(TEXT("Extension on virtual shader source file name \"%s\" is wrong. Only .h is allowed for shared headers that are shared between C++ and shader code."), *FString(VirtualFilePath));
			ReportVirtualShaderFilePathError(CompileErrors, Error);
			bSuccess = false;
		}	
	}
	else if (VirtualFilePath.StartsWith(TEXT("/ThirdParty/")))
	{
		// Third party includes don't have naming convention restrictions
	}
	else
	{
		if ((Extension != TEXT("usf") && Extension != TEXT("ush")) || VirtualFilePath.EndsWith(TEXT(".usf.usf")))
		{
			FString Error = FString::Printf(TEXT("Extension on virtual shader source file name \"%s\" is wrong. Only .usf or .ush allowed."), *FString(VirtualFilePath));
			ReportVirtualShaderFilePathError(CompileErrors, Error);
			bSuccess = false;
		}
	}

	return bSuccess;
}

const IShaderFormat* FindShaderFormat(FName Format, const TArray<const IShaderFormat*>& ShaderFormats)
{
	for (int32 Index = 0; Index < ShaderFormats.Num(); Index++)
	{
		TArray<FName> Formats;

		ShaderFormats[Index]->GetSupportedFormats(Formats);

		for (int32 FormatIndex = 0; FormatIndex < Formats.Num(); FormatIndex++)
		{
			if (Formats[FormatIndex] == Format)
			{
				return ShaderFormats[Index];
			}
		}
	}

	return nullptr;
}

#if PLATFORM_WINDOWS
static bool ExceptionCodeToString(DWORD ExceptionCode, FString& OutStr)
{
#define EXCEPTION_CODE_CASE_STR(CODE) case CODE: OutStr = TEXT(#CODE); return true;
	const DWORD CPlusPlusExceptionCode = 0xE06D7363;
	switch (ExceptionCode)
	{
		EXCEPTION_CODE_CASE_STR(EXCEPTION_ACCESS_VIOLATION);
		EXCEPTION_CODE_CASE_STR(EXCEPTION_ARRAY_BOUNDS_EXCEEDED);
		EXCEPTION_CODE_CASE_STR(EXCEPTION_BREAKPOINT);
		EXCEPTION_CODE_CASE_STR(EXCEPTION_DATATYPE_MISALIGNMENT);
		EXCEPTION_CODE_CASE_STR(EXCEPTION_FLT_DENORMAL_OPERAND);
		EXCEPTION_CODE_CASE_STR(EXCEPTION_FLT_DIVIDE_BY_ZERO);
		EXCEPTION_CODE_CASE_STR(EXCEPTION_FLT_INEXACT_RESULT);
		EXCEPTION_CODE_CASE_STR(EXCEPTION_FLT_INVALID_OPERATION);
		EXCEPTION_CODE_CASE_STR(EXCEPTION_FLT_OVERFLOW);
		EXCEPTION_CODE_CASE_STR(EXCEPTION_FLT_STACK_CHECK);
		EXCEPTION_CODE_CASE_STR(EXCEPTION_FLT_UNDERFLOW);
		EXCEPTION_CODE_CASE_STR(EXCEPTION_GUARD_PAGE);
		EXCEPTION_CODE_CASE_STR(EXCEPTION_ILLEGAL_INSTRUCTION);
		EXCEPTION_CODE_CASE_STR(EXCEPTION_IN_PAGE_ERROR);
		EXCEPTION_CODE_CASE_STR(EXCEPTION_INT_DIVIDE_BY_ZERO);
		EXCEPTION_CODE_CASE_STR(EXCEPTION_INT_OVERFLOW);
		EXCEPTION_CODE_CASE_STR(EXCEPTION_INVALID_DISPOSITION);
		EXCEPTION_CODE_CASE_STR(EXCEPTION_INVALID_HANDLE);
		EXCEPTION_CODE_CASE_STR(EXCEPTION_NONCONTINUABLE_EXCEPTION);
		EXCEPTION_CODE_CASE_STR(EXCEPTION_PRIV_INSTRUCTION);
		EXCEPTION_CODE_CASE_STR(EXCEPTION_SINGLE_STEP);
		EXCEPTION_CODE_CASE_STR(EXCEPTION_STACK_OVERFLOW);
		EXCEPTION_CODE_CASE_STR(STATUS_UNWIND_CONSOLIDATE);
		case CPlusPlusExceptionCode: OutStr = TEXT("CPP_EXCEPTION"); return true;
		default: return false;
	}
#undef EXCEPTION_CODE_CASE_STR
}

int HandleShaderCompileException(Windows::LPEXCEPTION_POINTERS Info, FString& OutExMsg, FString& OutCallStack)
{
	const DWORD AssertExceptionCode = 0x00004000;
	FString ExCodeStr;
	OutCallStack = "";
	if (Info->ExceptionRecord->ExceptionCode == AssertExceptionCode)
	{
		// In the case of an assert the assert handler populates the GErrorHist global.
		// This contains a readable assert message that may be followed by a callstack; so we can use that to populate
		// our message/callstack and save some time as well as getting the properly formatted assert message.
		const TCHAR* CallstackStart = FCString::Strfind(GErrorHist, TEXT("0x"));
		if (CallstackStart && CallstackStart > GErrorHist)
		{
			OutExMsg = FString::ConstructFromPtrSize(GErrorHist, CallstackStart - GErrorHist);
			OutCallStack = CallstackStart;
		}
		else
		{
			OutExMsg = GErrorHist;
		}
	}
	else
	{
		if (ExceptionCodeToString(Info->ExceptionRecord->ExceptionCode, ExCodeStr))
		{
			OutExMsg = FString::Printf(
				TEXT("Exception: %s, address=0x%016" UINT64_x_FMT "\n"),
				*ExCodeStr,
				(uint64)Info->ExceptionRecord->ExceptionAddress);
		}
		else
		{
			OutExMsg = FString::Printf(
				TEXT("Exception code: 0x%08x, address=0x%016" UINT64_x_FMT "\n"),
				(uint32)Info->ExceptionRecord->ExceptionCode,
				(uint64)Info->ExceptionRecord->ExceptionAddress);
		}
	}

	if (OutCallStack.Len() == 0)
	{
		ANSICHAR CallStack[32768];
		FMemory::Memzero(CallStack);
		FPlatformStackWalk::StackWalkAndDump(CallStack, ARRAYSIZE(CallStack), Info->ExceptionRecord->ExceptionAddress);
		OutCallStack = ANSI_TO_TCHAR(CallStack);
	}

	return EXCEPTION_EXECUTE_HANDLER;
}
#endif


class FInternalShaderCompilerFunctions
{
public:

	static void PreprocessShaderInternal(
		const IShaderFormat* Backend, 
		FShaderCompileJob& Job, 
		const FShaderCompilerEnvironment& Environment)
	{
		if (!GBreakOnPreprocessJob.IsEmpty())
		{
			if (Job.Input.DebugGroupName.Contains(GBreakOnPreprocessJob, ESearchCase::IgnoreCase))
			{
				UE_DEBUG_BREAK();
			}
		}
		double PreprocessStartTime = FPlatformTime::Seconds();
		Job.PreprocessOutput.bSucceeded = Backend->PreprocessShader(Job.Input, Environment, Job.PreprocessOutput);
		Job.PreprocessOutput.PreprocessTime = FPlatformTime::Seconds() - PreprocessStartTime;

		if (Job.PreprocessOutput.bSucceeded && Backend->RequiresSecondaryCompile(Job.Input, Environment, Job.PreprocessOutput))
		{
			PreprocessStartTime = FPlatformTime::Seconds();
			Job.SecondaryPreprocessOutput = MakeUnique<FShaderPreprocessOutput>();
			Job.SecondaryPreprocessOutput->bIsSecondary = true;
			Job.SecondaryPreprocessOutput->bSucceeded = Backend->PreprocessShader(Job.Input, Environment, *Job.SecondaryPreprocessOutput);
			Job.SecondaryPreprocessOutput->PreprocessTime = FPlatformTime::Seconds() - PreprocessStartTime;

			// Mark the main job preprocess output as failed if the secondary preprocess fails, even if the primary succeeds
			// This means the job submission code only needs to check one flag to determine whether to actually submit the job 
			// (no point in trying to compile if either of the two preprocess steps fail)
			Job.PreprocessOutput.bSucceeded &= Job.SecondaryPreprocessOutput->bSucceeded;
		}
	}

	static bool PreprocessShaderInternal(const IShaderFormat* Backend, FShaderCompileJob& Job)
	{
		if (IsValidRef(Job.Input.SharedEnvironment))
		{
			// only create new environment & merge if necessary, save some allocs
			// (need a copy here as we don't want to merge the environment in place like we do in the compile path
			// and affect what is passed to the workers)
			FShaderCompilerEnvironment MergedEnvironment = Job.Input.Environment;
			MergedEnvironment.Merge(*Job.Input.SharedEnvironment);
			PreprocessShaderInternal(Backend, Job, MergedEnvironment);
		}
		else
		{
			PreprocessShaderInternal(Backend, Job, Job.Input.Environment);
		}

		bool bSucceeded = true;

		auto PostPreprocessFunc = [&Job, &bSucceeded](FShaderPreprocessOutput& PreprocessOutput, FStringView DiagPrefix = FStringView())
			{
				if (PreprocessOutput.bSucceeded)
				{
					if (!Job.Input.Environment.CompilerFlags.Contains(CFLAG_DisableSourceStripping))
					{
						// if the preprocessed job cache is enabled we strip the preprocessed code if not explicitly disabled; this removes comments, 
						// line directives and blank lines to improve deduplication (and populates data required to remap diagnostic messages to correct
						// filenames and line numbers)
						PreprocessOutput.StripCode(Job.Input.NeedsOriginalShaderSource());
						FShaderCompilerInputHash Hash = Job.GetInputHash();
						// Replace the placeholder debug hash value appended in StripCode with the real job input hash
						FShaderSource::FViewType DebugHashStr = GetShaderSourceDebugHashPrefix();
						FShaderSource::FViewType SourceView = PreprocessOutput.GetSourceView();
						int32 DebugHashLoc = SourceView.Find(DebugHashStr) + DebugHashStr.Len();
						int32 NewlineLoc = SourceView.Find(ANSITEXTVIEW("\n"), DebugHashLoc);
						FShaderSource::TStringBuilder<2 * sizeof(FShaderCompilerInputHash::ByteArray) + 1> HashStr;
						HashStr << Hash;
						// handle case where we didn't add the debug hash association (this can be disabled by a cvar for debugging purposes)
						if (NewlineLoc - DebugHashLoc == HashStr.Len())
						{
							FMemory::Memcpy(PreprocessOutput.EditSource().GetData() + DebugHashLoc, HashStr.GetData(), sizeof(FShaderSource::CharType) * HashStr.Len());
						}
					}
					// always compress the code after stripping to minimize memory footprint
					PreprocessOutput.CompressCode();
				}
				
				// append a prefix to secondary job error messages to distinguish them from the primary compilation
				if (!DiagPrefix.IsEmpty())
				{
					for (FShaderCompilerError& Error : PreprocessOutput.Errors)
					{
						Error.StrippedErrorMessage = DiagPrefix + Error.StrippedErrorMessage;
					}
				}
				Job.Output.Errors.Append(PreprocessOutput.Errors);
				bSucceeded &= PreprocessOutput.bSucceeded;
			};

		PostPreprocessFunc(Job.PreprocessOutput);
		if (Job.SecondaryPreprocessOutput.IsValid())
		{
			TStringBuilder<128> DiagPrefix;
			DiagPrefix << "(" << Backend->GetSecondaryCompileName() << ") ";
			PostPreprocessFunc(*Job.SecondaryPreprocessOutput, DiagPrefix);
		}

		return bSucceeded;
	}

	static bool CombineCodeOutputs(const IShaderFormat* Compiler, FShaderCompileJob& Job)
	{
		check(Job.SecondaryOutput.IsValid());

		// Pack shader code results together
		// [int32 key][uint32 primary length][uint32 secondary length][full primary shader code][full secondary shader code]
		TArray<uint8> CombinedSource;

		int32 PackedShaderKey = Compiler->GetPackedShaderKey();
		CombinedSource.Append(reinterpret_cast<const uint8*>(&PackedShaderKey), sizeof(PackedShaderKey));

		TArrayView<const uint8> PrimaryCode = Job.Output.ShaderCode.GetReadView();
		const uint32 PrimaryLength = PrimaryCode.NumBytes();
		TArrayView<const uint8> SecondaryCode = Job.SecondaryOutput->ShaderCode.GetReadView();
		const uint32 SecondaryLength = SecondaryCode.NumBytes();

		CombinedSource.Reserve(PrimaryLength + sizeof(PrimaryLength) + SecondaryLength + sizeof(SecondaryLength));
		CombinedSource.Append(reinterpret_cast<const uint8*>(&PrimaryLength), sizeof(PrimaryLength));
		CombinedSource.Append(reinterpret_cast<const uint8*>(&SecondaryLength), sizeof(SecondaryLength));
		CombinedSource.Append(PrimaryCode);
		CombinedSource.Append(SecondaryCode);

		// this could be cleaner, but we currently require backends with secondary outputs to manually merge their symbols during the compilation process
		// as such we need to save off the symbol buffer before we reset the shader code struct below, then re-set the buffer when finalizing the new combined code
		FCompressedBuffer SymbolsBuffer = Job.Output.GetFinalizedCodeResource().GetSymbolsBuffer();
		// Replace Output shader code with the combined result; need to reset first since the code struct is in the finalized state and write access is disallowed
		Job.Output.ShaderCode = {};
		Job.Output.ShaderCode.GetWriteAccess() = MoveTemp(CombinedSource);
		Job.Output.ShaderCode.FinalizeShaderCode(SymbolsBuffer);

		// note: we intentionally keep the secondary output object around so duplicate/cache hit jobs can also dump debug info. it's a bit of memory overhead but preferable
		// to requiring that enabling/disabling debug information mutates the DDC key (which would be necessary if we cleared it, since we output separate debug information 
		// for the secondary jobs).

		// validate that the UsedAttributes array, if populated, matches for both jobs. this is necessary since it's used for interpolator removal on later stages, and such
		// stages may not have a secondary compilation (and so would need to have matching attributes to be run in a pipeline with two different versions of preceding stages)
		// if we end up in cases where this check is being hit and it's not resolvable by fixing the code to ensure the same attributes are used in both primary and secondary compilation,
		// then we could potentially merge these here instead, but that _seems_ unlikely to be the case in practice.
		if (Job.Output.UsedAttributes != Job.SecondaryOutput->UsedAttributes)
		{
			Job.Output.AddErrorf(TEXT("Mismatch in UsedAttributes between primary and secondary compilation; these must match since later stages may not have a secondary compile and so require both versions of the earlier stage to output the same set of attributes."));
			return false;
		}
		return true;
	}

	static void InvokeCompile(const IShaderFormat* Compiler, const FShaderCompilerInput& CompileInput, FShaderCompileJob& Job, FString& OutExceptionMsg, FString& OutExceptionCallstack)
	{
#if PLATFORM_WINDOWS
		__try
#endif
		{
			// decompress if necessary; this is a no-op if source is not compressed.
			Job.PreprocessOutput.DecompressCode();

			if (Job.SecondaryOutput.IsValid())
			{
				check(Job.SecondaryPreprocessOutput.IsValid());
				Job.SecondaryPreprocessOutput->DecompressCode();
				Compiler->CompilePreprocessedShader(CompileInput, Job.PreprocessOutput, *Job.SecondaryPreprocessOutput, Job.Output, *Job.SecondaryOutput);
			}
			else
			{
				Compiler->CompilePreprocessedShader(CompileInput, Job.PreprocessOutput, Job.Output);
			}
		}
#if PLATFORM_WINDOWS
		__except(HandleShaderCompileException(GetExceptionInformation(), OutExceptionMsg, OutExceptionCallstack))
		{
			Job.Output.bSucceeded = false;
		}
#endif
	}

	static void FinalizeShaderCompileJob(FShaderCompilerOutput& Output, const FShaderCompilerInput& Input)
	{
		Output.Target = Input.Target;
		// also set the job input hash on the output struct for validation purposes
		Output.ValidateInputHash = Input.Hash;
		Output.ShaderCode.FinalizeShaderCode();
	}

	static void CompileShaderInternal(const IShaderFormat* Compiler, FShaderCompileJob& Job, FString& OutExceptionMsg, FString& OutExceptionCallstack, int32* CompileCount)
	{
		double TimeStart = FPlatformTime::Seconds();

		// the only intended case where we can get to the compile step on a job which failed preprocessing is 
		// when running the dumped debug compile for such a job. this is intended to allow inspection of the
		// contents of the FShaderCompileInput (and by extension FShaderCompilerEnvironment(s)) for the job;
		// so having a check here is a convenience to break the debugger at a point where these things can
		// easily be inspected.
		check(Job.PreprocessOutput.GetSucceeded());

		Job.Output.Errors.Append(Job.PreprocessOutput.Errors);
		
		if (Job.SecondaryPreprocessOutput.IsValid())
		{
			Job.SecondaryOutput = MakeUnique<FShaderCompilerOutput>();
		}

		// Pass a locally-merged input to the platform compiler instead of mutating Job.Input;
		// CompileShaderInternal can be re-entered (pipeline interpolator-removal retry, SCW fallback).
		const FShaderCompilerInput* CompileInput = &Job.Input;
		FShaderCompilerInput MergedInput;
		if (IsValidRef(Job.Input.SharedEnvironment))
		{
			MergedInput = Job.Input;
			MergedInput.Environment.Merge(*Job.Input.SharedEnvironment);
			CompileInput = &MergedInput;
		}

		InvokeCompile(Compiler, *CompileInput, Job, OutExceptionMsg, OutExceptionCallstack);
		if (Job.SecondaryOutput.IsValid())
		{
			Job.Output.bSucceeded = Job.Output.bSucceeded && Job.SecondaryOutput->bSucceeded;
			// ensure the target field is set on the job output struct as we use it for validation during serialization
			FinalizeShaderCompileJob(*Job.SecondaryOutput, Job.Input);
			if (Job.SecondaryOutput->bSucceeded)
			{
				Job.SecondaryOutput->GenerateOutputHash();
			}
			Job.Output.bSucceeded &= CombineCodeOutputs(Compiler, Job);
		}

		FinalizeShaderCompileJob(Job.Output, Job.Input);

		// clear out the modified source/entry point fields if they aren't needed to avoid the data transfer/ddc overhead
		if (!Job.Input.NeedsOriginalShaderSource())
		{
			Job.Output.ModifiedShaderSource.Empty();
			Job.Output.ModifiedEntryPointName.Empty();
		}

		if (Job.Output.bSucceeded)
		{
			Job.Output.GenerateOutputHash();

			if (Job.Input.CompressionFormat != NAME_None)
			{
				Job.Output.CompressOutput(Job.Input.CompressionFormat, Job.Input.OodleCompressor, Job.Input.OodleLevel);
			}
		}
		Job.Output.CompileTime = FPlatformTime::Seconds() - TimeStart;

		if (CompileCount)
		{
			++(*CompileCount);
		}
	}
};

bool ConditionalPreprocessShader(FShaderCommonCompileJob* Job)
{
	return PreprocessShader(Job);
}

bool PreprocessShader(FShaderCommonCompileJob* Job)
{
	static ITargetPlatformManagerModule& TargetPlatformManager = GetTargetPlatformManagerRef();
	if (FShaderCompileJob* SingleJob = Job->GetSingleShaderJob())
	{
		const IShaderFormat* ShaderFormat = TargetPlatformManager.FindShaderFormat(SingleJob->Input.ShaderFormat);
		return FInternalShaderCompilerFunctions::PreprocessShaderInternal(ShaderFormat, *SingleJob);
	}
	else if (FShaderPipelineCompileJob* PipelineJob = Job->GetShaderPipelineJob())
	{
		bool bAnyFailed = false;
		for (FShaderCompileJob* StageJob : PipelineJob->StageJobs)
		{
			const IShaderFormat* ShaderFormat = TargetPlatformManager.FindShaderFormat(StageJob->Input.ShaderFormat);

			if (!bAnyFailed)
			{
				bAnyFailed |= !FInternalShaderCompilerFunctions::PreprocessShaderInternal(ShaderFormat, *StageJob);
			}
			else
			{
				// skip subsequent stage preprocessing if a prior stage failed to avoid unnecessary work, but log an error to indicate this
				StageJob->Output.AddErrorf(
					TEXT("Preprocessing %s stage skipped due to earlier stage preprocessing failure."),
					GetShaderFrequencyString(StageJob->Input.Target.GetFrequency())
				);
			}
		}
		return !bAnyFailed;
	}

	checkf(0, TEXT("Unknown shader compile job type or bad job pointer"));
	return false;
}

static void CompileShader(const TArray<const IShaderFormat*>& ShaderFormats, FShaderCompilerInput& Input, FShaderCompilerOutput& Output, int32* CompileCount)
{
	FShaderCompileJob TempJob;
	TempJob.Input = Input;
	
	CompileShader(ShaderFormats, TempJob, CompileCount);
	Output = TempJob.Output;
}

void CompileShader(const TArray<const IShaderFormat*>& ShaderFormats, FShaderCompileJob& Job, int32* CompileCount)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(CompileShader);

	const IShaderFormat* Compiler = FindShaderFormat(Job.Input.ShaderFormat, ShaderFormats);
	if (!Compiler)
	{
		UE_LOGF(LogShaders, Fatal, "Can't compile shaders for format %ls, couldn't load compiler dll", *Job.Input.ShaderFormat.ToString());
	}

	FString ExceptionMsg;
	FString ExceptionCallstack;
	FInternalShaderCompilerFunctions::CompileShaderInternal(Compiler, Job, ExceptionMsg, ExceptionCallstack, CompileCount);
	if (!Job.Output.bSucceeded && (!ExceptionMsg.IsEmpty() || !ExceptionCallstack.IsEmpty()))
	{
		Job.Output.AddErrorf(
			TEXT("Exception encountered in platform compiler: %s\nException Callstack:\n%s"),
			*ExceptionMsg,
			*ExceptionCallstack
		);
	}

	Job.bSucceeded = Job.Output.bSucceeded;
}

// Compiles all shader pipeline stage jobs and returns true if all succeeded.
static bool CompileShaderPipelineStages(const TArray<const IShaderFormat*>& ShaderFormats, FShaderPipelineCompileJob* PipelineJob, int32* CompileCount, bool* bOutInterpolatorRemovalFailed = nullptr)
{
	FShaderCompileJob* CurrentJob = nullptr;
	for (int32 Index = 0; Index < PipelineJob->StageJobs.Num(); ++Index)
	{
		auto* PreviousJob = CurrentJob;
		CurrentJob = PipelineJob->StageJobs[Index]->GetSingleShaderJob();

		bool bFirstJob = PreviousJob == nullptr;
		if (bFirstJob)
		{
			CurrentJob->Input.bIncludeUsedOutputs = false;
			// Flag should be set on the first job when the FShaderPipelineCompileJob was constructed, to ensure the flag is included when computing the input hash.
			check(CurrentJob->Input.bCompilingForShaderPipeline == true);
		}

		if (bOutInterpolatorRemovalFailed && !bFirstJob && PreviousJob->Output.bSupportsQueryingUsedAttributes)
		{
			CurrentJob->Input.bIncludeUsedOutputs = true;
			CurrentJob->Input.bCompilingForShaderPipeline = true;
			CurrentJob->Input.UsedOutputs = PreviousJob->Output.UsedAttributes;
		}

		// Compile the shader directly through the platform dll (directly from the shader dir as the working directory)
		CompileShader(ShaderFormats, *CurrentJob, CompileCount);

		CurrentJob->bSucceeded = CurrentJob->Output.bSucceeded;
		if (!CurrentJob->Output.bSucceeded)
		{
			// Can't carry on compiling the pipeline
			// Set values used for validation on the pipeline jobs that we're skipping before returning
			for (int32 SkipIndex = Index + 1; SkipIndex < PipelineJob->StageJobs.Num(); SkipIndex++)
			{
				FShaderCompileJob& Job = *PipelineJob->StageJobs[SkipIndex];
				FInternalShaderCompilerFunctions::FinalizeShaderCompileJob(Job.Output, Job.Input);
				if (Job.SecondaryOutput.IsValid())
				{
					FInternalShaderCompilerFunctions::FinalizeShaderCompileJob(*Job.SecondaryOutput, Job.Input);
				}
			}
			return false;
		}
		else if (CurrentJob->Output.bInterpolatorRemovalFailed && bOutInterpolatorRemovalFailed)
		{
			*bOutInterpolatorRemovalFailed = true;
		}
	}

	return true;
}

// Returns false if the specified pipeline job contains any shader stage other than Vertex or Pixel.
static bool HasOnlyVertexOrPixelShaderStages(const FShaderPipelineCompileJob& PipelineJob)
{
	for (const TRefCountPtr<FShaderCompileJob>& StageJob : PipelineJob.StageJobs)
	{
		const EShaderFrequency Stage = StageJob->Input.Target.GetFrequency();
		if (Stage != SF_Vertex && Stage != SF_Pixel)
		{
			return false;
		}
	}
	return true;
}

// Resets the specified shader compiler output, but retains all errors so they can be reported as warnings.
static void ResetShaderCompilerOutputWithRetainedErrors(FShaderCompilerOutput& Output)
{
	TArray<FShaderCompilerError> RetainedErrors = MoveTemp(Output.Errors);
	bool bInterpolatorRemovalFailed = Output.bInterpolatorRemovalFailed;

	Output = FShaderCompilerOutput();

	Output.Errors = MoveTemp(RetainedErrors);
	Output.bInterpolatorRemovalFailed = bInterpolatorRemovalFailed;
}

void CompileShaderPipeline(const TArray<const IShaderFormat*>& ShaderFormats, FShaderPipelineCompileJob* PipelineJob, int32* CompileCount)
{
	checkf(PipelineJob->StageJobs.Num() > 0, TEXT("Pipeline %s has zero jobs!"), PipelineJob->Key.ShaderPipeline->GetName());

	// This tells the shader compiler we do want to remove unused outputs
	const bool bEnableRemovingUnused = HasOnlyVertexOrPixelShaderStages(*PipelineJob);
	if (bEnableRemovingUnused)
	{
		bool bInterpolatorRemovalFailed = false;
		if (!CompileShaderPipelineStages(ShaderFormats, PipelineJob, CompileCount, &bInterpolatorRemovalFailed))
		{
			return;
		}

		// If interpolator removal failed in any shader stage, compile pipeline again with interpolator removal disabled.
		// Otherwise, pipeline stages might end up with mismatching sets of interpolators.
		// If the pipeline consists of only a single stage (for DepthOnly pipelines for instance), there's no need to compile again, just take the successful result.
		if (bInterpolatorRemovalFailed && PipelineJob->StageJobs.Num() > 1)
		{
			// Reset compile states and strip interpolator removal option from each pipeline stage job
			PipelineJob->ForEachSingleShaderJob(
				[](FShaderCompileJob& StageJob)
				{
					// Reset information about used outputs
					StageJob.Input.bIncludeUsedOutputs = false;
					StageJob.Input.UsedOutputs.Empty();

					// Strip interpolator removal option from input, so our next compilation attempt avoids this optimization for the entire pipeline
					StageJob.Input.Environment.CompilerFlags.Remove(CFLAG_ForceRemoveUnusedInterpolators);

					// Reset job output before compiling again but retain errors so they can be reported as warnings
					ResetShaderCompilerOutputWithRetainedErrors(StageJob.Output);
				}
			);

			if (!CompileShaderPipelineStages(ShaderFormats, PipelineJob, CompileCount))
			{
				return;
			}
		}
	}
	else
	{
		if (!CompileShaderPipelineStages(ShaderFormats, PipelineJob, CompileCount))
		{
			return;
		}
	}

	PipelineJob->bSucceeded = true;
}

static void InternalGetShaderIncludes(const TCHAR* EntryPointVirtualFilePath, const TCHAR* VirtualFilePath, TArray<FString>& IncludeVirtualFilePaths, EShaderPlatform ShaderPlatform, uint32 DepthLimit, bool AddToIncludeFile, const FName* ShaderPlatformName, bool bPreprocessDependencies);

/**
* Add a new entry to the list of shader source files
* Only unique entries which can be loaded are added as well as their #include files
*
* @param OutVirtualFilePaths - [out] list of shader source files to add to
* @param ShaderFilename - shader file to add
*/
void AddShaderSourceFileEntry(TArray<FString>& OutVirtualFilePaths, FString VirtualFilePath, EShaderPlatform ShaderPlatform, const FName* ShaderPlatformName)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(AddShaderSourceFileEntry);

	check(CheckVirtualShaderFilePath(VirtualFilePath));
	if (!OutVirtualFilePaths.Contains(VirtualFilePath))
	{
		OutVirtualFilePaths.Add(VirtualFilePath);

		const uint32 DepthLimit = 100;
		const bool bPreprocessDependencies = true;
		InternalGetShaderIncludes(*VirtualFilePath, *VirtualFilePath, OutVirtualFilePaths, ShaderPlatform, DepthLimit, false, ShaderPlatformName, bPreprocessDependencies);
	}
}

/**
* Generates a list of virtual paths of all shader source that engine needs to load.
*
* @param OutVirtualFilePaths - [out] list of shader source files to add to
*/
void GetAllVirtualShaderSourcePaths(TArray<FString>& OutVirtualFilePaths, EShaderPlatform ShaderPlatform, const FName* ShaderPlatformName)
{
	// add all shader source files for hashing
	for( TLinkedList<FVertexFactoryType*>::TIterator FactoryIt(FVertexFactoryType::GetTypeList()); FactoryIt; FactoryIt.Next() )
	{
		FVertexFactoryType* VertexFactoryType = *FactoryIt;
		if( VertexFactoryType )
		{
			FString ShaderFilename(VertexFactoryType->GetShaderFilename());
			AddShaderSourceFileEntry(OutVirtualFilePaths, ShaderFilename, ShaderPlatform, ShaderPlatformName);
			if (VertexFactoryType->IncludesFwdShaderFile())
			{
				AddShaderSourceFileEntry(OutVirtualFilePaths, VertexFactoryType->GetShaderFwdFilename(), ShaderPlatform, ShaderPlatformName);
			}
		}
	}
	for( TLinkedList<FShaderType*>::TIterator ShaderIt(FShaderType::GetTypeList()); ShaderIt; ShaderIt.Next() )
	{
		FShaderType* ShaderType = *ShaderIt;
		if(ShaderType)
		{
			FString ShaderFilename(ShaderType->GetShaderFilename());
			AddShaderSourceFileEntry(OutVirtualFilePaths, ShaderFilename, ShaderPlatform, ShaderPlatformName);
		}
	}

	//#todo-rco: No need to loop through Shader Pipeline Types (yet)

	// Always add ShaderVersion.ush, so if shader forgets to include it, it will still won't break DDC.
	AddShaderSourceFileEntry(OutVirtualFilePaths, FString("/Engine/Public/ShaderVersion.ush"), ShaderPlatform, ShaderPlatformName);
	AddShaderSourceFileEntry(OutVirtualFilePaths, FString("/Engine/Private/MaterialTemplate.ush"), ShaderPlatform, ShaderPlatformName);
	AddShaderSourceFileEntry(OutVirtualFilePaths, FString("/Engine/Private/Common.ush"), ShaderPlatform, ShaderPlatformName);
	AddShaderSourceFileEntry(OutVirtualFilePaths, FString("/Engine/Private/Definitions.usf"), ShaderPlatform, ShaderPlatformName);
}

/**
* Kick off SHA verification for all shader source files
*/
void VerifyShaderSourceFiles(EShaderPlatform ShaderPlatform)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(VerifyShaderSourceFiles);

#if WITH_EDITORONLY_DATA
	if (!FPlatformProperties::RequiresCookedData() && AllowShaderCompiling())
	{
		// get the list of shader files that can be used
		TArray<FString> VirtualShaderSourcePaths;
		GetAllVirtualShaderSourcePaths(VirtualShaderSourcePaths, ShaderPlatform);
		FScopedSlowTask SlowTask((float)VirtualShaderSourcePaths.Num());
		for( int32 ShaderFileIdx=0; ShaderFileIdx < VirtualShaderSourcePaths.Num(); ShaderFileIdx++ )
		{
			SlowTask.EnterProgressFrame(1);
			// load each shader source file. This will cache the shader source data after it has been verified
			LoadShaderSourceFile(*VirtualShaderSourcePaths[ShaderFileIdx], ShaderPlatform, nullptr, nullptr);
		}
	}
#endif // WITH_EDITORONLY_DATA
}

static void LogShaderSourceDirectoryMappings()
{
	for (const auto& Iter : GShaderSourceDirectoryMappings)
	{
		UE_LOGF(LogShaders, Log, "Shader directory mapping %ls -> %ls", *Iter.Key, *Iter.Value);
	}
}

FString GetShaderSourceFilePath(const FString& VirtualFilePath, TArray<FShaderCompilerError>* CompileErrors)
{
	// Make sure the .usf extension is correctly set.
	if (!CheckVirtualShaderFilePath(VirtualFilePath, CompileErrors))
	{
		return FString();
	}

	// We don't cache the output of this function because only used in LoadShaderSourceFile that is cached, or when there
	// is shader compilation errors.
	
	FString RealFilePath;

	// Look if this virtual shader source file match any directory mapping.
	const TMap<FString, FString>& ShaderSourceDirectoryMappings = GShaderSourceDirectoryMappings;
	FString ParentVirtualDirectoryPath = FPaths::GetPath(VirtualFilePath);
	FString RelativeVirtualDirectoryPath = FPaths::GetCleanFilename(VirtualFilePath);
	while (!ParentVirtualDirectoryPath.IsEmpty())
	{
		if (ShaderSourceDirectoryMappings.Contains(ParentVirtualDirectoryPath))
		{
			RealFilePath = FPaths::Combine(
				*ShaderSourceDirectoryMappings.Find(ParentVirtualDirectoryPath),
				RelativeVirtualDirectoryPath);
			break;
		}

		RelativeVirtualDirectoryPath = FPaths::GetCleanFilename(ParentVirtualDirectoryPath) / RelativeVirtualDirectoryPath;
		ParentVirtualDirectoryPath = FPaths::GetPath(ParentVirtualDirectoryPath);
	}

	// Make sure a directory mapping has matched.
	if (RealFilePath.IsEmpty())
	{
		FString Error = FString::Printf(TEXT("Can't map virtual shader source path \"%s\"."), *VirtualFilePath);
		Error += TEXT("\nDirectory mappings are:");
		for (const auto& Iter : ShaderSourceDirectoryMappings)
		{
			Error += FString::Printf(TEXT("\n  %s -> %s"), *Iter.Key, *Iter.Value);
		}

		ReportVirtualShaderFilePathError(CompileErrors, Error);
	}
	
	return RealFilePath;
}

FString ParseVirtualShaderFilename(const FString& InFilename)
{
	FString ShaderDir = FString(FPlatformProcess::ShaderDir());
	ShaderDir.ReplaceInline(TEXT("\\"), TEXT("/"), ESearchCase::CaseSensitive);
	int32 CharIndex = ShaderDir.Find(TEXT("/"), ESearchCase::CaseSensitive, ESearchDir::FromEnd, ShaderDir.Len() - 1);
	if (CharIndex != INDEX_NONE)
	{
		ShaderDir.RightInline(ShaderDir.Len() - CharIndex, EAllowShrinking::No);
	}

	FString RelativeFilename = InFilename.Replace(TEXT("\\"), TEXT("/"), ESearchCase::CaseSensitive);
	// remove leading "/" because this makes path absolute on Linux (and Mac).
	if (RelativeFilename.Len() > 0 && RelativeFilename[0] == TEXT('/'))
	{
		RelativeFilename.RightInline(RelativeFilename.Len() - 1, EAllowShrinking::No);
	}
	RelativeFilename = IFileManager::Get().ConvertToRelativePath(*RelativeFilename);
	CharIndex = RelativeFilename.Find(ShaderDir);
	if (CharIndex != INDEX_NONE)
	{
		CharIndex += ShaderDir.Len();
		if (RelativeFilename.GetCharArray()[CharIndex] == TEXT('/'))
		{
			CharIndex++;
		}
		if (RelativeFilename.Contains(TEXT("WorkingDirectory")))
		{
			const int32 NumDirsToSkip = 3;
			int32 NumDirsSkipped = 0;
			int32 NewCharIndex = CharIndex;

			do
			{
				NewCharIndex = RelativeFilename.Find(TEXT("/"), ESearchCase::CaseSensitive, ESearchDir::FromStart, CharIndex);
				CharIndex = (NewCharIndex == INDEX_NONE) ? CharIndex : NewCharIndex + 1;
			}
			while (NewCharIndex != INDEX_NONE && ++NumDirsSkipped < NumDirsToSkip);
		}
		RelativeFilename.MidInline(CharIndex, RelativeFilename.Len() - CharIndex, EAllowShrinking::No);
	}

	// add leading "/" to the relative filename because that's what virtual shader path expects
	FString OutputFilename;
	if (RelativeFilename.Len() > 0 && RelativeFilename[0] != TEXT('/'))
	{
		OutputFilename = TEXT("/") + RelativeFilename;
	}
	else
	{
		OutputFilename = RelativeFilename;
	}
	check(CheckVirtualShaderFilePath(OutputFilename));
	return OutputFilename;
}

bool ReplaceVirtualFilePathForShaderPlatform(FString& InOutVirtualFilePath, EShaderPlatform ShaderPlatform)
{
	// as of 2021-03-01, it'd be safe to access just the include directory without the lock... but the lock (and copy) is here for the consistency's and future-proofness' sake
	const FString PlatformIncludeDirectory( [ShaderPlatform]()
		{
			FRWScopeLock ShaderHashAccessLock(GShaderHashAccessRWLock, SLT_ReadOnly);
			return GShaderHashCache.GetPlatformIncludeDirectory(ShaderPlatform);
		}()
	);

	if (PlatformIncludeDirectory.IsEmpty())
	{
		return false;
	}

	struct FEntry
	{
		const TCHAR* Prefix;
		const TCHAR* Visibility;
	};
	const FEntry VirtualPlatformPrefixes[] =
	{
		{TEXT("/Platform/Private/"), TEXT("Private")},
		{TEXT("/Platform/Public/"),  TEXT("Public")}
	};

	for (const FEntry& Entry : VirtualPlatformPrefixes)
	{
		if (InOutVirtualFilePath.StartsWith(Entry.Prefix))
		{
			// PlatformIncludeDirectory already contains leading and trailing slash (which we need to remove)
			FString CandidatePath = FString::Printf(TEXT("/Platform%s"), *PlatformIncludeDirectory);
			CandidatePath.RemoveFromEnd(TEXT("/"));

			// If a directory mapping exists for the candidate path, then commit the replacement
			if (GShaderSourceDirectoryMappings.Contains(CandidatePath))
			{
				InOutVirtualFilePath = FString::Printf(TEXT("/Platform%s%s/%s"),
					*PlatformIncludeDirectory, // This already contains leading and trailing slash
					Entry.Visibility,
					*InOutVirtualFilePath.RightChop(FCString::Strlen(Entry.Prefix)));

				return true;
			}
		}
	}

	return false;
}

bool ReplaceVirtualFilePathForShaderAutogen(FString& InOutVirtualFilePath, EShaderPlatform ShaderPlatform, const FName* InShaderPlatformName)
{
	const FStringView ShaderAutogenStem = TEXTVIEW("/Engine/Generated/ShaderAutogen/");

	// Tweak the autogen path
	// for examples, if it starts with "/Engine/Generated/ShaderAutogen/" change it to "ShaderAutogen/PCD3D_SM5/"
	if (!FCString::Strnicmp(*InOutVirtualFilePath, ShaderAutogenStem.GetData(), ShaderAutogenStem.Len()))
	{
		check(FDataDrivenShaderPlatformInfo::IsValid(ShaderPlatform) || InShaderPlatformName != nullptr);
		FName ShaderPlatformName = FDataDrivenShaderPlatformInfo::IsValid(ShaderPlatform) ? FDataDrivenShaderPlatformInfo::GetName(ShaderPlatform) : *InShaderPlatformName;
		TStringBuilder<1024> OutputShaderName;

		// Append the prefix.
		OutputShaderName.Append(TEXTVIEW("/ShaderAutogen/"));

		// Append the platform name.

		FString PlatformNameString = ShaderPlatformName.ToString();
		OutputShaderName.Append(*PlatformNameString, PlatformNameString.Len());

		OutputShaderName.AppendChar(TEXT('/'));

		// Append the relative name (the substring after "/Engine/Generated/ShaderAutogen/").
		const TCHAR* RelativeShaderName = *InOutVirtualFilePath + ShaderAutogenStem.Len();
		OutputShaderName.Append(RelativeShaderName, InOutVirtualFilePath.Len() - ShaderAutogenStem.Len());

		InOutVirtualFilePath = OutputShaderName;
		return true;
	}

	return false;
}

void FixupShaderFilePath(FString& VirtualFilePath, EShaderPlatform ShaderPlatform, const FName* ShaderPlatformName)
{
	// Always substitute virtual platform path before accessing GShaderFileCache to get platform-specific file.
	ReplaceVirtualFilePathForShaderPlatform(VirtualFilePath, ShaderPlatform);

	// Fixup autogen file
	ReplaceVirtualFilePathForShaderAutogen(VirtualFilePath, ShaderPlatform, ShaderPlatformName);
}

inline bool IsEndOfLine(TCHAR C)
{
	return C == TEXT('\r') || C == TEXT('\n');
}

inline bool CommentStripNeedsHandling(TCHAR C)
{
	return IsEndOfLine(C) || C == TEXT('/') || C == 0;
}

inline int NewlineCharCount(TCHAR First, TCHAR Second)
{
	return ((First + Second) == TEXT('\r') + TEXT('\n')) ? 2 : 1;
}

// Given an FString containing the contents of a shader source file, populates the given array with contents of
// that source file with all comments stripped. This is needed since the STB preprocessor itself does not strip 
// comments.
void ShaderConvertAndStripComments(const FString& ShaderSource, TArray<ANSICHAR>& OutStripped, EConvertAndStripFlags Flags)
{
	bool bSkipSimdPadding = EnumHasAnyFlags(Flags, EConvertAndStripFlags::NoSimdPadding);
	// need extra null chars at the end of the stripped string - one null terminator plus 15 optional padding characters for SIMD-safe reads
	const uint8 NullCharCount = bSkipSimdPadding ? 1 : 16;
	
	// STB preprocessor does not strip comments, so we do so here before returning the loaded source
	// Doing so is barely more costly than the memcopy we require anyways so has negligible overhead.
	// Reserve worst case (i.e. assuming there are no comments at all) to avoid reallocation
	int32 BufferSize = ShaderSource.Len() + NullCharCount;
	OutStripped.SetNumUninitialized(BufferSize);

	ANSICHAR* CurrentOut = OutStripped.GetData();

	const TCHAR* const Start = ShaderSource.GetCharArray().GetData();
	const TCHAR* const End = Start + ShaderSource.Len();

	// We rely on null termination to avoid the need to check Current < End in some cases
	check(*End == TEXT('\0'));

	const TCHAR* Current = Start;

#if PLATFORM_ALWAYS_HAS_SSE4_2
	__m128i CharCR = _mm_set1_epi8('\r');			// Carriage return
	__m128i CharLF = _mm_set1_epi8('\n');			// Line feed (newline)
	__m128i CharSlash = _mm_set1_epi8('/');
	__m128i CharStar = _mm_set1_epi8('*');

	// We process 15 characters at a time, so we can find comment starts (needs access to pairs of characters)
	const TCHAR* EndSse = End - 16;
	for (; Current < EndSse; )
	{
		__m128i First8 = _mm_loadu_si128((const __m128i*)Current);
		__m128i Second8 = _mm_loadu_si128((const __m128i*)(Current + 8));
		__m128i CurrentWord = _mm_packus_epi16(First8, Second8);

		int32 CRMask = _mm_movemask_epi8(_mm_cmpeq_epi8(CurrentWord, CharCR));
		int32 SlashMask = _mm_movemask_epi8(_mm_cmpeq_epi8(CurrentWord, CharSlash));
		int32 StarMask = _mm_movemask_epi8(_mm_cmpeq_epi8(CurrentWord, CharStar));

		// If we encounter a carriage return, fall back to slower single character path that handles CR/LF combos
		if (CRMask)
		{
			// Go back one character if first character in current word is CR, and previous character was LF, so
			// the single character parser can treat it as a newline pair.
			if ((CRMask & 1) && (Current > Start) && *(Current - 1) == '\n')
			{
				Current--;
				CurrentOut--;
			}
			break;
		}

		// Echo the current word
		_mm_storeu_si128((__m128i*)CurrentOut, CurrentWord);

		// Check if there is a comment start, meaning a slash followed by slash or star, which we can detect by shifting right
		// a mask containing both slash and star, and seeing if that overlaps with a slash.
		int32 CommentStartMask = SlashMask & ((SlashMask | StarMask) >> 1);
		if (!CommentStartMask)
		{
			// If no potential comment start, advance 15 characters and parse again
			CurrentOut += 15;
			Current += 15;
			continue;
		}

		// Advance input to contents of comment, output to end of non-comment characters
		int32 CommentOffset = _tzcnt_u32(CommentStartMask);
		Current += CommentOffset + 2;
		CurrentOut += CommentOffset;

		if (*(Current - 1) == '/')
		{
			// Single line comment, advance to newline
			bool bFoundNewline = false;

			for (; Current < EndSse;)
			{
				First8 = _mm_loadu_si128((const __m128i*)Current);
				Second8 = _mm_loadu_si128((const __m128i*)(Current + 8));
				CurrentWord = _mm_packus_epi16(First8, Second8);

				CRMask = _mm_movemask_epi8(_mm_cmpeq_epi8(CurrentWord, CharCR));
				int32 LFMask = _mm_movemask_epi8(_mm_cmpeq_epi8(CurrentWord, CharLF));
				int32 EitherMask = CRMask | LFMask;
				if (EitherMask)
				{
					int32 NewlineOffset = _tzcnt_u32(EitherMask);
					Current += NewlineOffset;
					bFoundNewline = true;
					break;
				}
				else
				{
					Current += 16;
				}
			}

			if (!bFoundNewline)
			{
				// Ran out of input buffer we can safely scan with SSE -- resume comment parsing in single character parser.
				goto SingleLineCommentParse;
			}
			if (CRMask)
			{
				// Hit a CR.  Stop and fall back to single character parser.  Note that we don't need to worry about rewinding for
				// a newline pair here, because we stop on either that's encountered first, so we haven't emitted a newline yet.
				break;
			}
		}
		else
		{
			// Multi line comment, skip to end of comment, writing newlines
			bool bFoundEnd = false;

			for (; Current < EndSse;)
			{
				First8 = _mm_loadu_si128((const __m128i*)Current);
				Second8 = _mm_loadu_si128((const __m128i*)(Current + 8));
				CurrentWord = _mm_packus_epi16(First8, Second8);

				// Fall back to single character parsing if we hit a CR
				CRMask = _mm_movemask_epi8(_mm_cmpeq_epi8(CurrentWord, CharCR));
				if (CRMask)
				{
					// Go back one character if this is the first CR, and previous character was LF
					if ((CRMask & 1) && (Current > Start) && *(Current - 1) == '\n')
					{
						Current--;
						CurrentOut--;
					}
					goto MultiLineCommentParse;
				}

				StarMask = _mm_movemask_epi8(_mm_cmpeq_epi8(CurrentWord, CharStar));
				SlashMask = _mm_movemask_epi8(_mm_cmpeq_epi8(CurrentWord, CharSlash));
				int32 LFMask = _mm_movemask_epi8(_mm_cmpeq_epi8(CurrentWord, CharLF));

				int32 CommentEndMask = StarMask & (SlashMask >> 1);
				if (CommentEndMask)
				{
					// Process any newlines before the comment end
					int32 CommentEndOffset = _tzcnt_u32(CommentEndMask);
					LFMask &= (0xffff >> (16 - CommentEndOffset));
					if (LFMask)
					{
						_mm_storeu_si128((__m128i*)CurrentOut, CharLF);
						CurrentOut += _mm_popcnt_u32(LFMask);
					}
					Current += CommentEndOffset + 2;
					bFoundEnd = true;
					break;
				}
				else
				{
					// No comment end -- process any newlines in the first 15 characters and continue
					LFMask &= 0x7fff;
					if (LFMask)
					{
						_mm_storeu_si128((__m128i*)CurrentOut, CharLF);
						CurrentOut += _mm_popcnt_u32(LFMask);
					}
					Current += 15;
				}
			}

			if (!bFoundEnd)
			{
				// Ran out of input buffer we can safely scan with SSE -- resume comment parsing in single character parser.
				goto MultiLineCommentParse;
			}
		}
	}
#endif	// PLATFORM_ALWAYS_HAS_SSE4_2

	for (; Current < End;)
	{
		// sanity check that we're not overrunning the buffer
		check(CurrentOut < (OutStripped.GetData() + BufferSize));
		// CommentStripNeedsHandling returns true when *Current == '\0';
		while (!CommentStripNeedsHandling(*Current))
		{
			// straight cast to ansichar; since this is a character in hlsl source that's not in a comment
			// we assume that it must be valid to do so. if this assumption is not valid the shader source was
			// broken/corrupt anyways.
			*CurrentOut++ = (ANSICHAR)(*Current++);
		}

		if (IsEndOfLine(*Current))
		{
			*CurrentOut++ = '\n';
			Current += NewlineCharCount(Current[0], Current[1]);
		}
		else if (Current[0] == '/')
		{
			if (Current[1] == '/')
			{
#if PLATFORM_ALWAYS_HAS_SSE4_2
				SingleLineCommentParse:
#endif
				while (!IsEndOfLine(*Current) && Current < End)
				{
					++Current;
				}
			}
			else if (Current[1] == '*')
			{
				Current += 2;
				while (Current < End)
				{
#if PLATFORM_ALWAYS_HAS_SSE4_2
					MultiLineCommentParse:
#endif
					if (Current[0] == '*' && Current[1] == '/')
					{
						Current += 2;
						break;
					}
					else if (IsEndOfLine(*Current))
					{
						*CurrentOut++ = '\n';
						Current += NewlineCharCount(Current[0], Current[1]);
					}
					else
					{
						++Current;
					}
				}
			}
			else
			{
				*CurrentOut++ = (ANSICHAR)(*Current++);
			}
		}
	}

	// Null terminate after comment-stripped copy, plus 15 zero padding characters for SSE safe reads
	check(CurrentOut + NullCharCount <= (OutStripped.GetData() + BufferSize));
	for (int32 TerminateAndPadIndex = 0; TerminateAndPadIndex < NullCharCount; TerminateAndPadIndex++)
	{
		*CurrentOut++ = 0;
	}

	// Set correct length after stripping but don't bother shrinking/reallocating, minor memory overhead to save time
	OutStripped.SetNum(CurrentOut - OutStripped.GetData(), EAllowShrinking::No);
}

bool LoadCachedShaderSourceFile(const TCHAR* InVirtualFilePath, EShaderPlatform ShaderPlatform, FShaderSharedStringPtr* OutFileContents, TArray<FShaderCompilerError>* OutCompileErrors, const FName* ShaderPlatformName, FShaderSharedAnsiStringPtr* OutStrippedContents)
{
#if WITH_EDITORONLY_DATA
	// it's not expected that cooked platforms get here, but if they do, this is the final out
	if (FPlatformProperties::RequiresCookedData())
	{
		return false;
	}

	bool bResult = false;

	STAT(double ShaderFileLoadingTime = 0);

	{
		SCOPE_SECONDS_COUNTER(ShaderFileLoadingTime);

		FString VirtualFilePath(InVirtualFilePath);
		FixupShaderFilePath(VirtualFilePath, ShaderPlatform, ShaderPlatformName);

		GShaderFileCache.FindOrTryProduceAndApply(VirtualFilePath,
			[&VirtualFilePath, OutCompileErrors](FShaderFileCacheEntry& CachedFile)
			{
				FString ShaderFilePath = GetShaderSourceFilePath(VirtualFilePath, OutCompileErrors);

				if (!ShaderFilePath.IsEmpty())
				{
					FShaderSharedStringPtr SourceFile = MakeShared<FString>();

					// verify SHA hash of shader files on load. missing entries trigger an error
					if (FFileHelper::LoadFileToString(*SourceFile, *ShaderFilePath, FFileHelper::EHashOptions::EnableVerify | FFileHelper::EHashOptions::ErrorMissingHash))
					{
						CachedFile.Source = MoveTemp(SourceFile);

						TArray<ANSICHAR>* StrippedSource = new TArray<ANSICHAR>;
						ShaderConvertAndStripComments(*CachedFile.Source, *StrippedSource);
						CachedFile.StrippedSource = MakeShareable(StrippedSource);

						return true;
					}
				}

				// Create an empty entry if missing files are being cached.
				return FMissingShaderFileCacheGuard::IsEnabled();
			},
			[&bResult, OutFileContents, OutStrippedContents](const FShaderFileCacheEntry& CachedFile)
			{
				if (CachedFile.Source.IsValid())
				{
					if (OutFileContents)
					{
						*OutFileContents = CachedFile.Source;
					}
					if (OutStrippedContents)
					{
						*OutStrippedContents = CachedFile.StrippedSource;
					}
					bResult = true;
				}
			}
		);
	}

	INC_FLOAT_STAT_BY(STAT_ShaderCompiling_LoadingShaderFiles,(float)ShaderFileLoadingTime);

	return bResult;
#else
	return false;
#endif // WITH_EDITORONLY_DATA
}

bool LoadShaderSourceFile(const TCHAR* InVirtualFilePath, EShaderPlatform ShaderPlatform, FString* OutFileContents, TArray<FShaderCompilerError>* OutCompileErrors, const FName* ShaderPlatformName, FShaderSharedAnsiStringPtr* OutStrippedContents) // TODO: const FString&
{
#if WITH_EDITORONLY_DATA
	FShaderSharedStringPtr LocalFileContents;
	FShaderSharedStringPtr* pFileContents = OutFileContents ? &LocalFileContents : nullptr;

	const bool bResult = LoadCachedShaderSourceFile(InVirtualFilePath, ShaderPlatform, pFileContents, OutCompileErrors, ShaderPlatformName, OutStrippedContents);

	if (OutFileContents && LocalFileContents.IsValid())
	{
		*OutFileContents = *LocalFileContents;
	}

	return bResult;
#else
	return false;
#endif
}

static FString FormatErrorCantFindSourceFile(const TCHAR* VirtualFilePath)
{
	return FString::Printf(TEXT("Couldn't find source file of virtual shader path \'%s\'"), VirtualFilePath);
}

void LoadShaderSourceFileChecked(const TCHAR* VirtualFilePath, EShaderPlatform ShaderPlatform, FString& OutFileContents, const FName* ShaderPlatformName)
{
	if (!LoadShaderSourceFile(VirtualFilePath, ShaderPlatform, &OutFileContents, nullptr, ShaderPlatformName))
	{
		UE_LOGF(LogShaders, Fatal, "%ls", *FormatErrorCantFindSourceFile(VirtualFilePath));
	}
}

/**
 * Walks InStr until we find either an end-of-line or TargetChar.
 */
const TCHAR* SkipToCharOnCurrentLine(const TCHAR* InStr, TCHAR TargetChar)
{
	const TCHAR* Str = InStr;
	if (Str)
	{
		while (*Str && *Str != TargetChar && *Str != TEXT('\n'))
		{
			++Str;
		}
		if (*Str != TargetChar)
		{
			Str = NULL;
		}
	}
	return Str;
}

/**
* Find the first valid preprocessor include directive in the given text.
* @return Pointer to start of first include directive if found.  nullptr otherwise.
*/
static const TCHAR* FindFirstInclude(const TCHAR* Text)
{
	const TCHAR *IncludeToken = TEXT("include");
	const uint32 IncludeTokenLength = FCString::Strlen(IncludeToken);
	const TCHAR* PreprocessorDirectiveStart = FCString::Strstr(Text, TEXT("#"));
	while (PreprocessorDirectiveStart)
	{
		// Eat any whitespace between # and the next token.
		const TCHAR* ParseHead = PreprocessorDirectiveStart + 1;
		while (*ParseHead == TEXT(' ') || *ParseHead == TEXT('\t'))
		{
			++ParseHead;
		}
		// Check for "include" token.
		if (FCString::Strnicmp(ParseHead, IncludeToken, IncludeTokenLength) == 0)
		{
			ParseHead += IncludeTokenLength;
		}
		// Need a trailing whitespace character to make a valid include directive.
		if (*ParseHead == TEXT(' ') || *ParseHead == TEXT('\t'))
		{
			return PreprocessorDirectiveStart;
		}
		// Look for the next preprocess directive.
		PreprocessorDirectiveStart = *PreprocessorDirectiveStart ? FCString::Strstr(PreprocessorDirectiveStart + 1, TEXT("#")) : nullptr;
	}
	return nullptr;
}

static void StringCopyToAnsiCharArray(const TCHAR* Text, int32 TextLen, TArray<ANSICHAR>& Out)
{
	Out.SetNumUninitialized(TextLen + 1);
	ANSICHAR* OutData = Out.GetData();
	for (int32 CharIndex = 0; CharIndex < TextLen; CharIndex++, OutData++, Text++)
	{
		*OutData = (ANSICHAR)*Text;
	}
	*OutData = 0;
}

// Allocates structure and adds root file dependency
static TUniquePtr<FShaderPreprocessDependencies> ShaderPreprocessDependenciesBegin(const TCHAR* VirtualFilePath)
{
	TUniquePtr<FShaderPreprocessDependencies> PreprocessDependencies = MakeUnique<FShaderPreprocessDependencies>();

	PreprocessDependencies->Dependencies.AddDefaulted();
	StringCopyToAnsiCharArray(VirtualFilePath, FCString::Strlen(VirtualFilePath), PreprocessDependencies->Dependencies[0].ResultPath);
	PreprocessDependencies->Dependencies[0].ResultPathHash = FCrc::Strihash_DEPRECATED(VirtualFilePath);

	return PreprocessDependencies;
}

// Adds finished dependencies to the cache
static void ShaderPreprocessDependenciesEnd(const TCHAR* VirtualFilePath, TUniquePtr<FShaderPreprocessDependencies> PreprocessDependencies, EShaderPlatform Platform)
{
	GShaderFileCache.FindAndApply(
		VirtualFilePath,
		[&PreprocessDependencies](FShaderFileCacheEntry& CachedFile)
		{
			// Another thread could have finished the job...  If not, set the dependencies.
			if (!CachedFile.Dependencies.IsValid())
			{
				CachedFile.Dependencies = MakeShareable(PreprocessDependencies.Release());
			}
		}
	);
}

static void AddPreprocessDependency(FShaderPreprocessDependencies& Dependencies, const FShaderPreprocessDependency& Dependency)
{
	check(Dependency.StrippedSource.IsValid());

	// First, check if the dependency already exists
	for (uint32 HashIndex = Dependencies.BySource.First(GetTypeHash(Dependency.PathInSourceHash)); Dependencies.BySource.IsValid(HashIndex); HashIndex = Dependencies.BySource.Next(HashIndex))
	{
		FShaderPreprocessDependency& TestDependency = Dependencies.Dependencies[HashIndex];

		// Subtract one from PathInSource.Num() to get length minus null terminator
		if (TestDependency.EqualsPathInSource(Dependency.PathInSource.GetData(), Dependency.PathInSource.Num() - 1, Dependency.PathInSourceHash, Dependency.ParentPath.GetData()))
		{
			// The result path better be the same for both
			check(!FCStringAnsi::Stricmp(TestDependency.ResultPath.GetData(), Dependency.ResultPath.GetData()));
			return;
		}
	}

	// Add the dependency
	int32 AddedIndex = Dependencies.Dependencies.Add(Dependency);
	Dependencies.BySource.Add(GetTypeHash(Dependency.PathInSourceHash), (uint32)AddedIndex);

	// Then check if the result path already exists, so we can point ResultPathUniqueIndex at the first instance of the result path
	uint32 ExistingResultIndex;
	for (ExistingResultIndex = Dependencies.ByResult.First(Dependency.ResultPathHash); Dependencies.ByResult.IsValid(ExistingResultIndex); ExistingResultIndex = Dependencies.ByResult.Next(ExistingResultIndex))
	{
		FShaderPreprocessDependency& TestDependency = Dependencies.Dependencies[ExistingResultIndex];
		if (TestDependency.EqualsResultPath(Dependency.ResultPath.GetData(), Dependency.ResultPathHash))
		{
			break;
		}
	}

	if (Dependencies.ByResult.IsValid(ExistingResultIndex))
	{
		// Reference existing result
		Dependencies.Dependencies[AddedIndex].ResultPathUniqueIndex = ExistingResultIndex;
	}
	else
	{
		// Add new result
		Dependencies.Dependencies[AddedIndex].ResultPathUniqueIndex = (uint32)AddedIndex;
		Dependencies.ByResult.Add(Dependency.ResultPathHash, (uint32)AddedIndex);
	}
}

/**
 * Recursively populates IncludeFilenames with the unique include filenames found in the shader file named Filename.
 */
static void InternalGetShaderIncludes(const TCHAR* EntryPointVirtualFilePath, const TCHAR* VirtualFilePath, FStringView FileContents, TArray<FString>& IncludeVirtualFilePaths, EShaderPlatform ShaderPlatform, uint32 DepthLimit, bool AddToIncludeFile, const FName* ShaderPlatformName, FShaderPreprocessDependencies* OutDependencies)
{
	//avoid an infinite loop with a 0 length string
	if (FileContents.Len() > 0)
	{
		if (AddToIncludeFile)
		{
			IncludeVirtualFilePaths.Add(VirtualFilePath);
		}

		//find the first include directive
		const TCHAR* IncludeBegin = FindFirstInclude(GetData(FileContents));

		uint32 SearchCount = 0;
		const uint32 MaxSearchCount = 200;
		//keep searching for includes as long as we are finding new ones and haven't exceeded the fixed limit
		while (IncludeBegin != NULL && SearchCount < MaxSearchCount && DepthLimit > 0)
		{
			//find the first double quotation after the include directive
			const TCHAR* IncludeFilenameBegin = SkipToCharOnCurrentLine(IncludeBegin, TEXT('\"'));

			if (IncludeFilenameBegin)
			{
				//find the trailing double quotation
				const TCHAR* IncludeFilenameEnd = SkipToCharOnCurrentLine(IncludeFilenameBegin + 1, TEXT('\"'));

				if (IncludeFilenameEnd)
				{
					//construct a string between the double quotations
					FString ExtractedIncludeFilename = FString::ConstructFromPtrSize(IncludeFilenameBegin + 1, (int32)(IncludeFilenameEnd - IncludeFilenameBegin - 1));

					// If the include is relative, then it must be relative to the current virtual file path.
					if (!ExtractedIncludeFilename.StartsWith(TEXT("/")))
					{
						ExtractedIncludeFilename = FPaths::GetPath(VirtualFilePath) / ExtractedIncludeFilename;

						// Collapse any relative directories to allow #include "../MyFile.ush"
						FPaths::CollapseRelativeDirectories(ExtractedIncludeFilename);
					}

					//CRC the template, not the filled out version so that this shader's CRC will be independent of which material references it.
					const TCHAR* MaterialTemplateName = TEXT("/Engine/Private/MaterialTemplate.ush");
					const TCHAR* MaterialGeneratedName = TEXT("/Engine/Generated/Material.ush");

					bool bIsMaterialTemplate = false;
					if (ExtractedIncludeFilename == MaterialGeneratedName)
					{
						ExtractedIncludeFilename = MaterialTemplateName;
						bIsMaterialTemplate = true;
					}

					bool bIsPlatformFile = ReplaceVirtualFilePathForShaderPlatform(ExtractedIncludeFilename, ShaderPlatform);

					// Fixup autogen file
					bIsPlatformFile |= ReplaceVirtualFilePathForShaderAutogen(ExtractedIncludeFilename, ShaderPlatform, ShaderPlatformName);

					// Ignore uniform buffer, vertex factory and instanced stereo includes
					bool bIgnoreInclude = ExtractedIncludeFilename.StartsWith(TEXT("/Engine/Generated/"));

					// Check virtual.
					bIgnoreInclude |= !CheckVirtualShaderFilePath(ExtractedIncludeFilename);

					// Include only platform specific files, which will be used by the target platform.
					{
						FRWScopeLock ShaderHashAccessLock(GShaderHashAccessRWLock, SLT_ReadOnly);
						bIgnoreInclude = bIgnoreInclude || GShaderHashCache.ShouldIgnoreInclude(ExtractedIncludeFilename, ShaderPlatform);
					}

					bIsPlatformFile |= FShaderHashCache::IsPlatformInclude(ExtractedIncludeFilename);

					//vertex factories need to be handled separately
					if (!bIgnoreInclude)
					{
						int32 SeenFilenameIndex = IncludeVirtualFilePaths.Find(ExtractedIncludeFilename);
						if (SeenFilenameIndex == INDEX_NONE)
						{
							// Preprocess dependencies don't include platform files.
							TUniquePtr<FShaderPreprocessDependencies> ExtractedIncludeDependencies;
							if (OutDependencies && !bIsPlatformFile)
							{
								ExtractedIncludeDependencies = ShaderPreprocessDependenciesBegin(*ExtractedIncludeFilename);
							}

							// First element in Dependencies is root file, so initialize the StrippedSource pointer in it
							FShaderSharedStringPtr IncludedFileContents;
							LoadCachedShaderSourceFile(*ExtractedIncludeFilename, ShaderPlatform, &IncludedFileContents, nullptr, ShaderPlatformName,
								ExtractedIncludeDependencies ? &ExtractedIncludeDependencies->Dependencies[0].StrippedSource : nullptr);

							if (IncludedFileContents.IsValid())
							{
								InternalGetShaderIncludes(EntryPointVirtualFilePath, *ExtractedIncludeFilename, *IncludedFileContents, IncludeVirtualFilePaths, ShaderPlatform, DepthLimit - 1, true, ShaderPlatformName, ExtractedIncludeDependencies.Get());
							}

							if (ExtractedIncludeDependencies)
							{
								// Some generated shaders are referenced as includes, and won't be found -- if so, just delete the dependencies
								if (ExtractedIncludeDependencies->Dependencies[0].StrippedSource.IsValid())
								{
									ShaderPreprocessDependenciesEnd(*ExtractedIncludeFilename, MoveTemp(ExtractedIncludeDependencies), ShaderPlatform);
								}
							}
						}

						if (OutDependencies)
						{
							// Preprocess dependencies don't include platform files.
							if (!bIsPlatformFile)
							{
								// The material template itself isn't added as a dependency, but child includes of it are.
								FShaderSharedAnsiStringPtr StrippedContents;
								if (!bIsMaterialTemplate && LoadShaderSourceFile(*ExtractedIncludeFilename, ShaderPlatform, nullptr, nullptr, nullptr, &StrippedContents))
								{
									// Add immediate dependency
									FShaderPreprocessDependency Dependency;
									Dependency.StrippedSource = StrippedContents;

									// If the parent is the material template, switch its name to the generated name, so include dependencies from
									// the material template to other non-procedural files can be cached.
									const TCHAR* ParentNonTemplate = VirtualFilePath == MaterialTemplateName ? MaterialGeneratedName : VirtualFilePath;

									// We want ResultPath to have consistent case, for the preprocessor which is case sensitive.  So we use the exact
									// string from the previously found array element if it exists.  If this is the first time it's encountered, it will
									// have been added to the array by the InternalGetShaderIncludes call above.
									const FString& ResultPath = SeenFilenameIndex == INDEX_NONE ? ExtractedIncludeFilename : IncludeVirtualFilePaths[SeenFilenameIndex];

									StringCopyToAnsiCharArray(IncludeFilenameBegin + 1, (int32)(IncludeFilenameEnd - IncludeFilenameBegin - 1), Dependency.PathInSource);
									StringCopyToAnsiCharArray(ParentNonTemplate, FCString::Strlen(ParentNonTemplate), Dependency.ParentPath);
									StringCopyToAnsiCharArray(*ResultPath, ResultPath.Len(), Dependency.ResultPath);
									Dependency.ResultPathHash = GetTypeHash(ResultPath);

									// Hash deliberately doesn't include null terminator, so we can generate hash from string view.  Xxhash is faster than
									// the normal case insensitive string hash, so we choose that.
									Dependency.PathInSourceHash = FXxHash64::HashBuffer(Dependency.PathInSource.GetData(), Dependency.PathInSource.Num() - 1);

									AddPreprocessDependency(*OutDependencies, Dependency);
								}

								// Add recursive dependencies from the child
								FShaderPreprocessDependenciesShared ChildDependenciesShared;
								if (GetShaderPreprocessDependencies(*ExtractedIncludeFilename, ShaderPlatform, ChildDependenciesShared))
								{
									const FShaderPreprocessDependencies& ChildDependencies = *ChildDependenciesShared;

									// Skip over first entry, which is the root file (its dependency is handled by the "add immediate dependency" code above)
									for (int32 DependencyIndex = 1; DependencyIndex < ChildDependencies.Dependencies.Num(); DependencyIndex++)
									{
										AddPreprocessDependency(*OutDependencies, ChildDependencies.Dependencies[DependencyIndex]);
									}
								}

							}  // if (!bIsPlatformFile)
						}  // if (OutDependencies)
					}  // if (!bIgnoreInclude)
				}
			}

			// Skip to the end of the line.
			IncludeBegin = SkipToCharOnCurrentLine(IncludeBegin, TEXT('\n'));
		
			//find the next include directive
			if (IncludeBegin && *IncludeBegin != 0)
			{
				IncludeBegin = FindFirstInclude(IncludeBegin + 1);
			}
			SearchCount++;
		}

		if (SearchCount == MaxSearchCount || DepthLimit == 0)
		{
			UE_LOGF(LogShaders, Warning, "GetShaderIncludes parsing terminated early to avoid infinite looping!\n Entrypoint \'%ls\' CurrentInclude \'%ls\' SearchCount %u Depth %u", 
				EntryPointVirtualFilePath, 
				VirtualFilePath,
				SearchCount, 
				DepthLimit);
		}
	}
}

bool GetShaderPreprocessDependencies(const TCHAR* VirtualFilePath, EShaderPlatform ShaderPlatform, FShaderPreprocessDependenciesShared& OutDependencies)
{
	bool bResult = false;
	GShaderFileCache.FindAndApply(VirtualFilePath,
		[&bResult, &OutDependencies](const FShaderFileCacheEntry& CachedFile)
		{
			if (CachedFile.Dependencies.IsValid())
			{
				OutDependencies = CachedFile.Dependencies;
				bResult = true;
			}
		}
	);
	return bResult;
}

/**
 * Recursively populates IncludeFilenames with the unique include filenames found in the shader file named Filename.
 */
static void InternalGetShaderIncludes(const TCHAR* EntryPointVirtualFilePath, const TCHAR* VirtualFilePath, TArray<FString>& IncludeVirtualFilePaths, EShaderPlatform ShaderPlatform, uint32 DepthLimit, bool AddToIncludeFile, const FName* ShaderPlatformName, bool bPreprocessDependencies)
{
	TUniquePtr<FShaderPreprocessDependencies> PreprocessDependencies;
	if (bPreprocessDependencies)
	{
		// Check if they've already been generated.  These are platform independent, so we only need to generate them once if multiple platforms are being cooked,
		// but in case we want to specialize them by platform in the future, the platform is passed in.
		FShaderPreprocessDependenciesShared OutDependenciesIgnored;
		if (!GetShaderPreprocessDependencies(VirtualFilePath, ShaderPlatform, OutDependenciesIgnored))
		{
			// Allocates dependency structure and adds root file element
			PreprocessDependencies = ShaderPreprocessDependenciesBegin(VirtualFilePath);
		}
	}

	// First element in Dependencies is root file, so initialize the StrippedSource pointer in it
	FShaderSharedStringPtr FileContents;
	LoadCachedShaderSourceFile(VirtualFilePath, ShaderPlatform, &FileContents, nullptr, ShaderPlatformName, PreprocessDependencies ? &PreprocessDependencies->Dependencies[0].StrippedSource : nullptr);

	if (FileContents.IsValid())
	{
		InternalGetShaderIncludes(EntryPointVirtualFilePath, VirtualFilePath, *FileContents, IncludeVirtualFilePaths, ShaderPlatform, DepthLimit, AddToIncludeFile, ShaderPlatformName, PreprocessDependencies.Get());
	}

	if (PreprocessDependencies)
	{
		// Adds completed dependency structure to shader cache map entry
		ShaderPreprocessDependenciesEnd(VirtualFilePath, MoveTemp(PreprocessDependencies), ShaderPlatform);
	}
}

void GetShaderIncludes(const TCHAR* EntryPointVirtualFilePath, const TCHAR* VirtualFilePath, TArray<FString>& IncludeVirtualFilePaths, EShaderPlatform ShaderPlatform, uint32 DepthLimit, const FName* ShaderPlatformName)
{
	InternalGetShaderIncludes(EntryPointVirtualFilePath, VirtualFilePath, IncludeVirtualFilePaths, ShaderPlatform, DepthLimit, false, ShaderPlatformName, false);
}

void GetShaderIncludes(const TCHAR* EntryPointVirtualFilePath, const TCHAR* VirtualFilePath, const FString& FileContents, TArray<FString>& IncludeVirtualFilePaths, EShaderPlatform ShaderPlatform, uint32 DepthLimit, const FName* ShaderPlatformName)
{
	InternalGetShaderIncludes(EntryPointVirtualFilePath, VirtualFilePath, FileContents, IncludeVirtualFilePaths, ShaderPlatform, DepthLimit, false, ShaderPlatformName, nullptr);
}

void HashShaderFileWithIncludes(FArchive& HashingArchive, const TCHAR* VirtualFilePath, const FString& FileContents, EShaderPlatform ShaderPlatform, bool bOnlyHashIncludedFiles)
{
	// deprecated
}

static bool TryUpdateSingleShaderFilehash(FSHA1& InOutHashState, const TCHAR* VirtualFilePath,
	EShaderPlatform ShaderPlatform, FString* OutErrorMessage)
{
	// Get the list of includes this file contains
	TArray<FString> IncludeVirtualFilePaths;
	GetShaderIncludes(VirtualFilePath, VirtualFilePath, IncludeVirtualFilePaths, ShaderPlatform);
#if WITH_EDITOR &&  !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	if ( UE_LOG_ACTIVE(LogShaders, Verbose) )
	{
		UE_LOGF(LogShaders, Verbose, "Generating hash of file %ls, ", VirtualFilePath);
	}
#endif
	for (int32 IncludeIndex = 0; IncludeIndex < IncludeVirtualFilePaths.Num(); IncludeIndex++)
	{
		// Load the include file and hash it
		FShaderSharedStringPtr IncludeFileContents;
		if (!LoadCachedShaderSourceFile(*IncludeVirtualFilePaths[IncludeIndex], ShaderPlatform, &IncludeFileContents, nullptr))
		{
			if (OutErrorMessage)
			{
				*OutErrorMessage = FormatErrorCantFindSourceFile(*IncludeVirtualFilePaths[IncludeIndex]);
			}
			return false;
		}
		InOutHashState.UpdateWithString(GetData(*IncludeFileContents), GetNum(*IncludeFileContents));
#if WITH_EDITOR &&  !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		if (UE_LOG_ACTIVE(LogShaders, Verbose))
		{
			FSHA1 HashStateCopy = InOutHashState;
			FSHAHash IncrementalHash = HashStateCopy.Finalize();
			
			UE_LOGF(LogShaders, Verbose, "Processing include file for %ls, %ls, %ls", VirtualFilePath, *IncludeVirtualFilePaths[IncludeIndex], *BytesToHex(IncrementalHash.Hash, 20));
		}
#endif
	}

	// Load the source file and hash it
	FShaderSharedStringPtr FileContents;
	if (!LoadCachedShaderSourceFile(VirtualFilePath, ShaderPlatform, &FileContents, nullptr))
	{
		if (OutErrorMessage)
		{
			*OutErrorMessage = FormatErrorCantFindSourceFile(VirtualFilePath);
		}
		return false;
	}
	checkf(FileContents.IsValid(), TEXT("LoadCachedShaderSourceFile(\"%s\") returned true but did not provide a valid shared pointer for the shader content"), VirtualFilePath);
	InOutHashState.UpdateWithString(GetData(*FileContents), GetNum(*FileContents));
	if (OutErrorMessage)
	{
		OutErrorMessage->Reset();
	}
	return true;
}

/** 
* Prevents multiple threads from trying to redundantly call UpdateSingleShaderFilehash in GetShaderFileHash / GetShaderFilesHash.
* Must be used in conjunction with GShaderHashAccessRWLock, which protects actual GShaderHashCache operations.
*/
static FCriticalSection GShaderFileHashCalculationGuard;

const FSHAHash& GetShaderFileHash(const TCHAR* VirtualFilePath, EShaderPlatform ShaderPlatform)
{
	FString ErrorMessage;
	const FSHAHash* Hash = TryGetShaderFileHash(VirtualFilePath, ShaderPlatform, &ErrorMessage);
	if (!Hash)
	{
		UE_LOGF(LogShaders, Fatal, "%ls", *ErrorMessage);
		static FSHAHash EmptyHash;
		return EmptyHash;
	}
	return *Hash;
}

const FSHAHash* TryGetShaderFileHash(const TCHAR* VirtualFilePath, EShaderPlatform ShaderPlatform,
	FString* OutErrorMessage)
{
	// Make sure we are only accessing GShaderHashCache from one thread
	//check(IsInGameThread() || IsAsyncLoading());
	STAT(double HashTime = 0);
	{
		SCOPE_SECONDS_COUNTER(HashTime);

		{
			FRWScopeLock ShaderHashAccessLock(GShaderHashAccessRWLock, SLT_ReadOnly);
			const FSHAHash* CachedHash = GShaderHashCache.FindHash(ShaderPlatform, VirtualFilePath);
			// If a hash for this filename has been cached, use that
			if (CachedHash)
			{
				if (OutErrorMessage) OutErrorMessage->Reset();
				return CachedHash;
			}
		}

		// We don't want UpdateSingleShaderFilehash to be called redundantly from multiple threads,
		// while minimiziong GShaderHashAccessRWLock exclusive lock time.
		// We can use a dedicated critical section around the hash calculation and cache update, 
		// while keeping the cache itself available for reading.
		FScopeLock FileHashCalculationAccessLock(&GShaderFileHashCalculationGuard);

		// Double-check the cache while holding exclusive lock as another thread may have added the item we're looking for
		const FSHAHash* CachedHash = GShaderHashCache.FindHash(ShaderPlatform, VirtualFilePath);
		if (CachedHash)
		{
			if (OutErrorMessage) OutErrorMessage->Reset();
			return CachedHash;
		}

		FSHA1 HashState;
		bool bSucceeded = TryUpdateSingleShaderFilehash(HashState, VirtualFilePath, ShaderPlatform, OutErrorMessage);
		if (!bSucceeded)
		{
			return nullptr;
		}
		HashState.Final();

		// Update the hash cache
		FRWScopeLock ShaderHashAccessLock(GShaderHashAccessRWLock, SLT_Write);
		FSHAHash& NewHash = GShaderHashCache.AddHash(ShaderPlatform, VirtualFilePath);
		HashState.GetHash(&NewHash.Hash[0]);

#if WITH_EDITOR &&  !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		UE_LOGF(LogShaders, Verbose, "Final hash for file %ls, %ls", VirtualFilePath,*BytesToHex(&NewHash.Hash[0], 20));
#endif
		INC_FLOAT_STAT_BY(STAT_ShaderCompiling_HashingShaderFiles, (float)HashTime);
		return &NewHash;
	}
}

/**
 * Calculates a Hash for the given filename and its includes if it does not already exist in the Hash cache.
 * @param Filename - shader file to Hash
 * @param ShaderPlatform - shader platform to Hash
 */
const FSHAHash& GetShaderFilesHash(const TArray<FString>& VirtualFilePaths, EShaderPlatform ShaderPlatform)
{
	// Make sure we are only accessing GShaderHashCache from one thread
	//check(IsInGameThread() || IsAsyncLoading());
	STAT(double HashTime = 0);

	{
		SCOPE_SECONDS_COUNTER(HashTime);

		FString Key;
		for (const FString& Filename : VirtualFilePaths)
		{
			Key += Filename;
		}

		{
			FRWScopeLock ShaderHashAccessLock(GShaderHashAccessRWLock, SLT_ReadOnly);
			const FSHAHash* CachedHash = GShaderHashCache.FindHash(ShaderPlatform, Key);

			// If a hash for this filename has been cached, use that
			if (CachedHash)
			{
				return *CachedHash;
			}
		}

		// We don't want UpdateSingleShaderFilehash to be called redundantly from multiple threads,
		// while minimiziong GShaderHashAccessRWLock exclusive lock time.
		// We can use a dedicated critical section around the hash calculation and cache update, 
		// while keeping the cache itself available for reading.
		FScopeLock FileHashCalculationAccessLock(&GShaderFileHashCalculationGuard);

		// Double-check the cache while holding exclusive lock as another thread may have added the item we're looking for
		const FSHAHash* CachedHash = GShaderHashCache.FindHash(ShaderPlatform, Key);
		if (CachedHash)
		{
			return *CachedHash;
		}

		FSHA1 HashState;
		for (const FString& VirtualFilePath : VirtualFilePaths)
		{
			FString ErrorMessage;
			if (!TryUpdateSingleShaderFilehash(HashState, *VirtualFilePath, ShaderPlatform, &ErrorMessage))
			{
				UE_LOGF(LogShaders, Fatal, "%ls", *ErrorMessage);
			}
		}
		HashState.Final();

		// Update the hash cache
		FRWScopeLock ShaderHashAccessLock(GShaderHashAccessRWLock, SLT_Write);
		FSHAHash& NewHash = GShaderHashCache.AddHash(ShaderPlatform, Key);
		HashState.GetHash(&NewHash.Hash[0]);

		INC_FLOAT_STAT_BY(STAT_ShaderCompiling_HashingShaderFiles, (float)HashTime);
		return NewHash;
	}
}

void PrimeShaderFileHashCache(TArrayView<const FString> VirtualFilePaths, EShaderPlatform ShaderPlatform)
{
	STAT(double HashTime = 0);
	{
		SCOPE_SECONDS_COUNTER(HashTime);

		// Single read-lock pass: collect all paths not yet present in the cache.
		TArray<const FString*> UncachedPaths;
		UncachedPaths.Reserve(VirtualFilePaths.Num());
		{
			FRWScopeLock ShaderHashAccessLock(GShaderHashAccessRWLock, SLT_ReadOnly);
			for (const FString& VirtualFilePath : VirtualFilePaths)
			{
				if (!GShaderHashCache.FindHash(ShaderPlatform, VirtualFilePath))
				{
					UncachedPaths.Add(&VirtualFilePath);
				}
			}
		}

		if (UncachedPaths.IsEmpty())
		{
			return;
		}

		// Hold the calculation guard for the entire batch so no other thread redundantly
		// computes any of the same hashes while we are working through the list.
		FScopeLock FileHashCalculationAccessLock(&GShaderFileHashCalculationGuard);

		// Pre-allocate one slot per path so parallel tasks write to disjoint indices with no contention.
		// FilePath == nullptr serves as the sentinel for "skipped or failed".
		struct FComputedHash
		{
			const FString* FilePath = nullptr;
			FSHAHash Hash;
		};
		TArray<FComputedHash> ComputedHashes;
		ComputedHashes.SetNum(UncachedPaths.Num());

		// Compute hashes in parallel; each task owns its slot in ComputedHashes.
		// FindHash reads below are safe without RWLock: GShaderFileHashCalculationGuard
		// ensures no other thread writes new cache entries during this section.
		ParallelFor(UncachedPaths.Num(), [&](int32 Index)
		{
			const FString* VirtualFilePath = UncachedPaths[Index];

			// Double-check: a thread that held the guard before us may have already populated this entry.
			if (GShaderHashCache.FindHash(ShaderPlatform, *VirtualFilePath))
			{
				return;
			}

			FSHA1 HashState;
			FString ErrorMessage;
			if (TryUpdateSingleShaderFilehash(HashState, **VirtualFilePath, ShaderPlatform, &ErrorMessage))
			{
				HashState.Final();
				ComputedHashes[Index].FilePath = VirtualFilePath;
				HashState.GetHash(&ComputedHashes[Index].Hash.Hash[0]);
			}
			else
			{
				UE_LOGF(LogShaders, Warning, "PrimeShaderFileHashCache: failed to hash '%ls': %ls", **VirtualFilePath, *ErrorMessage);
			}
		});

		// Single write-lock pass to insert all newly computed hashes at once.
		FRWScopeLock ShaderHashAccessLock(GShaderHashAccessRWLock, SLT_Write);
		for (const FComputedHash& Entry : ComputedHashes)
		{
			if (Entry.FilePath)
			{
				FSHAHash& NewHash = GShaderHashCache.AddHash(ShaderPlatform, *Entry.FilePath);
				NewHash = Entry.Hash;
#if WITH_EDITOR && !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
				UE_LOGF(LogShaders, Verbose, "Final hash for file %ls, %ls", *(*Entry.FilePath), *BytesToHex(&NewHash.Hash[0], 20));
#endif
			}
		}

		INC_FLOAT_STAT_BY(STAT_ShaderCompiling_HashingShaderFiles, (float)HashTime);
	}
}

#if WITH_EDITOR
void BuildShaderFileToUniformBufferMap(TMap<FString, TArray<const TCHAR*> >& ShaderFileToUniformBufferVariables)
{
	if (!FPlatformProperties::RequiresCookedData())
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(BuildShaderFileToUniformBufferMap);

		TArray<FString> ShaderSourceFiles;
		GetAllVirtualShaderSourcePaths(ShaderSourceFiles, GMaxRHIShaderPlatform);

		FScopedSlowTask SlowTask((float)ShaderSourceFiles.Num());

		// Cache UB access strings, make it case sensitive for faster search
		struct FShaderVariable
		{
			FShaderVariable(const TCHAR* ShaderVariable) :
				OriginalShaderVariable(ShaderVariable), 
				SearchKey(FString(ShaderVariable).ToUpper() + TEXT(".")),
				// The shader preprocessor inserts a space after a #define replacement, make sure we detect the uniform buffer reference
				SearchKeyWithSpace(FString(ShaderVariable).ToUpper() + TEXT(" ."))
			{}

			bool operator<(const FShaderVariable& Other) const
			{
				return FCString::Strcmp(OriginalShaderVariable, Other.OriginalShaderVariable) < 0;
			}

			const TCHAR* OriginalShaderVariable;
			FString SearchKey;
			FString SearchKeyWithSpace;
		};
		// Cache each UB
		TArray<FShaderVariable> SearchKeys;
		for (TLinkedList<FShaderParametersMetadata*>::TIterator StructIt(FShaderParametersMetadata::GetStructList()); StructIt; StructIt.Next())
		{
			SearchKeys.Add(FShaderVariable(StructIt->GetShaderVariableName()));
		}

		// Sort SearchKeys for determinism in the generated ShaderFileToUniformBufferVariables maps, to improve consistency for A/B testing.
		// Order of items in FShaderParametersMetadata::GetStructList() is otherwise dependent on arbitrary startup constructor order.
		SearchKeys.Sort();

		TArray<UE::Tasks::TTask<void>> Tasks;
		Tasks.Reserve(ShaderSourceFiles.Num());

		// Just make sure that all the TArray inside the map won't move while being used by async tasks
		for (int32 FileIndex = 0; FileIndex < ShaderSourceFiles.Num(); FileIndex++)
		{
			ShaderFileToUniformBufferVariables.FindOrAdd(ShaderSourceFiles[FileIndex]);
		}

		// Find for each shader file which UBs it needs
		for (int32 FileIndex = 0; FileIndex < ShaderSourceFiles.Num(); FileIndex++)
		{
			SlowTask.EnterProgressFrame(1);

 			FString ShaderFileContents;
			LoadShaderSourceFileChecked(*ShaderSourceFiles[FileIndex], GMaxRHIShaderPlatform, ShaderFileContents);

			Tasks.Emplace(
				UE::Tasks::Launch(
					TEXT("SearchKeysInShaderContent"),
					[&SearchKeys, &ShaderSourceFiles, FileIndex, &ShaderFileToUniformBufferVariables, ShaderFileContents = MoveTemp(ShaderFileContents)]() mutable
					{
						// To allow case sensitive search which is way faster on some platforms (no need to look up locale, etc)
						ShaderFileContents.ToUpperInline();

						TArray<const TCHAR*>* ReferencedUniformBuffers = ShaderFileToUniformBufferVariables.Find(ShaderSourceFiles[FileIndex]);

						for (int32 SearchKeyIndex = 0; SearchKeyIndex < SearchKeys.Num(); ++SearchKeyIndex)
						{
							// Searching for the uniform buffer shader variable being accessed with '.'
							if (ShaderFileContents.Contains(SearchKeys[SearchKeyIndex].SearchKey, ESearchCase::CaseSensitive)
								|| ShaderFileContents.Contains(SearchKeys[SearchKeyIndex].SearchKeyWithSpace, ESearchCase::CaseSensitive))
								{
									ReferencedUniformBuffers->AddUnique(SearchKeys[SearchKeyIndex].OriginalShaderVariable);
							}
						}
					}
				)
			);
		}
		UE::Tasks::Wait(Tasks);
	}
}
#endif // WITH_EDITOR

void InitializeShaderHashCache()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(InitializeShaderHashCache);
	FRWScopeLock ShaderHashAccessLock(GShaderHashAccessRWLock, SLT_Write);
	GShaderHashCache.Initialize();
}

void UpdateIncludeDirectoryForPreviewPlatform(EShaderPlatform PreviewPlatform, EShaderPlatform ActualPlatform)
{
	FRWScopeLock ShaderHashAccessLock(GShaderHashAccessRWLock, SLT_Write);
	GShaderHashCache.UpdateIncludeDirectoryForPreviewPlatform(PreviewPlatform, ActualPlatform);
}

void CheckShaderHashCacheInclude(const FString& VirtualFilePath, EShaderPlatform ShaderPlatform, const FString& ShaderFormatName)
{
	FRWScopeLock ShaderHashAccessLock(GShaderHashAccessRWLock, SLT_ReadOnly);
	bool bIgnoreInclude = GShaderHashCache.ShouldIgnoreInclude(VirtualFilePath, ShaderPlatform);

	checkf(!bIgnoreInclude,
		TEXT("Shader compiler is trying to include %s, which is not located in IShaderFormat::GetPlatformIncludeDirectory for %s."),
		*VirtualFilePath,
		*ShaderFormatName);
}

void InitializeShaderTypes()
{
	UE_LOGF(LogShaders, Log, "InitializeShaderTypes() begin");
	
	FMissingShaderFileCacheGuard Guard;

	LogShaderSourceDirectoryMappings();

	TMap<FString, TArray<const TCHAR*> > ShaderFileToUniformBufferVariables;
#if WITH_EDITOR
	BuildShaderFileToUniformBufferMap(ShaderFileToUniformBufferVariables);
#endif // WITH_EDITOR

	FShaderType::Initialize(ShaderFileToUniformBufferVariables);
	FVertexFactoryType::Initialize(ShaderFileToUniformBufferVariables);

	FShaderPipelineType::Initialize();

	UE_LOGF(LogShaders, Log, "InitializeShaderTypes() end");
}

/**
 * Flushes the shader file and CRC cache, and regenerates the binary shader files if necessary.
 * Allows shader source files to be re-read properly even if they've been modified since startup.
 */
void FlushShaderFileCache()
{
	UE_LOGF(LogShaders, Log, "FlushShaderFileCache() begin");

	{
		FRWScopeLock ShaderHashAccessLock(GShaderHashAccessRWLock, SLT_Write);
		GShaderHashCache.Empty();
	}

	GShaderFileCache.Empty();

	UE_LOGF(LogShaders, Log, "FlushShaderFileCache() end");
}

void InvalidateShaderFileCacheEntry(const TCHAR* InVirtualFilePath, EShaderPlatform InShaderPlatform, const FName* InShaderPlatformName)
{
	FString VirtualFilePath(InVirtualFilePath);
	FixupShaderFilePath(VirtualFilePath, InShaderPlatform, InShaderPlatformName);

	GShaderFileCache.Remove(VirtualFilePath);

	{
		FRWScopeLock ShaderHashAccessLock(GShaderHashAccessRWLock, SLT_Write);
		GShaderHashCache.RemoveHash(InShaderPlatform, VirtualFilePath);
	}
}

#if WITH_EDITOR

void UpdateReferencedUniformBufferNames(
	TArrayView<const FShaderType*> OutdatedShaderTypes,
	TArrayView<const FVertexFactoryType*> OutdatedFactoryTypes,
	TArrayView<const FShaderPipelineType*> OutdatedShaderPipelineTypes)
{
	if (!FPlatformProperties::RequiresCookedData())
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UpdateReferencedUniformBufferNames);

		LogShaderSourceDirectoryMappings();

		TMap<FString, TArray<const TCHAR*> > ShaderFileToUniformBufferVariables;
		BuildShaderFileToUniformBufferMap(ShaderFileToUniformBufferVariables);

		for (const FShaderPipelineType* PipelineType : OutdatedShaderPipelineTypes)
		{
			for (const FShaderType* ShaderType : PipelineType->GetStages())
			{
				const_cast<FShaderType*>(ShaderType)->UpdateReferencedUniformBufferNames(ShaderFileToUniformBufferVariables);
			}
		}

		for (const FShaderType* ShaderType : OutdatedShaderTypes)
		{
			const_cast<FShaderType*>(ShaderType)->UpdateReferencedUniformBufferNames(ShaderFileToUniformBufferVariables);
		}

		for (const FVertexFactoryType* VertexFactoryType : OutdatedFactoryTypes)
		{
			const_cast<FVertexFactoryType*>(VertexFactoryType)->UpdateReferencedUniformBufferNames(ShaderFileToUniformBufferVariables);
		}
	}
}

void GenerateReferencedUniformBuffers(
	const TCHAR* SourceFilename,
	const TCHAR* ShaderTypeName,
	const TMap<FString, TArray<const TCHAR*> >& ShaderFileToUniformBufferVariables,
	TSet<const FShaderParametersMetadata*>& UniformBuffers)
{
	TArray<FString> FilesToSearch;
	GetShaderIncludes(SourceFilename, SourceFilename, FilesToSearch, GMaxRHIShaderPlatform);
	FilesToSearch.Emplace(SourceFilename);

	for (const FString& FileToSearch : FilesToSearch)
	{
		const TArray<const TCHAR*>& FoundUniformBufferVariables = ShaderFileToUniformBufferVariables.FindChecked(FileToSearch);
		for (const TCHAR* UniformBufferName : FoundUniformBufferVariables)
		{
			const FShaderParametersMetadata* UniformBufferStruct = FindUniformBufferStructByName(UniformBufferName);
			if (UniformBufferStruct)
			{
				UniformBuffers.Emplace(UniformBufferStruct);
			}
		}
	}
}

namespace {
// anonymous namespace

class FFrozenMaterialLayoutHashCache
{
public:
	FSHAHash Get(const FTypeLayoutDesc& TypeDesc, FPlatformTypeLayoutParameters LayoutParams)
	{
		{
			FReadScopeLock ReadScope(Lock);

			if (const FPlatformCache* Platform = Algo::FindBy(Platforms, LayoutParams, &FPlatformCache::Parameters))
			{
				if (const FSHAHash* Hash = Platform->Cache.Find(&TypeDesc))
				{
					return *Hash;
				}
			}
		}

		FSHAHash Hash = Freeze::HashLayout(TypeDesc, LayoutParams);

		{
			FWriteScopeLock WriteScope(Lock);

			FPlatformCache* Platform = Algo::FindBy(Platforms, LayoutParams, &FPlatformCache::Parameters);
			if (!Platform)
			{
				Platform = &Platforms.AddDefaulted_GetRef();
				Platform->Parameters = LayoutParams;
			}


			Platform->Cache.FindOrAdd(&TypeDesc, Hash);
		}

		return Hash;
	}

private:
	struct FPlatformCache
	{
		FPlatformTypeLayoutParameters Parameters;
		TMap<const FTypeLayoutDesc*, FSHAHash> Cache;
	};

	FRWLock Lock;
	TArray<FPlatformCache, TInlineAllocator<8>> Platforms;
};

} // anonymous namespace


FSHAHash GetShaderTypeLayoutHash(const FTypeLayoutDesc& TypeDesc, FPlatformTypeLayoutParameters LayoutParameters)
{
	static FFrozenMaterialLayoutHashCache GFrozenMaterialLayoutHashes;
	return GFrozenMaterialLayoutHashes.Get(TypeDesc, LayoutParameters);
}

void AppendKeyStringShaderDependencies(
	TConstArrayView<FShaderTypeDependency> ShaderTypeDependencies,
	FPlatformTypeLayoutParameters LayoutParams,
	FString& OutKeyString,
	bool bIncludeSourceHashes)
{
	// Simplified interface if we only have ShaderTypeDependencies
	AppendKeyStringShaderDependencies(
		ShaderTypeDependencies,
		TConstArrayView<FShaderPipelineTypeDependency>(),
		TConstArrayView<FVertexFactoryTypeDependency>(),
		LayoutParams,
		OutKeyString,
		bIncludeSourceHashes);
}

void AppendKeyStringShaderDependencies(
	TConstArrayView<FShaderTypeDependency> ShaderTypeDependencies,
	TConstArrayView<FShaderPipelineTypeDependency> ShaderPipelineTypeDependencies,
	TConstArrayView<FVertexFactoryTypeDependency> VertexFactoryTypeDependencies,
	FPlatformTypeLayoutParameters LayoutParams,
	FString& OutKeyString,
	bool bIncludeSourceHashes)
{
	FShaderKeyGenerator KeyGen(OutKeyString);
	AppendShaderDependencies(KeyGen, ShaderTypeDependencies, ShaderPipelineTypeDependencies,
		VertexFactoryTypeDependencies, LayoutParams, bIncludeSourceHashes);
}

void AppendShaderDependencies(
	FShaderKeyGenerator& KeyGen,
	TConstArrayView<FShaderTypeDependency> ShaderTypeDependencies,
	TConstArrayView<FShaderPipelineTypeDependency> ShaderPipelineTypeDependencies,
	TConstArrayView<FVertexFactoryTypeDependency> VertexFactoryTypeDependencies,
	FPlatformTypeLayoutParameters LayoutParams,
	bool bIncludeSourceHashes)
{
	FMemMark MemMark(FMemStack::Get());
	using FMemStackSetAllocator = TSetAllocator<TSparseArrayAllocator<TMemStackAllocator<>, TMemStackAllocator<>>, TMemStackAllocator<>>;
	TSet<const FShaderParametersMetadata*, DefaultKeyFuncs<const FShaderParametersMetadata*>, FMemStackSetAllocator> ReferencedUniformBuffers;
	ReferencedUniformBuffers.Reserve(128);

	const FShaderType* LastShaderType = nullptr;
	for (const FShaderTypeDependency& ShaderTypeDependency : ShaderTypeDependencies)
	{
		const FShaderType* ShaderType = FindShaderTypeByName(ShaderTypeDependency.ShaderTypeName);
		if (ShaderType == nullptr)
		{
			// If we're reading a serialized dependency that no longer exists, simply continue.
			// This will generate a new key and invalidate the asset, causing the asset to be re-cooked.
			UE_LOGF(
				LogShaders, Display, "Failed to find FShaderType for dependency %s (total in the NameToTypeMap: %d)",
				ShaderTypeDependency.ShaderTypeName.GetDebugString().String.Get(), FShaderType::GetNameToTypeMap().Num()
			);
			continue;
		}

		KeyGen.AppendSeparator();
		KeyGen.Append(ShaderType->GetName());
		KeyGen.Append(ShaderTypeDependency.PermutationId);
		KeyGen.AppendSeparator();
		ERayTracingPayloadType RayTracingPayloadType = ShaderType->GetRayTracingPayloadType(ShaderTypeDependency.PermutationId);
		KeyGen.Append(static_cast<uint32>(RayTracingPayloadType));
		KeyGen.AppendSeparator();
		KeyGen.Append(GetRayTracingPayloadTypeMaxSize(RayTracingPayloadType));

		if (bIncludeSourceHashes)
		{
			// Add the type's source hash so that we can invalidate cached shaders when .usf changes are made
			KeyGen.Append(ShaderTypeDependency.SourceHash);
		}

		if (const FShaderParametersMetadata* ParameterStructMetadata = ShaderType->GetRootParametersMetadata())
		{
			ParameterStructMetadata->Append(KeyGen);
		}

		KeyGen.Append(GetShaderTypeLayoutHash(ShaderType->GetLayout(), LayoutParams));

		if (ShaderType != LastShaderType)
		{
			for (const FShaderParametersMetadata* UniformBuffer : ShaderType->GetReferencedUniformBuffers())
			{
				ReferencedUniformBuffers.Add(UniformBuffer);
			}

			LastShaderType = ShaderType;
		}
	}

	// Add the inputs for any shader pipelines that are stored inline in the shader map
	for (const FShaderPipelineTypeDependency& Dependency : ShaderPipelineTypeDependencies)
	{
		const FShaderPipelineType* ShaderPipelineType = FShaderPipelineType::GetShaderPipelineTypeByName(Dependency.ShaderPipelineTypeName);
		if (ShaderPipelineType == nullptr)
		{
			// If we're reading a serialized dependency that no longer exists, simply continue.
			// This will generate a new key and invalidate the asset, causing the asset to be re-cooked.
			UE_LOGF(
				LogShaders, Display, "Failed to find FShaderPipelineType for dependency %s (total in the NameToTypeMap: %d)",
				Dependency.ShaderPipelineTypeName.GetDebugString().String.Get(), FShaderType::GetNameToTypeMap().Num()
			);
			continue;
		}

		KeyGen.AppendSeparator();
		KeyGen.Append(ShaderPipelineType->GetName());

		if (bIncludeSourceHashes)
		{
			KeyGen.Append(Dependency.StagesSourceHash);
		}

		for (const FShaderType* ShaderType : ShaderPipelineType->GetStages())
		{
			if (const FShaderParametersMetadata* ParameterStructMetadata = ShaderType->GetRootParametersMetadata())
			{
				ParameterStructMetadata->Append(KeyGen);
			}

			for (const FShaderParametersMetadata* UniformBuffer : ShaderType->GetReferencedUniformBuffers())
			{
				ReferencedUniformBuffers.Add(UniformBuffer);
			}
		}
	}

	for (const FVertexFactoryTypeDependency& VFDependency : VertexFactoryTypeDependencies)
	{
		KeyGen.AppendSeparator();

		const FVertexFactoryType* VertexFactoryType = FVertexFactoryType::GetVFByName(VFDependency.VertexFactoryTypeName);

		KeyGen.Append(VertexFactoryType->GetName());

		if (bIncludeSourceHashes)
		{
			KeyGen.Append(VFDependency.VFSourceHash);
		}

		for (int32 Frequency = 0; Frequency < SF_NumFrequencies; Frequency++)
		{
			const FTypeLayoutDesc* ParameterLayout = VertexFactoryType->GetShaderParameterLayout((EShaderFrequency)Frequency);
			if (ParameterLayout)
			{
				const FSHAHash LayoutHash = GetShaderTypeLayoutHash(*ParameterLayout, LayoutParams);
				KeyGen.Append(LayoutHash);
			}
		}

		for (const FShaderParametersMetadata* UniformBuffer : VertexFactoryType->GetReferencedUniformBuffers())
		{
			ReferencedUniformBuffers.Add(UniformBuffer);
		}
	}

	struct FUbSortByLayoutSignature
	{
		bool operator()(const FShaderParametersMetadata& A, const FShaderParametersMetadata& B) const
		{
			return A.GetLayoutSignature() < B.GetLayoutSignature();
		}
	};
	// sort the referenced uniform buffers by the stable layout signature; for ddc keys we care about stability not alphabetical ordering by name
	ReferencedUniformBuffers.StableSort(FUbSortByLayoutSignature());

	// Save uniform buffer member info so we can detect when layout has changed
	for (const FShaderParametersMetadata* UniformBufferMetadata : ReferencedUniformBuffers)
	{
		UniformBufferMetadata->Append(KeyGen);
	}
}

#endif // WITH_EDITOR

FString MakeInjectedShaderCodeBlock(const TCHAR* BlockName, const FString& CodeToInject)
{
	return FString::Printf(TEXT("#line 1 \"%s\"\n%s"), BlockName, *CodeToInject);
}

FString FShaderCompilerInput::GetOrCreateShaderDebugInfoPath() const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FShaderCompilerInput::GetOrCreateShaderDebugInfoPath);

	FString OutDumpDebugInfoPath = FPaths::Combine(DumpDebugInfoRootPath, DebugGroupName + DebugExtension);

	// Sanitize the name to be used as a path
	// List mostly comes from set of characters not allowed by windows in a path.  Just try to rename a file and type one of these for the list.
	OutDumpDebugInfoPath.ReplaceInline(TEXT("<"), TEXT("("));
	OutDumpDebugInfoPath.ReplaceInline(TEXT(">"), TEXT(")"));
	OutDumpDebugInfoPath.ReplaceInline(TEXT("::"), TEXT("=="));
	OutDumpDebugInfoPath.ReplaceInline(TEXT("|"), TEXT("_"));
	OutDumpDebugInfoPath.ReplaceInline(TEXT("*"), TEXT("-"));
	OutDumpDebugInfoPath.ReplaceInline(TEXT("?"), TEXT("!"));
	OutDumpDebugInfoPath.ReplaceInline(TEXT("\""), TEXT("\'"));

	if (!IFileManager::Get().DirectoryExists(*OutDumpDebugInfoPath))
	{
		if (!IFileManager::Get().MakeDirectory(*OutDumpDebugInfoPath, true))
		{
			const uint32 ErrorCode = FPlatformMisc::GetLastError();
			UE_LOGF(LogShaders, Warning, "Last Error %u: Failed to create directory for shader debug info '%ls'. Try enabling large file paths or r.DumpShaderDebugShortNames.", ErrorCode, *OutDumpDebugInfoPath);
			return FString();
		}
	}

	return OutDumpDebugInfoPath;
}

FString FShaderCompilerError::GetErrorStringWithSourceLocation() const
{
	if (!ErrorVirtualFilePath.IsEmpty() && !ErrorLineString.IsEmpty())
	{
		return ErrorVirtualFilePath + TEXT("(") + ErrorLineString + TEXT("): ") + StrippedErrorMessage;
	}
	else
	{
		return StrippedErrorMessage;
	}
}

FString FShaderCompilerError::GetErrorStringWithLineMarker() const
{
	if (HasLineMarker())
	{
		// Append highlighted line and its marker to the same error message with line terminators
		// to get a similar multiline error output as with DXC
		return (GetErrorStringWithSourceLocation() + LINE_TERMINATOR + TEXT("\t") + HighlightedLine + LINE_TERMINATOR + TEXT("\t") + HighlightedLineMarker);
	}
	else
	{
		return GetErrorStringWithSourceLocation();
	}
}

FString FShaderCompilerError::GetErrorString(bool bOmitLineMarker) const
{
	return bOmitLineMarker ? GetErrorStringWithSourceLocation() : GetErrorStringWithLineMarker();
}

static bool ExtractSourceLocationFromCompilerMessage(FString& CompilerMessage, FString& OutFilePath, int32& OutRow, int32& OutColumn, const TCHAR* LeftBracket, const TCHAR* MiddleBracket, const TCHAR* RightBracket)
{
	// Ignore ':' character from absolute paths in Windows format
	const int32 StartPosition = (CompilerMessage.Len() >= 3 && FChar::IsAlpha(CompilerMessage[0]) && CompilerMessage[1] == TEXT(':') && (CompilerMessage[2] == TEXT('/') || CompilerMessage[2] == TEXT('\\')) ? 3 : 0);

	const int32 LeftBracketLen = FCString::Strlen(LeftBracket);
	const int32 LeftPosition = CompilerMessage.Find(LeftBracket, ESearchCase::IgnoreCase, ESearchDir::FromStart, StartPosition);
	if (LeftPosition == INDEX_NONE || LeftPosition == StartPosition || LeftPosition + LeftBracketLen >= CompilerMessage.Len() || !FChar::IsDigit(CompilerMessage[LeftPosition + LeftBracketLen]))
	{
		return false;
	}

	const int32 MiddleBracketLen = FCString::Strlen(MiddleBracket);
	const int32 MiddlePosition = CompilerMessage.Find(MiddleBracket, ESearchCase::IgnoreCase, ESearchDir::FromStart, LeftPosition + LeftBracketLen);
	if (MiddlePosition == INDEX_NONE || MiddlePosition + MiddleBracketLen >= CompilerMessage.Len() || !FChar::IsDigit(CompilerMessage[MiddlePosition + MiddleBracketLen]))
	{
		return false;
	}

	const int32 RightBracketLen = FCString::Strlen(RightBracket);
	const int32 RightPosition = CompilerMessage.Find(RightBracket, ESearchCase::IgnoreCase, ESearchDir::FromStart, MiddlePosition + MiddleBracketLen);
	if (RightPosition == INDEX_NONE || RightPosition >= CompilerMessage.Len())
	{
		return false;
	}

	// Extract file path, row, and column from compiler message
	OutFilePath = CompilerMessage.Left(LeftPosition);
	LexFromString(OutRow, *CompilerMessage.Mid(LeftPosition + LeftBracketLen, MiddlePosition - LeftPosition - LeftBracketLen));
	LexFromString(OutColumn, *CompilerMessage.Mid(MiddlePosition + MiddleBracketLen, RightPosition - MiddlePosition - MiddleBracketLen));

	// Remove extracted information from compiler message
	CompilerMessage = CompilerMessage.Right(CompilerMessage.Len() - RightPosition - RightBracketLen);

	return true;
}

bool FShaderCompilerError::ExtractSourceLocation()
{
	// Ignore this call if a file path and line string is already provided
	if (!StrippedErrorMessage.IsEmpty() && ErrorVirtualFilePath.IsEmpty() && ErrorLineString.IsEmpty())
	{
		auto ExtractSourceLineFromStrippedErrorMessage = [this](const TCHAR* LeftBracket, const TCHAR* MiddleBracket, const TCHAR* RightBracket) -> bool
		{
			int32 Row = 0, Column = 0;
			if (ExtractSourceLocationFromCompilerMessage(StrippedErrorMessage, ErrorVirtualFilePath, Row, Column, LeftBracket, MiddleBracket, RightBracket))
			{
				// Format error line string to MSVC format to be able to jump to the source location with a double click in VisualStudio
				ErrorLineString = FString::Printf(TEXT("%d,%d"), Row, Column);
				return true;
			}
			return false;
		};

		// Extract from Clang format, e.g. "Filename:3:12: error:"
		if (ExtractSourceLineFromStrippedErrorMessage(TEXT(":"), TEXT(":"), TEXT(": ")))
		{
			return true;
		}

		// Extract from MSVC format, e.g. "Filename(3,12) : error: "
		if (ExtractSourceLineFromStrippedErrorMessage(TEXT("("), TEXT(","), TEXT(") : ")))
		{
			return true;
		}
	}
	return false;
}

void FShaderCompilerError::ExtractSourceLocations(TArray<FShaderCompilerError>& InOutErrors)
{
	FString CurrentLine, CurrentColumn; // Local to loop but hoisted for performance.
	FString PreviousLine, PreviousColumn;

	for (int ErrorIndex = 0; ErrorIndex < InOutErrors.Num(); ++ErrorIndex)
	{
		FShaderCompilerError& CurrentError = InOutErrors[ErrorIndex];

		CurrentError.ExtractSourceLocation();

		if (!CurrentError.ErrorLineString.Split(TEXT(","), &CurrentLine, &CurrentColumn))
		{
			PreviousLine.Reset();
			PreviousColumn.Reset();
			continue;
		}

		if (!CurrentLine.IsNumeric() || !CurrentColumn.IsNumeric())
		{
			PreviousLine.Reset();
			PreviousColumn.Reset();
			continue;
		}

		// The shader compiler may omit line marker info after the first error for that line/column. Copy this information from the previous error
		// if the line/column matches.
		if (!CurrentError.HasLineMarker() && ErrorIndex > 0 && PreviousLine == CurrentLine && PreviousColumn == CurrentColumn)
		{
			FShaderCompilerError& PreviousError = InOutErrors[ErrorIndex - 1];
			if (PreviousError.HasLineMarker())
			{
				// Issue pertains to same code. Copy marker.
				CurrentError.HighlightedLine = PreviousError.HighlightedLine;
				CurrentError.HighlightedLineMarker = PreviousError.HighlightedLineMarker;
			}
		}

		PreviousLine = MoveTemp(CurrentLine);
		PreviousColumn = MoveTemp(CurrentColumn);
	}
}

FString FShaderCompilerError::GetShaderSourceFilePath(TArray<FShaderCompilerError>* InOutErrors) const
{
	// Always return error file path as-is if it doesn't denote a virtual path.
	// We don't wont to report errors when accessing a compile error's message.
	if (ErrorVirtualFilePath.IsEmpty() || ErrorVirtualFilePath[0] != TEXT('/'))
	{
		return ErrorVirtualFilePath;
	}
	else
	{
		return ::GetShaderSourceFilePath(ErrorVirtualFilePath, InOutErrors);
	}
}

const TMap<FString, FString>& AllShaderSourceDirectoryMappings()
{
	return GShaderSourceDirectoryMappings;
}

void ResetAllShaderSourceDirectoryMappings()
{
	GShaderSourceDirectoryMappings.Reset();
}

void AddShaderSourceDirectoryMapping(const FString& VirtualShaderDirectory, const FString& RealShaderDirectory)
{
	check(IsInGameThread());

	if (FPlatformProperties::RequiresCookedData() || !AllowShaderCompiling())
	{
		return;
	}

	// Do sanity checks of the virtual shader directory to map.
	checkf(
		VirtualShaderDirectory.StartsWith(TEXT("/")) &&
		!VirtualShaderDirectory.EndsWith(TEXT("/")) &&
		!VirtualShaderDirectory.Contains(FString(".")),
		TEXT("VirtualShaderDirectory = \"%s\""),
		*VirtualShaderDirectory
	);

	// Detect collisions with any other mappings.
	check(!GShaderSourceDirectoryMappings.Contains(VirtualShaderDirectory));

	// Make sure the real directory to map exists.
	bool bDirectoryExists = FPaths::DirectoryExists(RealShaderDirectory);
	if (!bDirectoryExists)
	{
		UE_LOGF(LogShaders, Log, "Directory %ls", *RealShaderDirectory);
	}
	checkf(bDirectoryExists, TEXT("FPaths::DirectoryExists(%s %s) %s"), *RealShaderDirectory, *FPaths::ConvertRelativePathToFull(RealShaderDirectory), FPlatformProcess::ComputerName());

	// Make sure the Generated directory does not exist, because is reserved for C++ generated shader source
	// by the FShaderCompilerEnvironment::IncludeVirtualPathToContentsMap member.
	checkf(!FPaths::DirectoryExists(RealShaderDirectory / TEXT("Generated")),
		TEXT("\"%s/Generated\" is not permitted to exist since C++ generated shader file would be mapped to this directory."), *RealShaderDirectory);

	UE_LOGF(LogShaders, Log, "Mapping virtual shader directory %ls to %ls",
		*VirtualShaderDirectory, *RealShaderDirectory);
	GShaderSourceDirectoryMappings.Add(VirtualShaderDirectory, RealShaderDirectory);
}

void AddShaderSourceSharedVirtualDirectory(const FString& VirtualShaderDirectory)
{
	check(IsInGameThread());
	if (FPlatformProperties::RequiresCookedData() || !AllowShaderCompiling())
	{
		return;
	}

	// Do sanity checks of the virtual shader directory to map.
	checkf(
		VirtualShaderDirectory.StartsWith(TEXT("/")) &&
		VirtualShaderDirectory.EndsWith(TEXT("/")) &&
		!VirtualShaderDirectory.Contains(FString(".")),
		TEXT("Shared VirtualShaderDirectory = \"%s\" must start and end with '/' and contain no '.' characters."),
		*VirtualShaderDirectory
	);

	// Detect collisions with any other mappings.
	check(!GShaderSourceSharedVirtualDirectories.Contains(VirtualShaderDirectory));

	// Add to the list of shared directories
	GShaderSourceSharedVirtualDirectories.Add(VirtualShaderDirectory);
}

void FShaderCode::Compress(FName ShaderCompressionFormat, FOodleDataCompression::ECompressor InOodleCompressor, FOodleDataCompression::ECompressionLevel InOodleLevel)
{
	checkf(OptionalDataSize == -1, TEXT("FShaderCode::Compress() was called before calling FShaderCode::FinalizeShaderCode()"));

	check(ShaderCompressionFormat == NAME_Oodle); // We now force shaders to compress with oodle (even if they are uncompressed)

	TArray<uint8> Compressed;
	// conventional formats will fail if the compressed size isn't enough, Oodle needs a more precise estimate
	TConstArrayView<uint8> Code = ShaderCodeResource.GetCodeView();
	int32 CompressedSize = (ShaderCompressionFormat != NAME_Oodle) ? Code.NumBytes() : FOodleDataCompression::CompressedBufferSizeNeeded(Code.NumBytes());
	Compressed.AddUninitialized(CompressedSize);

	// non-Oodle format names use the old API, for NAME_Oodle we replace the call with the custom invocation
	bool bCompressed = false;
	if (ShaderCompressionFormat != NAME_Oodle)
	{
		bCompressed = FCompression::CompressMemory(ShaderCompressionFormat, Compressed.GetData(), CompressedSize, Code.GetData(), Code.Num(), COMPRESS_BiasSize);
	}
	else
	{
		CompressedSize = FOodleDataCompression::Compress(Compressed.GetData(), CompressedSize, Code.GetData(), Code.Num(),
			InOodleCompressor, InOodleLevel);
		bCompressed = CompressedSize != 0;
	}

	// there is code that assumes that if CompressedSize == CodeSize, the shader isn't compressed. Because of that, do not accept equal compressed size (very unlikely anyway)
	if (bCompressed && CompressedSize < Code.Num())
	{
		// cache the ShaderCodeSize since it will no longer possible to get it as the reader will fail to parse the compressed data
		FShaderCodeReader Wrapper(Code);
		ShaderCodeSize = Wrapper.GetShaderCodeSize();
		checkf(ShaderCodeSize >= 0, TEXT("Unable to determine ShaderCodeSize from uncompressed code"));

		// finalize the compression
		CompressionFormat = ShaderCompressionFormat;
		OodleCompressor = InOodleCompressor;
		OodleLevel = InOodleLevel;
		UncompressedSize = Code.Num();

		Compressed.SetNum(CompressedSize);
		ShaderCodeResource.Code = MakeSharedBufferFromArray(MoveTemp(Compressed));
	}
}

const FShaderCodeResource& FShaderCode::GetFinalizedResource(EShaderFrequency Frequency, FShaderHash OutputHash) const
{
	// shader code must be finalized prior to calling this function
	// the finalize process will have created the code FSharedBuffer on the resource already
	// Log the actual values on failure to distinguish a logic bug (plausible OptionalDataSize/Frequency)
	// from memory corruption (e.g. 0xcdcdcdcd / out-of-range Frequency).
	checkf(OptionalDataSize == -1,
		TEXT("FShaderCode not finalized when GetFinalizedResource was called. OptionalDataSize=%d (0x%08x), Frequency=%s, OutputHash=%s, HeaderPopulated=%d, CodeSize=%llu, UncompressedSize=%d"),
		OptionalDataSize, (uint32)OptionalDataSize,
		GetShaderFrequencyString(Frequency), *OutputHash.ToString(),
		ShaderCodeResource.Header ? 1 : 0,
		ShaderCodeResource.Code.GetSize(),
		UncompressedSize);

	// If the header is already populated, resource has already been finalized, early out
	if (ShaderCodeResource.Header)
	{
		// sanity check
		const EShaderFrequency HeaderFrequency = ShaderCodeResource.GetFrequency();
		checkf(HeaderFrequency == Frequency,
			TEXT("FShaderCodeResource frequency mismatch. Header.Frequency=%s (%d), expected Frequency=%s (%d), OutputHash=%s, Header.UncompressedSize=%d, Header.ShaderCodeSize=%d, CodeSize=%llu, OptionalDataSize=%d"),
			GetShaderFrequencyString(HeaderFrequency), (int32)HeaderFrequency,
			GetShaderFrequencyString(Frequency), (int32)Frequency,
			*OutputHash.ToString(),
			ShaderCodeResource.GetUncompressedSize(),
			ShaderCodeResource.GetShaderCodeSize(),
			ShaderCodeResource.Code.GetSize(),
			OptionalDataSize);
		return ShaderCodeResource;
	}

	// Validate that compression settings used for this ShaderCode by the compilation process match what is expected
	FName ShaderCompressionFormat = GetShaderCompressionFormat();
	if (ShaderCompressionFormat != NAME_None)
	{
		// we trust that SCWs also obeyed by the same CVar, so we expect a compressed shader code at this point
		// However, if we see an uncompressed shader, it perhaps means that SCW tried to compress it, but the result was worse than uncompressed. 
		// Because of that we special-case NAME_None here
		if (ShaderCompressionFormat != GetCompressionFormat())
		{
			if (GetCompressionFormat() != NAME_None)
			{
				UE_LOGF(LogShaders, Fatal, "Shader %ls is expected to be compressed with %ls, but it is compressed with %ls instead.",
					*OutputHash.ToString(),
					*ShaderCompressionFormat.ToString(),
					*GetCompressionFormat().ToString()
				);
				// unreachable
				return ShaderCodeResource;
			}

			// assume uncompressed due to worse ratio than the compression
			UE_LOGF(LogShaders, Verbose, "Shader %ls is expected to be compressed with %ls, but it arrived uncompressed (size=%d). Assuming compressing made it longer and storing uncompressed.",
				*OutputHash.ToString(),
				*ShaderCompressionFormat.ToString(),
				ShaderCodeWithOptionalData.Num()
			);
		}
		else if (ShaderCompressionFormat == NAME_Oodle)
		{
			// check if Oodle-specific settings match
			FOodleDataCompression::ECompressor OodleCompressorSetting;
			FOodleDataCompression::ECompressionLevel OodleLevelSetting;
			GetShaderCompressionOodleSettings(OodleCompressorSetting, OodleLevelSetting);

			if (GetOodleCompressor() != OodleCompressorSetting || GetOodleLevel() != OodleLevelSetting)
			{
				UE_LOGF(LogShaders, Fatal, "Shader %ls is expected to be compressed with Oodle compressor %d level %d, but it is compressed with compressor %d level %d instead.",
					*OutputHash.ToString(),
					static_cast<int32>(OodleCompressorSetting),
					static_cast<int32>(OodleLevelSetting),
					static_cast<int32>(GetOodleCompressor()),
					static_cast<int32>(GetOodleLevel())
				);
				// unreachable
				return ShaderCodeResource;
			}
		}
	}

	// Shader library/shader map usage expects uncompressed size to be set to the full code buffer size if uncompressed; so we need to apply that
	// transformation here (and reverse it when populating from a FShaderCodeResource, see mirroring code in SetFromResource below)
	ShaderCodeResource.PopulateHeader(UncompressedSize == 0 ? static_cast<int32>(ShaderCodeResource.Code.GetSize()) : UncompressedSize, GetShaderCodeSize(), Frequency);
	return ShaderCodeResource;
}


FArchive& operator<<(FArchive& Ar, FSharedBuffer& Buffer)
{
	uint64 Len = Buffer.GetSize();
	Ar << Len;

	if (Ar.IsLoading())
	{
		Buffer.Reset();

		if (Len > 0)
		{
			FUniqueBuffer BufTmp = FUniqueBuffer::Alloc(Len);
			Ar.Serialize(BufTmp.GetData(), Len);
			Buffer = BufTmp.MoveToShared();
		}
	}
	else if (Ar.IsSaving())
	{
		if (Len > 0)
		{
			Ar.Serialize(const_cast<void*>(Buffer.GetData()), Len);
		}
	}

	return Ar;
}

FArchive& operator<<(FArchive& Ar, FShaderCodeResource& Resource)
{
	Ar << Resource.Header;
	Ar << Resource.Code;
	Ar << Resource.Symbols;
	return Ar;
}

FArchive& operator<<(FArchive& Ar, FShaderCode& Output)
{
	if (Ar.IsLoading())
	{
		Output.OptionalDataSize = -1;
	}
	else
	{
		Output.FinalizeShaderCode();
	}

	// Note: this serialize is used to pass between UE and the shader compile worker, recompile both when modifying
	FSharedBuffer& CodeBuffer = Output.ShaderCodeResource.Code;
	Ar << CodeBuffer;

	FCompressedBuffer& SymbolsBuffer = Output.ShaderCodeResource.Symbols;
	Ar << SymbolsBuffer;

	Ar << Output.UncompressedSize;
	{
		FString CompressionFormatString(Output.CompressionFormat.ToString());
		Ar << CompressionFormatString;
		Output.CompressionFormat = FName(*CompressionFormatString);
	}
	Ar << reinterpret_cast<uint8&>(Output.OodleCompressor);
	Ar << reinterpret_cast<uint8&>(Output.OodleLevel);
	Ar << Output.ShaderCodeSize;
	checkf(Output.UncompressedSize == 0 || Output.ShaderCodeSize > 0, TEXT("FShaderCode::operator<<(): invalid shader code size for a compressed shader: ShaderCodeSize=%d, UncompressedSize=%d"), Output.ShaderCodeSize, Output.UncompressedSize);
	return Ar;
}

FArchive& operator<<(FArchive& Ar, FShaderCompilerInput& Input)
{
	auto SerializeName = [&Ar](FName& Name)
		{
			FString NameString(Name.ToString());
			Ar << NameString;
			Name = FName(*NameString);
		};

	// Note: this serialize is used to pass between UE and the shader compile worker, recompile both when modifying
	Ar << Input.Target;
	SerializeName(Input.ShaderFormat);
	SerializeName(Input.CompressionFormat);
	SerializeName(Input.ShaderPlatformName);
	SerializeName(Input.PreferredShaderCodeFormat);
	Ar << Input.VirtualSourceFilePath;
	Ar << Input.EntryPointName;
	Ar << Input.ShaderName;
	Ar << Input.SupportedHardwareMask;
	Ar << Input.UsedOutputs;
	// VSDisallowedUniformBufferNames and NonVSDisallowedUniformBufferNames are not used in FShaderCompileJob::GetInputHash because these do not influence
	// the result of the shader compilation and are only there to perform validation wrt what material uniform buffer is used
	Ar << Input.VSDisallowedUniformBufferNames;
	Ar << Input.NonVSDisallowedUniformBufferNames;
	Ar << Input.DumpDebugInfoRootPath;
	Ar << Input.DumpDebugInfoPath;
	Ar << Input.DebugInfoFlags;
	Ar << Input.DebugExtension;
	Ar << Input.DebugGroupName;
	Ar << Input.Hash;
	Input.Environment.SerializeCompilationDependencies(Ar);
	Ar << Input.ExtraSettings;
	Ar << reinterpret_cast<uint8&>(Input.OodleCompressor);
	Ar << reinterpret_cast<uint8&>(Input.OodleLevel);
	Ar << Input.bCompilingForShaderPipeline;
	Ar << Input.bIncludeUsedOutputs;
	Ar << Input.bBindlessEnabled;

	// Note: skipping Input.SharedEnvironment, which is handled by FShaderCompileUtilities::DoWriteTasks in order to maintain sharing

	return Ar;
}

const TCHAR* LexToString(EShaderCompileJobStatus Status)
{
	switch (Status)
	{
	case EShaderCompileJobStatus::Unset: return TEXT("Unset");
	case EShaderCompileJobStatus::Ready: return TEXT("Ready");
	case EShaderCompileJobStatus::Skipped: return TEXT("Skipped");
	case EShaderCompileJobStatus::Cancelled: return TEXT("Cancelled");
	case EShaderCompileJobStatus::PendingDDC: return TEXT("PendingDDC");
	case EShaderCompileJobStatus::Queued: return TEXT("Queued");
	case EShaderCompileJobStatus::Pending: return TEXT("Pending");
	case EShaderCompileJobStatus::Completed: return TEXT("Completed");
	case EShaderCompileJobStatus::Released: return TEXT("Released");
	default: return TEXT("(unknown)");
	}
}

const TCHAR* LexToString(EShaderCompileJobCompletionMethod Method)
{
	switch (Method)
	{
	case EShaderCompileJobCompletionMethod::Unset: return TEXT("Unset");
	case EShaderCompileJobCompletionMethod::CacheHit: return TEXT("CacheHit");
	case EShaderCompileJobCompletionMethod::DdcHit: return TEXT("DdcHit");
	case EShaderCompileJobCompletionMethod::Duplicate: return TEXT("Duplicate");
	case EShaderCompileJobCompletionMethod::LocalExecution: return TEXT("LocalExecution");
	case EShaderCompileJobCompletionMethod::DistributedExecution: return TEXT("DistributedExecution");
	default: return TEXT("(unknown)");
	}
}

FShaderCompilerInputHash FShaderPipelineCompileJob::GetInputHash()
{
	if (bInputHashSet)
	{
		return InputHash;
	}
	static_assert(sizeof(FShaderCompilerInputHash) == 32);
	int256 CombinedHash = 0u;
	for (int32 Index = 0; Index < StageJobs.Num(); ++Index)
	{
		if (StageJobs[Index])
		{
			const FShaderCompilerInputHash StageHash = StageJobs[Index]->GetInputHash();
			const FShaderCompilerInputHash::ByteArray& StageHashBytes = StageHash.GetBytes();
			static_assert(sizeof(StageHashBytes) == sizeof(int256));
			CombinedHash += int256(StageHashBytes, sizeof(StageHashBytes));
		}
	}

	InputHash = FShaderCompilerInputHash(reinterpret_cast<FShaderCompilerInputHash::ByteArray&>(*CombinedHash.GetBits()));
	bInputHashSet = true;
	return InputHash;
}

// The Id of 0 is reserved for global shaders
FThreadSafeCounter FShaderCommonCompileJob::JobIdCounter(2);

uint32 FShaderCommonCompileJob::GetNextJobId()
{
	uint32 Id = JobIdCounter.Increment();
	if (Id == UINT_MAX)
	{
		JobIdCounter.Set(2);
	}
	return Id;
}

FString FShaderCompileJobKey::ToString() const
{
	return FString::Printf(
		TEXT("ShaderType:%s VertexFactoryType:%s PermutationId:%d"),
		ShaderType ? ShaderType->GetName() : TEXT("None"),
		VFType ? VFType->GetName() : TEXT("None"),
		PermutationId);
}

struct FShaderVirtualFileContents
{
	const FString* Wide;
	const TArray<ANSICHAR>* Ansi;

	FShaderVirtualFileContents(const FString* InWide)
		: Wide(InWide), Ansi(nullptr)
	{}

	FShaderVirtualFileContents(const TArray<ANSICHAR>* InAnsi)
		: Wide(nullptr), Ansi(InAnsi)
	{}
};

FShaderCompilerInputHash FShaderCompileJob::GetInputHash()
{
	if (bInputHashSet)
	{
		return InputHash;
	}

	FMemoryHasherBlake3 Hasher;

	FGuid ShaderCacheVersionLocal = UE_SHADER_CACHE_VERSION;
	Hasher << ShaderCacheVersionLocal;

	uint32 FormatVersion = GetTargetPlatformManagerRef().ShaderFormatVersion(Input.ShaderFormat);
	Hasher << FormatVersion;

	Hasher << Input.PreferredShaderCodeFormat;
	
	FShaderTarget Target = Input.Target;
	Hasher << Target;
	Hasher << Input.EntryPointName;

	// Include this flag, so shader pipeline jobs get a different hash from single shader jobs, even if the preprocessed shader is otherwise the same.
	Hasher << Input.bCompilingForShaderPipeline;

	Hasher << Input.bBindlessEnabled;

	FShaderCompilerEnvironment MergedEnvironment = Input.Environment;
	if (Input.SharedEnvironment)
	{
		MergedEnvironment.Merge(*Input.SharedEnvironment);
	}
	MergedEnvironment.SerializeCompilationDependencies(Hasher);
	
	auto HashDirectives = [&Hasher](const FString* Directive)
	{
		check(Directive && !Directive->IsEmpty());
		// const_cast due to serialization API requiring non-const. better than not having const correctness in the API.
		Hasher << const_cast<FString&>(*Directive);
	};
	// Hash all UESHADERMETADATA_ directives encountered during preprocessing (assume these may be used to modify compilation behaviour)
	PreprocessOutput.VisitDirectives(HashDirectives);

	Hasher << PreprocessOutput.EditSource();
	if (SecondaryPreprocessOutput.IsValid())
	{
		Hasher << SecondaryPreprocessOutput->EditSource();
	}

	if (Input.RootParametersStructure)
	{
		FBlake3Hash LayoutSignature = Input.RootParametersStructure->GetLayoutSignature();
		Hasher << LayoutSignature;
	}

	InputHash = Hasher.Finalize();

	bInputHashSet = true;
	Input.Hash = InputHash;
	return InputHash;
}

void FShaderCompileJob::SerializeOutput(FShaderCacheSerializeContext& Ctx, int32 CodeIndex)
{
	Output.bSerializingForCache = true;
	FArchive& Ar = Ctx.GetMainArchive();
	bool bIsSaving = Ar.IsSaving();
	bool bIsLoading = Ar.IsLoading();

	// Snapshot for the inconsistency check below: if Output.Target.Frequency drifts between the
	// Ar << Output write (which serializes Output.Target into ShaderObjectData) and the
	// GetFinalizedCodeResource call (which stamps the Header from current Output.Target.Frequency),
	// the cache record ends up internally inconsistent and trips the GetFrequency() == Frequency
	// assert on a future load. Hunt for the producer.
	const EShaderFrequency PreSaveFrequency = bIsSaving ? Output.Target.GetFrequency() : SF_NumFrequencies;

	Ar << Output;

	FShaderCodeResource CodeResource;
	if (bIsSaving)
	{
		CodeResource = Output.GetFinalizedCodeResource();
	}

	check(Ctx.EnableCustomCodeSerialize());
	Ctx.SerializeCode(CodeResource, CodeIndex);

	if (bIsSaving)
	{
		const EShaderFrequency HeaderFrequency = CodeResource.GetFrequency();
		const EShaderFrequency CurrentFrequency = Output.Target.GetFrequency();
		const EShaderFrequency InputFrequency = Input.Target.GetFrequency();
		checkf(PreSaveFrequency == HeaderFrequency && PreSaveFrequency == CurrentFrequency && PreSaveFrequency == InputFrequency,
			TEXT("Inconsistent shader code cache record being written. PreSave=%s Header=%s Current=%s Input=%s, InputHash=%s, OutputHash=%s"),
			GetShaderFrequencyString(PreSaveFrequency), GetShaderFrequencyString(HeaderFrequency),
			GetShaderFrequencyString(CurrentFrequency), GetShaderFrequencyString(InputFrequency),
			*LexToString(InputHash), *Output.OutputHash.ToString());
	}

	// we intentionally re-set the internal ShaderCode even when saving; GetCodeResource moves the code array into the
	// FShaderCodeResource's internal array and this moves it back (preventing an unnecessary temporary copy of the code)
	Output.SetCodeFromResource(MoveTemp(CodeResource));
	
	// output hash is now serialized as part of the output, as the shader code is compressed in SCWs
	checkf(!Output.bSucceeded || Output.OutputHash != FShaderHash(), TEXT("Successful compile job does not have an OutputHash generated."));
	checkf(Output.Target == Input.Target, TEXT("Output FShaderTarget does not match the input struct; incorrect results associated with job?"));

	if (bIsLoading)
	{
		bSucceeded = Output.bSucceeded;
	}
}

void FShaderCompileJob::OnComplete(FShaderDebugDataContext& Ctx)
{
	const IShaderFormat* ShaderFormat = nullptr;
	FCompressedBuffer SymbolsBuffer = Output.bSucceeded ? Output.GetFinalizedCodeResource().GetSymbolsBuffer() : FCompressedBuffer();
	bool bHasSecondary = SecondaryOutput.IsValid();

	const bool bHasSymbols = SymbolsBuffer.GetRawSize() > 0;
	if (bHasSymbols || Input.DumpDebugInfoEnabled() || bHasSecondary)
	{
		// don't bother looking up the shader format unless needed, this can have non-trivial cost due to thread safety
		// and the possibility of available target platforms needing to be refreshed
		ShaderFormat = GetTargetPlatformManagerRef().FindShaderFormat(Input.ShaderFormat);
	}

	// For jobs which applied source stripping, we need to remap error messages whether or not the job was actually the one that ran
	// the compilation step.
	if (!Input.Environment.CompilerFlags.Contains(CFLAG_DisableSourceStripping))
	{
		PreprocessOutput.RemapErrors(Output);
		if (bHasSecondary)
		{
			check(SecondaryPreprocessOutput.IsValid());
			SecondaryPreprocessOutput->RemapErrors(*SecondaryOutput);
		}
	}

	if (bHasSecondary)
	{
		check(ShaderFormat);

		// append a prefix to secondary job error messages to distinguish them from the primary compilation
		FStringView SecondaryName = ShaderFormat->GetSecondaryCompileName();
		if (!SecondaryName.IsEmpty())
		{
			TStringBuilder<128> Prefix;
			Prefix << "(" << SecondaryName << ") ";
			for (FShaderCompilerError& Error : SecondaryOutput->Errors)
			{
				Error.StrippedErrorMessage.InsertAt(0, Prefix.ToString());
			}
		}
		Output.Errors.Append(SecondaryOutput->Errors);
	}

	if (Input.NeedsOriginalShaderSource())
	{
		// Decompress the code if needed by debug info or source extraction
		PreprocessOutput.DecompressCode();
		if (SecondaryPreprocessOutput.IsValid())
		{
			SecondaryPreprocessOutput->DecompressCode();
		}
	}

	auto AppendPreprocessInfo = [this](const FShaderPreprocessOutput& InPreprocessOutput)
		{
			// intentionally adding to main output's diagnostic data array and preprocess time here; at this point 
			// outputs are already merged and we want to store the merged results of both of these in the cache.
			Output.PreprocessTime += InPreprocessOutput.PreprocessTime;
			Output.ShaderDiagnosticDatas.Append(InPreprocessOutput.GetDiagnosticDatas());
		};

	AppendPreprocessInfo(PreprocessOutput);
	if (SecondaryPreprocessOutput.IsValid())
	{
		AppendPreprocessInfo(*SecondaryPreprocessOutput);
	}

	// output debug info for the job if either the job was the one which actually executed the compile, or debug info is requested 
	// for all jobs including ones which hit the job cache (or matched and was deduplicated with another in-flight job)
	// note that depending on shaderformat implementation this may not necessarily be _all_ debug artifacts, since some require
	// running compilation to generate (these will always be output by the compile step, and always only be generated for the single
	// job which executed compilation)
	if (Input.DumpDebugInfoEnabled())
	{
		bool bShouldDump = CVarDumpDebugInfoForCacheHits.GetValueOnAnyThread() || !JobStatusPtr->WasCompilationSkipped();

		check(ShaderFormat);
		if (!PreprocessOutput.bSucceeded)
		{
			// output minimal debug info for preprocessing failures
			ShaderFormat->OutputDebugDataMinimal(Input, Ctx);
		}
		else if (bShouldDump)
			// if we only want debug info for jobs which actually compiled, check the CompileTime
			// (jobs deserialized from the cache/wait list/ddc will have a compiletime of 0.0)
		{
			if (SecondaryPreprocessOutput.IsValid() && SecondaryOutput.IsValid())
			{
				ShaderFormat->OutputDebugData(Input, PreprocessOutput, *SecondaryPreprocessOutput, Output, *SecondaryOutput, Ctx);
			}
			else
			{
				ShaderFormat->OutputDebugData(Input, PreprocessOutput, Output, Ctx);
			}
		}

		if (bShouldDump && !Ctx.bIsPipeline)
		{
			// only need to output the debug compile input/cmdline here if we're not part of a pipeline, otherwise 
			// it's done at the pipeline level and symlinked in the appropriate place
			// note that we can't use FShaderCompilerInput::bCompilingForShaderPipeline as that is not set for all stages of a pipeline job
			FShaderCompileWorkerUtil::DumpDebugCompileInput(*this, Ctx);
		}
	}

	if (bHasSymbols)
	{
		check(ShaderFormat);
		ShaderFormat->NotifyShaderCompiled(SymbolsBuffer, Input.ShaderFormat, Input.GenerateDebugInfo());
	}
}

void FShaderCompileJob::AppendDebugName(FStringBuilderBase& OutName) const
{
	OutName << (Input.DumpDebugInfoPath.IsEmpty() ? Input.DebugGroupName : Input.DumpDebugInfoPath);
}

static void DumpShaderCompileJobArtifact(FShaderCompileJob& Job, const TCHAR* InFilename)
{
	// Serialize compile job to binary file. Can be loaded and analyzed in debugger with commandlet FLoadShaderCompileJobCommandlet.
	const FString DebugInfoPath = Job.Input.GetOrCreateShaderDebugInfoPath();
	const FString JobDumpFilename = FPaths::Combine(DebugInfoPath, InFilename);
	if (TUniquePtr<FArchive> JobDumpFile = TUniquePtr<FArchive>(IFileManager::Get().CreateFileWriter(*JobDumpFilename)))
	{
		Job.SerializeArtifact(*JobDumpFile);
	}
}

void FShaderCompileJob::SerializeWorkerOutput(FArchive& Ar)
{
	SerializeWorkerOutputInner(Ar);
}

void FShaderCompileJob::SerializeWorkerOutputInner(FArchive& Ar, bool bSerializeForArtifact)
{
	Ar << Output;

	bool bSecondaryOutput = SecondaryOutput.IsValid();
	Ar << bSecondaryOutput;

	if (!bSerializeForArtifact)
	{
		if (!ensureMsgf(Output.ValidateInputHash == Input.Hash, TEXT("Output.ValidateInputHash does not match Input.Hash; incorrect results associated with job?")))
		{
			DumpShaderCompileJobArtifact(*this, TEXT("ShaderCompileJob-InputHashMismatch"));
			UE_LOGF(LogShaders, Fatal, "Cannot continue serializing shader compile jobs after input hash mismatch in primary output");
		}
	}
	// empty validation hash after running validation to avoid impacting output deduplication
	Output.ValidateInputHash = FShaderCompilerInputHash();

	if (bSecondaryOutput)
	{
		if (Ar.IsLoading())
		{
			SecondaryOutput = MakeUnique<FShaderCompilerOutput>();
		}
		Ar << *SecondaryOutput;
		if (!bSerializeForArtifact)
		{
			if (!ensureMsgf(SecondaryOutput->ValidateInputHash == Input.Hash, TEXT("SecondaryOutput.ValidateInputHash does not match Input.Hash; incorrect results associated with job?")))
			{
				DumpShaderCompileJobArtifact(*this, TEXT("ShaderCompileJob-SecondaryOutput-InputHashMismatch"));
				UE_LOGF(LogShaders, Fatal, "Cannot continue serializing shader compile jobs after input hash mismatch in secondary output");
			}
		}
		// empty validation hash after running validation to avoid impacting output deduplication
		SecondaryOutput->ValidateInputHash = FShaderCompilerInputHash();
	}

	bool bSucceededTemp = (bool)bSucceeded;
	Ar << bSucceededTemp;
	bSucceeded = bSucceededTemp;

	if (!bSerializeForArtifact)
	{
		if (!ensureMsgf(Output.Target == Input.Target, TEXT("Output FShaderTarget does not match the input struct; incorrect results associated with job?")))
		{
			DumpShaderCompileJobArtifact(*this, TEXT("ShaderCompileJob-ShaderTargetMismatch"));
			UE_LOGF(LogShaders, Fatal, "Cannot continue serializing shader compile jobs after target mismatch in primary output");
		}
		if (bSecondaryOutput)
		{
			if (!ensureMsgf(SecondaryOutput->Target == Input.Target, TEXT("SecondaryOutput FShaderTarget does not match the input struct; incorrect results associated with job?")))
			{
				DumpShaderCompileJobArtifact(*this, TEXT("ShaderCompileJob-SecondaryOutput-ShaderTargetMismatch"));
				UE_LOGF(LogShaders, Fatal, "Cannot continue serializing shader compile jobs after target mismatch in secondary output");
			}
		}
	}
}

void FShaderCompileJob::SerializeWorkerInput(FArchive& Ar)
{
	Ar << Input;

	Ar << PreprocessOutput;

	bool bSecondaryPreprocessOutput = SecondaryPreprocessOutput.IsValid();
	Ar << bSecondaryPreprocessOutput;

	if (bSecondaryPreprocessOutput)
	{
		if (Ar.IsLoading())
		{
			SecondaryPreprocessOutput = MakeUnique<FShaderPreprocessOutput>();
		}
		Ar << *SecondaryPreprocessOutput;
	}
}

void FShaderCompileJob::SerializeWorkerInputNoSource(FArchive& Ar)
{
	FShaderSource SourceTemp = MoveTemp(PreprocessOutput.PreprocessedSource);
	bool bHasSecondary = SecondaryPreprocessOutput.IsValid();
	FShaderSource SecondarySourceTemp = bHasSecondary ? MoveTemp(SecondaryPreprocessOutput->PreprocessedSource) : FShaderSource();

	// we still want to serialize the decompressed size so when using source file virtualization via UBA we can just pass the source binary blob
	// as-is as a virtual input file.
	PreprocessOutput.PreprocessedSource.DecompressedCharCount = SourceTemp.DecompressedCharCount;
	if (bHasSecondary)
	{
		SecondaryPreprocessOutput->PreprocessedSource.DecompressedCharCount = SecondarySourceTemp.DecompressedCharCount;
	}

	SerializeWorkerInput(Ar);

	PreprocessOutput.PreprocessedSource = MoveTemp(SourceTemp);
	if (bHasSecondary)
	{
		SecondaryPreprocessOutput->PreprocessedSource = MoveTemp(SecondarySourceTemp);
	}
}

void FShaderCompileJob::SerializeArtifact(FArchive& Ar)
{
	SerializeWorkerInput(Ar);

	constexpr bool bSerializeForArtifact = true;
	SerializeWorkerOutputInner(Ar, bSerializeForArtifact);
}

FStringView FShaderCompileJob::GetFinalSourceView() const
{
	 // any modifications to the source done as part of the compile step will be written to the "ModifiedShaderSource" field
	// always return empty string if source extraction was not requested; this will prevent bloat of material DDC data in the case where debug info is enabled 
	// or Output.ModifiedShaderSource is unset (since the preprocess output unstripped source will always be set)
	if (Input.ExtraSettings.bExtractShaderSource)
	{
		// if there are no such modifications, return the "unstripped" version of the source code (with comments & line directives maintained),
		// otherwise return whatever the final modified source is as input to the compiler by the backend.
		return Output.ModifiedShaderSource.IsEmpty() ? PreprocessOutput.GetUnstrippedSourceView() : FStringView(Output.ModifiedShaderSource);
	}
	else
	{
		return FStringView();
	}
}

void FShaderCompileJob::AppendDiagnostics(FString& OutDiagnostics, int32 InJobIndex, int32 InNumJobs, const TCHAR* Indentation) const
{
	if (!OutDiagnostics.IsEmpty())
	{
		OutDiagnostics.AppendChar(TEXT('\n'));
	}
	OutDiagnostics.Appendf(
		TEXT("%sJob [%d/%d]: %s (PermutationId=%d)"),
		Indentation == nullptr ? TEXT("") : Indentation, InJobIndex + 1, InNumJobs, *Input.GenerateShaderName(), Key.PermutationId
	);
}

FShaderPipelineCompileJob::FShaderPipelineCompileJob(int32 NumStages)
	: FShaderCommonCompileJob(Type, 0u, 0u, EShaderCompileJobPriority::Num)
{
	StageJobs.Empty(NumStages);
	for (int32 StageIndex = 0; StageIndex < NumStages; ++StageIndex)
	{
		StageJobs.Add(new FShaderCompileJob());
	}

	if (StageJobs.Num())
	{
		// Set this flag on first job in constructor, so it's included during input hash computation.  Flag is set conditionally for other stage jobs in CompileShaderPipeline.
		StageJobs[0]->Input.bCompilingForShaderPipeline = true;
	}
}

FShaderPipelineCompileJob::FShaderPipelineCompileJob(uint32 InHash, uint32 InId, EShaderCompileJobPriority InPriroity, const FShaderPipelineCompileJobKey& InKey) :
	FShaderCommonCompileJob(Type, InHash, InId, InPriroity),
	Key(InKey)
{
	const auto& Stages = InKey.ShaderPipeline->GetStages();
	StageJobs.Empty(Stages.Num());
	for (const FShaderType* ShaderType : Stages)
	{
		const FShaderCompileJobKey StageKey(ShaderType, InKey.Platform, InKey.VFType, InKey.PermutationId);
		StageJobs.Add(new FShaderCompileJob(StageKey.MakeHash(InId), InId, InPriroity, StageKey));
	}
	
	if (StageJobs.Num())
	{
		// Set this flag on first job in constructor, so it's included during input hash computation.  Flag is set conditionally for other stage jobs in CompileShaderPipeline.
		StageJobs[0]->Input.bCompilingForShaderPipeline = true;
	}
}

void FShaderPipelineCompileJob::SerializeOutput(FShaderCacheSerializeContext& Ctx)
{
	bool bAllStagesSucceeded = true;

	Ctx.ReserveCode(StageJobs.Num());
	
	for (int32 Index = 0, Num = StageJobs.Num(); Index < Num; ++Index)
	{
		StageJobs[Index]->SerializeOutput(Ctx, Index);
		bAllStagesSucceeded = bAllStagesSucceeded && StageJobs[Index]->bSucceeded;
	}

	if (Ctx.GetMainArchive().IsLoading())
	{
		bSucceeded = bAllStagesSucceeded;
	}
}

void FShaderPipelineCompileJob::OnComplete(FShaderDebugDataContext& Ctx)
{
	Ctx.bIsPipeline = true;
	for (int32 Index = 0, Num = StageJobs.Num(); Index < Num; ++Index)
	{
		StageJobs[Index]->OnComplete(Ctx);
	}

	bool bShouldDump = !Ctx.DebugSourceFiles.IsEmpty() && (CVarDumpDebugInfoForCacheHits.GetValueOnAnyThread() || !JobStatusPtr->WasCompilationSkipped());
	if (bShouldDump)
	{
		// validate that all expected debug source files for each pipeline job have been written out and in the expected order
		int32 DebugSourceIndex = 0, DebugSourceCount = Ctx.DebugSourceFiles.Num();
		TArray<const FShaderCompileJob*> MissingStages;
		for (int32 Index = 0, Num = StageJobs.Num(); Index < Num; ++Index)
		{
			FShaderCompileJob* StageJob = StageJobs[Index];
			if (DebugSourceIndex >= DebugSourceCount || (Ctx.DebugSourceFiles[DebugSourceIndex++].Key != StageJob->Input.Target.GetFrequency()))
			{
				MissingStages.Add(StageJob);
				continue;
			}

			if (StageJob->SecondaryPreprocessOutput.IsValid() && (DebugSourceIndex >= DebugSourceCount || (Ctx.DebugSourceFiles[DebugSourceIndex++].Key != StageJob->Input.Target.GetFrequency())))
			{
				MissingStages.Add(StageJob);
			}
		}

		if (MissingStages.IsEmpty())
		{
			FShaderCompileWorkerUtil::DumpDebugCompileInput(*this, Ctx);
		}
		else
		{
			FString MissingStagesStr = FString::JoinBy(MissingStages, TEXT("\n"), [](const FShaderCompileJob* StageJob)
				{
					bool bDebugInfoEnabled = StageJob->Input.DumpDebugInfoEnabled();
					return FString::Printf(TEXT("%s (debug info %s)"), *StageJob->Input.DebugGroupName, (bDebugInfoEnabled ? TEXT("enabled") : TEXT("disabled")));
				}
			);
			UE_LOGF(LogShaders, Warning, "Cannot dump debug compile input for pipeline job; the following stage job(s) failed to dump source files: \n%ls", *MissingStagesStr);
		}
	}
}

void FShaderPipelineCompileJob::AppendDebugName(FStringBuilderBase& OutName) const
{
	OutName << "Pipeline Job\n";
	for (int32 Index = 0, Num = StageJobs.Num(); Index < Num; ++Index)
	{
		OutName << "    Stage " << Index << ": ";
		StageJobs[Index]->AppendDebugName(OutName);
		OutName << TEXT("\n");
	}
}

void FShaderPipelineCompileJob::AppendDiagnostics(FString& OutDiagnostics, int32 InJobIndex, int32 InNumJobs, const TCHAR* Indentation) const
{
	FString NewIndentation = Indentation == nullptr ? TEXT("") : Indentation;
	if (!OutDiagnostics.IsEmpty())
	{
		OutDiagnostics.AppendChar(TEXT('\n'));
	}
	OutDiagnostics.Appendf(
		TEXT("%sPipeline Job [%d/%d] (PermutationId=%d):"),
		*NewIndentation, InJobIndex + 1, InNumJobs, Key.PermutationId
	);
	NewIndentation += TEXT("  ");
	for (int32 StageJobIndex = 0; StageJobIndex < StageJobs.Num(); ++StageJobIndex)
	{
		StageJobs[StageJobIndex]->AppendDiagnostics(OutDiagnostics, StageJobIndex, StageJobs.Num(), *NewIndentation);
	}
}

static const TMap<FString, ECompilerFlags>& GetCompilerFlagsDictionary()
{
	static const TMap<FString, ECompilerFlags> CompilerFlagsDictionary =
	{
		#define SHADER_COMPILER_FLAGS_ENTRY(Name) { TEXT(#Name), CFLAG_##Name },
		#include "ShaderCompilerFlags.inl"
	};
	return CompilerFlagsDictionary;
}

void LexFromString(ECompilerFlags& OutValue, const TCHAR* InString)
{
	if (const ECompilerFlags* Flag = GetCompilerFlagsDictionary().Find(InString))
	{
		OutValue = *Flag;
	}
	else
	{
		OutValue = CFLAG_Max;
	}
}

const TCHAR* LexToString(ECompilerFlags InValue)
{
	if (const FString* String = GetCompilerFlagsDictionary().FindKey(InValue))
	{
		return **String;
	}
	else
	{
		return nullptr;
	}
}
