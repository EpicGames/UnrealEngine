// Copyright Epic Games, Inc. All Rights Reserved.

#include "ShaderDiagnostics.h"

#include "DataDrivenShaderPlatformInfo.h"
#include "ShaderCompilerJobTypes.h"

static TAutoConsoleVariable<int32> CVarShaderDevelopmentMode(
	TEXT("r.ShaderDevelopmentMode"),
	0,
	TEXT("0: Default, 1: Enable various shader development utilities, such as the ability to retry on failed shader compile, and extra logging as shaders are compiled."),
	ECVF_Default);

static TAutoConsoleVariable<int32> CVarShowShaderWarnings(
	TEXT("r.ShowShaderCompilerWarnings"),
	0,
	TEXT("When set to 1, will display all warnings. Note that this flag is ignored if r.ShaderDevelopmentMode=1 (in dev mode warnings are shown by default).")
	);

static TAutoConsoleVariable<int32> CVarShaderWarningsFilter(
	TEXT("r.ShaderCompilerWarningsFilter"),
	2,
	TEXT("Additional filtering for shader warnings; 2=show all, 1=show global shader warnings only, 0=show no shader warnings.")
);


static FString ConvertToNativePlatformAbsolutePath(const TCHAR* InPath)
{
	FString Path = IFileManager::Get().ConvertToAbsolutePathForExternalAppForRead(InPath);
	FPaths::MakePlatformFilename(Path);
	return Path;
}

int32 FShaderDiagnosticInfo::AddAndProcessErrorsForFailedJobFiltered(FShaderCompileJob& CurrentJob, const TCHAR* FilterMessage)
{
	int32 NumAddedErrors = 0;

	bool bReportedDebugInfo = false;

	for (int32 ErrorIndex = 0; ErrorIndex < CurrentJob.Output.Errors.Num(); ErrorIndex++)
	{
		FShaderCompilerError& CurrentError = CurrentJob.Output.Errors[ErrorIndex];
		FString CurrentErrorString = CurrentError.GetErrorString();

		// Include warnings if LogShaders is unsuppressed, otherwise only include filtered messages
		if (UE_LOG_ACTIVE(LogShaders, Log) || FilterMessage == nullptr || CurrentError.StrippedErrorMessage.Contains(FilterMessage))
		{
			// Extract source location from error message if the shader backend doesn't provide it separated from the stripped message
			CurrentError.ExtractSourceLocation();

			// Remap filenames
			if (CurrentError.ErrorVirtualFilePath == TEXT("/Engine/Generated/Material.ush"))
			{
				// MaterialTemplate.usf is dynamically included as Material.usf
				// Currently the material translator does not add new lines when filling out MaterialTemplate.usf,
				// So we don't need the actual filled out version to find the line of a code bug.
				CurrentError.ErrorVirtualFilePath = TEXT("/Engine/Private/MaterialTemplate.ush");
			}
			else if (CurrentError.ErrorVirtualFilePath.Contains(TEXT("memory")))
			{
				check(CurrentJob.Key.ShaderType);

				// Files passed to the shader compiler through memory will be named memory
				// Only the shader's main file is passed through memory without a filename
				CurrentError.ErrorVirtualFilePath = FString(CurrentJob.Key.ShaderType->GetShaderFilename());
			}
			else if (CurrentError.ErrorVirtualFilePath == TEXT("/Engine/Generated/VertexFactory.ush"))
			{
				// VertexFactory.usf is dynamically included from whichever vertex factory the shader was compiled with.
				check(CurrentJob.Key.VFType);
				CurrentError.ErrorVirtualFilePath = FString(CurrentJob.Key.VFType->GetShaderFilename());
			}
			else if (CurrentError.ErrorVirtualFilePath == TEXT("/Engine/Generated/VertexFactoryFwd.ush"))
			{
				// VertexFactoryFwd.usf is dynamically included from whichever vertex factory the shader was compiled with.
				check(CurrentJob.Key.VFType);
				check(CurrentJob.Key.VFType->IncludesFwdShaderFile());
				CurrentError.ErrorVirtualFilePath = FString(CurrentJob.Key.VFType->GetShaderFwdFilename());
			}
			else if (CurrentError.ErrorVirtualFilePath == TEXT("") && CurrentJob.Key.ShaderType)
			{
				// Some shader compiler errors won't have a file and line number, so we just assume the error happened in file containing the entrypoint function.
				CurrentError.ErrorVirtualFilePath = FString(CurrentJob.Key.ShaderType->GetShaderFilename());
			}

			uint32 ErrorHash = GetTypeHash(CurrentErrorString);
			if (!UniqueErrorHashes.Contains(ErrorHash))
			{
				// build up additional info in a "prefix" string; only do this once for each unique error
				FString UniqueErrorPrefix;

				// If we dumped the shader info, add it before the first error string
				if (!GIsBuildMachine && !bReportedDebugInfo && CurrentJob.Input.DumpDebugInfoPath.Len() > 0)
				{
					const FString DebugInfoPath = ConvertToNativePlatformAbsolutePath(*CurrentJob.Input.DumpDebugInfoPath);
					UniqueErrorPrefix += FString::Printf(TEXT("Shader debug info dumped to: \"%s\"\n\n"), *DebugInfoPath);
					bReportedDebugInfo = true;
				}

				TArray<FShaderCompilerError> SecondaryErrorsFromFilePath;
				const FString ShaderFilePath = ConvertToNativePlatformAbsolutePath(*CurrentError.GetShaderSourceFilePath(&SecondaryErrorsFromFilePath));
				const FString ShaderErrorLineString = CurrentError.ErrorLineString.IsEmpty() ? TEXT("0") : *CurrentError.ErrorLineString;
				if (CurrentJob.Key.ShaderType)
				{
					// Construct a path that will enable VS.NET to find the shader file, relative to the solution
					UniqueErrorPrefix += FString::Printf(TEXT("%s(%s): Shader %s, Permutation %d, VF %s:\n\t"),
						*ShaderFilePath,
						*ShaderErrorLineString,
						CurrentJob.Key.ShaderType->GetName(),
						CurrentJob.Key.PermutationId,
						CurrentJob.Key.VFType ? CurrentJob.Key.VFType->GetName() : TEXT("None"));
				}
				else
				{
					UniqueErrorPrefix += FString::Printf(TEXT("%s(%s): "),
						*ShaderFilePath,
						*ShaderErrorLineString);
				}

				// Append secondary errors resulting from invalid file path
				for (const FShaderCompilerError& SecondaryError : SecondaryErrorsFromFilePath)
				{
					FString SecondaryErrorString = SecondaryError.GetErrorString();
					CurrentErrorString.AppendChar(TEXT('\n'));
					CurrentErrorString.Append(SecondaryErrorString);
				}

				UniqueErrorHashes.Add(ErrorHash);
				UniqueErrors.Add(UniqueErrorPrefix + CurrentErrorString);
				ErrorJobs.AddUnique(&CurrentJob);
			}
			++NumAddedErrors;
		}
	}

	return NumAddedErrors;
}

FString GetSingleJobCompilationDump(const FShaderCompileJob* SingleJob)
{
	if (!SingleJob)
	{
		return TEXT("Internal error, not a Job!");
	}
	FString String = SingleJob->Input.GenerateShaderName();
	if (SingleJob->Key.VFType)
	{
		String += FString::Printf(TEXT(" VF '%s'"), SingleJob->Key.VFType->GetName());
	}
	String += FString::Printf(TEXT(" Type '%s'"), SingleJob->Key.ShaderType->GetName());
	String += FString::Printf(TEXT(" '%s' Entry '%s' Permutation %i "), *SingleJob->Input.VirtualSourceFilePath, *SingleJob->Input.EntryPointName, SingleJob->Key.PermutationId);
	return String;
}

bool IsShaderDevelopmentModeEnabled()
{
	return (CVarShaderDevelopmentMode.GetValueOnAnyThread() != 0);
}

bool ShouldShowShaderWarnings()
{
	// show warnings if explicitly requested via the r.ShowShaderCompilerWarnings cvar, or if shader dev mode is enabled.
	// in either case additional optional filtering happens at the job level via r.ShaderCompilerWarningsFilter (this can
	// be used to show just only global shader warnings, or disable warning prints entirely for shader dev mode)
	return (CVarShowShaderWarnings.GetValueOnAnyThread() != 0) || IsShaderDevelopmentModeEnabled();
}

// Returns true if the shader compile job or pipeline compile job requested to always log its diagnostic messages.
static bool AreDiagnosticMessagesForced(const FShaderCommonCompileJob& InJob)
{
	bool bOutAreDiagnosticMessagesForced = false;
	InJob.ForEachSingleShaderJob([&bOutAreDiagnosticMessagesForced](const FShaderCompileJob& SingleJob) {
		if (SingleJob.Output.bForceLogDiagnosticMessages)
		{
			bOutAreDiagnosticMessagesForced = true;
		}
	});
	return bOutAreDiagnosticMessagesForced;
}

FShaderDiagnosticInfo::FShaderDiagnosticInfo(const TArray<FShaderCommonCompileJobPtr>& Jobs)
{
	// Gather unique errors
	for (const FShaderCommonCompileJobPtr& Job : Jobs)
	{
		if (!Job->bSucceeded)
		{
			AddAndProcessErrorsForJob(*Job);
		}
		else if (ShouldShowShaderWarnings() || AreDiagnosticMessagesForced(*Job))
		{
			AddWarningsForJob(*Job);
		}
	}

	for (EShaderPlatform ErrorPlatform : ErrorPlatforms)
	{
		if (TargetShaderPlatformString.IsEmpty())
		{
			TargetShaderPlatformString = FDataDrivenShaderPlatformInfo::GetName(ErrorPlatform).ToString();
		}
		else
		{
			TargetShaderPlatformString += FString(", ") + FDataDrivenShaderPlatformInfo::GetName(ErrorPlatform).ToString();
		}
	}
}

void FShaderDiagnosticInfo::AddAndProcessErrorsForJob(FShaderCommonCompileJob& Job)
{
	Job.ForEachSingleShaderJob([this](FShaderCompileJob& Job)
		{
			ErrorPlatforms.AddUnique((EShaderPlatform)Job.Input.Target.Platform);

			if (Job.Output.Errors.Num() == 0)
			{
				// Job hard crashed
				FString InternalErrorStr = FString::Printf(TEXT("Internal Error!\n\t%s"), *GetSingleJobCompilationDump(&Job));
				UniqueErrors.Add(InternalErrorStr);
				UniqueErrorHashes.Add(GetTypeHash(InternalErrorStr));
			}

			// If we filter all error messages because they are interpreted as warnings, we have to assume all error messages are in fact errors and not warnings.
			// In that case, add jobs again without a filter; e.g. when the stripped message starts with "Internal exception".
			if (AddAndProcessErrorsForFailedJobFiltered(Job, TEXT("error")) == 0)
			{
				AddAndProcessErrorsForFailedJobFiltered(Job, nullptr);
			}

		});
}

void FShaderDiagnosticInfo::AddWarningsForJob(const FShaderCommonCompileJob& Job)
{
	Job.ForEachSingleShaderJob([this](const FShaderCompileJob& Job)
		{
			bool bIsGlobalShader = Job.Key.ShaderType->GetTypeForDynamicCast() == FShaderType::EShaderTypeForDynamicCast::Global;
			int32 FilterValue = CVarShaderWarningsFilter.GetValueOnAnyThread();
			bool bFilter = (FilterValue == 0) || ((FilterValue == 1) && !bIsGlobalShader);
			// Append the "errors" to the UniqueWarnings array if the job succeeded; there's nothing distinguishing errors
			// from warnings in the compile job so any errors that exist on a successful job are in fact warnings. 
			// Note that if the failed any warnings will already be interspersed with the errors in the UniqueErrors array.
			if (Job.bSucceeded && !bFilter)
			{
				for (const FShaderCompilerError& Warning : Job.Output.Errors)
				{
					UniqueWarnings.AddUnique(Warning.GetErrorString());
				}
			}
		});
}