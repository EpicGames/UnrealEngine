// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	EditorShaderCodeArchive.cpp: Implementation of FEditorShaderCodeArchive
=============================================================================*/

#include "ShaderLibrary/EditorShaderCodeArchive.h"

#if WITH_EDITORONLY_DATA

#include "Async/ParallelFor.h"
#include "Cooker/CookArtifact.h"
#include "HAL/FileManager.h"
#include "Interfaces/IShaderFormat.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#include "Internationalization/Regex.h"
#include "Math/UnitConversion.h"
#include "Misc/Compression.h"
#include "RHIStrings.h"
#include "PipelineCacheUtilities.h"
#include "Serialization/CompactBinarySerialization.h"
#include "Serialization/CompactBinaryWriter.h"
#include "Shader.h"
#include "ShaderCodeLibrary.h"
#include "ShaderCompilerCore.h"
#include "ShaderLibrary/ShaderCodeLibraryUtilities.h"
#include "ShaderPipelineCache.h"
#include "UObject/Class.h"
#include "UObject/Object.h"

using namespace UE::ShaderLibrary::Private;

// Compares only the shader code portion of two (potentially compressed) shader buffers,
// excluding optional data attachments which may legitimately differ between shader types
// that share the same bytecode. Returns true if the shader code portions are equal.
static bool ShaderCodeBytesAreEqual(
	FMemoryView BufferA, uint32 UncompressedSizeA,
	FMemoryView BufferB, uint32 UncompressedSizeB)
{
	FName CompressionFormat = GetShaderCompressionFormat();

	auto DecompressIfNeeded = [CompressionFormat](FMemoryView CompressedView, uint32 UncompressedSize, TArray<uint8>& OutDecompressed) -> TConstArrayView<uint8>
	{
		bool bIsCompressed = (CompressionFormat != NAME_None) && (CompressedView.GetSize() != UncompressedSize);
		if (bIsCompressed)
		{
			OutDecompressed.SetNumUninitialized(UncompressedSize);
			bool bDecompressSuccess = FCompression::UncompressMemory(CompressionFormat,
				OutDecompressed.GetData(), UncompressedSize,
				CompressedView.GetData(), CompressedView.GetSize());
			if (!bDecompressSuccess)
			{
				UE_LOGF(LogShaderLibrary, Fatal, "Failed to decompress shader code buffer for collision check, aborting");
			}
			return TConstArrayView<uint8>(OutDecompressed.GetData(), UncompressedSize);
		}
		return TConstArrayView<uint8>(static_cast<const uint8*>(CompressedView.GetData()), static_cast<int32>(CompressedView.GetSize()));
	};

	TArray<uint8> DecompressedA, DecompressedB;
	TConstArrayView<uint8> ViewA = DecompressIfNeeded(BufferA, UncompressedSizeA, DecompressedA);
	TConstArrayView<uint8> ViewB = DecompressIfNeeded(BufferB, UncompressedSizeB, DecompressedB);

	FShaderCodeReader ReaderA(ViewA);
	int32 ShaderCodeSizeA = ReaderA.GetShaderCodeSize();
	FShaderCodeReader ReaderB(ViewB);
	int32 ShaderCodeSizeB = ReaderB.GetShaderCodeSize();

	FMemoryView CodeA(ViewA.GetData(), FMath::Min(ShaderCodeSizeA, ViewA.Num()));
	FMemoryView CodeB(ViewB.GetData(), FMath::Min(ShaderCodeSizeB, ViewB.Num()));
	return CodeA.EqualBytes(CodeB);
}

// Returns the shader format interface for the specified format and platform name.
// This is used to determine whether native shader libraries are supported in FEditorShaderCodeArchive::SupportsShaderArchives().
static const IShaderFormat* GetShaderFormatFromName(FName InFormatAndPlatformName)
{
	// Parse library format name into shader format name, e.g. "SF_VULKAN_ES31_ANDROID-VULKAN_ES3_1_ANDROID" to "SF_VULKAN_ES31_ANDROID".
	TArray<FString> Components;
	const FString Name = InFormatAndPlatformName.ToString();
	Name.ParseIntoArray(Components, TEXT("-"));
	if (!ensureMsgf(Components.Num() == 2, TEXT("Failed to parse shader format and platform from name '%s'"), *InFormatAndPlatformName.ToString()))
	{
		return nullptr;
	}
	const FName ShaderFormatName(Components[0]);
	return GetTargetPlatformManagerRef().FindShaderFormat(ShaderFormatName);
}

FEditorShaderCodeArchive::FEditorShaderCodeArchive(FName InFormatAndPlatformName)
	: FormatAndPlatformName(InFormatAndPlatformName)
	, Format(GetShaderFormatFromName(InFormatAndPlatformName))
{
	SerializedShaders.ShaderHashTable.Clear(0x10000);
	SerializedShaders.ShaderMapHashTable.Clear(0x10000);
}

FEditorShaderCodeArchive::~FEditorShaderCodeArchive()
{
	// dummy
}

bool FEditorShaderCodeArchive::SupportsShaderArchives() const
{
	return Format && Format->SupportsShaderArchives();
}

void FEditorShaderCodeArchive::OpenLibrary(FString const& Name)
{
	check(LibraryName.Len() == 0);
	check(Name.Len() > 0);
	LibraryName = Name;
	SerializedShaders.Empty();
	ShaderCode.Empty();
}

void FEditorShaderCodeArchive::CloseLibrary(FString const& Name)
{
	check(LibraryName == Name);
	LibraryName = TEXT("");
}

int32 FEditorShaderCodeArchive::AddShaderCode(const FShaderMapResourceCode* Code,
	const FShaderMapAssetPaths& AssociatedAssets, ECookShaderLibrarySource CookSource, FShaderCodeStats* OutCodeStats)
{
	int32 ShaderMapIndex = INDEX_NONE;

	if (AssociatedAssets.Num() == 0 && LibraryName != TEXT("Global"))
	{
		UE_LOGF(LogShaderLibrary, Warning, "Shadermap %ls does not have assets associated with it, library layout may be inconsistent between builds", *Code->ResourceHash.ToString());
	}

	const int32 NumShaders = Code->ShaderCodeResources.Num();
	if (SerializedShaders.FindOrAddShaderMapEditor(Code->ResourceHash, ShaderMapIndex, &AssociatedAssets, CookSource))
	{
		if (OutCodeStats)
		{
			OutCodeStats->NumShaderMaps++;
		}
		FShaderMapEntry& ShaderMapEntry = SerializedShaders.ShaderMapEntries[ShaderMapIndex];
		ShaderMapEntry.NumShaders = NumShaders;
		ShaderMapEntry.ShaderIndicesOffset = SerializedShaders.ShaderIndices.AddZeroed(NumShaders);

		for (int32 i = 0; i < NumShaders; ++i)
		{
			int32 ShaderIndex = INDEX_NONE;
			if (SerializedShaders.FindOrAddShader(Code->ShaderHashes[i], ShaderIndex))
			{
				const FShaderCodeResource& SourceShaderResource = Code->ShaderCodeResources[i];
				FShaderCodeEntry& SerializedShaderEntry = SerializedShaders.ShaderEntries[ShaderIndex];
				SerializedShaderEntry.SetFrequency(SourceShaderResource.GetFrequency());
				FSharedBuffer CodeBuffer = SourceShaderResource.GetCodeBuffer();
				SerializedShaderEntry.Size = CodeBuffer.GetSize();
				SerializedShaderEntry.SetUncompressedSize(SourceShaderResource.GetUncompressedSize());
				check(SerializedShaderEntry.Size > 0);
				ShaderCode.Add(CodeBuffer);
				check(ShaderCode.Num() == SerializedShaders.ShaderEntries.Num());
			}
			else
			{
				// Verify the existing shader with the same hash has identical shader code.
				// The full code buffer includes optional data (attachments) after ShaderCodeSize
				// that may legitimately differ between shader types sharing the same bytecode.
				// Fast path: skip if full buffers match. Slow path: decompress and compare only
				// the shader code portion. Guard against empty existing code (after CopyToArchiveAndClear).
				FMemoryView ExistingView = ShaderCode[ShaderIndex].GetView();
				if (ExistingView.GetSize() > 0 && !ExistingView.EqualBytes(Code->ShaderCodeResources[i].GetCodeBuffer().GetView()))
				{
					const FShaderCodeEntry& ExistingEntry = SerializedShaders.ShaderEntries[ShaderIndex];
					FSharedBuffer IncomingCodeBuffer = Code->ShaderCodeResources[i].GetCodeBuffer();
					if (!ShaderCodeBytesAreEqual(
						ExistingView, ExistingEntry.GetUncompressedSize(),
						IncomingCodeBuffer.GetView(), Code->ShaderCodeResources[i].GetUncompressedSize()))
					{
						UE_LOGF(LogShaderLibrary, Fatal,
							"Shader hash collision: two different shaders produced hash %ls",
							*Code->ShaderHashes[i].ToString());
					}
				}
			}
			SerializedShaders.ShaderIndices[ShaderMapEntry.ShaderIndicesOffset + i] = ShaderIndex;
		}
	}
	else
	{
		// Verify the existing shader map with the same ResourceHash has identical shader content.
		// ResourceHash is a simple sum of shader hashes, so different shader sets could theoretically
		// produce the same sum, though this is exceedingly unlikely (this is just a safety measure).
		// The element-wise comparison below is valid because ShaderIndices are stored in the same
		// order as Code->ShaderHashes (see the insertion path above). Do not sort either side.
		const FShaderMapEntry& ExistingEntry = SerializedShaders.ShaderMapEntries[ShaderMapIndex];
		if (ExistingEntry.NumShaders != static_cast<uint32>(NumShaders))
		{
			UE_LOGF(LogShaderLibrary, Fatal,
				"Shader map hash collision: ResourceHash %ls maps to existing shader map with %u shaders, but incoming has %d shaders",
				*Code->ResourceHash.ToString(), ExistingEntry.NumShaders, NumShaders);
		}
		else
		{
			for (uint32 ShaderIndex = 0; ShaderIndex < ExistingEntry.NumShaders; ++ShaderIndex)
			{
				if (SerializedShaders.ShaderHashes[SerializedShaders.ShaderIndices[ExistingEntry.ShaderIndicesOffset + ShaderIndex]] != Code->ShaderHashes[ShaderIndex])
				{
					UE_LOGF(LogShaderLibrary, Fatal,
						"Shader map hash collision: ResourceHash %ls maps to existing shader map with matching shader count (%u), but shader hashes differ at index %u",
						*Code->ResourceHash.ToString(), ExistingEntry.NumShaders, ShaderIndex);
				}
			}
		}
	}

	for (int32 i = 0; i < NumShaders; ++i)
	{
		for (uint64 ShaderTypeHash : Code->ShaderEditorOnlyDataEntries[i].ShaderTypeHashes)
		{
			int32 ShaderIndex = SerializedShaders.FindShader(Code->ShaderHashes[i]);

			TArray<uint64>& SerializedShaderTypes = SerializedShaders.ShaderTypes[ShaderIndex].Data;
			const int32 HashIndex = Algo::LowerBound(SerializedShaderTypes, ShaderTypeHash);
			if (HashIndex >= SerializedShaderTypes.Num() || SerializedShaderTypes[HashIndex] != ShaderTypeHash)
			{
				SerializedShaderTypes.Insert(ShaderTypeHash, HashIndex);
				if (OutCodeStats)
				{
					OutCodeStats->AddEntry(ShaderTypeHash, SerializedShaders.ShaderEntries[ShaderIndex].Size);
				}
			}
		}
	}

	// always mark the shadermap dirty, because it might have gotten new asset associations
	MarkShaderMapDirty(ShaderMapIndex);
	return ShaderMapIndex;
}

void FEditorShaderCodeArchive::MarkShaderMapDirty(int32 ShaderMapIndex)
{
	if (bHasCopiedAndCleared)
	{
		ShaderMapsToCopy.Add(ShaderMapIndex);
	}
}

void FEditorShaderCodeArchive::MarkAllShaderMapsDirty()
{
	if (bHasCopiedAndCleared)
	{
		for (int32 ShaderMapIndex = 0; ShaderMapIndex < SerializedShaders.ShaderMapHashes.Num(); ++ShaderMapIndex)
		{
			ShaderMapsToCopy.Add(ShaderMapIndex);
		}
	}
}

bool FEditorShaderCodeArchive::HasDataToCopy() const
{
	// After the first copy dirty elements are added to ShaderMapsToCopy; during the first copy every ShaderMap in SerializedShaders is copied
	return bHasCopiedAndCleared ? !ShaderMapsToCopy.IsEmpty() : !SerializedShaders.ShaderMapHashes.IsEmpty();
}

void FEditorShaderCodeArchive::CopyToCompactBinary(FCbWriter& Writer, FSerializedShaderArchive& TransferArchive, TArray<uint8>& TransferCode)
{
	Writer.BeginObject();
	Writer << "SerializedShaders" << TransferArchive;
	Writer << "ShaderCode";
	Writer.AddBinary(FMemoryView(TransferCode.GetData(), TransferCode.Num()));
	Writer.EndObject();
}

bool FEditorShaderCodeArchive::AppendFromCompactBinary(FCbFieldView Field, ECookShaderLibrarySource CookSource, FShaderCodeStats* OutCodeStats)
{
	FSerializedShaderArchive TransferArchive;
	if (!LoadFromCompactBinary(Field["SerializedShaders"], TransferArchive))
	{
		return false;
	}

	FMemoryView FlatShaderCodeView = Field["ShaderCode"].AsBinaryView();
	TConstArrayView64<uint8> TransferCode(reinterpret_cast<const uint8*>(FlatShaderCodeView.GetData()), FlatShaderCodeView.GetSize());
	return AppendFromArchive(TransferArchive, TransferCode, CookSource, OutCodeStats);
}

void FEditorShaderCodeArchive::CopyToArchiveAndClear(FSerializedShaderArchive& TargetArchive, TArray<uint8>& TargetFlatShaderCode,
	bool& bOutRanOutOfRoom, int64 MaxShaderSize, int64 MaxShaderCount, ECookShaderLibrarySource CookSource)
{
	bOutRanOutOfRoom = false;
	if (!bHasCopiedAndCleared)
	{
		// First CopyAndClear adds all shadermaps to ShaderMapsToCopy, because as an optimization we do not write to
		// ShaderMapsToCopy until the first CopyAndClear call.
		for (int32 ShaderMapIndex = 0; ShaderMapIndex < SerializedShaders.ShaderMapHashes.Num(); ++ShaderMapIndex)
		{
			ShaderMapsToCopy.Add(ShaderMapIndex);
		}
	}

	TArray<int32> LocalShaderMapsToCopy = ShaderMapsToCopy.Array();
	LocalShaderMapsToCopy.Sort(); // Maintain the same order that the shadermaps were added in
	int32 NumShadersSentWithCode = 0;

	const FSerializedShaderArchive& SourceArchive = this->SerializedShaders;
	TArray<FSharedBuffer>& SourceShaderCodes = this->ShaderCode;
	for (int32 SourceShaderMapIndex : LocalShaderMapsToCopy)
	{
		const FShaderHash& SourceShaderMapHash = SourceArchive.ShaderMapHashes[SourceShaderMapIndex];
		const FShaderMapEntry& SourceEntry = SourceArchive.ShaderMapEntries[SourceShaderMapIndex];
		const FShaderMapAssetPaths* const SourceAssetPaths
			= SourceArchive.GetShaderMapAssetAssociations().FindShaderMap(SourceShaderMapHash);

		int32 TargetShaderMapIndex;
		const bool bIsNewShaderMap = TargetArchive.FindOrAddShaderMapEditor(SourceShaderMapHash, TargetShaderMapIndex,
			SourceAssetPaths, CookSource);
		if (!bIsNewShaderMap)
		{
			continue;
		}
		FShaderMapEntry& TargetEntry = TargetArchive.ShaderMapEntries[TargetShaderMapIndex];
		const int32 NumShaders = SourceEntry.NumShaders;

		TargetEntry.NumShaders = NumShaders;
		TargetEntry.ShaderIndicesOffset = TargetArchive.ShaderIndices.Num();
		TargetArchive.ShaderIndices.AddUninitialized(NumShaders);

		for (int32 ShaderIndexIndex = 0; ShaderIndexIndex < NumShaders; ++ShaderIndexIndex)
		{
			const uint32 SourceShaderIndex = SourceArchive.ShaderIndices[SourceEntry.ShaderIndicesOffset + ShaderIndexIndex];
			const FShaderHash& SourceShaderHash = SourceArchive.ShaderHashes[SourceShaderIndex];
			int32 TargetShaderIndex;
			const bool bIsNewShader = TargetArchive.FindOrAddShader(SourceShaderHash, TargetShaderIndex);
			TargetArchive.ShaderIndices[TargetEntry.ShaderIndicesOffset + ShaderIndexIndex] = TargetShaderIndex;
			if (bIsNewShader)
			{
				// We rely on the index of the newly added shader being at the end of the list of ShaderEntries,
				// so we can pop off the added index if we overflow below
				check(TargetShaderIndex == TargetArchive.ShaderEntries.Num() - 1);

				const FShaderCodeEntry& SourceShaderEntry = SourceArchive.ShaderEntries[SourceShaderIndex];
				FSharedBuffer& SourceShaderCode = SourceShaderCodes[SourceShaderIndex];
				FShaderCodeEntry& TargetShaderEntry = TargetArchive.ShaderEntries[TargetShaderIndex];

				TargetShaderEntry = SourceShaderEntry;
				if (SourceShaderCode.GetSize() > 0)
				{
					TargetShaderEntry.Offset = TargetFlatShaderCode.Num();
					check(SourceShaderEntry.Size == SourceShaderCode.GetSize());
					if ((MaxShaderSize > 0 && (int64)TargetFlatShaderCode.Num() + (int64)SourceShaderCode.GetSize() > MaxShaderSize) ||
						(MaxShaderCount > 0 && NumShadersSentWithCode + 1 > MaxShaderCount))
					{
						// We have to stop here to avoid overflowing the shader limit. Send the shaders we have accumulated
						// but do not send&clear any other data.
						// Remove all ShaderMap data
						TargetArchive.EmptyShaderMaps();
						// Remove the ShaderEntry we just added; we are not adding its code so we have to remove it from the
						// list of shaders contained by the targetarchive
						check(TargetShaderIndex == TargetArchive.ShaderHashes.Num() - 1);
						TargetArchive.RemoveLastAddedShader();
						bOutRanOutOfRoom = true;
						// Keep any shaders we added before the one we just added
						return;
					}

					// Append shader code to flat target array and keep track of how many shaders have been sent to target archive
					TargetFlatShaderCode.Append(MakeArrayView(reinterpret_cast<const uint8*>(SourceShaderCode.GetData()), SourceShaderCode.GetSize()));
					++NumShadersSentWithCode;

					// Reset the ShaderCode reference to (potentially) save memory in the local process. The consumer of the TargetArchive and
					// TargetFlatShaderCode will be the only one that needs to read it.
					SourceShaderCode.Reset();
				}
				else
				{
					// The shadercode was already copied in an earlier CopyAndClear operation; we just need to note that
					// ShaderMaps in this call to CopyAndClear reference it.
					TargetShaderEntry.Offset = INDEX_NONE;
				}
			}

			for (uint64 ShaderTypeHash : SourceArchive.ShaderTypes[SourceShaderIndex].Data)
			{
				TArray<uint64>& TargetShaderTypes = TargetArchive.ShaderTypes[TargetShaderIndex].Data;
				const int32 HashIndex = Algo::LowerBound(TargetShaderTypes, ShaderTypeHash);
				if (HashIndex >= TargetShaderTypes.Num() || TargetShaderTypes[HashIndex] != ShaderTypeHash)
				{
					TargetShaderTypes.Insert(ShaderTypeHash, HashIndex);
				}
			}
		}
	}

	ShaderMapsToCopy.Empty();
	bHasCopiedAndCleared = true;
}

bool FEditorShaderCodeArchive::AppendFromArchive(
	const FSerializedShaderArchive& SourceArchive, TConstArrayView64<uint8> SourceFlatShaderCode, ECookShaderLibrarySource CookSource, FShaderCodeStats* OutCodeStats)
{
	bool bOk = true;
	FSerializedShaderArchive& TargetArchive = this->SerializedShaders;
	TArray<FSharedBuffer>& TargetShaderCodes = this->ShaderCode;

	// Add all the shaders; we can sometimes get messages that send the shaders in advance without sending the shadermaps that use them
	for (int32 SourceShaderIndex = 0; SourceShaderIndex < SourceArchive.ShaderHashes.Num(); ++SourceShaderIndex)
	{
		const FShaderHash& SourceShaderHash = SourceArchive.ShaderHashes[SourceShaderIndex];
		int32 TargetShaderIndex = INDEX_NONE;
		const bool bShaderIsNew = TargetArchive.FindOrAddShader(SourceShaderHash, TargetShaderIndex);
		const TArray<uint64>& ShaderTypeHashes = SourceArchive.ShaderTypes[SourceShaderIndex].Data;
		const FShaderCodeEntry& SourceShaderEntry = SourceArchive.ShaderEntries[SourceShaderIndex];

		TArray<uint64>& ExistingShaderTypeHashes = TargetArchive.ShaderTypes[TargetShaderIndex].Data;
		for (uint64 ShaderTypeHash : ShaderTypeHashes)
		{
			const int32 HashIndex = Algo::LowerBound(ExistingShaderTypeHashes, ShaderTypeHash);
			if (HashIndex >= ExistingShaderTypeHashes.Num() || ExistingShaderTypeHashes[HashIndex] != ShaderTypeHash)
			{
				ExistingShaderTypeHashes.Insert(ShaderTypeHash, HashIndex);
				if (OutCodeStats)
				{
					OutCodeStats->AddEntry(ShaderTypeHash, SourceShaderEntry.Size);
				}
			}
		}

		if (!bShaderIsNew)
		{
			// Verify the existing shader with the same hash has identical bytecode.
			// Guard against empty target (possible after CopyToArchiveAndClear) and missing source.
			FMemoryView ExistingView = TargetShaderCodes[TargetShaderIndex].GetView();
			if (ExistingView.GetSize() > 0 && SourceShaderEntry.Offset != INDEX_NONE
				&& SourceShaderEntry.Offset + SourceShaderEntry.Size <= static_cast<uint64>(SourceFlatShaderCode.Num()))
			{
				FMemoryView IncomingView(SourceFlatShaderCode.GetData() + SourceShaderEntry.Offset, SourceShaderEntry.Size);
				if (!ExistingView.EqualBytes(IncomingView))
				{
					const FShaderCodeEntry& TargetEntry = TargetArchive.ShaderEntries[TargetShaderIndex];
					if (!ShaderCodeBytesAreEqual(
						ExistingView, TargetEntry.GetUncompressedSize(),
						IncomingView, SourceShaderEntry.GetUncompressedSize()))
					{
						UE_LOGF(LogShaderLibrary, Fatal,
							"Shader hash collision: two different shaders produced hash %ls",
							*SourceShaderHash.ToString());
					}
				}
			}
			continue;
		}
		check(TargetShaderIndex == TargetArchive.ShaderEntries.Num() - 1 && TargetShaderCodes.Num() == TargetArchive.ShaderEntries.Num() - 1);

		FShaderCodeEntry& TargetShaderEntry = TargetArchive.ShaderEntries[TargetShaderIndex];
		TargetShaderEntry = SourceShaderEntry;
		TargetShaderEntry.Offset = 0;

		if (SourceShaderEntry.Offset == INDEX_NONE)
		{
			UE_LOGF(LogShaderLibrary, Error, "ShaderMapLibrary transfer received from a remote machine has incomplete record for Shader %ls. "
				"The remote machine thought this machine already had the shader and did not send the ShaderCode for the Shader, but the shader is not found. "
				"The ShaderMaps using the shader will be corrupt.",
				*SourceShaderHash.ToString());
			bOk = false;
			TargetShaderEntry.Size = 0;
		}
		else if (SourceShaderEntry.Offset + SourceShaderEntry.Size > static_cast<uint64>(SourceFlatShaderCode.Num()))
		{
			UE_LOGF(LogShaderLibrary, Error, "ShaderMapLibrary transfer received from a remote machine has corrupt record for Shader %ls. "
				"The (Offset, Size) specified by the ShaderEntry does not fit in the TransferCode (an array of bytes that should contain the shader code for all transferred shaders. "
				"The ShaderMaps using the shader will be corrupt.",
				*SourceShaderHash.ToString());
			bOk = false;
			TargetShaderEntry.Size = 0;
		}
		else
		{
			// Copy from source's flat list to the target's separate FSharedBuffer for each shader
			TargetShaderCodes.Add(FSharedBuffer::Clone(SourceFlatShaderCode.GetData() + SourceShaderEntry.Offset, SourceShaderEntry.Size));
		}
	}

	// Add all the shadermaps
	for (int32 SourceShaderMapIndex = 0; SourceShaderMapIndex < SourceArchive.ShaderMapHashes.Num(); ++SourceShaderMapIndex)
	{
		const FShaderHash& SourceShaderMapHash = SourceArchive.ShaderMapHashes[SourceShaderMapIndex];
		const FShaderMapAssetPaths* const SourceAssetPaths
			= SourceArchive.GetShaderMapAssetAssociations().FindShaderMap(SourceShaderMapHash);
		int32 TargetShaderMapIndex;
		const bool bIsNewShaderMap = TargetArchive.FindOrAddShaderMapEditor(SourceShaderMapHash, TargetShaderMapIndex,
			SourceAssetPaths, CookSource);
		// always mark the shadermap dirty, because it might have gotten new asset associations
		MarkShaderMapDirty(TargetShaderMapIndex);
		if (!bIsNewShaderMap)
		{
			// Verify the existing shader map with the same hash has identical shader content.
			const FShaderMapEntry& TargetEntry = TargetArchive.ShaderMapEntries[TargetShaderMapIndex];
			const FShaderMapEntry& SourceEntry = SourceArchive.ShaderMapEntries[SourceShaderMapIndex];
			if (TargetEntry.NumShaders != SourceEntry.NumShaders)
			{
				UE_LOGF(LogShaderLibrary, Fatal,
					"Shader map hash collision: ResourceHash %ls maps to existing shader map with %u shaders, but incoming has %u shaders",
					*SourceShaderMapHash.ToString(), TargetEntry.NumShaders, SourceEntry.NumShaders);
			}
			else
			{
				for (uint32 ShaderIndex = 0; ShaderIndex < TargetEntry.NumShaders; ++ShaderIndex)
				{
					if (TargetArchive.ShaderHashes[TargetArchive.ShaderIndices[TargetEntry.ShaderIndicesOffset + ShaderIndex]]
						!= SourceArchive.ShaderHashes[SourceArchive.ShaderIndices[SourceEntry.ShaderIndicesOffset + ShaderIndex]])
					{
						UE_LOGF(LogShaderLibrary, Fatal,
							"Shader map hash collision: ResourceHash %ls maps to existing shader map with matching shader count (%u), but shader hashes differ at index %u",
							*SourceShaderMapHash.ToString(), TargetEntry.NumShaders, ShaderIndex);
					}
				}
			}
			continue;
		}

		const FShaderMapEntry& SourceEntry = SourceArchive.ShaderMapEntries[SourceShaderMapIndex];
		FShaderMapEntry& TargetEntry = TargetArchive.ShaderMapEntries[TargetShaderMapIndex];
		const int32 NumShaders = SourceEntry.NumShaders;

		TargetEntry.NumShaders = NumShaders;
		TargetEntry.ShaderIndicesOffset = TargetArchive.ShaderIndices.Num();
		TargetArchive.ShaderIndices.AddUninitialized(NumShaders);

		for (int32 ShaderIndexIndex = 0; ShaderIndexIndex < NumShaders; ++ShaderIndexIndex)
		{
			const uint32 SourceShaderIndex = SourceArchive.ShaderIndices[SourceEntry.ShaderIndicesOffset + ShaderIndexIndex];
			const FShaderHash& SourceShaderHash = SourceArchive.ShaderHashes[SourceShaderIndex];
			int32 TargetShaderIndex;
			const bool bShaderIsNew = TargetArchive.FindOrAddShader(SourceShaderHash, TargetShaderIndex);
			TargetArchive.ShaderIndices[TargetEntry.ShaderIndicesOffset + ShaderIndexIndex] = TargetShaderIndex;
			// Every shader in the SourceArchive should have already been added by the loop above over SourceArchive.ShaderHashes
			check(!bShaderIsNew);
		}
		if (OutCodeStats)
		{
			OutCodeStats->NumShaderMaps++;
		}
	}
	return bOk;
}

TUniquePtr<FEditorShaderCodeArchive> FEditorShaderCodeArchive::CreateArchiveFromFilteredAssets(const FString& NewLibraryName,
	FShaderMapAssetAssociations::FAssetFilterFunctionRef ShouldKeepAsset,
	FShaderMapAssetAssociations::FShaderMapFilterFunction ShouldExcludeShaderMap)
{
	checkf(!bHasCopiedAndCleared, TEXT("It is not valid to call CreateArchiveFromFilteredAssets on an FEditorShaderCodeArchive that has sent its shaders to another process by calling CopyAndClear."));

	TUniquePtr<FEditorShaderCodeArchive> NewArchive(new FEditorShaderCodeArchive(FormatAndPlatformName));
	NewArchive->OpenLibrary(NewLibraryName);

	TArray<int32> ShaderCodeEntriesNeeded;	// this array is filled with the indices from the existing ShaderCode that will need to be taken
	NewArchive->SerializedShaders.CreateFromFilteredAssets(SerializedShaders, ShaderCodeEntriesNeeded, ShouldKeepAsset, ShouldExcludeShaderMap);

	// extra integrity check
	checkf(ShaderCodeEntriesNeeded.Num() == NewArchive->SerializedShaders.ShaderHashes.Num(),
		TEXT("FSerializedShaderArchive for the new chunk did not create a valid shader code mapping"));
	checkf(ShaderCodeEntriesNeeded.Num() == NewArchive->SerializedShaders.ShaderEntries.Num(),
		TEXT("FSerializedShaderArchive for the new chunk did not create a valid shader code mapping"));

	// copy the shader code
	NewArchive->ShaderCode.Empty();
	for (int32 NewArchiveIdx = 0, NumIndices = ShaderCodeEntriesNeeded.Num(); NewArchiveIdx < NumIndices; ++NewArchiveIdx)
	{
		FSharedBuffer& SourceShaderCodeEntry = ShaderCode[ShaderCodeEntriesNeeded[NewArchiveIdx]];
		check(SourceShaderCodeEntry.GetSize() > 0);
		NewArchive->ShaderCode.Add(SourceShaderCodeEntry);
	}

	return NewArchive;
}

#if WITH_EDITOR

TUniquePtr<FEditorShaderCodeArchive> FEditorShaderCodeArchive::CreateArchiveFromAssetsReferencedByStaging()
{
	return CreateArchiveFromFilteredAssets(
		LibraryName,
		[](FName AssetName, const FShaderMapAssetAssociations::FAssociatedAssetData& AssetData) -> bool
		{
			return AssetData.bReferencedByStaging;
		});
}

TUniquePtr<FEditorShaderCodeArchive> FEditorShaderCodeArchive::CreateArchiveFromAssetsReferencedByOplog()
{
	return CreateArchiveFromFilteredAssets(
		LibraryName,
		[](FName AssetName, const FShaderMapAssetAssociations::FAssociatedAssetData& AssetData) -> bool
		{
			return AssetData.bReferencedByOplog;
		});
}

TUniquePtr<FEditorShaderCodeArchive> FEditorShaderCodeArchive::CreateArchiveFromAssetsOnlyReferencedByOplog()
{
	return CreateArchiveFromFilteredAssets(
		LibraryName,
		[](FName AssetName, const FShaderMapAssetAssociations::FAssociatedAssetData& AssetData) -> bool
		{
			return AssetData.bReferencedByOplog && !AssetData.bReferencedByStaging;
		});
}

#endif // WITH_EDITOR

TUniquePtr<FEditorShaderCodeArchive> FEditorShaderCodeArchive::CreateChunk(int32 ChunkId, const TSet<FName>& PackagesInChunk)
{
	return CreateNamedChunk(UE::ShaderLibrary::Private::GetShaderLibraryNameForChunk(LibraryName, ChunkId), PackagesInChunk);
}

TUniquePtr<FEditorShaderCodeArchive> FEditorShaderCodeArchive::CreateNamedChunk(const FString& ChunkName, const TSet<FName>& PackagesInChunk, FShaderMapAssetAssociations::FShaderMapFilterFunction ShouldExcludeShaderMap)
{
	return CreateArchiveFromFilteredAssets(
		ChunkName,
		[&PackagesInChunk](FName AssetName, const FShaderMapAssetAssociations::FAssociatedAssetData& AssetData) -> bool
		{
			return PackagesInChunk.Contains(AssetName);
		},
		ShouldExcludeShaderMap);
}

int32 FEditorShaderCodeArchive::AddShaderCode(int32 OtherShaderMapIndex, const FEditorShaderCodeArchive& OtherArchive,
	ECookShaderLibrarySource CookSource)
{
	checkf(!OtherArchive.bHasCopiedAndCleared,
		TEXT("It is not valid to call AddShaderCode from an FEditorShaderCodeArchive that has sent its shaders to another process by calling CopyAndClear."));

	const FSerializedShaderArchive& OtherShaders = OtherArchive.SerializedShaders;
	const FShaderHash& OtherShaderMapHash = OtherShaders.ShaderMapHashes[OtherShaderMapIndex];
	const FShaderMapAssetPaths* OtherAssets = OtherShaders.GetShaderMapAssetAssociations().FindShaderMap(OtherShaderMapHash);
	int32 ShaderMapIndex = 0;
	if (SerializedShaders.FindOrAddShaderMapEditor(OtherShaderMapHash, ShaderMapIndex, OtherAssets, CookSource))
	{
		const FShaderMapEntry& PrevShaderMapEntry = OtherShaders.ShaderMapEntries[OtherShaderMapIndex];
		FShaderMapEntry& ShaderMapEntry = SerializedShaders.ShaderMapEntries[ShaderMapIndex];
		ShaderMapEntry.NumShaders = PrevShaderMapEntry.NumShaders;
		ShaderMapEntry.ShaderIndicesOffset = SerializedShaders.ShaderIndices.AddZeroed(ShaderMapEntry.NumShaders);

		for (uint32 i = 0; i < ShaderMapEntry.NumShaders; ++i)
		{
			const int32 OtherShaderIndex = OtherShaders.ShaderIndices[PrevShaderMapEntry.ShaderIndicesOffset + i];
			int32 ShaderIndex = 0;
			if (SerializedShaders.FindOrAddShader(OtherShaders.ShaderHashes[OtherShaderIndex], ShaderIndex))
			{
				const FShaderCodeEntry& OtherShaderEntry = OtherShaders.ShaderEntries[OtherShaderIndex];
				SerializedShaders.ShaderEntries[ShaderIndex] = OtherShaderEntry;
				SerializedShaders.ShaderTypes[ShaderIndex] = OtherShaders.ShaderTypes[OtherShaderIndex];

				const FSharedBuffer& OtherShaderCodeEntry = OtherArchive.ShaderCode[OtherShaderIndex];
				check(OtherShaderCodeEntry.GetSize() > 0);
				ShaderCode.Add(OtherShaderCodeEntry);
				check(ShaderCode.Num() == SerializedShaders.ShaderEntries.Num());
			}
			else
			{
				// Verify the existing shader with the same hash has identical shader code.
				// Guard against empty existing code (possible after CopyToArchiveAndClear).
				FMemoryView ExistingView = ShaderCode[ShaderIndex].GetView();
				if (ExistingView.GetSize() > 0 && !ExistingView.EqualBytes(OtherArchive.ShaderCode[OtherShaderIndex].GetView()))
				{
					const FShaderCodeEntry& ExistingEntry = SerializedShaders.ShaderEntries[ShaderIndex];
					const FShaderCodeEntry& OtherEntry = OtherShaders.ShaderEntries[OtherShaderIndex];
					if (!ShaderCodeBytesAreEqual(
						ExistingView, ExistingEntry.GetUncompressedSize(),
						OtherArchive.ShaderCode[OtherShaderIndex].GetView(), OtherEntry.GetUncompressedSize()))
					{
						UE_LOGF(LogShaderLibrary, Fatal,
							"Shader hash collision: two different shaders produced hash %ls",
							*OtherShaders.ShaderHashes[OtherShaderIndex].ToString());
					}
				}
			}
			SerializedShaders.ShaderIndices[ShaderMapEntry.ShaderIndicesOffset + i] = ShaderIndex;
		}
	}
	// always mark the shadermap dirty, because it might have gotten new asset associations
	MarkShaderMapDirty(ShaderMapIndex);
	return ShaderMapIndex;
}

int32 FEditorShaderCodeArchive::PrepareAddShaderCode(
	int32 OtherShaderMapIndex,
	const FSerializedShaderArchive& OtherShaders,
	TArray<FShaderCodeReadInfo>& ReadInfo,
	ECookShaderLibrarySource CookSource,
	FShaderCodeStats* OutCodeStats)
{
	int32 ShaderMapIndex = 0;
	const FShaderHash& OtherShaderMapHash = OtherShaders.ShaderMapHashes[OtherShaderMapIndex];
	const FShaderMapAssetPaths* OtherAssets = OtherShaders.GetShaderMapAssetAssociations().FindShaderMap(OtherShaderMapHash);
	bool bIsNew = SerializedShaders.FindOrAddShaderMapEditor(OtherShaderMapHash, ShaderMapIndex, OtherAssets, CookSource);
	if (bIsNew)
	{
		if (OutCodeStats)
		{
			OutCodeStats->NumShaderMaps++;
		}

		const FShaderMapEntry& OtherEntry = OtherShaders.ShaderMapEntries[OtherShaderMapIndex];
		FShaderMapEntry& ShaderMapEntry = SerializedShaders.ShaderMapEntries[ShaderMapIndex];
		ShaderMapEntry.NumShaders = OtherEntry.NumShaders;
		ShaderMapEntry.ShaderIndicesOffset = SerializedShaders.ShaderIndices.AddZeroed(ShaderMapEntry.NumShaders);

		for (uint32 i = 0; i < ShaderMapEntry.NumShaders; ++i)
		{
			const int32 OtherShaderIndex = OtherShaders.ShaderIndices[OtherEntry.ShaderIndicesOffset + i];
			int32 ShaderIndex = 0;
			if (SerializedShaders.FindOrAddShader(OtherShaders.ShaderHashes[OtherShaderIndex], ShaderIndex))
			{
				const FShaderCodeEntry& OtherShaderEntry = OtherShaders.ShaderEntries[OtherShaderIndex];
				SerializedShaders.ShaderEntries[ShaderIndex] = OtherShaderEntry;

				check(ShaderCode.Num() == SerializedShaders.GetNumShaders() - 1);

				// Save information required to read in the old shader code
				check(OtherShaderEntry.Size > 0);
				FShaderCodeReadInfo Info{ ShaderCode.Num(), OtherShaderEntry.Offset, OtherShaderEntry.Size };
				ReadInfo.Add(Info);

				// We will add this old shader code in a separate pass
				ShaderCode.AddZeroed(1);
			}

			TArray<uint64>& SerializedShaderTypes = SerializedShaders.ShaderTypes[ShaderIndex].Data;
			for (uint64 OtherShaderTypeHash : OtherShaders.ShaderTypes[OtherShaderIndex].Data)
			{
				const int32 HashIndex = Algo::LowerBound(SerializedShaderTypes, OtherShaderTypeHash);
				if (HashIndex >= SerializedShaderTypes.Num() || SerializedShaderTypes[HashIndex] != OtherShaderTypeHash)
				{
					SerializedShaderTypes.Insert(OtherShaderTypeHash, HashIndex);
					if (OutCodeStats)
					{
						OutCodeStats->AddEntry(OtherShaderTypeHash, SerializedShaders.ShaderEntries[ShaderIndex].Size);
					}
				}
			}
			SerializedShaders.ShaderIndices[ShaderMapEntry.ShaderIndicesOffset + i] = ShaderIndex;
		}
	}
	// always mark the shadermap dirty, because it might have gotten new asset associations
	MarkShaderMapDirty(ShaderMapIndex);
	return ShaderMapIndex;
}

FEditorShaderCodeArchive::EShaderCodeReadStatus FEditorShaderCodeArchive::ReadShaderCodeBlock(FArchive& Ar, int64 StartOffset, const TArrayView<FShaderCodeReadInfo>& BlockReadInfo)
{
	if (BlockReadInfo.Num() > 0)
	{
		const FShaderCodeReadInfo& First = BlockReadInfo[0];
		const FShaderCodeReadInfo& Last = BlockReadInfo[BlockReadInfo.Num() - 1];
		int64 ReadSize = Last.Offset - First.Offset + Last.Size;

		FUniqueBuffer BlockOfShaderCode = FUniqueBuffer::Alloc(ReadSize);
		int64 FileLocation = First.Offset + StartOffset;
		if (FileLocation != Ar.Tell())
		{
			Ar.Seek(FileLocation);
		}
		Ar.Serialize(BlockOfShaderCode.GetData(), ReadSize);

		for (auto& Block : BlockReadInfo)
		{
			FUniqueBuffer Code = FUniqueBuffer::Clone(static_cast<const uint8*>(BlockOfShaderCode.GetData()) + Block.Offset - First.Offset, Block.Size);
			ShaderCode[Block.ShaderCodeIndex] = Code.MoveToShared();
		}

		if (Ar.GetError())
		{
			return EShaderCodeReadStatus::MalformedArchive;
		}
	}
	return EShaderCodeReadStatus::OK;
}

FEditorShaderCodeArchive::EShaderCodeReadStatus FEditorShaderCodeArchive::ReadShaderCode(FArchive& Ar, TArray<FShaderCodeReadInfo>& ReadInfo)
{
	EShaderCodeReadStatus Status = EShaderCodeReadStatus::OK;
	if (ReadInfo.Num() > 0)
	{
		// Sort by offset
		ReadInfo.Sort([](FShaderCodeReadInfo const& A, FShaderCodeReadInfo const& B) { return A.Offset < B.Offset; });

		// Fill in all the shader code data, reading old data in blocks
		const int64 DesiredBlockSize = 10 * 1024 * 1024;
		int64 PrevCookedShadersCodeStart = Ar.Tell();

		// Setup starting block, then iterate the rest
		int32 BlockStartIndex = 0;
		int32 IndexCount = 1;

		for (int64 i = 1; i < ReadInfo.Num(); ++i)
		{
			const FShaderCodeReadInfo& Next = ReadInfo[i];

			// See how much we'd have to read to add this block, if it adds too much, read and start a new block, otherwise include in current block
			int64 PotentialSize = Next.Offset - ReadInfo[BlockStartIndex].Offset + Next.Size;
			if (PotentialSize > DesiredBlockSize)
			{
				Status = ReadShaderCodeBlock(Ar, PrevCookedShadersCodeStart, MakeArrayView(&ReadInfo[BlockStartIndex], IndexCount));
				BlockStartIndex = i;
				IndexCount = 1;
				if (Status != EShaderCodeReadStatus::OK)
				{
					return Status;
				}
			}
			else
			{
				IndexCount += 1;
			}
		}

		// Read last block
		Status = ReadShaderCodeBlock(Ar, PrevCookedShadersCodeStart, MakeArrayView(&ReadInfo[BlockStartIndex], IndexCount));
	}
	return Status;
}

FEditorShaderCodeArchive::EShaderCodeReadStatus FEditorShaderCodeArchive::AddShaderCodeLibraryByName(
	const FString& BaseDir, const FString& InLibraryName, ECookShaderLibrarySource CookSource, FShaderCodeStats* OutCodeStats)
{
	using namespace UE::ShaderLibrary;

	bool bFoundAny = false;
	for (ESaveToDiskTarget Target : { ESaveToDiskTarget::Staging, ESaveToDiskTarget::CookCache })
	{
		const FString CodeArchiveFileName = GetCodeArchiveFilename(BaseDir, InLibraryName, FormatAndPlatformName, Target);
		TUniquePtr<FArchive> PrevCookedAr(CreateShaderFileReader(*CodeArchiveFileName));
		if (!PrevCookedAr)
		{
			continue;
		}
		bFoundAny = true;

		if (!FSerializedShaderArchive::SerializeHeaderVersion(*PrevCookedAr))
		{
			return EShaderCodeReadStatus::VersionMismatch;
		}

		FSerializedShaderArchive PrevCookedShaders;

		*PrevCookedAr << PrevCookedShaders;

		// check if it also contains the asset info file
		const FString AssetInfoFileName = GetShaderAssetInfoFilename(BaseDir, InLibraryName, FormatAndPlatformName, Target);
		TUniquePtr<FArchive> ShaderAssetInfoReader(CreateShaderFileReader(*AssetInfoFileName));
		if (PrevCookedShaders.LoadAssetInfo(ShaderAssetInfoReader.Get(), CookSource))
		{
			UE_LOGF(LogShaderLibrary, Display, "Loaded asset info %ls for the shader library %ls: %d entries",
				*AssetInfoFileName,
				*CodeArchiveFileName,
				PrevCookedShaders.GetShaderMapAssetAssociations().ViewShaderMaps().Num()
			);
		}
		else
		{
			FString ErrorMessage = FString::Printf(
				TEXT("Could not find or load asset info %s for the shader library %s"),
				*AssetInfoFileName, *CodeArchiveFileName);
			if (CookSource == ECookShaderLibrarySource::PreviousIncremental)
			{
				// In incremental cooks, we require the AssetInfo so we know which shadermaps to keep, lack of it
				// should report a cook error.
				UE_LOGF(LogShaderLibrary, Error, "%ls", *ErrorMessage);
			}
			else
			{
				UE_LOGF(LogShaderLibrary, Warning, "%ls", *ErrorMessage);
			}
		}

		// check if it also contains the shader type info file
		FString TypeInfoFileName = GetShaderTypeInfoFilename(BaseDir, InLibraryName, FormatAndPlatformName, Target);
		TUniquePtr<FArchive> ShaderTypeInfoReader(CreateShaderFileReader(*TypeInfoFileName));
		if (ShaderTypeInfoReader)
		{
			*ShaderTypeInfoReader << PrevCookedShaders.ShaderTypes;
			check(PrevCookedShaders.ShaderTypes.Num() == PrevCookedShaders.ShaderEntries.Num());
			UE_LOGF(LogShaderLibrary, Display, "Loaded shader type info %ls for the shader library %ls: %d entries",
				*TypeInfoFileName,
				*CodeArchiveFileName,
				PrevCookedShaders.ShaderTypes.Num()
			);
		}
		else
		{
			UE_LOGF(LogShaderLibrary, Warning, "Could not load shader type info %ls for the shader library %ls",
				*TypeInfoFileName,
				*CodeArchiveFileName
			);
		}

		// Setup current shader library information and accumulate the reads we will need to do. Then read them.
		TArray<FShaderCodeReadInfo> ReadInfo;
		for (int32 PrevShaderMapIndex = 0; PrevShaderMapIndex < PrevCookedShaders.ShaderMapEntries.Num(); ++PrevShaderMapIndex)
		{
			PrepareAddShaderCode(PrevShaderMapIndex, PrevCookedShaders, ReadInfo, CookSource, OutCodeStats);
		}
		EShaderCodeReadStatus Status = ReadShaderCode(*PrevCookedAr, ReadInfo);
		if (Status != EShaderCodeReadStatus::OK)
		{
			return Status;
		}
	}
	return bFoundAny ? EShaderCodeReadStatus::OK : EShaderCodeReadStatus::FileNotFound;
}

bool FEditorShaderCodeArchive::LoadExistingShaderCodeLibrary(FString const& MetaDataDir, ECookShaderLibrarySource CookSource)
{
	using namespace UE::ShaderLibrary;

	const FString BaseDir = MetaDataDir / TEXT("ShaderLibrarySource");
	const EShaderCodeReadStatus Status = AddShaderCodeLibraryByName(BaseDir, LibraryName, CookSource);
	switch (Status)
	{
	case EShaderCodeReadStatus::FileNotFound:
	{
		UE_LOGF(LogShaderLibrary, Error, "Failed to load shader code library from %ls; file not found",
			*GetCodeArchiveFilename(BaseDir, LibraryName, FormatAndPlatformName, ESaveToDiskTarget::Staging));
		return false;
	}
	case EShaderCodeReadStatus::MalformedArchive:
	{
		UE_LOGF(LogShaderLibrary, Error, "Failed to deserialized code from library %ls",
			*GetCodeArchiveFilename(BaseDir, LibraryName, FormatAndPlatformName, ESaveToDiskTarget::Staging));
		return false;
	}
	case EShaderCodeReadStatus::VersionMismatch:
	{
		UE_LOGF(LogShaderLibrary, Error, "Failed to deserialize shader code from %ls because the archive format is incompatible with the current version %u",
			*GetCodeArchiveFilename(BaseDir, LibraryName, FormatAndPlatformName, ESaveToDiskTarget::Staging), GShaderCodeArchiveVersion);
		return false;
	}
	case EShaderCodeReadStatus::OK:
	default:
		return true;
	}
}

void FEditorShaderCodeArchive::AddExistingShaderCodeLibrary(FString const& OutputDir, ECookShaderLibrarySource CookSource, FShaderCodeStats* OutCodeStats)
{
	check(LibraryName.Len() > 0);
	static const FRegexPattern ShaderCodePattern(ShaderCodePatternStr);
	static const FRegexPattern ShaderArchivePattern(ShaderArchivePatternStr);

	const FString FormatNameStr = FormatAndPlatformName.ToString();
	const FString ShaderIntermediateLocation = GetShaderCodeIntermediatePath() / FormatNameStr;

	TArray<FString> ShaderFiles;
	ShaderFindFiles(ShaderFiles, *ShaderIntermediateLocation, *ShaderExtension);
	ShaderFindFiles(ShaderFiles, *OutputDir, *ShaderExtension);
	ShaderFindFiles(ShaderFiles, *OutputDir, *CookCacheShaderExtension);

	for (const FString& ShaderFileName : ShaderFiles)
	{
		FRegexMatcher FindShaderFormat(ShaderCodePattern, ShaderFileName);
		if (FindShaderFormat.FindNext())
		{
			const FString& FoundLibraryName = FindShaderFormat.GetCaptureGroup(1);
			const FString& FoundFormatName = FindShaderFormat.GetCaptureGroup(2);

			if (FoundLibraryName.StartsWith(LibraryName) && FoundFormatName.Equals(FormatNameStr, ESearchCase::IgnoreCase))
			{
				AddShaderCodeLibraryByName(OutputDir, FoundLibraryName, CookSource, OutCodeStats);
			}
		}

		FRegexMatcher FindArchiveFormat(ShaderArchivePattern, ShaderFileName);
		if (FindArchiveFormat.FindNext())
		{
			const FString& FoundArchiveName = FindArchiveFormat.GetCaptureGroup(1);
			const FString& FoundFormatName = FindArchiveFormat.GetCaptureGroup(2);

			if (FoundArchiveName.StartsWith(LibraryName) && FoundFormatName.Equals(FormatNameStr, ESearchCase::IgnoreCase))
			{
				AddShaderCodeLibraryByName(OutputDir, FoundArchiveName, CookSource, OutCodeStats);
			}
		}
	}
}

#if WITH_EDITOR
void FEditorShaderCodeArchive::InitializeReferenceTracking()
{
	for (TPair<FName, FShaderMapAssetAssociations::FAssociatedAssetData>& Pair
		: SerializedShaders.GetShaderMapAssetAssociations().ViewAssets())
	{
		Pair.Value.bReferencedByOplog = false;
		Pair.Value.bReferencedByStaging = false;
	}
}

void FEditorShaderCodeArchive::UpdateOplogPackages(UE::Cook::Artifact::FUpdateOplogPackagesContext& Context)
{
	using FShaderMapAssetAssociations = FShaderMapAssetAssociations;
	using FOplogPackageData = UE::Cook::Artifact::FOplogPackageData;

	const TMap<FName, FOplogPackageData>& OplogPackages = Context.GetOplogPackages();
	FShaderMapAssetAssociations& AssetMap = SerializedShaders.GetShaderMapAssetAssociations();

	for (TPair<FName, FShaderMapAssetAssociations::FAssociatedAssetData>& Pair : AssetMap.ViewAssets())
	{
		FName PackageName = Pair.Key;
		if (PackageName.IsNone())
		{
			// NAME_None is used for shader maps that are re-added on every incremental cook. It will not be found in
			// the list of PackageNames passed in by the cooker, so mark it as referenced by staging and oplog as a
			// special case so that we keep these globally added shaders for staging.
			Pair.Value.bReferencedByOplog = true;
			Pair.Value.bReferencedByStaging = true;
		}
		else
		{
			const FOplogPackageData* OplogData = OplogPackages.Find(PackageName);
			if (OplogData)
			{
				Pair.Value.bReferencedByOplog = true;
				Pair.Value.bReferencedByStaging |=
					OplogData->GetReferenceType() == UE::Cook::EPackageReferenceType::Stage;
			}
		}
	}
}

void FEditorShaderCodeArchive::Prune()
{
	if (!UE::ShaderLibrary::Private::bIncrementalCookPruningEnabled)
	{
		return;
	}

	TUniquePtr<FEditorShaderCodeArchive> PrunedArchive = this->CreateArchiveFromAssetsReferencedByOplog();
	AssignFrom(*PrunedArchive);
}
#endif // WITH_EDITOR

void FEditorShaderCodeArchive::AssignFrom(FEditorShaderCodeArchive& InArchive)
{
	checkf(this->FormatAndPlatformName == InArchive.FormatAndPlatformName,
		TEXT("Cannot assign shader code archive due to `FormatAndPlatformName` mismatch: Expected '%s', but got '%s'"), *this->FormatAndPlatformName.ToString(), *InArchive.FormatAndPlatformName.ToString());
	checkf(this->Format == InArchive.Format,
		TEXT("Cannot assign shader code archive due to `Format` mismatch: Expected %p, but got %p"), this->Format, InArchive.Format);

	this->LibraryName = MoveTemp(InArchive.LibraryName);
	this->SerializedShaders = MoveTemp(InArchive.SerializedShaders);
	this->ShaderCode = MoveTemp(InArchive.ShaderCode);
	this->ShaderMapsToCopy = MoveTemp(InArchive.ShaderMapsToCopy);
	this->bHasCopiedAndCleared = MoveTemp(InArchive.bHasCopiedAndCleared);
}

bool FEditorShaderCodeArchive::SaveToDisk(const FString& OutputDir, const FString& MetaOutputDir,
	ESaveToDiskTarget Target, ESaveToDiskSortOrder SortOrder, TArray<FString>* OutputFilenames)
{
	check(LibraryName.Len() > 0);
	checkf(!bHasCopiedAndCleared, TEXT("It is not valid to call SaveToDisk on an FEditorShaderCodeArchive that has sent its shaders to another process by calling CopyAndClear."));

	bool bSuccess = IFileManager::Get().MakeDirectory(*OutputDir, true);

	auto CopyFile = [this](const FString& DestinationPath, const FString& SourcePath, TArray<FString>* OutputFilenames) -> bool
		{
			uint32 Result = IFileManager::Get().Copy(*DestinationPath, *SourcePath, true, true);
			if (Result != COPY_OK)
			{
				UE_LOGF(LogShaderLibrary, Error, "FEditorShaderCodeArchive copying %ls to %ls failed. Failed to finalize Shared Shader Library %ls with format %ls",
					*SourcePath, *DestinationPath, *LibraryName, *FormatAndPlatformName.ToString());
				return false;
			}

			if (OutputFilenames)
			{
				OutputFilenames->Add(DestinationPath);
			}
			return true;
		};

	auto CopyFilesToOutputDir = [this, &bSuccess, &CopyFile, &OutputDir, &MetaOutputDir, Target](
		TFunction<FString(const FString&, const FString&, FName, ESaveToDiskTarget)> DestinationPathFunction,
		const FString& SourcePath, TArray<FString>* OutputFilenames) -> void
		{
			// Copy to output location - support for iterative native library cooking
			if (!CopyFile(DestinationPathFunction(OutputDir, LibraryName, FormatAndPlatformName, Target), SourcePath, OutputFilenames))
			{
				bSuccess = false;
			}
			else if (MetaOutputDir.Len())
			{
				if (!CopyFile(DestinationPathFunction(MetaOutputDir / TEXT("../ShaderLibrarySource"), LibraryName, FormatAndPlatformName, Target), SourcePath, nullptr))
				{
					bSuccess = false;
				}
			}
		};

	// Shader library
	if (bSuccess && SerializedShaders.GetNumShaderMaps() > 0)
	{
		// Write to intermediate files
		const FString IntermediatePathBase = GetShaderCodeIntermediatePath() / FormatAndPlatformName.ToString();

		// Save the ShaderMaps, Shaders, and the Shaders' byte code
		constexpr bool bSaveOnlyAssetInfo = false; // A currently unused setting that we might want to bring back.
		if (!bSaveOnlyAssetInfo)
		{
			switch (SortOrder)
			{
			case ESaveToDiskSortOrder::PackageLoad:
				// Leave shaders arranged in the order in which they were created when loading and saving packages. This
				// is better for runtime performance when the shader library we write is loaded directly from a loose
				// file rather than being loaded through IoStore.
				break;
			case ESaveToDiskSortOrder::ShaderHash:
				// Sort the shader library for deterministic output when no cook inputs have changed. Note that this
				// depends on the ShaderHashes, so if they change the shader library sorting will change.
				SerializedShaders.SortByHashes(ShaderCode);
				break;
			default:
				checkNoEntry();
				break;
			}
			SerializedShaders.Finalize();

			const FString IntermediateFormatPath = GetShaderCodeFilename(IntermediatePathBase, LibraryName, FormatAndPlatformName, Target);
			if (TUniquePtr<FArchive> FileWriter = TUniquePtr<FArchive>(IFileManager::Get().CreateFileWriter(*IntermediateFormatPath, FILEWRITE_NoFail)))
			{
				FSerializedShaderArchive::SerializeHeaderVersion(*FileWriter);

				// Write shader library
				*FileWriter << SerializedShaders;
				for (FSharedBuffer& Code : ShaderCode)
				{
					check(Code.GetSize() > 0);
					FileWriter->Serialize(const_cast<void*>(Code.GetData()), Code.GetSize());
				}

				FileWriter->Close();

				// Copies and changes filename from "ShaderCode-*.ushaderbytecode" (GetShaderCodeFilename) to "ShaderArchive-*.ushaderbytecode" (GetCodeArchiveFilename)
				CopyFilesToOutputDir(GetCodeArchiveFilename, IntermediateFormatPath, OutputFilenames);
			}

			const FString TypeInfoIntermediatePath = GetShaderTypeInfoFilename(IntermediatePathBase, LibraryName, FormatAndPlatformName, Target);
			if (TUniquePtr<FArchive> ShaderTypeFileWriter = TUniquePtr<FArchive>(IFileManager::Get().CreateFileWriter(*TypeInfoIntermediatePath, FILEWRITE_NoFail)))
			{
				*ShaderTypeFileWriter << SerializedShaders.ShaderTypes;
				ShaderTypeFileWriter->Close();

				// Copies "ShaderTypeInfo-*.stinfo" to output directory
				CopyFilesToOutputDir(GetShaderTypeInfoFilename, TypeInfoIntermediatePath, OutputFilenames);
			}
		}

		// Save asset info
		{
			const FString AssetInfoIntermediatePath = GetShaderAssetInfoFilename(IntermediatePathBase, LibraryName, FormatAndPlatformName, Target);
			if (TUniquePtr<FArchive> AssetInfoWriter = TUniquePtr<FArchive>(IFileManager::Get().CreateFileWriter(*AssetInfoIntermediatePath, FILEWRITE_NoFail)))
			{
				SerializedShaders.SaveAssetInfo(*AssetInfoWriter);
				AssetInfoWriter->Close();

				// Copies "ShaderAssetInfo-*.assetinfo.json" to output directory
				CopyFilesToOutputDir(GetShaderAssetInfoFilename, AssetInfoIntermediatePath, nullptr);
			}
		}
	}

	return bSuccess;
}

bool FEditorShaderCodeArchive::PackageNativeShaderLibrary(const FString& ShaderCodeDir, TArray<FString>* OutputFilenames)
{
	if (SerializedShaders.GetNumShaders() == 0)
	{
		return true;
	}
	checkf(!bHasCopiedAndCleared, TEXT("It is not valid to call PackageNativeShaderLibrary on an FEditorShaderCodeArchive that has sent its shaders to another process by calling CopyAndClear."));

	// If the shader format module was not loaded, we don't support packaging native libraries.
	// UnrealPak for instance only loads MetalShaderFormat, which is the only format that supports native libraries anyway.
	if (!Format)
	{
		return false;
	}

	const FString IntermediateFormatPath = GetShaderDebugFolder(GetShaderCodeIntermediatePath() / FormatAndPlatformName.ToString(), LibraryName, FormatAndPlatformName);
	const FString TempPath = IntermediateFormatPath / TEXT("NativeLibrary");

	IFileManager::Get().MakeDirectory(*TempPath, true);
	IFileManager::Get().MakeDirectory(*ShaderCodeDir, true);

	EShaderPlatform Platform = ShaderFormatToLegacyShaderPlatform(FormatAndPlatformName);
	const bool bOK = Format->CreateShaderArchive(LibraryName, FormatAndPlatformName, TempPath, ShaderCodeDir, IntermediateFormatPath, SerializedShaders, ShaderCode, OutputFilenames);

	if (bOK)
	{
		// Delete Shader code library as we now have native versions
		// [RCL] 2020-12-02 FIXME: check if this doesn't ruin iterative cooking during Launch On
		{
			FString OutputFilePath = GetCodeArchiveFilename(ShaderCodeDir, LibraryName, FormatAndPlatformName, ESaveToDiskTarget::Staging);
			IFileManager::Get().Delete(*OutputFilePath);
		}
	}

	// Clean up the saved directory of temporary files
	IFileManager::Get().DeleteDirectory(*IntermediateFormatPath, false, true);
	IFileManager::Get().DeleteDirectory(*TempPath, false, true);

	return bOK;
}

void FEditorShaderCodeArchive::MakePatchLibrary(TArray<FEditorShaderCodeArchive*> const& OldLibraries, FEditorShaderCodeArchive const& NewLibrary)
{
	for (int32 NewShaderMapIndex = 0; NewShaderMapIndex < NewLibrary.SerializedShaders.ShaderMapHashes.Num(); ++NewShaderMapIndex)
	{
		const FShaderHash& Hash = NewLibrary.SerializedShaders.ShaderMapHashes[NewShaderMapIndex];
		if (!HasShaderMap(Hash))
		{
			bool bInPreviousPatch = false;
			for (FEditorShaderCodeArchive const* OldLibrary : OldLibraries)
			{
				bInPreviousPatch |= OldLibrary->HasShaderMap(Hash);
				if (bInPreviousPatch)
				{
					break;
				}
			}
			if (!bInPreviousPatch)
			{
				AddShaderCode(NewShaderMapIndex, NewLibrary, ECookShaderLibrarySource::PatchBase);
			}
		}
	}
}

bool FEditorShaderCodeArchive::CreatePatchLibrary(FName InFormatAndPlatformName, FString const& LibraryName, TArray<FString> const& OldMetaDataDirs, FString const& NewMetaDataDir, FString const& OutDir, bool bNativeFormat)
{
	TArray<FEditorShaderCodeArchive*> OldLibraries;
	for (FString const& OldMetaDataDir : OldMetaDataDirs)
	{
		FEditorShaderCodeArchive* OldLibrary = new FEditorShaderCodeArchive(InFormatAndPlatformName);
		OldLibrary->OpenLibrary(LibraryName);
		if (OldLibrary->LoadExistingShaderCodeLibrary(OldMetaDataDir, ECookShaderLibrarySource::PatchBase))
		{
			OldLibraries.Add(OldLibrary);
		}
	}

	FEditorShaderCodeArchive NewLibrary(InFormatAndPlatformName);
	NewLibrary.OpenLibrary(LibraryName);
	bool bOK = NewLibrary.LoadExistingShaderCodeLibrary(NewMetaDataDir, ECookShaderLibrarySource::PatchBase);
	if (bOK)
	{
		FEditorShaderCodeArchive OutLibrary(InFormatAndPlatformName);
		OutLibrary.OpenLibrary(LibraryName);
		OutLibrary.MakePatchLibrary(OldLibraries, NewLibrary);
		bOK = OutLibrary.SerializedShaders.GetNumShaderMaps() > 0;
		if (bOK)
		{
			FString Empty;
			OutLibrary.FinishPopulate(OutDir, ECookShaderLibrarySource::PatchBase);
			bOK = OutLibrary.SaveToDisk(OutDir, Empty, ESaveToDiskTarget::Staging, ESaveToDiskSortOrder::PackageLoad);
			UE_CLOGF(!bOK, LogShaderLibrary, Error, "Failed to save %ls shader patch library %ls, %ls, %ls", bNativeFormat ? TEXT("native") : TEXT(""), *InFormatAndPlatformName.ToString(), *LibraryName, *OutDir);

			if (bOK && bNativeFormat && OutLibrary.SupportsShaderArchives())
			{
				bOK = OutLibrary.PackageNativeShaderLibrary(OutDir);
				UE_CLOGF(!bOK, LogShaderLibrary, Error, "Failed to package native shader patch library %ls, %ls, %ls", *InFormatAndPlatformName.ToString(), *LibraryName, *OutDir);
			}
		}
		else
		{
			UE_LOGF(LogShaderLibrary, Verbose, "No shaders to patch for library %ls, %ls, %ls", *InFormatAndPlatformName.ToString(), *LibraryName, *OutDir);
		}
	}
	else
	{
		UE_LOGF(LogShaderLibrary, Error, "Failed to open the shader library to patch against %ls, %ls, %ls", *InFormatAndPlatformName.ToString(), *LibraryName, *NewMetaDataDir);
	}

	for (FEditorShaderCodeArchive* Lib : OldLibraries)
	{
		delete Lib;
	}
	return bOK;
}

void FEditorShaderCodeArchive::DumpStatsAndDebugInfo()
{
	bool bUseExtendedDebugInfo = UE::ShaderLibrary::Private::GProduceExtendedStats != 0;

	UE_LOGF(LogShaderLibrary, Display, "");
	UE_LOGF(LogShaderLibrary, Display, "Shader Library '%ls' (%ls) Stats:", *LibraryName, *FormatAndPlatformName.ToString());
	UE_LOGF(LogShaderLibrary, Display, "=================");

	FSerializedShaderArchive::FDebugStats Stats;
	FSerializedShaderArchive::FExtendedDebugStats ExtendedStats;
	SerializedShaders.CollectStatsAndDebugInfo(Stats, bUseExtendedDebugInfo ? &ExtendedStats : nullptr);

	UE_LOGF(LogShaderLibrary, Display, "Assets: %d, Unique Shadermaps: %d (%.2f%%)",
		Stats.NumAssets, Stats.NumShaderMaps, (Stats.NumAssets > 0) ? 100.0 * static_cast<double>(Stats.NumShaderMaps) / static_cast<double>(Stats.NumAssets) : 0.0);
	UE_LOGF(LogShaderLibrary, Display, "Total Shaders: %d, Unique Shaders: %d (%.2f%%)",
		Stats.NumShaders, Stats.NumUniqueShaders, (Stats.NumShaders > 0) ? 100.0 * static_cast<double>(Stats.NumUniqueShaders) / static_cast<double>(Stats.NumShaders) : 0.0);
	UE_LOGF(LogShaderLibrary, Display, "Total Shader Size: %.2fmb, Unique Shaders Size: %.2fmb (%.2f%%)",
		FUnitConversion::Convert(static_cast<double>(Stats.ShadersSize), EUnit::Bytes, EUnit::Megabytes),
		FUnitConversion::Convert(static_cast<double>(Stats.ShadersUniqueSize), EUnit::Bytes, EUnit::Megabytes),
		(Stats.ShadersSize > 0) ? 100.0 * static_cast<double>(Stats.ShadersUniqueSize) / static_cast<double>(Stats.ShadersSize) : 0.0
	);

	if (bUseExtendedDebugInfo)
	{
		UE_LOGF(LogShaderLibrary, Display, "=== Extended info:");
		UE_LOGF(LogShaderLibrary, Display, "Minimum number of shaders in shadermap: %d", ExtendedStats.MinNumberOfShadersPerSM);
		UE_LOGF(LogShaderLibrary, Display, "Median number of shaders in shadermap: %d", ExtendedStats.MedianNumberOfShadersPerSM);
		UE_LOGF(LogShaderLibrary, Display, "Maximum number of shaders in shadermap: %d", ExtendedStats.MaxNumberofShadersPerSM);
		if (ExtendedStats.TopShaderUsages.Num() > 0)
		{
			FString UsageString;
			UE_LOGF(LogShaderLibrary, Display, "Number of shadermaps referencing top %d most shared shaders:", ExtendedStats.TopShaderUsages.Num());
			for (int IdxUsage = 0; IdxUsage < ExtendedStats.TopShaderUsages.Num() - 1; ++IdxUsage)
			{
				UsageString += FString::Printf(TEXT("%d, "), ExtendedStats.TopShaderUsages[IdxUsage]);
			}
			UE_LOGF(LogShaderLibrary, Display, "    %ls%d", *UsageString, ExtendedStats.TopShaderUsages[ExtendedStats.TopShaderUsages.Num() - 1]);

			// print per-frequency stats
			UE_LOGF(LogShaderLibrary, Display, "Unique shaders itemization (sorted by compressed size):");
			// sort by compressed size
			TArray<int32> SortedIndices;
			for (int32 IdxFreq = 0; IdxFreq < SF_NumFrequencies; ++IdxFreq)
			{
				SortedIndices.Add(IdxFreq);
			}
			SortedIndices.StableSort([&ExtendedStats](int32 IndexA, int32 IndexB) { return ExtendedStats.CompressedSizePerFrequency[IndexA] >= ExtendedStats.CompressedSizePerFrequency[IndexB]; });
			for (int32 Freq : SortedIndices)
			{
				if (ExtendedStats.NumShadersPerFrequency[Freq] > 0)
				{
					UE_LOGF(LogShaderLibrary, Display, "%ls: %d shaders (%.2f%%), compressed size: %.2f MB (%.2f KB avg per shader), uncompressed size: %.2f MB (%.2f KB avg per shader)",
						GetShaderFrequencyString(static_cast<EShaderFrequency>(Freq)),
						ExtendedStats.NumShadersPerFrequency[Freq],
						100.0 * ExtendedStats.NumShadersPerFrequency[Freq] / double(Stats.NumUniqueShaders),
						double(ExtendedStats.CompressedSizePerFrequency[Freq]) / (1024.0 * 1024.0), double(ExtendedStats.CompressedSizePerFrequency[Freq]) / (double(ExtendedStats.NumShadersPerFrequency[Freq]) * 1024.0),
						double(ExtendedStats.UncompressedSizePerFrequency[Freq]) / (1024.0 * 1024.0), double(ExtendedStats.UncompressedSizePerFrequency[Freq]) / (double(ExtendedStats.NumShadersPerFrequency[Freq]) * 1024.0)
					);
				}
			}

		}
		else
		{
			UE_LOGF(LogShaderLibrary, Display, "No shader usage info is provided");
		}

		const FString DebugLibFolder = FSerializedShaderArchive::MakeDebugDirectory(LibraryName, FormatAndPlatformName, *UE::ShaderLibrary::Private::GetShaderCodeIntermediatePath());
		{
			const FString DumpFilename = DebugLibFolder / TEXT("Dump.txt");
			if (TUniquePtr<FArchive> DumpAr = TUniquePtr<FArchive>(IFileManager::Get().CreateFileWriter(*DumpFilename)))
			{
				FTCHARToUTF8 Converter(*ExtendedStats.TextualRepresentation);
				DumpAr->Serialize(const_cast<UTF8CHAR*>(reinterpret_cast<const UTF8CHAR*>(Converter.Get())), Converter.Length());
				UE_LOGF(LogShaderLibrary, Display, "Textual dump saved to '%ls'", *DumpFilename);
			}
		}
#if 0 // creating a graphviz graph - maybe one day we'll return to this
		FString DebugGraphFolder = GetShaderDebugFolder(DebugInfoPath / FormatAndPlatformName.ToString(), LibraryName, FormatAndPlatformName);
		FString DebugGraphFile = DebugGraphFolder / TEXT("RelationshipGraph.gv");

		IFileManager::Get().MakeDirectory(*DebugGraphFolder, true);
		TUniquePtr<FArchive> GraphVizAr(IFileManager::Get().CreateFileWriter(*DebugGraphFile));

		TAnsiStringBuilder<512> LineBuffer;
		LineBuffer << "digraph ShaderLibrary {\n";
		GraphVizAr->Serialize(const_cast<ANSICHAR*>(LineBuffer.ToString()), LineBuffer.Len() * sizeof(ANSICHAR));
		for (TTuple<FString, FString>& Edge : RelationshipGraph)
		{
			LineBuffer.Reset();
			LineBuffer << "\t \"";
			LineBuffer << TCHAR_TO_UTF8(*Edge.Key);
			LineBuffer << "\" -> \"";
			LineBuffer << TCHAR_TO_UTF8(*Edge.Value);
			LineBuffer << "\";\n";
			GraphVizAr->Serialize(const_cast<ANSICHAR*>(LineBuffer.ToString()), LineBuffer.Len() * sizeof(ANSICHAR));
		}
		LineBuffer.Reset();
		LineBuffer << "}\n";
		GraphVizAr->Serialize(const_cast<ANSICHAR*>(LineBuffer.ToString()), LineBuffer.Len() * sizeof(ANSICHAR));
#endif//
	}

	UE_LOGF(LogShaderLibrary, Display, "=================");
}

void FEditorShaderCodeArchive::ForEachAssetReferencingShaderType(uint64 InShaderTypeHash, FShaderMapAssetAssociations::FAssetFilterFunctionRef AssetReferenceCallback) const
{
	check(SerializedShaders.ShaderTypes.Num() == SerializedShaders.ShaderHashes.Num());
	const int32 NumShaders = SerializedShaders.ShaderTypes.Num();

	// Create temporary array to quickly map from shader index to all its shadermap hashes
	TArray<TArray<FShaderHash>> ShaderIndexToShaderMapHashes;
	ShaderIndexToShaderMapHashes.SetNum(NumShaders);

	for (int32 ShaderMapIndex = 0; ShaderMapIndex < SerializedShaders.ShaderMapHashes.Num(); ++ShaderMapIndex)
	{
		// Put current shadermap's hash into the temporary array for all shaders it belongs to
		const FShaderHash& ShaderMapHash = SerializedShaders.ShaderMapHashes[ShaderMapIndex];
		const FShaderMapEntry& ShaderMapEntry = SerializedShaders.ShaderMapEntries[ShaderMapIndex];
		for (uint32 ShaderIter = 0; ShaderIter < ShaderMapEntry.NumShaders; ++ShaderIter)
		{
			const int32 ShaderIndex = SerializedShaders.ShaderIndices[ShaderMapEntry.ShaderIndicesOffset + ShaderIter];
			ShaderIndexToShaderMapHashes[ShaderIndex].Add(ShaderMapHash);
		}
	}

	// Find shader type hash in all of serialized shaders
	const FShaderMapAssetAssociations& ShaderMapAssetAssoc = SerializedShaders.GetShaderMapAssetAssociations();
	for (int32 ShaderIndex = 0; ShaderIndex < NumShaders; ++ShaderIndex)
	{
		if (!SerializedShaders.ShaderTypes[ShaderIndex].Data.Contains(InShaderTypeHash))
		{
			continue;
		}

		for (const FShaderHash& ShaderMapHash : ShaderIndexToShaderMapHashes[ShaderIndex])
		{
			const FShaderMapAssetPaths* ShaderMapAssetNames = ShaderMapAssetAssoc.FindShaderMap(ShaderMapHash);
			if (!ShaderMapAssetNames)
			{
				continue;
			}

			// Invoke input callback for each asset for this shadermap
			for (const FName& AssetName : *ShaderMapAssetNames)
			{
				const FShaderMapAssetAssociations::FAssociatedAssetData* AssetData = ShaderMapAssetAssoc.FindAsset(AssetName);
				if (!AssetData)
				{
					continue;
				}

				// Invoke the callback on the current asset data
				if (!AssetReferenceCallback(AssetName, *AssetData))
				{
					// Stop iterating
					return;
				}
			}
		}
	}
}

#endif // WITH_EDITORONLY_DATA
