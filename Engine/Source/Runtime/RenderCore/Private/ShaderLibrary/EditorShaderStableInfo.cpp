// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	EditorShaderStableInfo.cpp: Implementation of FEditorShaderStableInfo
=============================================================================*/

#include "EditorShaderStableInfo.h"
#include "ShaderLibrary/ShaderCodeLibraryUtilities.h"

#include "Async/ParallelFor.h"
#include "Containers/HashTable.h"
#include "Containers/Set.h"
#include "Containers/StringView.h"
#include "HAL/FileManager.h"
#include "Math/UnitConversion.h"
#include "Shader.h"
#include "ShaderCodeArchive.h"
#include "ShaderCompilerCore.h"
#include "ShaderPipelineCache.h"

#if WITH_EDITORONLY_DATA
#include "Interfaces/IShaderFormat.h"
#include "Interfaces/ITargetPlatform.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#endif

#if WITH_EDITOR
#include "PipelineCacheUtilities.h"
#include "UObject/Class.h"
#include "UObject/Object.h"
#include "Serialization/CompactBinarySerialization.h"
#include "Serialization/CompactBinaryWriter.h"
#include "RHIStrings.h"
#include "Cooker/CookArtifactReader.h"
#include "Misc/App.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/EngineVersion.h"
#endif

#if WITH_EDITOR

FEditorShaderStableInfo::FEditorShaderStableInfo(FName InFormat)
	: FormatName(InFormat)
{
}

void FEditorShaderStableInfo::OpenLibrary(FString const& Name)
{
	check(LibraryName.Len() == 0);
	check(Name.Len() > 0);
	LibraryName = Name;
	StableMap.Empty();
	StableMapsToCopy.Empty();
	bHasCopied = false;
}

void FEditorShaderStableInfo::CloseLibrary(FString const& Name)
{
	check(LibraryName == Name);
	LibraryName = TEXT("");
}

void FEditorShaderStableInfo::AddShader(FStableShaderKeyAndValue& StableKeyValue, EMergeRule MergeRule)
{
	const TUniquePtr<FStableShaderKeyAndValue>* Existing = StableMap.Find(StableKeyValue);
	if (Existing)
	{
		switch (MergeRule)
		{
		case EMergeRule::KeepExisting:
			return;
		case EMergeRule::OverwriteUnmodifiedWarnModified:
			if ((**Existing).OutputHash != StableKeyValue.OutputHash)
			{
				UE_LOGF(LogShaderLibrary, Warning, "Duplicate key in stable shader library, but different output, skipping new item:");
				UE_LOGF(LogShaderLibrary, Warning, "    Existing: %ls", *(**Existing).ToString());
				UE_LOGF(LogShaderLibrary, Warning, "    New     : %ls", *StableKeyValue.ToString());
				return;
			}
			*Existing->Get() = StableKeyValue;
			return; // Otherwise fall through to overwrite
		default:
			checkNoEntry();
			break;
		}
	}

	StableMap.Add(MakeUnique<FStableShaderKeyAndValue>(StableKeyValue));
	if (bHasCopied)
	{
		StableMapsToCopy.Add(MakeUnique<FStableShaderKeyAndValue>(StableKeyValue));
	}
}

void FEditorShaderStableInfo::AddShaderCodeLibraryFromDirectory(const FString& BaseDir, EMergeRule MergeRule)
{
	using namespace UE::ShaderLibrary::Private;

	TArray<FStableShaderKeyAndValue> StableKeys;
	if (UE::PipelineCacheUtilities::LoadStableKeysFile(GetStableInfoArchiveFilename(BaseDir, LibraryName, FormatName), StableKeys))
	{
		for (FStableShaderKeyAndValue& Item : StableKeys)
		{
			AddShader(Item, MergeRule);
		}
	}
}

void FEditorShaderStableInfo::AddExistingShaderCodeLibrary(FString const& OutputDir)
{
	using namespace UE::ShaderLibrary::Private;

	check(LibraryName.Len() > 0);

	bool bLibraryExistsInSavedShadersDir = false;
	{
		const FString ShaderIntermediateLocation = GetShaderCodeIntermediatePath() / FormatName.ToString();
		TArray<FString> ShaderFiles;
		ShaderFindFiles(ShaderFiles, *ShaderIntermediateLocation, *StableExtension);
		FString ExpectedFileNameText = LibraryName + TEXT("-") + FormatName.ToString() + TEXT(".");

		for (const FString& ShaderFileName : ShaderFiles)
		{
			if (ShaderFileName.Contains(ExpectedFileNameText))
			{
				bLibraryExistsInSavedShadersDir = true;
				break;
			}
		}
	}
	if (bLibraryExistsInSavedShadersDir)
	{
		AddShaderCodeLibraryFromDirectory(OutputDir, EMergeRule::KeepExisting);
	}
}

bool FEditorShaderStableInfo::SaveToDisk(FString const& OutputDir, FString& OutSCLCSVPath)
{
	using namespace UE::ShaderLibrary::Private;

	check(LibraryName.Len() > 0);
	OutSCLCSVPath = FString();

	if (StableMap.IsEmpty())
	{
		// Do not touch the disk if empty, but also don't consider this a failure.
		// It is entirely possible that during the cook no assets with shaders were cooked.
		return true;
	}

	bool bSuccess = IFileManager::Get().MakeDirectory(*OutputDir, true);

	EShaderPlatform Platform = ShaderFormatToLegacyShaderPlatform(FormatName);

	// Shader library
	if (bSuccess)
	{
		// Write to a intermediate file
		FString IntermediateFormatPath = GetStableInfoArchiveFilename(GetShaderCodeIntermediatePath() / FormatName.ToString(), LibraryName, FormatName);

		// Write directly to the file
		{
			if (!UE::PipelineCacheUtilities::SaveStableKeysFile(IntermediateFormatPath, StableMap))
			{
				UE_LOGF(LogShaderLibrary, Error, "Could not save stable map to file '%ls'", *IntermediateFormatPath);
			}

			// check that it works in a Debug build pm;u
			if (UE_BUILD_DEBUG)
			{
				TArray<FStableShaderKeyAndValue> LoadedBack;
				if (!UE::PipelineCacheUtilities::LoadStableKeysFile(IntermediateFormatPath, LoadedBack))
				{
					UE_LOGF(LogShaderLibrary, Error, "Saved stable map could not be loaded back (from file '%ls')", *IntermediateFormatPath);
				}
				else
				{
					if (LoadedBack.Num() != StableMap.Num())
					{
						UE_LOGF(LogShaderLibrary, Error, "Loaded stable map has a different number of entries (%d) than a saved one (%d)", LoadedBack.Num(), StableMap.Num());
					}
					else
					{
						for (FStableShaderKeyAndValue& Value : LoadedBack)
						{
							Value.ComputeKeyHash();
							if (!StableMap.Contains(Value))
							{
								UE_LOGF(LogShaderLibrary, Error, "Loaded stable map has an entry that is not present in the saved one");
								UE_LOGF(LogShaderLibrary, Error, "  %ls", *Value.ToString());
							}
						}
					}
				}
			}
		}

		// Only the primary cooker needs to write to the output directory, child cookers only write to the Saved directory
		FString OutputFilePath = GetStableInfoArchiveFilename(OutputDir, LibraryName, FormatName);

		if (FParse::Param(FCommandLine::Get(), TEXT("mirrorshkfiles")))
		{
			FString MirrorLocation;
			GConfig->GetString(TEXT("/Script/Engine.ShaderCompilerStats"), TEXT("SHKFilesLocation"), MirrorLocation, GGameIni);
			if (!MirrorLocation.IsEmpty())
			{
				FString TargetType = "Default";
				FParse::Value(FCommandLine::Get(), TEXT("target="), TargetType);
				if (TargetType == TEXT("Default"))
				{
					FParse::Value(FCommandLine::Get(), TEXT("targetplatform="), TargetType);
				}
				const FString CopyLocation = FPaths::Combine(*MirrorLocation, FString::Printf(TEXT("%s-%s-%d-%s-%s-%s.shk"), *FApp::GetBranchName(), FApp::GetProjectName(), FEngineVersion::Current().GetChangelist(), *TargetType, *LibraryName, *FormatName.ToString()));
				const uint32 CopyResult = IFileManager::Get().Copy(*CopyLocation, *IntermediateFormatPath, true, true);
				if (CopyResult == COPY_Fail)
				{
					UE_LOGF(LogShaderLibrary, Warning, "Copy from Src: %ls to Dest: %ls failed.", *IntermediateFormatPath, *CopyLocation);
				}
			}
		}

		// Copy to output location - support for iterative native library cooking
		uint32 Result = IFileManager::Get().Copy(*OutputFilePath, *IntermediateFormatPath, true, true);
		if (Result == COPY_OK)
		{
			OutSCLCSVPath = OutputFilePath;
		}
		else
		{
			UE_LOGF(LogShaderLibrary, Error, "FEditorShaderStableInfo copy failed to %ls. Failed to finalize Shared Shader Library %ls with format %ls", *OutputFilePath, *LibraryName, *FormatName.ToString());
			bSuccess = false;
		}
	}

	return bSuccess;
}

bool FEditorShaderStableInfo::HasDataToCopy() const
{
	// After the first copy dirty elements are added to StableMapsToCopy; during the first copy every element from StableMap is copied
	return bHasCopied ? !StableMapsToCopy.IsEmpty() : !StableMap.IsEmpty();
}

void FEditorShaderStableInfo::CopyToCompactBinary(FCbWriter& Writer)
{
	// First copy we use the whole stablemap, after that we'll only copy newly added elements
	const FStableShaderSet& StableMapBeingCopied = bHasCopied ? StableMapsToCopy : StableMap;

	// Deduplicate the hashes; there will be a significant number (up to 50%) of duplicate hashes.
	TArray<FShaderHash> Hashes;
	TMap<FShaderHash, int32> HashToIndex;
	auto IndexHash = [&Hashes, &HashToIndex](const FShaderHash& Hash)
		{
			if (HashToIndex.Find(Hash) == nullptr)
			{
				Hashes.Add(Hash);
				HashToIndex.Add(Hash, Hashes.Num() - 1);
			}
		};
	for (const TUniquePtr<FStableShaderKeyAndValue>& StableInfo : StableMapBeingCopied)
	{
		IndexHash(StableInfo->PipelineHash);
		IndexHash(StableInfo->OutputHash);
	}

	Writer.BeginObject();
	{
		Writer << "Hashes" << Hashes;
		Writer.BeginArray("StableShaderKeyAndValues");
		for (const TUniquePtr<FStableShaderKeyAndValue>& StableInfo : StableMapBeingCopied)
		{
			WriteToCompactBinary(Writer, *StableInfo, HashToIndex);
		}
		Writer.EndArray();
	}
	Writer.EndObject();

	StableMapsToCopy.Empty();
	bHasCopied = true;
}

bool FEditorShaderStableInfo::AppendFromCompactBinary(FCbFieldView Field)
{
	TArray<FShaderHash> Hashes;
	if (!LoadFromCompactBinary(Field["Hashes"], Hashes))
	{
		// Can't load the StableShaderKeyAndValues if the hashes for them are missing
		return false;
	}
	FCbFieldView StableShaderKeyAndValuesField = Field["StableShaderKeyAndValues"];
	int32 NumInfos = StableShaderKeyAndValuesField.AsArrayView().Num();
	bool bOk = !StableShaderKeyAndValuesField.HasError();
	for (FCbFieldView InfoView : StableShaderKeyAndValuesField)
	{
		FStableShaderKeyAndValue StableInfo;
		if (LoadFromCompactBinary(InfoView, StableInfo, Hashes))
		{
			AddShader(StableInfo, EMergeRule::OverwriteUnmodifiedWarnModified);
		}
		else
		{
			bOk = false;
		}
	}
	return bOk;
}

#endif //WITH_EDITOR
