// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	EditorShaderStableInfo.h: Declaration of FEditorShaderStableInfo
=============================================================================*/

#pragma once

#include "ShaderCodeLibrary.h"

#if WITH_EDITOR

/*
 * See also FEditorShaderCodeArchive.
 */
class FEditorShaderStableInfo
{
public:
	FEditorShaderStableInfo(FName InFormat);

	void OpenLibrary(FString const& Name);

	void CloseLibrary(FString const& Name);

	enum class EMergeRule
	{
		/** If the key already exists, do not modify it, keep the existing value. */
		KeepExisting,
		/**
		 * If the key already exists, compare whether the output hash is different. If different,
		 * log a warning and keep the existing value. If the same, overwrite the existing value
		 * with the new value.
		 */
		OverwriteUnmodifiedWarnModified,
	};

	void AddShader(FStableShaderKeyAndValue& StableKeyValue, EMergeRule MergeRule);

	void AddShaderCodeLibraryFromDirectory(const FString& BaseDir, EMergeRule MergeRule);

	FORCEINLINE void FinishPopulate(FString const& OutputDir)
	{
		AddExistingShaderCodeLibrary(OutputDir);
	}

	bool SaveToDisk(FString const& OutputDir, FString& OutSCLCSVPath);

	bool HasDataToCopy() const;

	void CopyToCompactBinary(FCbWriter& Writer);

	bool AppendFromCompactBinary(FCbFieldView Field);

private:
	void AddExistingShaderCodeLibrary(FString const& OutputDir);

private:
	FName FormatName;
	FString LibraryName;
	FStableShaderSet StableMap;
	/** Copies of elements in StableMap that have not yet been copied to the CookDirector. Used only by MultiprocessCookWorkers. */
	FStableShaderSet StableMapsToCopy;
	/** True if CopyToCompactBinary has been called, otherwise false. If false we avoid doing some tracking since it might never be used. */
	bool bHasCopied = false;

};

#endif // WITH_EDITOR
