// Copyright Epic Games, Inc. All Rights Reserved.

#include "ShaderCodeArchive.h"

#include "Async/ParallelFor.h"
#include "Compression/OodleDataCompression.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "Misc/Compression.h"
#include "Misc/FileHelper.h"
#include "Misc/MemStack.h"
#include "Misc/ScopeRWLock.h"
#include "Policies/PrettyJsonPrintPolicy.h"
#include "RHI.h"
#include "RenderUtils.h"
#include "RHICommandList.h"
#include "Serialization/JsonSerializer.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/MemoryWriter.h"
#include "ShaderCodeLibrary.h"
#include "ShaderCompilerCore.h"
#include "ShaderCore.h"
#include "ShaderLibrary/ShaderCodeLibraryUtilities.h"
#include "ShaderLibrary/ShaderCodeArchiveInternal.h"
#include "Stats/Stats.h"
#include "Serialization/StaticMemoryReader.h"
#include "ProfilingDebugging/IoStoreTrace.h"
#include "IO/IoDispatcherBackend.h"
#include "IO/IoDispatcherInternal.h"

#if WITH_EDITORONLY_DATA
#include "Algo/Sort.h"
#include "Misc/Optional.h"
#include "Misc/StringBuilder.h"
#include "Serialization/CompactBinarySerialization.h"
#include "Serialization/CompactBinaryWriter.h"
#include "Templates/Greater.h"
#endif

#if UE_SCA_VISUALIZE_SHADER_USAGE
#include "IImageWrapper.h"
#include "IImageWrapperModule.h"
#include "Modules/ModuleManager.h"
#endif // UE_SCA_VISUALIZE_SHADER_USAGE

int32 GShaderCodeLibraryAsyncLoadingPriority = int32(AIOP_Normal);
static FAutoConsoleVariableRef CVarShaderCodeLibraryAsyncLoadingPriority(
	TEXT("r.ShaderCodeLibrary.DefaultAsyncIOPriority"),
	GShaderCodeLibraryAsyncLoadingPriority,
	TEXT(""),
	ECVF_Default
);

int32 GShaderCodeLibraryAsyncLoadingAllowDontCache = 0;
static FAutoConsoleVariableRef CVarShaderCodeLibraryAsyncLoadingAllowDontCache(
	TEXT("r.ShaderCodeLibrary.AsyncIOAllowDontCache"),
	GShaderCodeLibraryAsyncLoadingAllowDontCache,
	TEXT(""),
	ECVF_Default
);

int32 GShaderCodeLibraryVisualizeShaderUsage = 0;
static FAutoConsoleVariableRef CVarShaderCodeLibraryVisualizeShaderUsage(
	TEXT("r.ShaderCodeLibrary.VisualizeShaderUsage"),
	GShaderCodeLibraryVisualizeShaderUsage,
	TEXT("If 1, a bitmap with the used shaders (for each shader library chunk) will be saved at the exit. Works in standalone games only."),
	ECVF_RenderThreadSafe | ECVF_ReadOnly
);

int32 GShaderCodeLibraryMaxShaderGroupSize = 1024 * 1024;	// decompressing 1MB of shaders takes about 0.1ms on PC (TR 3970x, Oodle Mermaid6).
static FAutoConsoleVariableRef CVarShaderCodeLibraryMaxShaderGroupSize(
	TEXT("r.ShaderCodeLibrary.MaxShaderGroupSize"),
	GShaderCodeLibraryMaxShaderGroupSize,
	TEXT("Max (uncompressed) size of a group of shaders to be compressed/decompressed together.")
	TEXT("If a group exceeds it, it will be evenly split into subgroups which strive to not exceed it. However, if a shader group is down to one shader and still exceeds the limit, the limit will be ignored."),
	ECVF_RenderThreadSafe | ECVF_ReadOnly
);

float GShaderCodeLibraryMaxShaderPreloadWaitTime = 0.001f;
static FAutoConsoleVariableRef CVarShaderCodeLibraryMaxShaderPreloadWaitTime(
	TEXT("r.ShaderCodeLibrary.MaxShaderPreloadWaitTime"),
	GShaderCodeLibraryMaxShaderPreloadWaitTime,
	TEXT("If we wait on shader preloads longer than this amount of seconds, we will log it as a warning."),
	ECVF_RenderThreadSafe | ECVF_ReadOnly
);

int32 GPreloadShaderPriority = 2;
static FAutoConsoleVariableRef CVarPreloadShaderPriority(
	TEXT("r.PreloadShaderPriority"),
	GPreloadShaderPriority,
	TEXT("Change PreloadShaderGroup I/O priority.\n")
	TEXT("0-Min\n")
	TEXT("1-Low\n")
	TEXT("2-Medium (Default)\n")
	TEXT("3-High\n")
	TEXT("4-Max\n"),
	ECVF_Default
);

#if !UE_BUILD_SHIPPING
float GShaderCodeLibraryOptionalShaderFailChance = 0.0f;
static FAutoConsoleVariableRef CVarShaderCodeLibraryOptionalShaderFailChance(
	TEXT("r.ShaderCodeLibrary.OptionalShaderFailChance"),
	GShaderCodeLibraryOptionalShaderFailChance,
	TEXT("Chance (0.0-1.0) of artificially failing creation of any optional (bRequired=false) shader, to test failure handling"),
	ECVF_Default
);
#endif

int32 GetShaderCodeArchivePriority()
{
	switch (GPreloadShaderPriority)
	{
		case 4:
			return IoDispatcherPriority_Max;
		case 3:
			return IoDispatcherPriority_High;
		case 1:
			return IoDispatcherPriority_Low;
		case 0:
			return IoDispatcherPriority_Min;
		default:
			return IoDispatcherPriority_Medium;
	}
}

#if RHI_RAYTRACING	// this function is only needed to check if we need to avoid excluding raytracing shaders
namespace
{
	bool IsCreateShadersOnLoadEnabled()
	{
		static IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.CreateShadersOnLoad"));
		return CVar && CVar->GetInt() != 0;
	}
}
#endif // RHI_RAYTRACING

int32 FSerializedShaderArchive::FindShaderMapWithKey(const FShaderHash& Hash, uint32 Key) const
{
	for (uint32 Index = ShaderMapHashTable.First(Key); ShaderMapHashTable.IsValid(Index); Index = ShaderMapHashTable.Next(Index))
	{
		if (ShaderMapHashes[Index] == Hash)
		{
			return Index;
		}
	}
	return INDEX_NONE;
}

int32 FSerializedShaderArchive::FindShaderMap(const FShaderHash& Hash) const
{
	const uint32 Key = GetTypeHash(Hash);
	return FindShaderMapWithKey(Hash, Key);
}

#if !USE_MMAPPED_SHADERARCHIVE
bool FSerializedShaderArchive::FindOrAddShaderMapNonEditorInternal(const FShaderHash& Hash, int32& OutIndex)
{
	const uint32 Key = GetTypeHash(Hash);
	int32 Index = FindShaderMapWithKey(Hash, Key);
	bool bAdded = Index == INDEX_NONE;
	if (bAdded)
	{
		Index = ShaderMapHashes.Add(Hash);
		ShaderMapEntries.AddDefaulted();
		check(ShaderMapEntries.Num() == ShaderMapHashes.Num());
		ShaderMapHashTable.Add(Key, Index);
	}
	OutIndex = Index;
	return bAdded;
}

bool FSerializedShaderArchive::FindOrAddShaderMap(const FShaderHash& Hash, int32& OutIndex,
	const FShaderMapAssetPaths* AssociatedAssets)
{
#if WITH_EDITORONLY_DATA
	FShaderMapAssetPaths Buffer;
	if (!AssociatedAssets)
	{
		AssociatedAssets = &Buffer;
	}
	return FindOrAddShaderMapEditor(Hash, OutIndex, AssociatedAssets, ECookShaderLibrarySource::CurrentCook);
#else
	return FindOrAddShaderMapNonEditor(Hash, OutIndex);
#endif
}

#if WITH_EDITORONLY_DATA
bool FSerializedShaderArchive::FindOrAddShaderMapEditor(const FShaderHash& Hash, int32& OutIndex,
	const FShaderMapAssetPaths* AssociatedAssets, ECookShaderLibrarySource CookSource)
{
	const bool bResult = FindOrAddShaderMapNonEditorInternal(Hash, OutIndex);

	FShaderMapAssetPaths AssociatedAssetsProxy;
	if (!AssociatedAssets || AssociatedAssets->IsEmpty())
	{
		UE_LOGF(LogShaderLibrary, Error,
			"FindOrAddShaderMapEditor must be called with non-empty AssociatedAssets using the ShaderMap. " "ShaderMaps without assets will be pruned. Pass NAME_None for ShaderMaps that are re-added on every cook. " "Hash %ls is being assigned to NAME_None and will be pruned next cook.",
			*Hash.ToString());
		AssociatedAssetsProxy.Add(NAME_None);
		AssociatedAssets = &AssociatedAssetsProxy;
	}

	for (FName Asset : *AssociatedAssets)
	{
		FShaderMapAssetAssociations::FAssociatedAssetData* AssetData = ShaderMapAssetAssociations.FindAsset(Asset);
		if (AssetData)
		{
			if (UE::ShaderLibrary::Private::bIncrementalCookPruningEnabled)
			{
				// When loading ShaderArchive data from a previous cook, if we have recooked the Asset and
				// already have data from the current cook, do not add the stale references.
				if (CookSource >= ECookShaderLibrarySource::PreviousIncremental
					&& AssetData->LatestSource < ECookShaderLibrarySource::PreviousIncremental)
				{
					continue;
				}
				// When recooking an asset, the first time we add a ShaderMap for it remove any Assets added from a
				// previous cook.
				if (CookSource == ECookShaderLibrarySource::CurrentCook
					&& AssetData->LatestSource >= ECookShaderLibrarySource::PreviousIncremental)
				{
					ShaderMapAssetAssociations.RemoveAsset(Asset);
					// Add the current ShaderMap to ShaderMapAssetAssociations.
					AssetData = &ShaderMapAssetAssociations.FindOrAddAsset(Asset, Hash);
				}
				else
				{
					// We keep both the previous and current ShaderMap in ShaderMapAssetAssociations, so add the current
					// ShaderMap to the existing ShaderMaps.
					ShaderMapAssetAssociations.FindOrAddAsset(Asset, Hash);
				}
			}
			else
			{
				// We keep both the previous and current ShaderMap in ShaderMapAssetAssociations, so add the current
				// ShaderMap to the existing ShaderMaps.
				ShaderMapAssetAssociations.FindOrAddAsset(Asset, Hash);
			}

			// Report the lower of previous and current source as the current source; lower sources are newer
			// and we report the newest.
			AssetData->MergeSource(CookSource);
		}
		else if (Asset.IsNone() && UE::ShaderLibrary::Private::bIncrementalCookPruningEnabled
			&& CookSource >= ECookShaderLibrarySource::PreviousIncremental)
		{
			// ShaderMaps that are associated with asset name NAME_None have a contract that they are never loaded from
			// incremental and instead are always added in every cook. Add support to enforce this contract in the case
			// that some shaders previously existed for NAME_None but in the current cook they were not added. We need
			// to not add the stale references in that case.
			continue;
		}
		else
		{
			// Add the new Asset to ShaderMapAssetAssociations with the current ShaderMap.
			ShaderMapAssetAssociations.FindOrAddAsset(Asset, Hash).LatestSource = CookSource;
		}
	}

	return bResult;
}
#else // WITH_EDITOR
bool FSerializedShaderArchive::FindOrAddShaderMapNonEditor(const FShaderHash& Hash, int32& OutIndex)
{
	return FindOrAddShaderMapNonEditorInternal(Hash, OutIndex);
}
#endif // #else !WITH_EDITOR

#endif // !USE_MMAPPED_SHADERARCHIVE

int32 FSerializedShaderArchive::FindShaderWithKey(const FShaderHash& Hash, uint32 Key) const
{
	for (uint32 Index = ShaderHashTable.First(Key); ShaderHashTable.IsValid(Index); Index = ShaderHashTable.Next(Index))
	{
		if (ShaderHashes[Index] == Hash)
		{
			return Index;
		}
	}
	return INDEX_NONE;
}

int32 FSerializedShaderArchive::FindShader(const FShaderHash& Hash) const
{
	const uint32 Key = GetTypeHash(Hash);
	return FindShaderWithKey(Hash, Key);
}

#if !USE_MMAPPED_SHADERARCHIVE
bool FSerializedShaderArchive::FindOrAddShader(const FShaderHash& Hash, int32& OutIndex)
{
	const uint32 Key = GetTypeHash(Hash);
	OutIndex = FindShaderWithKey(Hash, Key);
	if (OutIndex == INDEX_NONE)
	{
		OutIndex = ShaderHashes.Add(Hash);
		ShaderEntries.AddDefaulted();
		check(ShaderEntries.Num() == ShaderHashes.Num());
		ShaderHashTable.Add(Key, OutIndex);
#if WITH_EDITORONLY_DATA
		ShaderTypes.AddDefaulted();
		checkf(ShaderTypes.Num() == ShaderEntries.Num(), TEXT("Mismatch in number of shader (%d) and shader type (%d) entries when adding shader to archive"), ShaderEntries.Num(), ShaderTypes.Num());
#endif
		return true;
	}

	return false;
}

void FSerializedShaderArchive::RemoveLastAddedShader()
{
	check(!ShaderEntries.IsEmpty() && ShaderEntries.Num() == ShaderHashes.Num());
	int32 ShaderIndex = ShaderEntries.Num() - 1;
	const uint32 Key = GetTypeHash(ShaderHashes[ShaderIndex]);
	ShaderHashTable.Remove(Key, ShaderIndex);
	ShaderHashes.RemoveAt(ShaderIndex);
	ShaderEntries.RemoveAt(ShaderIndex);
#if WITH_EDITORONLY_DATA
	ShaderTypes.RemoveAt(ShaderIndex);
	checkf(ShaderTypes.Num() == ShaderEntries.Num(), TEXT("Mismatch in number of shader (%d) and shader type (%d) entries when removing last added shader from archive"), ShaderEntries.Num(), ShaderTypes.Num());
#endif
}

#if WITH_EDITORONLY_DATA
FCbWriter& operator<<(FCbWriter& Writer, const FSerializedShaderArchive& Archive)
{
	TArray64<uint8> SerializedBytes;
	{
		FMemoryWriter64 SerializeArchive(SerializedBytes);
		const_cast<FSerializedShaderArchive&>(Archive).Serialize(SerializeArchive);
	}

	checkf(Archive.ShaderTypes.Num() == Archive.ShaderEntries.Num(), TEXT("Mismatch in number of shader (%d) and shader type (%d) entries when serializing shader archive"), Archive.ShaderEntries.Num(), Archive.ShaderTypes.Num());

	Writer.BeginObject();
	{
		Writer << "SerializedBytes";
		Writer.AddBinary(FMemoryView(SerializedBytes.GetData(), SerializedBytes.Num()));
		SerializedBytes.Empty();

		// Serialize is meant for runtime fields only, so copy the editor-only fields separately
		Writer.BeginArray("ShaderMapAssetAssociations");
		for (const TPair<FName, FShaderMapAssetAssociations::FAssociatedAssetData>& Pair : Archive.ShaderMapAssetAssociations.ViewAssets())
		{
			Writer.BeginArray();
			Writer << Pair.Key;
			Writer << (uint8)Pair.Value.LatestSource;
			Writer << Pair.Value.ShaderMaps;
			Writer.EndArray();
		}
		Writer.EndArray();

		Writer.BeginArray("ShaderTypes");
		for (const FSerializedShaderArchive::FShaderTypeHashes& SingleShaderTypes : Archive.ShaderTypes)
		{
			Writer.BeginArray();
			for (uint64 ShaderTypeHash : SingleShaderTypes.Data)
			{
				Writer << ShaderTypeHash;
			}
			Writer.EndArray();
		}
		Writer.EndArray();
	}
	Writer.EndObject();
	return Writer;
}

bool LoadFromCompactBinary(FCbFieldView Field, FSerializedShaderArchive& OutArchive)
{
	FMemoryView SerializedBytes = Field["SerializedBytes"].AsBinaryView();
	{
		FMemoryReaderView SerializeArchive(SerializedBytes);
		OutArchive.Serialize(SerializeArchive);
		if (SerializeArchive.IsError())
		{
			OutArchive = FSerializedShaderArchive();
			return false;
		}
	}

	bool bOk = true;
	{
		FCbFieldView ShaderCodeToAssetsField = Field["ShaderCodeToAssets"];
		if (ShaderCodeToAssetsField.AsArrayView().Num() > 0)
		{
			// Legacy field, no longer supported
			OutArchive = FSerializedShaderArchive();
			return false;
		}
	}

	{

		FCbFieldView ShaderMapAssetAssociationsField = Field["ShaderMapAssetAssociations"];
		int32 NumAssets = ShaderMapAssetAssociationsField.AsArrayView().Num() / 2;
		bOk = !ShaderMapAssetAssociationsField.HasError() && bOk;
		OutArchive.ShaderMapAssetAssociations.ReserveAssets(NumAssets);

		for (FCbFieldView AssetFieldIter : ShaderMapAssetAssociationsField)
		{
			FCbArrayView AssetField = AssetFieldIter.AsArrayView();
			if (AssetField.Num() < 3)
			{
				bOk = false;
				continue;
			}
			FCbFieldViewIterator ArrayIter = AssetField.CreateViewIterator();
			FName AssetName;
			TArray<FShaderHash> ShaderMaps;
			ShaderCodeArchive::ECookShaderLibrarySource LatestSource = ShaderCodeArchive::ECookShaderLibrarySource::CurrentCook;
			bOk = LoadFromCompactBinary(*ArrayIter++, AssetName) && bOk;
			uint8 LatestSourceInt = 0xFF;
			if (LoadFromCompactBinary(*ArrayIter++, LatestSourceInt)
				&& LatestSourceInt >= (uint8)ShaderCodeArchive::ECookShaderLibrarySource::CurrentCook
				&& LatestSourceInt <= (uint8)ShaderCodeArchive::ECookShaderLibrarySource::PreviousIncremental)
			{
				LatestSource = (ShaderCodeArchive::ECookShaderLibrarySource)LatestSourceInt;
			}
			else
			{
				bOk = false;
			}
			bOk = LoadFromCompactBinary(*ArrayIter++, ShaderMaps) && bOk;
			if (bOk && ShaderMaps.Num() > 0)
			{
				FShaderMapAssetAssociations::FAssociatedAssetData* AssetData;
				AssetData = &OutArchive.ShaderMapAssetAssociations.FindOrAddAsset(AssetName, ShaderMaps[0]);
				AssetData->LatestSource = LatestSource;
				for (int32 Index = 1; Index < ShaderMaps.Num(); ++Index)
				{
					OutArchive.ShaderMapAssetAssociations.FindOrAddAsset(AssetName, ShaderMaps[Index]);
				}
			}
		}
	}

	{
		FCbFieldView ShaderTypesField = Field["ShaderTypes"];
		bOk = !ShaderTypesField.HasError() && bOk;
		OutArchive.ShaderTypes.Empty(ShaderTypesField.AsArrayView().Num());
		FCbFieldViewIterator It = ShaderTypesField.CreateViewIterator();
		while (It)
		{
			bOk = !It->HasError() && bOk;
			FSerializedShaderArchive::FShaderTypeHashes& ShaderTypeHashes = OutArchive.ShaderTypes.AddDefaulted_GetRef();
			FCbArrayView ArrayView = It.AsArrayView();
			ShaderTypeHashes.Data.Reserve(ArrayView.Num());
			for (FCbFieldViewIterator ArrayIt = It.AsArrayView().CreateViewIterator(); ArrayIt; ++ArrayIt)
			{
				ShaderTypeHashes.Data.Add(ArrayIt.AsUInt64());
			}
			It++;
		}

		checkf(OutArchive.ShaderTypes.Num() == OutArchive.ShaderEntries.Num(), 
			TEXT("Mismatch in number of shader (%d) and shader type (%d) entries when loading shader archive from compact binary"),
			OutArchive.ShaderEntries.Num(), 
			OutArchive.ShaderTypes.Num());
	}

	return bOk;
}
#endif //WITH_EDITORONLY_DATA
#endif //!USE_MMAPPED_SHADERARCHIVE


#if UE_SCA_VISUALIZE_SHADER_USAGE
void FShaderUsageVisualizer::Initialize(const int32 InNumShaders)
{
	FScopeLock Lock(&VisualizeLock);
	NumShaders = InNumShaders;
}

void FShaderUsageVisualizer::SaveShaderUsageBitmap(const FString& Name, EShaderPlatform ShaderPlatform)
{
	if (GShaderCodeLibraryVisualizeShaderUsage)
	{
		if (NumShaders)
		{
			if (IImageWrapperModule* ImageWrapperModule = FModuleManager::Get().GetModulePtr<IImageWrapperModule>(TEXT("ImageWrapper")))
			{
				if (TSharedPtr<IImageWrapper> PNGImageWrapper = ImageWrapperModule->CreateImageWrapper(EImageFormat::PNG))
				{
					UE_LOGF(LogShaderLibrary, Log, "Creating shader usage bitmap for archive %ls (NumShaders: %d, preloaded %d, created %d)", *Name, NumShaders, PreloadedShaders.Num(), CreatedShaders.Num());

					// find a value close to sqrt(NumShaders)
					int32 ImageDimension = static_cast<int32>(FMath::Sqrt(static_cast<float>(NumShaders))) + 1;
					TArray<FColor> ShaderUsageBitmap;
					ShaderUsageBitmap.Reserve(ImageDimension * ImageDimension);

					// map legend:
					FColor UnusedShaderColor(128, 128, 128);	// unused shaders - this is the majority of the bitmap content
					FColor PreloadedShaderColor(192, 192, 192);	// preloaded shaders - including those that weren't explicitly so, but they happened to be grouped with shaders we needed
					FColor ExplicitlyPreloadedShaderColor(0, 255, 0);	// shaders we explicitly wanted to preload - they can become the majority under certain circumstances. Pure white can blend with some viewer's background
					FColor PreloadedAndDecompressedShaderColor(0, 0, 255);	// shaders that we wanted to preload and that got decompressed (as part of the creating them or their neighbor in group)
					FColor NotPreloadedButDecompressedShaderColor(255, 0, 0);	// shaders that we decompressed just because they were grouped together with others. We did not want to preload them at all.
					FColor CreatedShaderColor(255, 255, 255);	// created shaders - in practice, always few and far between. Blue is more noticeable on a largely bright background than magenta

					for (int32 Idx = 0; Idx < NumShaders; ++Idx)
					{
						ShaderUsageBitmap.Add(UnusedShaderColor);
					}
					// the rest can be zero/transparent
					ShaderUsageBitmap.AddZeroed(ImageDimension * ImageDimension - NumShaders);
					check(ShaderUsageBitmap.Num() == ImageDimension * ImageDimension);

					{
						// in case this ever gets called runtime
						FScopeLock Lock(&VisualizeLock);

						// fill preloaded ones first
						for (int32 ShaderIdx : PreloadedShaders)
						{
							ShaderUsageBitmap[ShaderIdx] = PreloadedShaderColor;
						}

						// explicitly preloaded shaders
						for (int32 ShaderIdx : ExplicitlyPreloadedShaders)
						{
							ShaderUsageBitmap[ShaderIdx] = ExplicitlyPreloadedShaderColor;
						}

						// fill decompressed ones, but mark up those that we didn't ask to preload differently
						for (int32 ShaderIdx : DecompressedShaders)
						{
							bool bShaderWasRequestedToBePreloaded = ExplicitlyPreloadedShaders.Contains(ShaderIdx);
							ShaderUsageBitmap[ShaderIdx] = bShaderWasRequestedToBePreloaded ? PreloadedAndDecompressedShaderColor : NotPreloadedButDecompressedShaderColor;
						}

						for (int32 ShaderIdx : CreatedShaders)
						{
							ShaderUsageBitmap[ShaderIdx] = CreatedShaderColor;
						}
					}

					bool bSet = PNGImageWrapper->SetRaw(ShaderUsageBitmap.GetData(), ShaderUsageBitmap.Num() * sizeof(FColor),
						ImageDimension, ImageDimension, ERGBFormat::BGRA, 8);

					if (bSet)
					{
						TArray64<uint8> CompressedData = PNGImageWrapper->GetCompressed(100);

						const FString Filename = FString::Printf(TEXT("%s_%s_RuntimeShaderUsage_%s.png"), *Name, *LexToString(ShaderPlatform), *FDateTime::Now().ToString());
						const FString SaveDir = FPaths::Combine(FPaths::ProjectSavedDir(), TEXT("Profiling"));
						const FString FilePath = FPaths::Combine(SaveDir, Filename);

						if (!FFileHelper::SaveArrayToFile(CompressedData, *FilePath))
						{
							UE_LOGF(LogShaderLibrary, Warning, "Couldn't write shader usage bitmap %ls", *FilePath);
						}
						else
						{
							UE_LOGF(LogShaderLibrary, Log, "Saved shader usage bitmap %ls. Legend: shaders not loaded from disk - dark grey, loaded to RAM explicitly - green, loaded as part of their compressed group - bright grey, decompressed: blue if they were loaded explicitly, red if just because they were a part of the group. Actually created shaders - white", 
								*FilePath);
						}
					}
					else
					{
						UE_LOGF(LogShaderLibrary, Warning, "Error creating shader usage bitmap for archive %ls (NumShaders: %d, preloaded %d, created %d) - cannot create a PNG image", *Name, NumShaders, PreloadedShaders.Num(), CreatedShaders.Num());
					}
				}
				else
				{
					UE_LOGF(LogShaderLibrary, Warning, "Couldn't create shader usage bitmap for archive %ls (NumShaders: %d, preloaded %d, created %d) - cannot create ImageWrapper for PNG format", *Name, NumShaders, PreloadedShaders.Num(), CreatedShaders.Num());
				}
			}
			else
			{
				UE_LOGF(LogShaderLibrary, Warning, "Couldn't create shader usage bitmap for archive %ls (NumShaders: %d, preloaded %d, created %d) - no ImageWrapper module", *Name, NumShaders, PreloadedShaders.Num(), CreatedShaders.Num());
			}
		}
	}
}
#endif


static FString GetDecompressionFailureExtraMessage()
{
	FString Brand = FPlatformMisc::GetCPUBrand();

	// most common offender is 13900K
	//	also seems to be a problem on 14900K and maybe 12900K
	//	not sure if 700K is affected
	// don't try to filter the brand string, just always log it if we hit a fatal error
	#if 0
	bool Is900K =
	 ( Brand.Find(TEXT("900K")) != INDEX_NONE ) ||
	 ( Brand.Find(TEXT("900-K")) != INDEX_NONE ) ||
	 ( Brand.Find(TEXT("900 K")) != INDEX_NONE );

	if ( ! Is900K )
	{
		return FString();
	}
	#endif

	return FString::Printf(TEXT("The CPU (%s) may be unstable; for details see http://www.radgametools.com/oodleintel.htm"),*Brand);
}

void ShaderCodeArchive::DecompressShaderWithOodle(uint8* OutDecompressedShader, int64 UncompressedSize, const uint8* CompressedShaderCode, int64 CompressedSize)
{
	// Iostore always compresses with Oodle.
	bool bSucceed = FCompression::UncompressMemory(NAME_Oodle, OutDecompressedShader, UncompressedSize, CompressedShaderCode, CompressedSize);
	if (!bSucceed)
	{
		UE_LOGF(LogShaderLibrary, Fatal, "ShaderCodeArchive::DecompressShader(): Could not decompress shader with Oodle. %ls",
			*GetDecompressionFailureExtraMessage());
	}
}

namespace
{
	void DecompressShadergroupWithOodleAndExtraLogging(
			const int32 GroupIndex, const FIoChunkId& GroupHash, FIoStoreShaderGroupEntry& Entry, const int32 ShaderIndex, const uint64 ShaderInGroupIndex, const FShaderHash& ShaderHash, 
			uint8* OutDecompressedShader, int64 UncompressedSize, const uint8* CompressedShaderCode, int64 CompressedSize)
	{
		bool bSucceed = FCompression::UncompressMemory(NAME_Oodle, OutDecompressedShader, UncompressedSize, CompressedShaderCode, CompressedSize);
		if (!bSucceed)
		{
			UE_LOG(LogShaderLibrary, Fatal, TEXT("DecompressShaderWithOodleAndExtraLogging(): Could not decompress shader group with Oodle. Group Index: %d Group IoStoreHash:%s Group NumShaders: %d Shader Index: %d Shader In-group Index: %" UINT64_FMT " Shader Hash: %s. %s"),
				GroupIndex,
				*LexToString(GroupHash),
				Entry.NumShaders,
				ShaderIndex,
				ShaderInGroupIndex,
				*LexToString(ShaderHash),
				*GetDecompressionFailureExtraMessage()
				);
		}
	}
}

bool ShaderCodeArchive::CompressShaderWithOodle(uint8* OutCompressedShader, int64& OutCompressedSize, const uint8* InUncompressedShaderCode, int64 InUncompressedSize, FOodleDataCompression::ECompressor InOodleCompressor, FOodleDataCompression::ECompressionLevel InOodleLevel)
{
	if (OutCompressedShader)
	{
		OutCompressedSize = FOodleDataCompression::Compress(OutCompressedShader, OutCompressedSize, InUncompressedShaderCode, InUncompressedSize, InOodleCompressor, InOodleLevel);
		check(OutCompressedSize != 0);
		return OutCompressedSize != 0;
	}
	else
	{
		// Just requesting an estimate.
		OutCompressedSize = FOodleDataCompression::CompressedBufferSizeNeeded(InUncompressedSize);
		return false;
	}	
}

void FSerializedShaderArchive::DecompressShader(int32 Index, const TArray<FSharedBuffer>& ShaderCode, TArray<uint8>& OutDecompressedShader) const
{
	const FShaderCodeEntry& Entry = ShaderEntries[Index];
	if (!Entry.IsCompressed())
	{
		OutDecompressedShader.SetNum(Entry.Size, EAllowShrinking::No);
		FMemory::Memcpy(OutDecompressedShader.GetData(), ShaderCode[Index].GetData(), Entry.Size);
	}
	else
	{
		OutDecompressedShader.SetNum(0, EAllowShrinking::No);
		(void)FSerializedShaderArchive::DecompressShaderEntryAndAppend(Entry, static_cast<const uint8*>(ShaderCode[Index].GetData()), OutDecompressedShader);
	}
}

void FSerializedShaderArchive::DecompressShaderEntryAndAppend(const FShaderCodeEntry& InEntry, const uint8* InCompressedShaderCode, TArray<uint8>& OutDecompressedShader)
{
	const int32 StartOffset = OutDecompressedShader.Num();
	OutDecompressedShader.SetNum(StartOffset + InEntry.GetUncompressedSize(), EAllowShrinking::No);
	if (!InEntry.IsCompressed())
	{
		FMemory::Memcpy(OutDecompressedShader.GetData() + StartOffset, InCompressedShaderCode, InEntry.GetUncompressedSize());
	}
	else
	{
		ShaderCodeArchive::DecompressShaderWithOodle(OutDecompressedShader.GetData() + StartOffset, InEntry.GetUncompressedSize(), InCompressedShaderCode, InEntry.Size);
	}
}

#if !USE_MMAPPED_SHADERARCHIVE
#if WITH_EDITORONLY_DATA
void FSerializedShaderArchive::SortByHashes(TArray<FSharedBuffer>& InOutShaderCodeForShaderIndex)
{
	const uint32 NumShaders = ShaderEntries.Num();
	const uint32 NumShaderMaps = ShaderMapEntries.Num();
	check(ShaderHashes.Num() == NumShaders);
	check(ShaderTypes.Num() == NumShaders);
	check(InOutShaderCodeForShaderIndex.Num() == NumShaders);
	check(ShaderMapHashes.Num() == NumShaderMaps);

	// Find the SortOrder of Shaders
	TArray<uint32> ShaderSortOrder;
	ShaderSortOrder.SetNum(NumShaders);
	for (uint32 Index = 0; Index < NumShaders; ++Index)
	{
		ShaderSortOrder[Index] = Index;
	}
	Algo::Sort(ShaderSortOrder, [this](uint32 AIndex, uint32 BIndex)
		{
			return ShaderHashes[AIndex] < ShaderHashes[BIndex];
		});
	TMap<uint32, uint32> ShaderIndexRemap;
	ShaderIndexRemap.Reserve(NumShaders);
	for (uint32 NewIndex = 0; NewIndex < NumShaders; ++NewIndex)
	{
		ShaderIndexRemap.Add(ShaderSortOrder[NewIndex], NewIndex);
	}

	// Remap all values that store an index into ShaderEntries (except ShaderHashTable, that's done below).
	for (uint32& ShaderIndex : ShaderIndices)
	{
		ShaderIndex = ShaderIndexRemap.FindChecked(ShaderIndex);
	}

	// Shuffle Shaders into the new order
	TArrayType<FShaderCodeEntry> NewShaderEntries;
	TArrayType<FShaderHash> NewShaderHashes;
	TArray<FShaderTypeHashes> NewShaderTypes;
	TArrayType<FSharedBuffer> NewShaderCodeForShaderIndex;
	NewShaderEntries.SetNum(NumShaders);
	NewShaderHashes.SetNum(NumShaders);
	NewShaderTypes.SetNum(NumShaders);
	NewShaderCodeForShaderIndex.SetNum(NumShaders);
	for (uint32 NewIndex = 0; NewIndex < NumShaders; ++NewIndex)
	{
		const uint32 OldIndex = ShaderSortOrder[NewIndex];
		NewShaderEntries[NewIndex] = MoveTemp(ShaderEntries[OldIndex]);
		NewShaderHashes[NewIndex] = MoveTemp(ShaderHashes[OldIndex]);
		NewShaderTypes[NewIndex] = MoveTemp(ShaderTypes[OldIndex]);
		NewShaderCodeForShaderIndex[NewIndex] = MoveTemp(InOutShaderCodeForShaderIndex[OldIndex]);
	}
	ShaderHashes = MoveTemp(NewShaderHashes);
	ShaderEntries = MoveTemp(NewShaderEntries);
	ShaderTypes = MoveTemp(NewShaderTypes);
	InOutShaderCodeForShaderIndex = MoveTemp(NewShaderCodeForShaderIndex);

	// Remap ShaderHashTable
	ShaderHashTable.Clear(); // Resets values but does not deallocate
	for (uint32 NewIndex = 0; NewIndex < NumShaders; ++NewIndex)
	{
		const uint32 Key = GetTypeHash(ShaderHashes[NewIndex]);
		ShaderHashTable.Add(Key, NewIndex);
	}

	// PreloadEntries are recreated in deterministic order during Finalize, so we do not need to sort them.

	// Sort each ShaderMap's internal data
	TArrayView<uint32> ShaderIndicesView = TArrayView<uint32>(ShaderIndices);
	for (FShaderMapEntry& Entry : ShaderMapEntries)
	{
		if (Entry.NumShaders > 0)
		{
			TArrayView<uint32> Indices = ShaderIndicesView.Mid(Entry.ShaderIndicesOffset, Entry.NumShaders);
			Algo::Sort(Indices);
		}
		// PreloadIndices are recreated during Finalize, so we do not need to sort them.
	}

	// Find the SortOrder of ShaderMaps
	TArray<uint32> ShaderMapSortOrder;
	ShaderMapSortOrder.SetNum(NumShaderMaps);
	for (uint32 Index = 0; Index < NumShaderMaps; ++Index)
	{
		ShaderMapSortOrder[Index] = Index;
	}
	Algo::Sort(ShaderMapSortOrder, [this](uint32 AIndex, uint32 BIndex)
		{
			return ShaderMapHashes[AIndex] < ShaderMapHashes[BIndex];
		});

	// Shuffle ShaderMaps into the new order
	TArrayType<FShaderMapEntry> NewShaderMapEntries;
	TArrayType<FShaderHash> NewShaderMapHashes;
	TArrayType<uint32> NewShaderIndices;
	NewShaderMapEntries.SetNum(NumShaderMaps);
	NewShaderMapHashes.SetNum(NumShaderMaps);
	NewShaderIndices.Reserve(ShaderIndices.Num()); // Reserve rather than SetNum so we can use Append.
	for (uint32 NewIndex = 0; NewIndex < NumShaderMaps; ++NewIndex)
	{
		const uint32 OldIndex = ShaderMapSortOrder[NewIndex];
		FShaderMapEntry& Entry = NewShaderMapEntries[NewIndex];
		Entry = MoveTemp(ShaderMapEntries[OldIndex]);
		NewShaderMapHashes[NewIndex] = MoveTemp(ShaderMapHashes[OldIndex]);

		if (Entry.NumShaders > 0)
		{
			uint32 OldShaderIndicesOffset = Entry.ShaderIndicesOffset;
			uint32 NewShaderIndicesOffset = NewShaderIndices.Num();
			TArrayView<uint32> Indices = ShaderIndicesView.Mid(OldShaderIndicesOffset, Entry.NumShaders);
			NewShaderIndices.Append(Indices);
			Entry.ShaderIndicesOffset = NewShaderIndicesOffset;
		}
	}
	ShaderMapEntries = MoveTemp(NewShaderMapEntries);
	ShaderMapHashes = MoveTemp(NewShaderMapHashes);
	ShaderIndices = MoveTemp(NewShaderIndices);

	// Remap ShaderMapHashTable
	ShaderMapHashTable.Clear(); // Resets values but does not deallocate
	for (uint32 NewIndex = 0; NewIndex < NumShaderMaps; ++NewIndex)
	{
		const uint32 Key = GetTypeHash(ShaderMapHashes[NewIndex]);
		ShaderMapHashTable.Add(Key, NewIndex);
	}
}
#endif // WITH_EDITORONLY_DATA

void FSerializedShaderArchive::Finalize()
{
	// Set the correct offsets
	{
		uint64 Offset = 0u;
		for (FShaderCodeEntry& Entry : ShaderEntries)
		{
			Entry.Offset = Offset;
			Offset += Entry.Size;
		}
	}

	constexpr int32 MaxByteGapAllowedInAPreload = 1024;
	PreloadEntries.Empty();
	for (FShaderMapEntry& ShaderMapEntry : ShaderMapEntries)
	{
		check(ShaderMapEntry.NumShaders > 0u);
		TArray<FFileCachePreloadEntry> SortedPreloadEntries;
		SortedPreloadEntries.Empty(ShaderMapEntry.NumShaders + 1);
		for (uint32 i = 0; i < ShaderMapEntry.NumShaders; ++i)
		{
			const int32 ShaderIndex = ShaderIndices[ShaderMapEntry.ShaderIndicesOffset + i];
			const FShaderCodeEntry& ShaderEntry = ShaderEntries[ShaderIndex];
			SortedPreloadEntries.Add(FFileCachePreloadEntry(ShaderEntry.Offset, ShaderEntry.Size));
		}
		SortedPreloadEntries.Sort([](const FFileCachePreloadEntry& Lhs, const FFileCachePreloadEntry& Rhs) { return Lhs.Offset < Rhs.Offset; });
		SortedPreloadEntries.Add(FFileCachePreloadEntry(INT64_MAX, 0));

		ShaderMapEntry.FirstPreloadIndex = PreloadEntries.Num();
		FFileCachePreloadEntry CurrentPreloadEntry = SortedPreloadEntries[0];
		for (uint32 PreloadIndex = 1; PreloadIndex <= ShaderMapEntry.NumShaders; ++PreloadIndex)
		{
			const FFileCachePreloadEntry& PreloadEntry = SortedPreloadEntries[PreloadIndex];
			const int64 Gap = PreloadEntry.Offset - CurrentPreloadEntry.Offset - CurrentPreloadEntry.Size;
			checkf(Gap >= 0, TEXT("Overlapping preload entries, [%lld-%lld), [%lld-%lld)"),
				CurrentPreloadEntry.Offset, CurrentPreloadEntry.Offset + CurrentPreloadEntry.Size, PreloadEntry.Offset, PreloadEntry.Offset + PreloadEntry.Size);
			if (Gap > MaxByteGapAllowedInAPreload)
			{
				++ShaderMapEntry.NumPreloadEntries;
				PreloadEntries.Add(CurrentPreloadEntry);
				CurrentPreloadEntry = PreloadEntry;
			}
			else
			{
				CurrentPreloadEntry.Size = PreloadEntry.Offset + PreloadEntry.Size - CurrentPreloadEntry.Offset;
			}
		}
		check(ShaderMapEntry.NumPreloadEntries > 0u);
		check(CurrentPreloadEntry.Size == 0);
	}
}
#endif // !USE_MMAPPED_SHADERARCHIVE

static void MakeDebugDirectoryTreeGuarded(const TCHAR* InPath)
{
	if (!IFileManager::Get().MakeDirectory(InPath, true))
	{
		UE_LOGF(LogShaderLibrary, Warning, "Failed to create directory tree for debug output: %ls", InPath);
	}
}

FString FSerializedShaderArchive::MakeDebugDirectory(const FString LibraryName, const FName& FormatAndPlatformName, const TCHAR* BasePath)
{
	using namespace UE::ShaderLibrary::Private;
	FString DebugLibFolder = BasePath != nullptr ? BasePath : GetShaderDebugInfoPath();
	if (!LibraryName.IsEmpty() && FormatAndPlatformName != NAME_None)
	{
		DebugLibFolder = GetShaderDebugFolder(DebugLibFolder / FormatAndPlatformName.ToString(), LibraryName, FormatAndPlatformName);
	}
	MakeDebugDirectoryTreeGuarded(*DebugLibFolder);
	return DebugLibFolder;
}

bool FSerializedShaderArchive::SerializeHeaderVersion(FArchive& Ar)
{
	using UE::ShaderLibrary::Private::GShaderCodeArchiveVersion;
	uint32 Version = Ar.IsLoading() ? 0u : GShaderCodeArchiveVersion;
	Ar << Version;
	return Version == GShaderCodeArchiveVersion;
}

void FSerializedShaderArchive::Serialize(FArchive& Ar)
{
#if USE_MMAPPED_SHADERARCHIVE
	auto SerializeMappedToArrayView = [](auto& ArrayView, FStaticMemoryReader& Ar)
	{
        using ArrayType = std::remove_cvref_t <decltype(ArrayView)>;
        typename ArrayType::SizeType SerializeNum;
        
        Ar << SerializeNum;

        uint64 ArrayBytes = SerializeNum * sizeof(typename ArrayType::ElementType);
        uint64 Offset = Ar.Tell();

        ArrayView = TArrayView<typename ArrayType::ElementType >((typename ArrayType::ElementType*)(Ar.GetData() + Offset), SerializeNum);
        Ar.Seek(Offset + ArrayBytes);
	};
    
    FStaticMemoryReader& MemReaderAr = static_cast<FStaticMemoryReader&>(Ar);
    if (Ar.GetArchiveName() != TEXT("FStaticMemoryReader"))
    {
        UE_LOGF(LogShaderLibrary, Fatal, "mmapped shader archive must be serialized via FStaticMemoryReader");
    }
    else
    {
        SerializeMappedToArrayView(ShaderMapHashes, MemReaderAr);
        SerializeMappedToArrayView(ShaderHashes, MemReaderAr);
        SerializeMappedToArrayView(ShaderMapEntries, MemReaderAr);
        SerializeMappedToArrayView(ShaderEntries, MemReaderAr);
        SerializeMappedToArrayView(PreloadEntries, MemReaderAr);
        SerializeMappedToArrayView(ShaderIndices, MemReaderAr);
    }
#else
	Ar << ShaderMapHashes;
	Ar << ShaderHashes;
	Ar << ShaderMapEntries;
	Ar << ShaderEntries;
	Ar << PreloadEntries;
	Ar << ShaderIndices;
#endif

	check(ShaderHashes.Num() == ShaderEntries.Num());
	check(ShaderMapHashes.Num() == ShaderMapEntries.Num());

	if (Ar.IsLoading())
	{
		{
			const uint32 HashSize = FMath::Min<uint32>(0x10000, 1u << FMath::CeilLogTwo(ShaderMapHashes.Num()));
			ShaderMapHashTable.Clear(HashSize, ShaderMapHashes.Num());
			for (int32 Index = 0; Index < ShaderMapHashes.Num(); ++Index)
			{
				const uint32 Key = GetTypeHash(ShaderMapHashes[Index]);
				ShaderMapHashTable.Add(Key, Index);
			}
		}
		{
			const uint32 HashSize = FMath::Min<uint32>(0x10000, 1u << FMath::CeilLogTwo(ShaderHashes.Num()));
			ShaderHashTable.Clear(HashSize, ShaderHashes.Num());
			for (int32 Index = 0; Index < ShaderHashes.Num(); ++Index)
			{
				const uint32 Key = GetTypeHash(ShaderHashes[Index]);
				ShaderHashTable.Add(Key, Index);
			}
		}
	}
}

#if WITH_EDITORONLY_DATA
void FSerializedShaderArchive::SaveAssetInfo(FArchive& Ar)
{
	if (Ar.IsSaving())
	{
		FString JsonTcharText;
		{
			TSharedRef<TJsonWriter<TCHAR, TPrettyJsonPrintPolicy<TCHAR>>> Writer = TJsonWriterFactory<TCHAR, TPrettyJsonPrintPolicy<TCHAR>>::Create(&JsonTcharText);
			Writer->WriteObjectStart();

			Writer->WriteValue(TEXT("AssetInfoVersion"), static_cast<int32>(EAssetInfoVersion::CurrentVersion));

			TArray<const TPair<FShaderHash, FShaderMapAssetPaths>*> SortedData;
			ShaderMapAssetAssociations.SortForSaving();
			SortedData.Reserve(ShaderMapAssetAssociations.ViewShaderMaps().Num());
			for (const TPair<FShaderHash, FShaderMapAssetPaths>& Pair : ShaderMapAssetAssociations.ViewShaderMaps())
			{
				SortedData.Add(&Pair);
			}

			Writer->WriteArrayStart(TEXT("ShaderCodeToAssets"));
			for (const TPair<FShaderHash, FShaderMapAssetPaths>* Pair : SortedData)
			{
				Writer->WriteObjectStart();
				const FShaderHash& Hash = Pair->Key;
				Writer->WriteValue(TEXT("ShaderMapHash"), Hash.ToString());
				const FShaderMapAssetPaths& Assets = Pair->Value;

				Writer->WriteArrayStart(TEXT("Assets"));
				for (FShaderMapAssetPaths::TConstIterator AssetIter(Assets); AssetIter; ++AssetIter)
				{
					Writer->WriteValue((*AssetIter).ToString());
				}
				Writer->WriteArrayEnd();
				Writer->WriteObjectEnd();
			}
			Writer->WriteArrayEnd();

			Writer->WriteObjectEnd();
			Writer->Close();
		}

		FTCHARToUTF8 JsonUtf8(*JsonTcharText);
		Ar.Serialize(const_cast<void *>(reinterpret_cast<const void*>(JsonUtf8.Get())), JsonUtf8.Length() * sizeof(UTF8CHAR));
	}
}

bool FSerializedShaderArchive::LoadAssetInfo(const FString& Filename, ECookShaderLibrarySource CookSource)
{
	TUniquePtr<FArchive> Reader(IFileManager::Get().CreateFileReader(*Filename));
	return LoadAssetInfo(Reader.Get(), CookSource);
}

bool FSerializedShaderArchive::LoadAssetInfo(FArchive* Ar, ECookShaderLibrarySource CookSource)
{
	if (!Ar)
	{
		return false;
	}

	FString JsonText;
	FFileHelper::LoadFileToString(JsonText, *Ar);

	TSharedPtr<FJsonObject> JsonObject;
	TSharedRef<TJsonReader<TCHAR>> Reader = TJsonReaderFactory<TCHAR>::Create(JsonText);

	// Attempt to deserialize JSON
	if (!FJsonSerializer::Deserialize(Reader, JsonObject) || !JsonObject.IsValid())
	{
		return false;
	}

	TSharedPtr<FJsonValue> AssetInfoVersion = JsonObject->Values.FindRef(TEXT("AssetInfoVersion"));
	if (!AssetInfoVersion.IsValid())
	{
		UE_LOGF(LogShaderLibrary, Warning, "Rejecting asset info file %ls: missing AssetInfoVersion (damaged file?)", 
			*Ar->GetArchiveName());
		return false;
	}
	
	const EAssetInfoVersion FileVersion = static_cast<EAssetInfoVersion>(static_cast<int64>(AssetInfoVersion->AsNumber()));
	if (FileVersion != EAssetInfoVersion::CurrentVersion)
	{
		UE_LOGF(LogShaderLibrary, Warning, "Rejecting asset info file %ls: expected version %d, got unsupported version %d.",
			*Ar->GetArchiveName(), static_cast<int32>(EAssetInfoVersion::CurrentVersion), static_cast<int32>(FileVersion));
		return false;
	}

	TSharedPtr<FJsonValue> AssetInfoArrayValue = JsonObject->Values.FindRef(TEXT("ShaderCodeToAssets"));
	if (!AssetInfoArrayValue.IsValid())
	{
		UE_LOGF(LogShaderLibrary, Warning, "Rejecting asset info file %ls: missing ShaderCodeToAssets array (damaged file?)",
			*Ar->GetArchiveName());
		return false;
	}
	
	TArray<TSharedPtr<FJsonValue>> AssetInfoArray = AssetInfoArrayValue->AsArray();
	UE_LOGF(LogShaderLibrary, Display, "Reading asset info file %ls: found %d existing mappings",
		*Ar->GetArchiveName(), AssetInfoArray.Num());

	for (int32 IdxPair = 0, NumPairs = AssetInfoArray.Num(); IdxPair < NumPairs; ++IdxPair)
	{
		TSharedPtr<FJsonObject> Pair = AssetInfoArray[IdxPair]->AsObject();
		if (UNLIKELY(!Pair.IsValid()))
		{
			UE_LOGF(LogShaderLibrary, Warning, "Rejecting asset info file %ls: ShaderCodeToAssets array contains unreadable mapping #%d (damaged file?)",
				*Ar->GetArchiveName(),
				IdxPair
				);
			return false;
		}

		TSharedPtr<FJsonValue> ShaderMapHashJson = Pair->Values.FindRef(TEXT("ShaderMapHash"));
		if (UNLIKELY(!ShaderMapHashJson.IsValid()))
		{
			UE_LOGF(LogShaderLibrary, Warning, "Rejecting asset info file %ls: ShaderCodeToAssets array contains unreadable ShaderMapHash for mapping %d (damaged file?)",
				*Ar->GetArchiveName(),
				IdxPair
				);
			return false;
		}

		FShaderHash ShaderMapHash;
		ShaderMapHash.FromString(ShaderMapHashJson->AsString());

		TSharedPtr<FJsonValue> AssetPathsArrayValue = Pair->Values.FindRef(TEXT("Assets"));
		if (UNLIKELY(!AssetPathsArrayValue.IsValid()))
		{
			UE_LOGF(LogShaderLibrary, Warning, "Rejecting asset info file %ls: ShaderCodeToAssets array contains unreadable Assets array for mapping %d (damaged file?)",
				*Ar->GetArchiveName(),
				IdxPair
			);
			return false;
		}

		TArray<TSharedPtr<FJsonValue>> AssetPathsArray = AssetPathsArrayValue->AsArray();
		for (TSharedPtr<FJsonValue>& AssetPath : AssetPathsArray)
		{
			FShaderMapAssetAssociations::FAssociatedAssetData& AssetData
				= ShaderMapAssetAssociations.FindOrAddAsset(FName(*AssetPath->AsString()), ShaderMapHash);
			AssetData.MergeSource(CookSource);
		}
	}

	return true;
}

void FSerializedShaderArchive::CreateAsChunkFrom(const FSerializedShaderArchive& Parent, const TSet<FName>& PackagesInChunk, TArray<int32>& OutShaderCodeEntriesNeeded)
{
	CreateFromFilteredAssets(Parent, OutShaderCodeEntriesNeeded,
		[&PackagesInChunk](FName AssetName, const FShaderMapAssetAssociations::FAssociatedAssetData& AssetData)
		{
			return PackagesInChunk.Contains(AssetName);
		});
}

void FSerializedShaderArchive::CreateFromFilteredAssets(const FSerializedShaderArchive& Parent, TArray<int32>& OutShaderCodeEntriesNeeded,
	FShaderMapAssetAssociations::FAssetFilterFunctionRef ShouldKeepAsset,
	FShaderMapAssetAssociations::FShaderMapFilterFunction ShouldExcludeShaderMap)
{
	// we should begin with a clean slate
	checkf(ShaderMapHashes.Num() == 0 && ShaderHashes.Num() == 0 && ShaderMapEntries.Num() == 0 && ShaderEntries.Num() == 0 && PreloadEntries.Num() == 0 && ShaderIndices.Num() == 0,
		TEXT("Expecting a new, uninitialized FSerializedShaderArchive instance for CreateFromFilteredAssets."));

	// go through parent's shadermap hashes in the order of their addition
	for (int32 IdxSM = 0, NumSMs = Parent.ShaderMapHashes.Num(); IdxSM < NumSMs; ++IdxSM)
	{
		const FShaderHash& ShaderMapHash = Parent.ShaderMapHashes[IdxSM];
		const FShaderMapAssetPaths* Assets = Parent.ShaderMapAssetAssociations.FindShaderMap(ShaderMapHash);
		bool bIncludeSM = false;
		if (UNLIKELY(Assets == nullptr))
		{
			if (UE::ShaderLibrary::Private::bIncrementalCookPruningEnabled)
			{
				// ShaderMaps with no Assets occur when we load a ShaderLibrary from a previous cook, and find that
				// one of the Assets it recorded has recompiled in the current cook, and it recompiled with different ShaderMaps
				// than were previously recorded. The old ShaderMaps are added, but their association with the Asset is removed.
				// These unreferenced ShaderMaps should be pruned from any CreateFromFilteredAssets call.
				bIncludeSM = false;
			}
			else
			{
				// In the behavior before IncrementalCookPruning was implemented, it should never be possible for a ShaderMap
				// to be lacking an asset. Log a warning and conservatively keep the ShaderMap in every CreateFromFilteredAssets call.
				UE_LOGF(LogShaderLibrary, Warning, "Shadermap %ls is not associated with any asset. Including it in CreateFromFilteredAssets", *ShaderMapHash.ToString());
				bIncludeSM = true;
			}
		}
		else if (!ShouldExcludeShaderMap || !ShouldExcludeShaderMap(ShaderMapHash))
		{
			// if any asset is in the chunk, include
			for (const FName& AssetName : *Assets)
			{
				const FShaderMapAssetAssociations::FAssociatedAssetData* AssetData = Parent.ShaderMapAssetAssociations.FindAsset(AssetName);
				check(AssetData); // FShaderMapAssetAssociations guarantees that the Assets using a Shader are present in its list of assets.
				if (ShouldKeepAsset(AssetName, *AssetData))
				{
					bIncludeSM = true;
					break;
				}
			}
		}

		if (bIncludeSM)
		{
			// add this shader map
			int32 ShaderMapIndex = INDEX_NONE;
			if (FindOrAddShaderMapEditor(ShaderMapHash, ShaderMapIndex, Assets, ECookShaderLibrarySource::CurrentCook))
			{
				// if we're in this scope, it means it's a new shadermap for the chunk and we need more information about it from the parent
				int32 ParentShaderMapIndex = IdxSM;
				const FShaderMapEntry& ParentShaderMapDescriptor = Parent.ShaderMapEntries[ParentShaderMapIndex];

				const int32 NumShaders = ParentShaderMapDescriptor.NumShaders;

				FShaderMapEntry& ShaderMapDescriptor = ShaderMapEntries[ShaderMapIndex];
				ShaderMapDescriptor.NumShaders = NumShaders;
				ShaderMapDescriptor.ShaderIndicesOffset = ShaderIndices.AddZeroed(NumShaders);

				// add shader by shader
				for (int32 ShaderIdx = 0; ShaderIdx < NumShaders; ++ShaderIdx)
				{
					int32 ParentShaderIndex = Parent.ShaderIndices[ParentShaderMapDescriptor.ShaderIndicesOffset + ShaderIdx];

					int32 ShaderIndex = INDEX_NONE;
					if (FindOrAddShader(Parent.ShaderHashes[ParentShaderIndex], ShaderIndex))
					{
						// new shader! add it to the mapping of parent shadercode entries to ours. and check the integrity of the mapping
						checkf(OutShaderCodeEntriesNeeded.Num() == ShaderIndex, TEXT("Mapping between the shader indices in a chunk and the whole archive is inconsistent"));
						OutShaderCodeEntriesNeeded.Add(ParentShaderIndex);

						// copy the entry as is
						ShaderEntries[ShaderIndex] = Parent.ShaderEntries[ParentShaderIndex];
						ShaderTypes[ShaderIndex] = Parent.ShaderTypes[ParentShaderIndex];
					}
					ShaderIndices[ShaderMapDescriptor.ShaderIndicesOffset + ShaderIdx] = ShaderIndex;
				}
			}
		}
	}
}

void FSerializedShaderArchive::CollectStatsAndDebugInfo(FDebugStats& OutDebugStats, FExtendedDebugStats* OutExtendedDebugStats)
{
	// collect the light-weight stats first
	FMemory::Memzero(OutDebugStats);
	OutDebugStats.NumUniqueShaders = ShaderHashes.Num();
	OutDebugStats.NumShaderMaps = ShaderMapHashes.Num();
	int32 TotalShaders = 0;
	int64 TotalShaderSize = 0;
	uint32 MinSMSizeInShaders = UINT_MAX;
	uint32 MaxSMSizeInShaders = 0;
	for (const FShaderMapEntry& SMEntry : ShaderMapEntries)
	{
		MinSMSizeInShaders = FMath::Min(MinSMSizeInShaders, SMEntry.NumShaders);
		MaxSMSizeInShaders = FMath::Max(MaxSMSizeInShaders, SMEntry.NumShaders);
		TotalShaders += SMEntry.NumShaders;

		const int32 ThisSMShaders = SMEntry.NumShaders;
		for (int32 ShaderIdx = 0; ShaderIdx < ThisSMShaders; ++ShaderIdx)
		{
			TotalShaderSize += ShaderEntries[ShaderIndices[SMEntry.ShaderIndicesOffset + ShaderIdx]].Size;
		}
	}
	OutDebugStats.NumShaders = TotalShaders;
	OutDebugStats.ShadersSize = TotalShaderSize;
	OutDebugStats.NumAssets = ShaderMapAssetAssociations.ViewAssets().Num();

	int64 ActuallySavedShaderSize = 0;
	for (const FShaderCodeEntry& ShaderEntry : ShaderEntries)
	{
		ActuallySavedShaderSize += ShaderEntry.Size;
	}
	OutDebugStats.ShadersUniqueSize = ActuallySavedShaderSize;

	// If OutExtendedDebugStats pointer is passed, we're asked to fill out a heavy-weight stats.
	if (OutExtendedDebugStats)
	{
		// textual rep
		DumpContentsInPlaintext(OutExtendedDebugStats->TextualRepresentation);

		OutExtendedDebugStats->MinNumberOfShadersPerSM = MinSMSizeInShaders;
		OutExtendedDebugStats->MaxNumberofShadersPerSM = MaxSMSizeInShaders;

		// median SM size in shaders
		TArray<int32> ShadersInSM;

		// shader usage
		TMap<int32, int32> ShaderToUsageMap;

		for (const FShaderMapEntry& SMEntry : ShaderMapEntries)
		{
			const int32 ThisSMShaders = SMEntry.NumShaders;
			ShadersInSM.Add(ThisSMShaders);

			for (int32 ShaderIdx = 0; ShaderIdx < ThisSMShaders; ++ShaderIdx)
			{
				int ShaderIndex = ShaderIndices[SMEntry.ShaderIndicesOffset + ShaderIdx];
				int32& Usage = ShaderToUsageMap.FindOrAdd(ShaderIndex, 0);
				++Usage;
			}
		}

		ShadersInSM.Sort();
		OutExtendedDebugStats->MedianNumberOfShadersPerSM = ShadersInSM.Num() ? ShadersInSM[ShadersInSM.Num() / 2] : 0;

		ShaderToUsageMap.ValueSort(TGreater<int32>());
		// add top 10 shaders
		for (const TTuple<int32, int32>& UsagePair : ShaderToUsageMap)
		{
			OutExtendedDebugStats->TopShaderUsages.Add(UsagePair.Value);
			if (OutExtendedDebugStats->TopShaderUsages.Num() >= 10)
			{
				break;
			}
		}

		// calculate per-frequency stats
		FMemory::Memzero(OutExtendedDebugStats->NumShadersPerFrequency);
		FMemory::Memzero(OutExtendedDebugStats->UncompressedSizePerFrequency);
		FMemory::Memzero(OutExtendedDebugStats->CompressedSizePerFrequency);
		for (const FShaderCodeEntry& ShaderEntry : ShaderEntries)
		{
			EShaderFrequency Frequency = ShaderEntry.GetFrequency();
			check(Frequency < UE_ARRAY_COUNT(OutExtendedDebugStats->NumShadersPerFrequency));
			++OutExtendedDebugStats->NumShadersPerFrequency[Frequency];
			check(Frequency < UE_ARRAY_COUNT(OutExtendedDebugStats->UncompressedSizePerFrequency));
			OutExtendedDebugStats->UncompressedSizePerFrequency[Frequency] += ShaderEntry.GetUncompressedSize();
			check(Frequency < UE_ARRAY_COUNT(OutExtendedDebugStats->UncompressedSizePerFrequency));
			OutExtendedDebugStats->CompressedSizePerFrequency[Frequency] += ShaderEntry.Size;
		}
	}

#if 0 // graph visualization - maybe one day we'll return to this
		// enumerate all shaders first (so they can be identified by people looking them up in other debug output)
		int32 IdxShaderNum = 0;
		for (const FShaderHash& ShaderHash : ShaderHashes)
		{
			FString Numeral = FString::Printf(TEXT("Shd_%d"), IdxShaderNum);
			OutRelationshipGraph->Add(TTuple<FString, FString>(Numeral, FString("Hash_") + ShaderHash.ToString()));
			++IdxShaderNum;
		}

		// add all assets if any
		for (TMap<FName, FShaderHash>::TConstIterator Iter(AssetToShaderCode); Iter; ++Iter)
		{
			int32 SMIndex = FindShaderMap(Iter.Value());			
			OutRelationshipGraph->Add(TTuple<FString, FString>(Iter.Key().ToString(), FString::Printf(TEXT("SM_%d"), SMIndex)));
		}

		// shadermaps to shaders
		int NumSMs = ShaderMapHashes.Num();
		for (int32 IdxSM = 0; IdxSM < NumSMs; ++IdxSM)
		{
			FString SMId = FString::Printf(TEXT("SM_%d"), IdxSM);
			const FShaderMapEntry& SMEntry = ShaderMapEntries[IdxSM];

			const int32 ThisSMShaders = SMEntry.NumShaders;
			for (int32 ShaderIdx = 0; ShaderIdx < ThisSMShaders; ++ShaderIdx)
			{
				FString ReferencedShader = FString::Printf(TEXT("Shd_%d"), ShaderIndices[SMEntry.ShaderIndicesOffset + ShaderIdx]);
				OutRelationshipGraph->Add(TTuple<FString, FString>(SMId, ReferencedShader));
			}
		}
#endif // 0
}

void FSerializedShaderArchive::DumpContentsInPlaintext(FString& OutText) const
{
	TStringBuilder<256> Out;
	Out << TEXT("FSerializedShaderArchive\n{\n");
	{
		Out << TEXT("\tShaderMapHashes\n\t{\n");
		for (int32 IdxMapHash = 0, NumMapHashes = ShaderMapHashes.Num(); IdxMapHash < NumMapHashes; ++IdxMapHash)
		{
			Out << TEXT("\t\t");
			Out << ShaderMapHashes[IdxMapHash].ToString();
			Out << TEXT("\n");
		}
		Out << TEXT("\t}\n");
	}

	{
		Out << TEXT("\tShaderHashes\n\t{\n");
		for (int32 IdxHash = 0, NumHashes = ShaderHashes.Num(); IdxHash < NumHashes; ++IdxHash)
		{
			Out << TEXT("\t\t");
			Out << ShaderHashes[IdxHash].ToString();
			Out << TEXT("\n");
		}
		Out << TEXT("\t}\n");
	}

	{
		Out << TEXT("\tShaderMapEntries\n\t{\n");
		for (int32 IdxEntry = 0, NumEntries = ShaderMapEntries.Num(); IdxEntry < NumEntries; ++IdxEntry)
		{
			Out << TEXT("\t\tFShaderMapEntry\n\t\t{\n");

			Out << TEXT("\t\t\tShaderIndicesOffset : ");
			Out << ShaderMapEntries[IdxEntry].ShaderIndicesOffset;
			Out << TEXT("\n");

			Out << TEXT("\t\t\tNumShaders : ");
			Out << ShaderMapEntries[IdxEntry].NumShaders;
			Out << TEXT("\n");

			Out << TEXT("\t\t\tFirstPreloadIndex : ");
			Out << ShaderMapEntries[IdxEntry].FirstPreloadIndex;
			Out << TEXT("\n");

			Out << TEXT("\t\t\tNumPreloadEntries : ");
			Out << ShaderMapEntries[IdxEntry].NumPreloadEntries;
			Out << TEXT("\n");

			Out << TEXT("\t\t}\n");
		}
		Out << TEXT("\t}\n");
	}

	{
		Out << TEXT("\tShaderEntries\n\t{\n");
		for (int32 IdxEntry = 0, NumEntries = ShaderEntries.Num(); IdxEntry < NumEntries; ++IdxEntry)
		{
			Out << TEXT("\t\tFShaderCodeEntry\n\t\t{\n");

			Out << TEXT("\t\t\tOffset : ");
			Out << ShaderEntries[IdxEntry].Offset;
			Out << TEXT("\n");

			Out << TEXT("\t\t\tSize : ");
			Out << ShaderEntries[IdxEntry].Size;
			Out << TEXT("\n");

			Out << TEXT("\t\t\tUncompressedSize : ");
			Out << ShaderEntries[IdxEntry].GetUncompressedSize();
			Out << TEXT("\n");

			Out << TEXT("\t\t\tFrequency : ");
			Out << (uint8)ShaderEntries[IdxEntry].GetFrequency();
			Out << TEXT("\n");

			Out << TEXT("\t\t}\n");
		}
		Out << TEXT("\t}\n");
	}

	{
		Out << TEXT("\tPreloadEntries\n\t{\n");
		for (int32 IdxEntry = 0, NumEntries = PreloadEntries.Num(); IdxEntry < NumEntries; ++IdxEntry)
		{
			Out << TEXT("\t\tFFileCachePreloadEntry\n\t\t{\n");

			Out << TEXT("\t\t\tOffset : ");
			Out << PreloadEntries[IdxEntry].Offset;
			Out << TEXT("\n");

			Out << TEXT("\t\t\tSize : ");
			Out << PreloadEntries[IdxEntry].Size;
			Out << TEXT("\n");

			Out << TEXT("\t\t}\n");
		}
		Out << TEXT("\t}\n");
	}

	{
		Out << TEXT("\tShaderIndices\n\t{\n");
		// split it by shadermaps
		int32 IdxSMEntry = 0;
		int32 NumShadersLeftInSM = ShaderMapEntries.Num() ? ShaderMapEntries[0].NumShaders : 0;
		bool bNewSM = true;
		for (int32 IdxEntry = 0, NumEntries = ShaderIndices.Num(); IdxEntry < NumEntries; ++IdxEntry)
		{
			if (UNLIKELY(bNewSM))
			{
				Out << TEXT("\t\t");
				bNewSM = false;
			}
			else
			{
				Out << TEXT(", ");
			}
			Out << ShaderIndices[IdxEntry];

			--NumShadersLeftInSM;
			while (NumShadersLeftInSM == 0)
			{
				bNewSM = true;
				++IdxSMEntry;
				if (IdxSMEntry >= ShaderMapEntries.Num())
				{
					break;
				}
				NumShadersLeftInSM = ShaderMapEntries[IdxSMEntry].NumShaders;
			}

			if (bNewSM)
			{
				Out << TEXT("\n");
			}
		}
		Out << TEXT("\t}\n");
	}

	Out << TEXT("}\n");
	OutText = FStringView(Out);
}

#endif // WITH_EDITORONLY_DATA

FShaderCodeArchive* FShaderCodeArchive::Create(EShaderPlatform InPlatform, FArchive& Ar, const FString& InDestFilePath, const FString& InLibraryDir, const FString& InLibraryName)
{
	FShaderCodeArchive* Library = new FShaderCodeArchive(InPlatform, InLibraryDir, InLibraryName);
	Ar << Library->SerializedShaders;
	Library->ShaderPreloads.SetNum(Library->SerializedShaders.GetNumShaders());
	Library->LibraryCodeOffset = Ar.Tell();

	// Open library for async reads
	Library->FileCacheHandle = IFileCacheHandle::CreateFileCacheHandle(*InDestFilePath);

	Library->DebugVisualizer.Initialize(Library->SerializedShaders.GetShaderEntries().Num());

	UE_LOGF(LogShaderLibrary, Display, "Using %ls for material shader code. Total %d unique shaders.", *InDestFilePath, Library->SerializedShaders.GetShaderEntries().Num());

	INC_DWORD_STAT_BY(STAT_Shaders_ShaderResourceMemory, Library->GetSizeBytes());

	return Library;
}

FShaderCodeArchive::FShaderCodeArchive(EShaderPlatform InPlatform, const FString& InLibraryDir, const FString& InLibraryName)
	: FRHIShaderLibrary(InPlatform, InLibraryName)
	, LibraryDir(InLibraryDir)
	, LibraryCodeOffset(0)
	, FileCacheHandle(nullptr)
{
}

FShaderCodeArchive::~FShaderCodeArchive()
{
	DEC_DWORD_STAT_BY(STAT_Shaders_ShaderResourceMemory, GetSizeBytes());
	Teardown();
}

void FShaderCodeArchive::Teardown()
{
	if (FileCacheHandle)
	{
		delete FileCacheHandle;
		FileCacheHandle = nullptr;
	}

	for (int32 ShaderIndex = 0; ShaderIndex < SerializedShaders.GetNumShaders(); ++ShaderIndex)
	{
		FShaderPreloadEntry& ShaderPreloadEntry = ShaderPreloads[ShaderIndex];
		if (ShaderPreloadEntry.Code)
		{
			const FShaderCodeEntry& ShaderEntry = SerializedShaders.GetShaderEntries()[ShaderIndex];
			FMemory::Free(ShaderPreloadEntry.Code);
			ShaderPreloadEntry.Code = nullptr;
			DEC_DWORD_STAT_BY(STAT_Shaders_ShaderPreloadMemory, ShaderEntry.Size);
#if (CSV_PROFILER_STATS && !UE_BUILD_SHIPPING) 
			TCsvPersistentCustomStat<float>* CsvStatPreloadedShaderMB = FCsvProfiler::Get()->GetOrCreatePersistentCustomStatFloat(TEXT("PreloadedShaderMB"), CSV_CATEGORY_INDEX(Shaders));
			CsvStatPreloadedShaderMB->Sub((float)ShaderEntry.Size / (1024.0f * 1024.0f));
#endif
		}
	}

	DebugVisualizer.SaveShaderUsageBitmap(GetName(), GetPlatform());
}

void FShaderCodeArchive::OnShaderPreloadFinished(int32 ShaderIndex, const IMemoryReadStreamRef& PreloadData)
{
	const FShaderCodeEntry& ShaderEntry = SerializedShaders.GetShaderEntries()[ShaderIndex];
	PreloadData->EnsureReadNonBlocking();		// Ensure data is ready before taking the lock
	{
		FWriteScopeLock Lock(ShaderPreloadLock);
		FShaderPreloadEntry& ShaderPreloadEntry = ShaderPreloads[ShaderIndex];
		PreloadData->CopyTo(ShaderPreloadEntry.Code, 0, ShaderEntry.Size);
		ShaderPreloadEntry.PreloadEvent.SafeRelease();
	}
}

struct FPreloadShaderTask
{
	explicit FPreloadShaderTask(FShaderCodeArchive* InArchive, int32 InShaderIndex, const IMemoryReadStreamRef& InData)
		: Archive(InArchive), Data(InData), ShaderIndex(InShaderIndex)
	{}

	FShaderCodeArchive* Archive;
	IMemoryReadStreamRef Data;
	int32 ShaderIndex;

	void DoTask(ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent)
	{
		Archive->OnShaderPreloadFinished(ShaderIndex, Data);
		Data.SafeRelease();
	}

	FORCEINLINE static ESubsequentsMode::Type GetSubsequentsMode() { return ESubsequentsMode::TrackSubsequents; }
	FORCEINLINE ENamedThreads::Type GetDesiredThread() { return ENamedThreads::AnyNormalThreadNormalTask; }
	FORCEINLINE TStatId GetStatId() const { return TStatId(); }
};

bool FShaderCodeArchive::PreloadShader(int32 ShaderIndex, FGraphEventArray& OutCompletionEvents)
{
	LLM_SCOPE(ELLMTag::Shaders);

	FWriteScopeLock Lock(ShaderPreloadLock);

	FShaderPreloadEntry& ShaderPreloadEntry = ShaderPreloads[ShaderIndex];
	checkf(!ShaderPreloadEntry.bNeverToBePreloaded, TEXT("We are preloading a shader that shouldn't be preloaded in this run (e.g. raytracing shader on D3D11)."));
	const uint32 ShaderNumRefs = ShaderPreloadEntry.NumRefs++;
	if (ShaderNumRefs == 0u)
	{
		check(!ShaderPreloadEntry.PreloadEvent);

		const FShaderCodeEntry& ShaderEntry = SerializedShaders.GetShaderEntries()[ShaderIndex];
		ShaderPreloadEntry.Code = FMemory::Malloc(ShaderEntry.Size);
		ShaderPreloadEntry.FramePreloadStarted = GFrameNumber;
		DebugVisualizer.MarkExplicitlyPreloadedForVisualization(ShaderIndex);

		const EAsyncIOPriorityAndFlags IOPriority = (EAsyncIOPriorityAndFlags)GShaderCodeLibraryAsyncLoadingPriority;

		FGraphEventArray ReadCompletionEvents;

		EAsyncIOPriorityAndFlags DontCache = GShaderCodeLibraryAsyncLoadingAllowDontCache ? AIOP_FLAG_DONTCACHE : AIOP_MIN;
		IMemoryReadStreamRef PreloadData = FileCacheHandle->ReadData(ReadCompletionEvents, LibraryCodeOffset + ShaderEntry.Offset, ShaderEntry.Size, IOPriority | DontCache);
		auto Task = TGraphTask<FPreloadShaderTask>::CreateTask(&ReadCompletionEvents).ConstructAndHold(this, ShaderIndex, MoveTemp(PreloadData));
		ShaderPreloadEntry.PreloadEvent = Task->GetCompletionEvent();
		Task->Unlock();

		INC_DWORD_STAT_BY(STAT_Shaders_ShaderPreloadMemory, ShaderEntry.Size);

#if (CSV_PROFILER_STATS && !UE_BUILD_SHIPPING) 
		TCsvPersistentCustomStat<float>* CsvStatPreloadedShaderMB = FCsvProfiler::Get()->GetOrCreatePersistentCustomStatFloat(TEXT("PreloadedShaderMB"), CSV_CATEGORY_INDEX(Shaders));
		CsvStatPreloadedShaderMB->Add((float)ShaderEntry.Size / (1024.0f * 1024.0f));
#endif
	}

	if (ShaderPreloadEntry.PreloadEvent)
	{
		OutCompletionEvents.Add(ShaderPreloadEntry.PreloadEvent);
	}
	return true;
}

bool FShaderCodeArchive::PreloadShaderMap(int32 ShaderMapIndex, FGraphEventArray& OutCompletionEvents)
{
	LLM_SCOPE(ELLMTag::Shaders);

	const FShaderMapEntry& ShaderMapEntry = SerializedShaders.GetShaderMapEntries()[ShaderMapIndex];
	const EAsyncIOPriorityAndFlags IOPriority = (EAsyncIOPriorityAndFlags)GShaderCodeLibraryAsyncLoadingPriority;
	const uint32 FrameNumber = GFrameNumber;
	uint32 PreloadMemory = 0u;
	
	FWriteScopeLock Lock(ShaderPreloadLock);

	TArrayView ShaderIndices = SerializedShaders.GetShaderIndices();
	for (uint32 i = 0u; i < ShaderMapEntry.NumShaders; ++i)
	{
		const int32 ShaderIndex = ShaderIndices[ShaderMapEntry.ShaderIndicesOffset + i];
		FShaderPreloadEntry& ShaderPreloadEntry = ShaderPreloads[ShaderIndex];
		const FShaderCodeEntry& ShaderEntry = SerializedShaders.GetShaderEntries()[ShaderIndex];

#if RHI_RAYTRACING
		if (!IsRayTracingAllowed() && !IsCreateShadersOnLoadEnabled() && IsRayTracingShaderFrequency(static_cast<EShaderFrequency>(ShaderEntry.GetFrequency())))
		{
			ShaderPreloadEntry.bNeverToBePreloaded = 1;
			continue;
		}
#endif

		const uint32 ShaderNumRefs = ShaderPreloadEntry.NumRefs++;
		if (ShaderNumRefs == 0u)
		{
			check(!ShaderPreloadEntry.PreloadEvent);
			ShaderPreloadEntry.Code = FMemory::Malloc(ShaderEntry.Size);
			ShaderPreloadEntry.FramePreloadStarted = FrameNumber;
			DebugVisualizer.MarkExplicitlyPreloadedForVisualization(ShaderIndex);
			PreloadMemory += ShaderEntry.Size;

			FGraphEventArray ReadCompletionEvents;
			EAsyncIOPriorityAndFlags DontCache = GShaderCodeLibraryAsyncLoadingAllowDontCache ? AIOP_FLAG_DONTCACHE : AIOP_MIN;
			IMemoryReadStreamRef PreloadData = FileCacheHandle->ReadData(ReadCompletionEvents, LibraryCodeOffset + ShaderEntry.Offset, ShaderEntry.Size, IOPriority | DontCache);
			auto Task = TGraphTask<FPreloadShaderTask>::CreateTask(&ReadCompletionEvents).ConstructAndHold(this, ShaderIndex, MoveTemp(PreloadData));
			ShaderPreloadEntry.PreloadEvent = Task->GetCompletionEvent();
			Task->Unlock();
			OutCompletionEvents.Add(ShaderPreloadEntry.PreloadEvent);
		}
		else if (ShaderPreloadEntry.PreloadEvent)
		{
			OutCompletionEvents.Add(ShaderPreloadEntry.PreloadEvent);
		}
	}

	INC_DWORD_STAT_BY(STAT_Shaders_ShaderPreloadMemory, PreloadMemory);

#if (CSV_PROFILER_STATS && !UE_BUILD_SHIPPING) 
	TCsvPersistentCustomStat<float>* CsvStatPreloadedShaderMB = FCsvProfiler::Get()->GetOrCreatePersistentCustomStatFloat(TEXT("PreloadedShaderMB"), CSV_CATEGORY_INDEX(Shaders));
	CsvStatPreloadedShaderMB->Add((float)PreloadMemory / (1024.0f * 1024.0f));
#endif
	return true;
}

bool FShaderCodeArchive::WaitForPreload(FShaderPreloadEntry& ShaderPreloadEntry)
{
	FGraphEventRef Event;
	{
		FReadScopeLock Lock(ShaderPreloadLock);
		if(ShaderPreloadEntry.NumRefs > 0u)
		{
			Event = ShaderPreloadEntry.PreloadEvent;
		}
		else
		{
			check(!ShaderPreloadEntry.PreloadEvent);
		}
	}

	const bool bNeedToWait = Event && !Event->IsComplete();
	if (bNeedToWait)
	{
		FTaskGraphInterface::Get().WaitUntilTaskCompletes(Event);
	}
	return bNeedToWait;
}

void FShaderCodeArchive::ReleasePreloadedShader(int32 ShaderIndex)
{
	FShaderPreloadEntry& ShaderPreloadEntry = ShaderPreloads[ShaderIndex];
	if (!ShaderPreloadEntry.bNeverToBePreloaded)
	{
		WaitForPreload(ShaderPreloadEntry);

		FWriteScopeLock Lock(ShaderPreloadLock);

		ShaderPreloadEntry.PreloadEvent.SafeRelease();

		const uint32 ShaderNumRefs = ShaderPreloadEntry.NumRefs--;
		check(ShaderPreloadEntry.Code);
		check(ShaderNumRefs > 0u);
		if (ShaderNumRefs == 1u)
		{
			FMemory::Free(ShaderPreloadEntry.Code);
			ShaderPreloadEntry.Code = nullptr;
			const FShaderCodeEntry& ShaderEntry = SerializedShaders.GetShaderEntries()[ShaderIndex];
			DEC_DWORD_STAT_BY(STAT_Shaders_ShaderPreloadMemory, ShaderEntry.Size);

#if (CSV_PROFILER_STATS && !UE_BUILD_SHIPPING) 
			TCsvPersistentCustomStat<float>* CsvStatPreloadedShaderMB = FCsvProfiler::Get()->GetOrCreatePersistentCustomStatFloat(TEXT("PreloadedShaderMB"), CSV_CATEGORY_INDEX(Shaders));
			CsvStatPreloadedShaderMB->Sub((float)ShaderEntry.Size / (1024.0f * 1024.0f));
#endif
		}
	}
}

TRefCountPtr<FRHIShader> FShaderCodeArchive::CreateShader(int32 Index, bool bRequired)
{
	LLM_SCOPE(ELLMTag::Shaders);

	TRefCountPtr<FRHIShader> Shader;

	FMemStackBase& MemStack = FMemStack::Get();
	FMemMark Mark(MemStack);

	const FShaderCodeEntry& ShaderEntry = SerializedShaders.GetShaderEntries()[Index];
	FShaderPreloadEntry& ShaderPreloadEntry = ShaderPreloads[Index];
	checkf(!ShaderPreloadEntry.bNeverToBePreloaded, TEXT("We are creating a shader that shouldn't be preloaded in this run (e.g. raytracing shader on D3D11)."));

	{
		FGraphEventArray Dummy;
		PreloadShader(Index, Dummy);
	}

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(BlockingShaderLoad);
		double TimeStarted = FPlatformTime::Seconds();
		const bool bNeededToWait = WaitForPreload(ShaderPreloadEntry);
		if (bNeededToWait)
		{
			const double WaitDuration = FPlatformTime::Seconds() - TimeStarted;
			// only complain if we spent more than 1ms waiting
			if (WaitDuration > GShaderCodeLibraryMaxShaderPreloadWaitTime)
			{
				UE_LOGF(LogShaderLibrary, Warning, "Spent %.2f ms in a blocking wait for shader preload, NumRefs: %d, FramePreloadStarted: %d, CurrentFrame: %d", WaitDuration * 1000.0, ShaderPreloadEntry.NumRefs, ShaderPreloadEntry.FramePreloadStarted, GFrameNumber);
			}
		}
	}

	const uint8* ShaderCode = (uint8*)ShaderPreloadEntry.Code;
	if (ShaderEntry.IsCompressed())
	{
		uint32 UncompressedSize = ShaderEntry.GetUncompressedSize();
		uint8* UncompressedCode = reinterpret_cast<uint8*>(MemStack.Alloc(UncompressedSize, 16));
		ShaderCodeArchive::DecompressShaderWithOodle(UncompressedCode, UncompressedSize, ShaderCode, ShaderEntry.Size);
		ShaderCode = (uint8*)UncompressedCode;
	}

	// detect the breach of contract early
	ensureAlwaysMsgf(IsInRenderingThread() || GRHISupportsMultithreadedShaderCreation, TEXT("More than one thread is creating shaders, but GRHISupportsMultithreadedShaderCreation is false."));

	FRHICreateShaderDesc CreateShaderDesc = FRHICreateShaderDesc(MakeArrayView(ShaderCode, ShaderEntry.GetUncompressedSize()), this);

	switch (ShaderEntry.GetFrequency())
	{
	case SF_Vertex: Shader = RHICreateVertexShader(CreateShaderDesc); CheckShaderCreation(Shader, Index); break;
	case SF_Mesh: Shader = RHICreateMeshShader(CreateShaderDesc); CheckShaderCreation(Shader, Index); break;
	case SF_Amplification: Shader = RHICreateAmplificationShader(CreateShaderDesc); CheckShaderCreation(Shader, Index); break;
	case SF_Pixel: Shader = RHICreatePixelShader(CreateShaderDesc); CheckShaderCreation(Shader, Index); break;
	case SF_Geometry: Shader = RHICreateGeometryShader(CreateShaderDesc); CheckShaderCreation(Shader, Index); break;
	case SF_Compute: Shader = RHICreateComputeShader(CreateShaderDesc); CheckShaderCreation(Shader, Index); break;
	case SF_WorkGraphRoot: Shader = RHICreateWorkGraphShader(CreateShaderDesc, SF_WorkGraphRoot); CheckShaderCreation(Shader, Index); break;
	case SF_WorkGraphComputeNode: Shader = RHICreateWorkGraphShader(CreateShaderDesc, SF_WorkGraphComputeNode); CheckShaderCreation(Shader, Index); break;
	case SF_RayGen: case SF_RayMiss: case SF_RayHitGroup: case SF_RayCallable:
#if RHI_RAYTRACING
		if (GRHISupportsRayTracing && GRHISupportsRayTracingShaders)
		{
			Shader = RHICreateRayTracingShader(CreateShaderDesc, ShaderEntry.GetFrequency());
			CheckShaderCreation(Shader, Index);
		}
#endif // RHI_RAYTRACING
		break;
	default: checkNoEntry(); break;
	}
	DebugVisualizer.MarkCreatedForVisualization(Index);

	// Release the reference we were holding
	ReleasePreloadedShader(Index);

	if (Shader)
	{
		INC_DWORD_STAT(STAT_Shaders_NumShadersCreated);

#if (CSV_PROFILER_STATS && !UE_BUILD_SHIPPING) 
		TCsvPersistentCustomStat<int>* CsvStatNumShadersCreated = FCsvProfiler::Get()->GetOrCreatePersistentCustomStatInt(TEXT("NumShadersCreated"), CSV_CATEGORY_INDEX(Shaders));
		CsvStatNumShadersCreated->Add(1);
#endif
		Shader->SetHash(SerializedShaders.GetShaderHashes()[Index]);
	}

	return Shader;
}

bool FShaderLibraryNameInfo::ParseFromFilename(const TCHAR* Filename)
{
	// Parse 
	// ShaderArchive-<MyProject>_Chunk<N>-<ShaderFormat>-<ShaderPlatform>
	// E.g. "ShaderArchive-Lyra_Chunk0-PCD3D_SM6-PCD3D_SM6
	// This also need to take projects with dashes into account
	// E.g. "ShaderArchive-b03cf9b5-4234-1233-9fcc-a5413a00f58f-PCD3D_SM5-PCD3D_SM5"
	const FString BaseFilename = FPaths::GetBaseFilename(Filename);

	const FStringView ShaderArchivePrefix = TEXTVIEW("ShaderArchive-");
	if (!BaseFilename.StartsWith(ShaderArchivePrefix))
	{
		return false;
	}

	const int32 LastDash = BaseFilename.Find(TEXT("-"), ESearchCase::IgnoreCase, ESearchDir::FromEnd);
	if (LastDash == INDEX_NONE)
	{
		return false;
	}

	const int32 SecondLastDash = BaseFilename.Find(TEXT("-"), ESearchCase::IgnoreCase, ESearchDir::FromEnd, LastDash);
	if (SecondLastDash == INDEX_NONE)
	{
		return false;
	}

	// Extract chunk name from the left after the "ShaderArchive-" prefix and shader format and platform names from the right
	ChunkName = BaseFilename.Mid(ShaderArchivePrefix.Len(), SecondLastDash - ShaderArchivePrefix.Len());
	ShaderFormatName = FName(BaseFilename.Mid(SecondLastDash + 1, LastDash - SecondLastDash - 1));
	ShaderPlatformName = FName(BaseFilename.Mid(LastDash + 1));

	return true;
}

FString FShaderLibraryNameInfo::GetAssetInfoFilename() const
{
	//@todo-lh - Should be merged with GetShaderAssetInfoFilename() in ShaderCodeLibrary.cpp
	return FString::Printf(TEXT("ShaderAssetInfo-%s-%s.assetinfo.json"), *ChunkName, *GetFormatAndPlatformName());
}

FString FShaderLibraryNameInfo::GetTypeInfoFilename() const
{
	//@todo-lh - Should be merged with GetShaderTypeInfoFilename() in ShaderCodeLibrary.cpp
	return FString::Printf(TEXT("ShaderTypeInfo-%s-%s.stinfo"), *ChunkName, *GetFormatAndPlatformName());
}

// Parse ID number from pakchunk name, e.g. return 100 and "iad" for "pakchunk100iad-Windows".
bool FShaderLibraryNameInfo::ParsePakChunkId(const FStringView& PakChunkName, int32& OutChunkId, FString& OutChunkIdSuffix)
{
	const FStringView PakchunkPrefix = TEXTVIEW("pakchunk");
	if (!PakChunkName.StartsWith(PakchunkPrefix))
	{
		return false;
	}

	// Parse ID number
	const int32 ChunkIdStart = PakchunkPrefix.Len();
	int32 ChunkIdEnd = ChunkIdStart;
	while (ChunkIdEnd < PakChunkName.Len() && FChar::IsDigit(PakChunkName[ChunkIdEnd]))
	{
		++ChunkIdEnd;
	}
	if (ChunkIdEnd == ChunkIdStart)
	{
		return false;
	}
	OutChunkId = FCString::Atoi(*FString(PakChunkName.Mid(ChunkIdStart, ChunkIdEnd - ChunkIdStart)));

	// Parse optional suffix
	int32 ChunkIdSuffixEnd = ChunkIdEnd;
	while (ChunkIdSuffixEnd < PakChunkName.Len() && PakChunkName[ChunkIdSuffixEnd] != TEXT('-'))
	{
		++ChunkIdSuffixEnd;
	}

	if (ChunkIdSuffixEnd == ChunkIdEnd)
	{
		OutChunkIdSuffix.Empty();
	}
	else
	{
		OutChunkIdSuffix = PakChunkName.Mid(ChunkIdEnd, ChunkIdSuffixEnd - ChunkIdEnd);
	}

	return true;
}

// public access deprecated; only use of this function is within this file so internal free function will be used going forward
FIoChunkId FIoStoreShaderCodeArchive::GetShaderCodeArchiveChunkId(const FString& LibraryName, FName FormatName)
{
	return ShaderCodeArchive::GetShaderCodeArchiveChunkId(LibraryName, FormatName);
}

// public access deprecated; only use of this function is within this file so internal template function will be used going forward
FIoChunkId FIoStoreShaderCodeArchive::GetShaderCodeChunkId(const FSHAHash& ShaderHash)
{
	return ShaderCodeArchive::GetShaderCodeChunkId(ShaderHash.Hash);
}

struct FShaderIndexToShaderMapIndicesPair
{
	uint32 ShaderIndex;
	TArray<int32> ShaderMapIndices;

	static bool Sort(const FShaderIndexToShaderMapIndicesPair& EntryA, const FShaderIndexToShaderMapIndicesPair& EntryB)
	{
		const TArray<int32>& A = EntryA.ShaderMapIndices;
		const TArray<int32>& B = EntryB.ShaderMapIndices;
		// if the number of shadermaps is the same, we need to sort "alphabetically"
		if (A.Num() == B.Num())
		{
			for (int32 Idx = 0, Num = A.Num(); Idx < Num; ++Idx)
			{
				if (A[Idx] != B[Idx])
				{
					return A[Idx] < B[Idx];
				}
			}

			return EntryA.ShaderIndex < EntryB.ShaderIndex;
		}

		return A.Num() < B.Num();
	}
};

class FShaderCodeArchiveProcessor
{
	const bool bSeparateRaytracingShaders;
	const uint32 MaxUncompressedShaderGroupSize;
	const TStringConversion<TStringConvert<TCHAR, UTF8CHAR>> Utf8FormatName;

	FIoStoreShaderCodeArchiveHeader& OutHeader;
	const FSerializedShaderArchive* SerializedShaders;

	TArray<FShaderIndexToShaderMapIndicesPair> ShaderToShadermapsArray;

	// for statistics
	struct FProcessingStats
	{
		uint64 TotalUncompressedMemory = 0;
		uint32 MinGroupSize = MAX_uint32;
		uint32 MaxGroupSize = 0;
		int32 GroupsThatAppendedToShaderIndices = 0;
	};
	FProcessingStats Stats;

	// We want to avoid adding group indices to ShaderIndices, however looking them up sequentially is to slow. Store them here for a future ParallelFor lookup.
	TArray<TArray<uint32>> StoredGroupShaderIndices;

public:
	FShaderCodeArchiveProcessor(const FName& Format, FIoStoreShaderCodeArchiveHeader& OutHeader);

	void ProcessSerializedShaders(const FSerializedShaderArchive& SerializedShaders);

private:
	// Populates the ShaderToShadermapsArray member via information from the archive header.
	void BuildShaderToShadermapsArray();

	// First stage of processing a streak of shaders all referenced by the same set of shadermaps.
	// We begin with separating raytracing and non-raytracing shaders, so we can avoid preloading RTX in non-RT runs.
	void SplitShaderGroupForRaytracing(TArray<uint32>& CurrentShaderGroup);

	// Second stage of processing shader group. Here we potentially split the group into smaller ones (as equally as possible),
	// striving to meet limit imposed by r.ShaderCodeLibrary.MaxShaderGroupSize.
	void SplitShaderGroupBySize(TArray<uint32>& CurrentShaderGroup);

	// Third and last stage of processing the shader group. We actually add it here, and do the book-keeping.
	void AddNewShaderGroup(TArray<uint32>& ShaderIndicesInGroup);

};

FShaderCodeArchiveProcessor::FShaderCodeArchiveProcessor(const FName& Format, FIoStoreShaderCodeArchiveHeader& OutHeader)
	: bSeparateRaytracingShaders(Format == FName("PCD3D_SM5"))
	// get the effective maximum uncompressed group size (cannot be lower than 1)
	, MaxUncompressedShaderGroupSize(FMath::Max(static_cast<uint32>(GShaderCodeLibraryMaxShaderGroupSize), 1U))
	, Utf8FormatName(Format.ToString())
	, OutHeader(OutHeader)
{
}

/** Tries to find whether SequenceToFind exists as a subsequence in ExistingIndices. Returns INDEX_NONE if not found. */
static int32 FindStartOfSequenceInArray(const TArrayView<const uint32> SuperSet, const TArrayView<const uint32> SequenceToFind)
{
	check(SequenceToFind.Num() > 0);

	const uint32 FirstSequenceEntry = SequenceToFind[0];
	const int32 SequenceLength = SequenceToFind.Num();
	for (int32 SuperSetIndex = 0, SuperSetLength = SuperSet.Num(); SuperSetIndex < SuperSetLength - SequenceLength + 1; ++SuperSetIndex)
	{
		if (LIKELY(SuperSet[SuperSetIndex] != FirstSequenceEntry))
		{
			continue;
		}

		// check the rest
		bool bFoundSequence = true;
		for (int32 SequenceIndex = 1; SequenceIndex < SequenceLength; ++SequenceIndex)
		{
			checkSlow(SuperSetIndex + SequenceIndex < SuperSetLength);
			if (LIKELY(SuperSet[SuperSetIndex + SequenceIndex] != SequenceToFind[SequenceIndex]))
			{
				bFoundSequence = false;
				break;
			}
		}

		if (UNLIKELY(bFoundSequence))
		{
			return SuperSetIndex;
		}
	}

	return INDEX_NONE;
}

void FShaderCodeArchiveProcessor::ProcessSerializedShaders(const FSerializedShaderArchive& InSerializedShaders)
{
	// Load up serialized shaders and populate archive header
	SerializedShaders = &InSerializedShaders;

	OutHeader.ShaderMapHashes = SerializedShaders->GetShaderMapHashes();
	OutHeader.ShaderHashes = SerializedShaders->GetShaderHashes();
	// shader group hashes will be populated later

	OutHeader.ShaderMapEntries.Empty(SerializedShaders->GetShaderMapEntries().Num());
	for (const FShaderMapEntry& ShaderMapEntry : SerializedShaders->GetShaderMapEntries())
	{
		FIoStoreShaderMapEntry& IoStoreShaderMapEntry = OutHeader.ShaderMapEntries.AddDefaulted_GetRef();
		IoStoreShaderMapEntry.ShaderIndicesOffset = ShaderMapEntry.ShaderIndicesOffset;
		IoStoreShaderMapEntry.NumShaders = ShaderMapEntry.NumShaders;
	}

	// indices should be copied before grouping as the groups will append to the array
	OutHeader.ShaderIndices = SerializedShaders->GetShaderIndices();

	// shader entries are copied, the remainder of the field will be assigned when splitting into groups
	OutHeader.ShaderEntries.Empty(SerializedShaders->GetShaderEntries().Num());
	for (const FShaderCodeEntry& ShaderEntry : SerializedShaders->GetShaderEntries())
	{
		FIoStoreShaderCodeEntry& IoStoreShaderEntry = OutHeader.ShaderEntries.AddDefaulted_GetRef();
		IoStoreShaderEntry.Frequency = ShaderEntry.GetFrequency();
	}

	BuildShaderToShadermapsArray();

	// now split this into streaks of shaders that are referenced by the same set of shadermaps and compress
	TArray<uint32> CurrentShaderGroup;
	TArray<int32> LastShadermapSetSeen;
	for (const FShaderIndexToShaderMapIndicesPair& ShaderIndexInfo : ShaderToShadermapsArray)
	{
		// if we have have just started the group, we don't check the last seen
		if (UNLIKELY(CurrentShaderGroup.IsEmpty()))
		{
			CurrentShaderGroup.Add(ShaderIndexInfo.ShaderIndex);
			LastShadermapSetSeen = ShaderIndexInfo.ShaderMapIndices;
		}
		else if (UNLIKELY(LastShadermapSetSeen != ShaderIndexInfo.ShaderMapIndices))
		{
			SplitShaderGroupForRaytracing(CurrentShaderGroup);

			// reset the collection, but don't forget to add to it the current element
			CurrentShaderGroup.SetNum(1);
			CurrentShaderGroup[0] = ShaderIndexInfo.ShaderIndex;
			LastShadermapSetSeen = ShaderIndexInfo.ShaderMapIndices;
		}
		else
		{
			// keep adding to the same group
			CurrentShaderGroup.Add(ShaderIndexInfo.ShaderIndex);
		}
	}
	// add the last group
	if (!CurrentShaderGroup.IsEmpty())
	{
		SplitShaderGroupForRaytracing(CurrentShaderGroup);
	}

	// now, try to see if we can look up group's indices in the existing ShaderIndicesArray to avoid storing them there.
	checkf(StoredGroupShaderIndices.Num() == OutHeader.ShaderGroupEntries.Num(), TEXT("We should have stored shader indices for all groups."));
	ParallelFor(StoredGroupShaderIndices.Num(),
		[this](int32 ShaderGroupIndex)
		{
			const TArray<uint32>& ShaderIndicesInGroup = StoredGroupShaderIndices[ShaderGroupIndex];
			FIoStoreShaderGroupEntry& GroupEntry = OutHeader.ShaderGroupEntries[ShaderGroupIndex];
			// See if we can find indices in that order somewhere in the ShaderIndices array already, to avoid adding new indices.
			// We are looking in the read-only original array, because there's no sense to look in OutHeader.ShaderIndices - groups don't overlap,
			// so we know that newly added (by some previous group) indices aren't useful for us.
			const int32 ExistingOffset = FindStartOfSequenceInArray(SerializedShaders->GetShaderIndices(), ShaderIndicesInGroup);
			if (ExistingOffset != INDEX_NONE)
			{
				GroupEntry.ShaderIndicesOffset = static_cast<uint32>(ExistingOffset);
			}
			else
			{
				GroupEntry.ShaderIndicesOffset = MAX_uint32;
			}
		}
	);

	// Now append all the groups that weren't found to the end of ShaderIndices, slow (we could have done that in above ParallelFor, with a lock), but good for determinism
	for (int32 ShaderGroupIndex = 0, NumGroups = OutHeader.ShaderGroupEntries.Num(); ShaderGroupIndex < NumGroups; ++ShaderGroupIndex)
	{
		FIoStoreShaderGroupEntry& GroupEntry = OutHeader.ShaderGroupEntries[ShaderGroupIndex];

		if (GroupEntry.ShaderIndicesOffset == MAX_uint32)
		{
			const TArray<uint32>& ShaderIndicesInGroup = StoredGroupShaderIndices[ShaderGroupIndex];
			GroupEntry.ShaderIndicesOffset = OutHeader.ShaderIndices.Num();
			OutHeader.ShaderIndices.Append(ShaderIndicesInGroup);
			Stats.GroupsThatAppendedToShaderIndices++;
		}
	}

	// Validate and log stats
	checkf(OutHeader.ShaderEntries.Num() == SerializedShaders->GetShaderEntries().Num(),
		TEXT("Error creating IoStoreShaderArchive header - shader entries differ (%d in IoStore, %d original). Bug in grouping logic?"),
		OutHeader.ShaderEntries.Num(), SerializedShaders->GetShaderEntries().Num());
	checkf(OutHeader.ShaderGroupIoHashes.Num() == OutHeader.ShaderGroupEntries.Num(),
		TEXT("Error creating IoStoreShaderArchive header - mismatch between shader group hashes and descriptors (%d descriptors, %d hashes). Bug in grouping logic?"),
		OutHeader.ShaderGroupEntries.Num(), OutHeader.ShaderGroupIoHashes.Num());

	const int32 NumShaderGroupEntries = OutHeader.ShaderGroupEntries.Num();
	UE_LOGF(LogShaderLibrary, Display,
		"Created IoStoreShaderArchive header: shaders grouped in %d groups (%d of them didn't need new indices), average uncompressed size %llu bytes, min %u bytes, max %u bytes (r.ShaderCodeLibrary.MaxShaderGroupSize=%u)",
		NumShaderGroupEntries,
		NumShaderGroupEntries - Stats.GroupsThatAppendedToShaderIndices,
		NumShaderGroupEntries > 0 ? Stats.TotalUncompressedMemory / static_cast<uint64>(NumShaderGroupEntries) : 0llu,
		Stats.MinGroupSize,
		Stats.MaxGroupSize,
		MaxUncompressedShaderGroupSize);
}

void FShaderCodeArchiveProcessor::BuildShaderToShadermapsArray()
{
	ShaderToShadermapsArray.AddDefaulted(OutHeader.ShaderEntries.Num());

	{
		FCriticalSection ShaderLocks[1024];
		// for each shader, find all the shadermaps it belongs to
		ParallelFor(OutHeader.ShaderMapEntries.Num(),
			[this, &ShaderLocks](int32 ShaderMapIndex)
			{
				const FIoStoreShaderMapEntry& ShaderMapEntry = OutHeader.ShaderMapEntries[ShaderMapIndex];
				const int32 ShaderIndicesRange[2] =
				{
					ShaderMapEntry.ShaderIndicesOffset,
					ShaderMapEntry.ShaderIndicesOffset + ShaderMapEntry.NumShaders
				};
				for (int32 ShaderIndicesEntryIndex = ShaderIndicesRange[0]; ShaderIndicesEntryIndex < ShaderIndicesRange[1]; ++ShaderIndicesEntryIndex)
				{
					// add this shadermap as a dependency.
					const int32 ShaderIndex = OutHeader.ShaderIndices[ShaderIndicesEntryIndex];
					const int32 ShaderLockNumber = ShaderIndex % UE_ARRAY_COUNT(ShaderLocks);
					FScopeLock Locker(&ShaderLocks[ShaderLockNumber]);
					ShaderToShadermapsArray[ShaderIndex].ShaderMapIndices.Add(ShaderMapIndex);
				}
			}
		);
	}

	// sort shadermaps entries in shaders
	{
		constexpr int32 kShaderSortedPerThread = 1024;
		const int32 NumThreads = (ShaderToShadermapsArray.Num() / kShaderSortedPerThread) + 1;
		ParallelFor(NumThreads,
			[this, kShaderSortedPerThread](int32 ThreadIndex)
			{
				const int32 ShaderIndexStart = ThreadIndex * kShaderSortedPerThread;
				const int32 ShaderIndexEnd = FMath::Min(ShaderIndexStart + kShaderSortedPerThread, ShaderToShadermapsArray.Num());
				for (int32 ShaderIndex = ShaderIndexStart; ShaderIndex < ShaderIndexEnd; ++ShaderIndex)
				{
					ShaderToShadermapsArray[ShaderIndex].ShaderMapIndices.Sort();
				}
			},
			EParallelForFlags::Unbalanced
		);
	}

	// now assigning the indices in the array so we can sort it
	for (int32 ShaderIndex = 0, Num = ShaderToShadermapsArray.Num(); ShaderIndex < Num; ++ShaderIndex)
	{
		// check that no shader is unreferenced
		checkf(!ShaderToShadermapsArray[ShaderIndex].ShaderMapIndices.IsEmpty(),
			TEXT("Error converting to IoStore archive: shader (index=%d) is not referenced by any of the shadermaps!"), ShaderIndex);
		ShaderToShadermapsArray[ShaderIndex].ShaderIndex = ShaderIndex;
	}

	// sort the mapping so the first are shaders that are referenced by a smaller number of shadermaps, then by index for determinism
	Algo::Sort(ShaderToShadermapsArray, FShaderIndexToShaderMapIndicesPair::Sort);
}

void FShaderCodeArchiveProcessor::SplitShaderGroupForRaytracing(TArray<uint32>& CurrentShaderGroup)
{
	if (!bSeparateRaytracingShaders)
	{
		SplitShaderGroupBySize(CurrentShaderGroup);
	}
	else
	{
		// The streak changed. Create the group, but first, determine if the group needs to be split in two because of the raytracing shaders.
		// We want to isolate them into separate groups so their preload can be skipped if raytracing is off.
		TArray<uint32> RaytracingShaders;
		TArray<uint32> NonraytracingShaders;
		for (int32 ShaderIndex : CurrentShaderGroup)
		{
			if (LIKELY(!IsRayTracingShaderFrequency(static_cast<EShaderFrequency>(OutHeader.ShaderEntries[ShaderIndex].Frequency))))
			{
				NonraytracingShaders.Add(ShaderIndex);
			}
			else
			{
				RaytracingShaders.Add(ShaderIndex);
			}
		}
		check(CurrentShaderGroup.Num() == NonraytracingShaders.Num() + RaytracingShaders.Num());

		if (LIKELY(!NonraytracingShaders.IsEmpty()))
		{
			SplitShaderGroupBySize(NonraytracingShaders);
		}
		if (UNLIKELY(!RaytracingShaders.IsEmpty()))
		{
			SplitShaderGroupBySize(RaytracingShaders);
		}
	}
}

void FShaderCodeArchiveProcessor::SplitShaderGroupBySize(TArray<uint32>& CurrentShaderGroup)
{
	// calculate current group size
	uint32 GroupSize = 0;
	const TArrayView<const FShaderCodeEntry> ShaderEntries = SerializedShaders->GetShaderEntries();
	for (uint32 ShaderIdx : CurrentShaderGroup)
	{
		GroupSize += ShaderEntries[ShaderIdx].GetUncompressedSize();
	}

	if (LIKELY(GroupSize <= MaxUncompressedShaderGroupSize || CurrentShaderGroup.Num() == 1))
	{
		AddNewShaderGroup(CurrentShaderGroup);
	}
	else
	{
		// split the shaders evenly into N new groups (don't allow more new groups than there are shaders)
		int32 NumNewGroups = FMath::Min(static_cast<int32>(GroupSize / MaxUncompressedShaderGroupSize + 1), CurrentShaderGroup.Num());
		checkf(NumNewGroups > 1, TEXT("Off by one error in group count calculation? NumNewGroups=%d, GroupSize=%u, MaxUncompressedShaderGroupSize=%u, CurrentShaderGroup.Num()=%d"), NumNewGroups, GroupSize, MaxUncompressedShaderGroupSize, CurrentShaderGroup.Num());

		TArray<TArray<uint32>> NewGroups;
		TArray<uint32> NewGroupSizes;
		NewGroups.AddDefaulted(NumNewGroups);
		NewGroupSizes.AddZeroed(NumNewGroups);

		// sort the shaders descending, as this is easier to split (greedy algorithm)
		CurrentShaderGroup.Sort(
			[this](const int32 ShaderIndexA, const int32 ShaderIndexB)
			{
				const TArrayView<const FShaderCodeEntry> ShaderEntries = SerializedShaders->GetShaderEntries();
				const FShaderCodeEntry& ShaderEntryA = ShaderEntries[ShaderIndexA];
				const FShaderCodeEntry& ShaderEntryB = ShaderEntries[ShaderIndexB];
				uint32 UncompressedSizeA = ShaderEntryA.GetUncompressedSize();
				uint32 UncompressedSizeB = ShaderEntryB.GetUncompressedSize();
				if (UncompressedSizeA != UncompressedSizeB)
				{
					return UncompressedSizeA > UncompressedSizeB;
				}
				if (ShaderEntryA.Size != ShaderEntryB.Size)
				{
					return ShaderEntryA.Size > ShaderEntryB.Size;
				}
				EShaderFrequency FrequencyA = ShaderEntryA.GetFrequency();
				EShaderFrequency FrequencyB = ShaderEntryB.GetFrequency();
				if (FrequencyA != FrequencyB)
				{
					return FrequencyA > FrequencyB;
				}
				return ShaderEntryA.Offset > ShaderEntryB.Offset;
			}
		);

		for (int32 ShaderIdx : CurrentShaderGroup)
		{
			// add the shader to the group of smallest size
			int32 SmallestNewGroupIdx = 0;
			for (int32 IdxNewGroup = 1; IdxNewGroup < NumNewGroups; ++IdxNewGroup)
			{
				if (NewGroupSizes[IdxNewGroup] < NewGroupSizes[SmallestNewGroupIdx])
				{
					SmallestNewGroupIdx = IdxNewGroup;
				}
			}

			NewGroups[SmallestNewGroupIdx].Add(ShaderIdx);
			NewGroupSizes[SmallestNewGroupIdx] += ShaderEntries[ShaderIdx].GetUncompressedSize();
		}

#if DO_CHECK // sanity checks
		uint32 NewGroupTotalSize = 0;
		for (uint32 NewGroupSize : NewGroupSizes)
		{
			NewGroupTotalSize += NewGroupSize;
		}
		checkf(NewGroupTotalSize == GroupSize, TEXT("Original shader group was of size %u bytes, which was larger than limit %u, but it was split into %d group of total size %u, which is not %u - sizes must agree"),
			GroupSize,
			MaxUncompressedShaderGroupSize,
			NumNewGroups,
			NewGroupTotalSize, GroupSize
		);
#endif

		for (TArray<uint32>& NewGroup : NewGroups)
		{
			// note there can be empty groups (take a very edge case of MaxUncompressedShaderGroupSize = 2 bytes and a shader group of 1 shader)
			if (!NewGroup.IsEmpty())
			{
				AddNewShaderGroup(NewGroup);
			}
		}
	}
}

void FShaderCodeArchiveProcessor::AddNewShaderGroup(TArray<uint32>& ShaderIndicesInGroup)
{
	// first, sort the shaders by uncompressed size, as this was found to compress better
	ShaderIndicesInGroup.Sort(
		[this](const int32 ShaderIndexA, const int32 ShaderIndexB)
		{
			const FShaderCodeEntry& ShaderEntryA = SerializedShaders->GetShaderEntries()[ShaderIndexA];
			const FShaderCodeEntry& ShaderEntryB = SerializedShaders->GetShaderEntries()[ShaderIndexB];
			uint32 UncompressedSizeA = ShaderEntryA.GetUncompressedSize();
			uint32 UncompressedSizeB = ShaderEntryB.GetUncompressedSize();
			if (UncompressedSizeA != UncompressedSizeB)
			{
				return UncompressedSizeA < UncompressedSizeB;
			}
			if (ShaderEntryA.Size != ShaderEntryB.Size)
			{
				return ShaderEntryA.Size < ShaderEntryB.Size;
			}
			EShaderFrequency FrequencyA = ShaderEntryA.GetFrequency();
			EShaderFrequency FrequencyB = ShaderEntryB.GetFrequency();
			if (FrequencyA != FrequencyB)
			{
				return FrequencyA < FrequencyB;
			}
			return ShaderEntryA.Offset < ShaderEntryB.Offset;
		}
	);

	// add a new group entry
	const int32 CurrentGroupIdx = OutHeader.ShaderGroupEntries.Num();
	FIoStoreShaderGroupEntry& GroupEntry = OutHeader.ShaderGroupEntries.AddDefaulted_GetRef();
	StoredGroupShaderIndices.Add(ShaderIndicesInGroup);
	GroupEntry.NumShaders = ShaderIndicesInGroup.Num();
	// ShaderIndicesOffset will be filled later, once we know all the groups (see comment about StoredGroupShaderIndices above).

	const TArrayView<const FShaderHash> ShaderHashes = SerializedShaders->GetShaderHashes();
	const TArrayView<const FShaderCodeEntry> ShaderEntries = SerializedShaders->GetShaderEntries();

	// update shader entries both with the group number and their uncompressed offset in the group
	// we use 128-bit xxhash explicitly here rather than FShaderHash as the IO chunk IDs are 11 bytes + 1 byte type
	FXxHash128Builder GroupHasher;
	uint32 CurrentGroupSize = 0;
	for (int32 ShaderIdxIdx = 0, NumIdxIdx = ShaderIndicesInGroup.Num(); ShaderIdxIdx < NumIdxIdx; ++ShaderIdxIdx)
	{
		int32 ShaderIndex = ShaderIndicesInGroup[ShaderIdxIdx];
		FIoStoreShaderCodeEntry& IoStoreShaderEntry = OutHeader.ShaderEntries[ShaderIndex];
		IoStoreShaderEntry.ShaderGroupIndex = CurrentGroupIdx;
		IoStoreShaderEntry.UncompressedOffsetInGroup = CurrentGroupSize;

		// group hash is constructed from hashing the shaders in the group.
		GroupHasher.Update(reinterpret_cast<const uint8*>(&ShaderHashes[ShaderIndex].Hash), sizeof(FShaderHash));
		// shader hash as of now excludes optional data, so we cannot rely on it, especially across the shader formats. Make the group hash a bit more robust by including the shader size in it.
		uint32 UncompressedSize = ShaderEntries[ShaderIndex].GetUncompressedSize();
		GroupHasher.Update(reinterpret_cast<const uint8*>(&UncompressedSize), sizeof(UncompressedSize));

		CurrentGroupSize += UncompressedSize;
	}

	// Shader hashes cannot be used to uniquely identify across shader formats due to aforementioned exclusion of optional data from it.
	// Include the shader format (in a platform-agnostic way) into the group hash to lower the risk of collision of shaders of different formats.
	GroupHasher.Update(reinterpret_cast<const uint8*>(Utf8FormatName.Get()), Utf8FormatName.Length());
	static_assert(sizeof(uint8) == sizeof(UTF8CHAR), "Unexpected UTF-8 char size.");

	GroupEntry.UncompressedSize = CurrentGroupSize;
	FXxHash128 Hash = GroupHasher.Finalize();
	uint8 HashBytes[16];
	Hash.ToByteArray(HashBytes);
	OutHeader.ShaderGroupIoHashes.Add(ShaderCodeArchive::GetShaderCodeChunkId(HashBytes));

	Stats.TotalUncompressedMemory += CurrentGroupSize;
	Stats.MinGroupSize = FMath::Min(Stats.MinGroupSize, CurrentGroupSize);
	Stats.MaxGroupSize = FMath::Max(Stats.MaxGroupSize, CurrentGroupSize);
}

void FIoStoreShaderCodeArchive::CreateIoStoreShaderCodeArchiveHeader(const FName& Format, const FSerializedShaderArchive& SerializedShaders, FIoStoreShaderCodeArchiveHeader& OutHeader)
{
	// Higher level description of the group splitting algo that follows:
	// We compress together shaders that are loaded together (all other strategies, like grouping similar shaders, were found to compress better but not reduce RAM usage).
	// For that, we find for each shader which shadermaps are referencing it (often times it will be just one, but for some simple and shared shaders it can be thousands).
	// We group the shaders by those sets of shadermaps - all shaders referenced by the same shadermap(s) are a candidate for being a single group. Then we potentially split this candidate group
	// into raytracing and non-raytracing groups (so we can avoid preloading RTX shaders run-time if RTX is disabled), and then each of those is potentially split further by size
	// (to avoid too large groups that will take too much time to decompress - this is regulated by r.ShaderCodeLibrary.MaxShaderGroupSize). The results of that process (note, it can still be
	// a single group) is added to the header.
	// Each group's indices, like in case of shadermaps, are stored in ShaderIndices array. Before we append a new group's indices however, we look if we can find an existing range that we can reuse.
	FShaderCodeArchiveProcessor ArchiveProcessor(Format, OutHeader);
	ArchiveProcessor.ProcessSerializedShaders(SerializedShaders);
}

FArchive& operator <<(FArchive& Ar, FIoStoreShaderCodeArchiveHeader& Ref)
{
	Ar << Ref.ShaderMapHashes;
	Ar << Ref.ShaderHashes;
	Ar << Ref.ShaderGroupIoHashes;
	Ar << Ref.ShaderMapEntries;
	Ar << Ref.ShaderEntries;
	Ar << Ref.ShaderGroupEntries;
	Ar << Ref.ShaderIndices;
	return Ar;
}

void FIoStoreShaderCodeArchive::SaveIoStoreShaderCodeArchive(const FIoStoreShaderCodeArchiveHeader& Header, FArchive& OutLibraryAr)
{
	uint32 Version = CurrentVersion;
	OutLibraryAr << Version;
	OutLibraryAr << const_cast<FIoStoreShaderCodeArchiveHeader &>(Header);
}

// Dumps all shader, shader map, and shader group hashes and how they are connected to a text file for debugging purposes.
static void DumpShaderCodeArchiveHashes(const FIoStoreShaderCodeArchiveHeader& InLibraryHeader, FArchive& Ar)
{
	FString DebugOutput;

	// Build shader-to-shadermap links for faster access
	TMap<FShaderHash, TSet<FShaderHash>> ShaderToShaderMaps;
	for (int32 ShaderMapIndex = 0; ShaderMapIndex < InLibraryHeader.ShaderMapEntries.Num(); ++ShaderMapIndex)
	{
		const FShaderHash& ShaderMapHash = InLibraryHeader.ShaderMapHashes[ShaderMapIndex];
		InLibraryHeader.ForEachShaderInShaderMap(ShaderMapIndex, [&](int32 ShaderIndex)
		{
			TSet<FShaderHash>& ShaderMapHahses = ShaderToShaderMaps.FindOrAdd(InLibraryHeader.ShaderHashes[ShaderIndex]);
			ShaderMapHahses.Add(ShaderMapHash);
		});
	}

	// Dump hashes of all IoChunks with their shaders and associated shadermaps
	DebugOutput.Appendf(TEXT("--------------- ShaderGroupIoHashes: ---------------\n"));
	for (int32 GroupIndex = 0; GroupIndex < InLibraryHeader.ShaderGroupIoHashes.Num(); ++GroupIndex)
	{
		const FIoChunkId& ShaderGroupChunkId = InLibraryHeader.ShaderGroupIoHashes[GroupIndex];
		const FIoStoreShaderGroupEntry& GroupEntry = InLibraryHeader.ShaderGroupEntries[GroupIndex];
		DebugOutput.Appendf(TEXT("IoChunk %s (%d shaders)\n"), *LexToString(ShaderGroupChunkId), GroupEntry.NumShaders);

		InLibraryHeader.ForEachShaderInGroup(GroupIndex, [&](int32 ShaderIndex)
		{
			const FShaderHash& ShaderHash = InLibraryHeader.ShaderHashes[ShaderIndex];
			DebugOutput.Appendf(TEXT("  %s"), *LexToString(ShaderHash));

			// Append all shadermaps that reference this shader
			if (const TSet<FShaderHash>* FoundShaderMaps = ShaderToShaderMaps.Find(ShaderHash))
			{
				DebugOutput.Append(TEXT(" (Referenced By Shadermaps: "));
				bool bIsFirstEntry = true;
				for (const FShaderHash& ShaderMapHash : *FoundShaderMaps)
				{
					if (!bIsFirstEntry)
					{
						DebugOutput.Append(TEXT(", "));
					}
					DebugOutput.Append(LexToString(ShaderMapHash));
					bIsFirstEntry = false;
				}
				DebugOutput.Append(TEXT(")"));
			}
			else
			{
				DebugOutput.Append(TEXT(" (No Shadermap References)"));
			}

			DebugOutput.Append(TEXT("\n"));
		});
		DebugOutput.Append(TEXT("\n"));
	}
	DebugOutput.Append(TEXT("\n\n\n"));

	// Dump hashes of all shadermaps with their shaders
	DebugOutput.Appendf(TEXT("--------------- ShaderMapHashes: ---------------\n"));
	for (int32 ShaderMapIndex = 0; ShaderMapIndex < InLibraryHeader.ShaderMapEntries.Num(); ++ShaderMapIndex)
	{
		const FIoStoreShaderMapEntry& ShaderMapEntry = InLibraryHeader.ShaderMapEntries[ShaderMapIndex];
		DebugOutput.Appendf(TEXT("ShaderMap %s (%d shaders)\n"), *LexToString(InLibraryHeader.ShaderMapHashes[ShaderMapIndex]), ShaderMapEntry.NumShaders);
		InLibraryHeader.ForEachShaderInShaderMap(ShaderMapIndex, [&](int32 ShaderIndex)
		{
			const FShaderHash& ShaderHash = InLibraryHeader.ShaderHashes[ShaderIndex];
			DebugOutput.Appendf(TEXT("  %s\n"), *LexToString(ShaderHash));
		});
		DebugOutput.Append(TEXT("\n"));
	}

	FTCHARToUTF8 DebugOutputUtf8Converter(*DebugOutput);
	Ar.Serialize(const_cast<UTF8CHAR*>(reinterpret_cast<const UTF8CHAR*>(DebugOutputUtf8Converter.Get())), DebugOutputUtf8Converter.Length());
}

FIoStoreShaderCodeArchive* FIoStoreShaderCodeArchive::Create(EShaderPlatform InPlatform, const FString& InLibraryName, FIoDispatcher& InIoDispatcher)
{
	const FName PlatformName = FName(FDataDrivenShaderPlatformInfo::GetShaderFormat(InPlatform).ToString() + TEXT("-") + FDataDrivenShaderPlatformInfo::GetName(InPlatform).ToString());
	FIoChunkId ChunkId = ShaderCodeArchive::GetShaderCodeArchiveChunkId(InLibraryName, PlatformName);
	if (InIoDispatcher.DoesChunkExist(ChunkId))
	{
		FIoBatch IoBatch = InIoDispatcher.NewBatch();
		FIoRequest IoRequest = IoBatch.Read(ChunkId, FIoReadOptions(), IoDispatcherPriority_Max);
		FEvent* Event = FPlatformProcess::GetSynchEventFromPool();
		IoBatch.IssueAndTriggerEvent(Event);
		Event->Wait();
		FPlatformProcess::ReturnSynchEventToPool(Event);
		const FIoBuffer& IoBuffer = IoRequest.GetResultOrDie();
		FMemoryReaderView Ar(MakeArrayView(IoBuffer.Data(), IoBuffer.DataSize()));
		uint32 Version = 0;
		Ar << Version;
		if (Version == CurrentVersion)
		{
			FIoStoreShaderCodeArchive* Library;
			if (IoStoreShaderCodeArchiveFactory)
			{
				Library = IoStoreShaderCodeArchiveFactory(InPlatform, InLibraryName, InIoDispatcher);
			}
			else
			{
				Library = new FIoStoreShaderCodeArchive(InPlatform, InLibraryName, InIoDispatcher);
			}
			Ar << Library->Header;
			{
				const uint32 HashSize = FMath::Min<uint32>(0x10000, 1u << FMath::CeilLogTwo(Library->Header.ShaderMapHashes.Num()));
				Library->ShaderMapHashTable.Clear(HashSize, Library->Header.ShaderMapHashes.Num());
				for (int32 Index = 0, Num = Library->Header.ShaderMapHashes.Num(); Index < Num; ++Index)
				{
					const uint32 Key = GetTypeHash(Library->Header.ShaderMapHashes[Index]);
					Library->ShaderMapHashTable.Add(Key, Index);
				}
			}
			{
				const uint32 HashSize = FMath::Min<uint32>(0x10000, 1u << FMath::CeilLogTwo(Library->Header.ShaderHashes.Num()));
				Library->ShaderHashTable.Clear(HashSize, Library->Header.ShaderHashes.Num());
				for (int32 Index = 0, Num = Library->Header.ShaderHashes.Num(); Index < Num; ++Index)
				{
					const uint32 Key = GetTypeHash(Library->Header.ShaderHashes[Index]);
					Library->ShaderHashTable.Add(Key, Index);
				}
			}

			Library->DebugVisualizer.Initialize(Library->Header.ShaderEntries.Num());

			// Dump debug info if enabled
			if (FParse::Param(FCommandLine::Get(), TEXT("DumpShaderLibraryHashes")))
			{
				if (!Library->Header.ShaderGroupIoHashes.IsEmpty() ||
					!Library->Header.ShaderHashes.IsEmpty() ||
					!Library->Header.ShaderMapHashes.IsEmpty())
				{
					const FString DumpPath = GetShaderDebugInfoPath();
					if (!FPaths::DirectoryExists(DumpPath))
					{
						MakeDebugDirectoryTreeGuarded(*DumpPath);
					}
					const FString DumpFilename = FPaths::Combine(DumpPath, FString::Printf(TEXT("ShaderHashes-%s-%s.txt"), *InLibraryName, *PlatformName.ToString()));
					UE_LOGF(LogShaderLibrary, Display, "Dump IoStoreShaderCodeArchive '%ls-%ls' header to file '%ls'",
						*InLibraryName, *PlatformName.ToString(), *DumpFilename);

					if (TUniquePtr<FArchive> DumpFile = TUniquePtr<FArchive>(IFileManager::Get().CreateFileWriter(*DumpFilename)))
					{
						DumpShaderCodeArchiveHashes(Library->Header, *DumpFile);
					}
				}
			}

			UE_LOGF(LogShaderLibrary, Display, "Using IoDispatcher for shader code library %ls. Total %d unique shaders.", *InLibraryName, Library->Header.ShaderEntries.Num());
			INC_DWORD_STAT_BY(STAT_Shaders_ShaderResourceMemory, Library->GetSizeBytes());
			return Library;
		}
	}
	return nullptr;
}

FIoStoreShaderCodeArchive::FCreateIoStoreShaderCodeArchiveDelegate FIoStoreShaderCodeArchive::IoStoreShaderCodeArchiveFactory;

void FIoStoreShaderCodeArchive::RegisterIoStoreShaderCodeArchiveFactory(FCreateIoStoreShaderCodeArchiveDelegate InFactory)
{
	IoStoreShaderCodeArchiveFactory = MoveTemp(InFactory);
}

FIoStoreShaderCodeArchive::FIoStoreShaderCodeArchive(EShaderPlatform InPlatform, const FString& InLibraryName, FIoDispatcher& InIoDispatcher)
	: FRHIShaderLibrary(InPlatform, InLibraryName)
	, IoDispatcher(InIoDispatcher)
{
}

FIoStoreShaderCodeArchive::~FIoStoreShaderCodeArchive()
{
	DEC_DWORD_STAT_BY(STAT_Shaders_ShaderResourceMemory, GetSizeBytes());
	Teardown();
}

void FIoStoreShaderCodeArchive::OnCloseShaderCode()
{
	// Release shader group data now since the backing IoStore memory is about to be unmounted.
	// The preload entries themselves are kept alive and will be cleaned up in Teardown().
	FWriteScopeLock Lock(PreloadedShaderGroupsLock);
	for (TMap<int32, FShaderGroupPreloadEntry*>::TIterator Iter(PreloadedShaderGroups); Iter; ++Iter)
	{
		Iter.Value()->ShaderGroupData.Reset();
	}
}

void FIoStoreShaderCodeArchive::Teardown()
{
	DebugVisualizer.SaveShaderUsageBitmap(GetName(), GetPlatform());
	uint32 DeletedPreloadEntryBytes = 0;
	FWriteScopeLock Lock(PreloadedShaderGroupsLock);
	for (TMap<int32, FShaderGroupPreloadEntry*>::TIterator Iter(PreloadedShaderGroups); Iter; ++Iter)
	{
		FShaderGroupPreloadEntry* PreloadEntry = Iter.Value();

#if UE_SCA_DEBUG_PRELOADING
		checkf(PreloadEntry->NumRefs == 0, TEXT("Group %d has still %d references on deletion. Group extended debug info: \n%s"), Iter.Key(), PreloadEntry->NumRefs,
			*PreloadEntry->DebugInfo);
#else
		checkf(PreloadEntry->NumRefs == 0, TEXT("Group %d has still %d references on deletion. Group extended debug info: \n%s"), Iter.Key(), PreloadEntry->NumRefs,
			TEXT("Not compiled in (set UE_SCA_DEBUG_PRELOADING to 1 in ShaderCodeArchive.h and recompile the game binary)"));
#endif

		const FIoStoreShaderGroupEntry& GroupEntry = Header.ShaderGroupEntries[Iter.Key()];
		DeletedPreloadEntryBytes += (GroupEntry.CompressedSize + sizeof(FShaderGroupPreloadEntry));

		delete PreloadEntry;
	}
	PreloadedShaderGroups.Empty();

	DEC_DWORD_STAT_BY(STAT_Shaders_ShaderPreloadMemory, DeletedPreloadEntryBytes);
#if (CSV_PROFILER_STATS && !UE_BUILD_SHIPPING) 
	TCsvPersistentCustomStat<float>* CsvStatPreloadedShaderMB = FCsvProfiler::Get()->GetOrCreatePersistentCustomStatFloat(TEXT("PreloadedShaderMB"), CSV_CATEGORY_INDEX(Shaders));
	CsvStatPreloadedShaderMB->Sub((float)DeletedPreloadEntryBytes / (1024.0f * 1024.0f));
#endif
}

void FIoStoreShaderCodeArchive::SetupPreloadEntryForLoading(int32 ShaderGroupIndex, FShaderGroupPreloadEntry& PreloadEntry)
{
	PreloadEntry.FramePreloadStarted = GFrameNumber;
	check(!PreloadEntry.PreloadEvent);
	PreloadEntry.PreloadEvent = FGraphEvent::CreateGraphEvent();

#if UE_SCA_VISUALIZE_SHADER_USAGE
	const FIoStoreShaderGroupEntry& GroupEntry = Header.ShaderGroupEntries[ShaderGroupIndex];
	for (uint32 ShaderIdxIdx = GroupEntry.ShaderIndicesOffset, StopBeforeIdxIdx = GroupEntry.ShaderIndicesOffset + GroupEntry.NumShaders; ShaderIdxIdx < StopBeforeIdxIdx; ++ShaderIdxIdx)
	{
		DebugVisualizer.MarkPreloadedForVisualization(Header.ShaderIndices[ShaderIdxIdx]);
	}
#endif // UE_SCA_VISUALIZE_SHADER_USAGE
}

bool FIoStoreShaderCodeArchive::PreloadShaderGroup(int32 ShaderGroupIndex, FGraphEventArray& OutCompletionEvents, 
#if UE_SCA_DEBUG_PRELOADING
	const FString& CallsiteInfo,
#endif
	FCoreDelegates::FAttachShaderReadRequestFunc* AttachShaderReadRequestFuncPtr)
{
	// should be called within LLMTag::Shaders scope
	FWriteScopeLock Lock(PreloadedShaderGroupsLock);
	FShaderGroupPreloadEntry& PreloadEntry = *FindOrAddPreloadEntry(ShaderGroupIndex);
	checkf(!PreloadEntry.bNeverToBePreloaded, TEXT("We are preloading a shader group (index=%d) that shouldn't be preloaded in this run (e.g. raytracing shaders on D3D11)."), ShaderGroupIndex);

	const uint32 NumRefs = PreloadEntry.NumRefs++;
#if UE_SCA_DEBUG_PRELOADING
	FString AppendInfo = FString::Printf(TEXT("PreloadShaderGroup: NumRefs %d -> %d    CallsiteInfo: %s\n"),
		NumRefs, PreloadEntry.NumRefs, *CallsiteInfo
	);
	PreloadEntry.DebugInfo.Append(AppendInfo);
#endif

	auto KickOffIoRequest = [this, AttachShaderReadRequestFuncPtr, &PreloadEntry, ShaderGroupIndex]()
		{
			SetupPreloadEntryForLoading(ShaderGroupIndex, PreloadEntry);

			// only global shaders and retries of asset shaders are going to hit this path, regular asset shaders will be preloaded with the package
			if (UNLIKELY(AttachShaderReadRequestFuncPtr == nullptr))
			{
#if USE_MMAPPED_SHADERARCHIVE
				PreloadEntry.ShaderGroupData->MappedRegionStatus = IoDispatcher.OpenMapped(Header.ShaderGroupIoHashes[ShaderGroupIndex], FIoReadOptions());
				if (PreloadEntry.ShaderGroupData->MappedRegionStatus.IsOk())
				{
					PreloadEntry.PreloadEvent->DispatchSubsequents();
					PreloadEntry.PreloadEvent.SafeRelease();
				}
				else
#endif
				{
					FIoBatch IoBatch = IoDispatcher.NewBatch();
					PreloadEntry.ShaderGroupData->IoRequest = IoBatch.Read(Header.ShaderGroupIoHashes[ShaderGroupIndex], FIoReadOptions(), GetShaderCodeArchivePriority());
					IoBatch.IssueAndDispatchSubsequents(PreloadEntry.PreloadEvent);
				}
			}
			else
			{
				FCoreDelegates::FShaderReadRequestResult Result = (*AttachShaderReadRequestFuncPtr)(Header.ShaderGroupIoHashes[ShaderGroupIndex], PreloadEntry.PreloadEvent);
#if USE_MMAPPED_SHADERARCHIVE
				PreloadEntry.ShaderGroupData->IoRequest = Result.Key;
				PreloadEntry.ShaderGroupData->MappedRegionStatus = Result.Value;
#else
				PreloadEntry.ShaderGroupData->IoRequest = Result;
#endif
			}
		};

	if (PreloadEntry.ShaderGroupData->IoRequest.Status() == FIoStatus::Invalid
#if USE_MMAPPED_SHADERARCHIVE
		&& !PreloadEntry.ShaderGroupData->MappedRegionStatus.IsOk()
#endif
		)
	{
		KickOffIoRequest();

		uint32 ShaderGroupSize = Header.ShaderGroupEntries[ShaderGroupIndex].CompressedSize + sizeof(FShaderGroupPreloadEntry);

		INC_DWORD_STAT_BY(STAT_Shaders_ShaderPreloadMemory, ShaderGroupSize);
#if (CSV_PROFILER_STATS && !UE_BUILD_SHIPPING) 
		TCsvPersistentCustomStat<float>* CsvStatPreloadedShaderMB = FCsvProfiler::Get()->GetOrCreatePersistentCustomStatFloat(TEXT("PreloadedShaderMB"), CSV_CATEGORY_INDEX(Shaders));
		CsvStatPreloadedShaderMB->Add((float)ShaderGroupSize / (1024.0f * 1024.0f));
#endif
	}
	else if (PreloadEntry.PreloadEvent.IsValid() && PreloadEntry.PreloadEvent->IsCompleted() && PreloadEntry.ShaderGroupData->IoRequest.GetResult() == nullptr
#if USE_MMAPPED_SHADERARCHIVE
		&& !PreloadEntry.ShaderGroupData->MappedRegionStatus.IsOk()
#endif
			 )
	{
		// failed request, re-kick in-place, without changing stats
		PreloadEntry.ShaderGroupData->IoRequest.Cancel();
		PreloadEntry.ShaderGroupData->IoRequest = FIoRequest();
		PreloadEntry.PreloadEvent.SafeRelease();

		KickOffIoRequest();
	}

	if (AttachShaderReadRequestFuncPtr == nullptr && PreloadEntry.PreloadEvent && !PreloadEntry.PreloadEvent->IsComplete())
	{
		OutCompletionEvents.Add(PreloadEntry.PreloadEvent);
	}
	return true;
}

void FIoStoreShaderCodeArchive::MarkPreloadEntrySkipped(int32 ShaderGroupIndex
#if UE_SCA_DEBUG_PRELOADING
	, const FString& CallsiteInfo
#endif
)
{
	// should be called within LLMTag::Shaders scope
	FWriteScopeLock Lock(PreloadedShaderGroupsLock);
	FShaderGroupPreloadEntry& PreloadEntry = *FindOrAddPreloadEntry(ShaderGroupIndex);
	const uint32 NumRefs = PreloadEntry.NumRefs++;
#if UE_SCA_DEBUG_PRELOADING
	FString AppendInfo = FString::Printf(TEXT("MarkPreloadEntrySkipped: NumRefs %d -> %d    CallsiteInfo: %s\n"),
		NumRefs, PreloadEntry.NumRefs, *CallsiteInfo
	);
	PreloadEntry.DebugInfo.Append(AppendInfo);
#endif
	if (NumRefs == 0u)
	{
		PreloadEntry.bNeverToBePreloaded = 1;
		uint32 ShaderGroupSize = sizeof(FShaderGroupPreloadEntry);
		
		INC_DWORD_STAT_BY(STAT_Shaders_ShaderPreloadMemory, ShaderGroupSize);
#if (CSV_PROFILER_STATS && !UE_BUILD_SHIPPING) 
		TCsvPersistentCustomStat<float>* CsvStatPreloadedShaderMB = FCsvProfiler::Get()->GetOrCreatePersistentCustomStatFloat(TEXT("PreloadedShaderMB"), CSV_CATEGORY_INDEX(Shaders));
		CsvStatPreloadedShaderMB->Add((float)ShaderGroupSize / (1024.0f * 1024.0f));
#endif
	}
}

uint32 FIoStoreShaderCodeArchive::GetShaderSizeBytes(int32 ShaderIndex) const
{
	return Header.GetShaderUncompressedSize(ShaderIndex);
}

void FIoStoreShaderCodeArchive::AddRefPreloadedShaderGroup(int32 ShaderGroupIndex)
{
	FWriteScopeLock Lock(PreloadedShaderGroupsLock);
	FShaderGroupPreloadEntry& PreloadEntry = *FindOrAddPreloadEntry(ShaderGroupIndex);
	PreloadEntry.NumRefs++;
}

void FIoStoreShaderCodeArchive::ReleasePreloadedShaderGroup(int32 ShaderGroupIndex)
{
	ReleasePreloadEntry(ShaderGroupIndex
#if UE_SCA_DEBUG_PRELOADING
		, FString::Printf(TEXT("ReleasePreloadedShaderGroup %d"), ShaderGroupIndex)
#endif
						);
}

bool FIoStoreShaderCodeArchive::IsPreloading(int32 ShaderIndex, FGraphEventArray& OutCompletionEvents)
{
	LLM_SCOPE(ELLMTag::Shaders);
	int32 ShaderGroupIndex = GetGroupIndexForShader(ShaderIndex);

	FReadScopeLock Lock(PreloadedShaderGroupsLock);
	FShaderGroupPreloadEntry** EntryPtrPtr = PreloadedShaderGroups.Find(ShaderGroupIndex);
	if (EntryPtrPtr)
	{
		FShaderGroupPreloadEntry& Entry = **EntryPtrPtr;
		if (Entry.PreloadEvent && !Entry.PreloadEvent->IsComplete())
		{
			OutCompletionEvents.Add(Entry.PreloadEvent);
			return true;
		}
	}
	return false;
}

bool FIoStoreShaderCodeArchive::PreloadShader(int32 ShaderIndex, FGraphEventArray& OutCompletionEvents)
{
	LLM_SCOPE(ELLMTag::Shaders);
	DebugVisualizer.MarkExplicitlyPreloadedForVisualization(ShaderIndex);
	return PreloadShaderGroup(GetGroupIndexForShader(ShaderIndex), OutCompletionEvents
#if UE_SCA_DEBUG_PRELOADING
		, FString::Printf(TEXT("PreloadShader %d"), ShaderIndex)
#endif
	);
}

bool FIoStoreShaderCodeArchive::GroupOnlyContainsRaytracingShaders(int32 ShaderGroupIndex) const
{
	const FIoStoreShaderGroupEntry& GroupEntry = Header.ShaderGroupEntries[ShaderGroupIndex];
	for (uint32 ShaderIdxIdx = GroupEntry.ShaderIndicesOffset, StopIdxIdx = GroupEntry.ShaderIndicesOffset + GroupEntry.NumShaders; ShaderIdxIdx < StopIdxIdx; ++ShaderIdxIdx)
	{
		int32 ShaderIndex = Header.ShaderIndices[ShaderIdxIdx];
		if (!IsRayTracingShaderFrequency(static_cast<EShaderFrequency>(Header.ShaderEntries[ShaderIndex].Frequency)))
		{
			return false;
		}
	}

	return true;
}

bool FIoStoreShaderCodeArchive::PreloadShaderMap(int32 ShaderMapIndex, FGraphEventArray& OutCompletionEvents)
{
	LLM_SCOPE(ELLMTag::Shaders);
	TRACE_IOSTORE_METADATA_SCOPE_TAG(*LibraryName);
	const FIoStoreShaderMapEntry& ShaderMapEntry = Header.ShaderMapEntries[ShaderMapIndex];

#if UE_SCA_DEBUG_PRELOADING
	FString Callsite = FString::Printf(TEXT("PreloadShaderMap %d"), ShaderMapIndex);
#endif
	for (uint32 i = 0u; i < ShaderMapEntry.NumShaders; ++i)
	{
		const int32 ShaderIndex = Header.ShaderIndices[ShaderMapEntry.ShaderIndicesOffset + i];
		const int32 ShaderGroupIndex = GetGroupIndexForShader(ShaderIndex);

#if RHI_RAYTRACING
		if (!IsRayTracingAllowed() && !IsCreateShadersOnLoadEnabled() && GroupOnlyContainsRaytracingShaders(ShaderGroupIndex))
		{
			MarkPreloadEntrySkipped(ShaderGroupIndex
#if UE_SCA_DEBUG_PRELOADING
				, Callsite
#endif
			);
			continue;
		}
#endif

		// only shaders we actually want to preload should be marked as such, not just everything in the group
		DebugVisualizer.MarkExplicitlyPreloadedForVisualization(ShaderIndex);
		PreloadShaderGroup(ShaderGroupIndex, OutCompletionEvents
#if UE_SCA_DEBUG_PRELOADING
			, Callsite
#endif
		);
	}

	return true;
}

bool FIoStoreShaderCodeArchive::ResolveShaderMap(int32 ShaderMapIndex, FCoreDelegates::FIoChunkIdResolvedFunc IoChunkIdResolvedFunc) const
{
	// Invoke callback for the shader group of each shader within the specified shadermap
	const FIoStoreShaderMapEntry& ShaderMapEntry = Header.ShaderMapEntries[ShaderMapIndex];
	for (uint32 Index = 0u; Index < ShaderMapEntry.NumShaders; ++Index)
	{
		const int32 ShaderIndex = Header.ShaderIndices[ShaderMapEntry.ShaderIndicesOffset + Index];
		const int32 ShaderGroupIndex = GetGroupIndexForShader(ShaderIndex);

#if RHI_RAYTRACING
		if (!IsRayTracingAllowed() && !IsCreateShadersOnLoadEnabled() && GroupOnlyContainsRaytracingShaders(ShaderGroupIndex))
		{
			continue;
		}
#endif

		const FIoChunkId& ShaderGroupChunkId = Header.ShaderGroupIoHashes[ShaderGroupIndex];
		IoChunkIdResolvedFunc(ShaderGroupChunkId);
	}
	return true;
}

bool FIoStoreShaderCodeArchive::PreloadShaderMap(int32 ShaderMapIndex, FCoreDelegates::FAttachShaderReadRequestFunc AttachShaderReadRequestFunc)
{
	LLM_SCOPE(ELLMTag::Shaders);
	TRACE_IOSTORE_METADATA_SCOPE_TAG(*LibraryName);
	const FIoStoreShaderMapEntry& ShaderMapEntry = Header.ShaderMapEntries[ShaderMapIndex];

	FGraphEventArray Dummy;
#if UE_SCA_DEBUG_PRELOADING
	FString Callsite = FString::Printf(TEXT("PreloadShaderMap(AttachShaderReadRequestFunc) %d"), ShaderMapIndex);
#endif
	for (uint32 i = 0u; i < ShaderMapEntry.NumShaders; ++i)
	{
		const int32 ShaderIndex = Header.ShaderIndices[ShaderMapEntry.ShaderIndicesOffset + i];
		const int32 ShaderGroupIndex = GetGroupIndexForShader(ShaderIndex);

#if RHI_RAYTRACING
		if (!IsRayTracingAllowed() && !IsCreateShadersOnLoadEnabled() && GroupOnlyContainsRaytracingShaders(ShaderGroupIndex))
		{
			MarkPreloadEntrySkipped(ShaderGroupIndex
#if UE_SCA_DEBUG_PRELOADING
				, Callsite
#endif
			);
			continue;
		}
#endif

		// only shaders we actually want to preload should be marked as such, not just everything in the group
		DebugVisualizer.MarkExplicitlyPreloadedForVisualization(ShaderIndex);
		PreloadShaderGroup(ShaderGroupIndex, Dummy, 
#if UE_SCA_DEBUG_PRELOADING
			Callsite,
#endif
			&AttachShaderReadRequestFunc);
	}

	return true;
}

void FIoStoreShaderCodeArchive::ReleasePreloadEntry(int32 ShaderGroupIndex
#if UE_SCA_DEBUG_PRELOADING
	, const FString& CallsiteInfo
#endif
)
{
	FWriteScopeLock Lock(PreloadedShaderGroupsLock);
	FShaderGroupPreloadEntry** ExistingEntry = PreloadedShaderGroups.Find(ShaderGroupIndex);
	ensureMsgf(ExistingEntry, TEXT("Preload entry for shader group %d should exist if we're asked to release it"), ShaderGroupIndex);
	if (ExistingEntry)
	{
		FShaderGroupPreloadEntry* PreloadEntry = *ExistingEntry;

		const uint32 ShaderNumRefs = PreloadEntry->NumRefs--;
		check(ShaderNumRefs > 0u);

#if UE_SCA_DEBUG_PRELOADING
		FString AppendInfo = FString::Printf(TEXT("ReleasePreloadEntry: NumRefs %d -> %d    CallsiteInfo: %s\n"),
			ShaderNumRefs, PreloadEntry->NumRefs, *CallsiteInfo
			);
		PreloadEntry->DebugInfo.Append(AppendInfo);
#endif

		if (ShaderNumRefs == 1u)
		{
			uint32 ShaderGroupLoadBytes = 0;
			if (!PreloadEntry->bNeverToBePreloaded)
			{
				PreloadEntry->ShaderGroupData.Reset();
				PreloadEntry->PreloadEvent.SafeRelease();
				const FIoStoreShaderGroupEntry& GroupEntry = Header.ShaderGroupEntries[ShaderGroupIndex];
				ShaderGroupLoadBytes = GroupEntry.CompressedSize + sizeof(FShaderGroupPreloadEntry);
			}
			else
			{
				ShaderGroupLoadBytes = sizeof(FShaderGroupPreloadEntry);
			}

#if UE_SCA_DEBUG_PRELOADING
			if (0)	// use this if you need comparison with all other groups
			{
				UE_LOGF(LogInit, Log, "Group %d has still %d references on deletion. Group extended debug info: \n%ls", ShaderGroupIndex, PreloadEntry->NumRefs,
					*PreloadEntry->DebugInfo);
			}
#endif

			delete PreloadEntry;
			PreloadedShaderGroups.Remove(ShaderGroupIndex);

			DEC_DWORD_STAT_BY(STAT_Shaders_ShaderPreloadMemory, ShaderGroupLoadBytes);
#if (CSV_PROFILER_STATS && !UE_BUILD_SHIPPING) 
			TCsvPersistentCustomStat<float>* CsvStatPreloadedShaderMB = FCsvProfiler::Get()->GetOrCreatePersistentCustomStatFloat(TEXT("PreloadedShaderMB"), CSV_CATEGORY_INDEX(Shaders));
			CsvStatPreloadedShaderMB->Sub((float)ShaderGroupLoadBytes / (1024.0f * 1024.0f));
#endif
		}
	}
}

void FIoStoreShaderCodeArchive::ReleasePreloadedShader(int32 ShaderIndex)
{
	ReleasePreloadEntry(GetGroupIndexForShader(ShaderIndex)
#if UE_SCA_DEBUG_PRELOADING
		, FString::Printf(TEXT("ReleasePreloadedShader %d"), ShaderIndex)
#endif
	);
}

bool FIoStoreShaderCodeArchive::IsShaderMapResolved(int32 ShaderMapIndex) const
{
	// Check availability of each IoChunk for the specified shadermap
	const FIoStoreShaderMapEntry& ShaderMapEntry = Header.ShaderMapEntries[ShaderMapIndex];
	for (uint32 ShaderIndex = 0; ShaderIndex < ShaderMapEntry.NumShaders; ++ShaderIndex)
	{
		const uint32 MappedShaderIndex = Header.ShaderIndices[ShaderMapEntry.ShaderIndicesOffset + ShaderIndex];
		const int32 GroupIndex = GetGroupIndexForShader(static_cast<int32>(MappedShaderIndex));
		const FIoChunkId& GroupChunkId = Header.ShaderGroupIoHashes[GroupIndex];

		if (!IoDispatcher.GetSizeForChunk(GroupChunkId).IsOk())
		{
			UE_LOGF(LogShaderLibrary, Warning, "Shadermap '%ls' (%d) not resolved in library %ls because IoChunk '%ls' is not available",
				*LexToString(Header.ShaderMapHashes[ShaderMapIndex]), ShaderMapIndex, *GetName(), *LexToString(GroupChunkId));
			return false;
		}
	}
	return true;
}

int32 FIoStoreShaderCodeArchive::FindShaderMapIndex(const FShaderHash& Hash)
{
	const uint32 Key = GetTypeHash(Hash);
	for (uint32 Index = ShaderMapHashTable.First(Key); ShaderMapHashTable.IsValid(Index); Index = ShaderMapHashTable.Next(Index))
	{
		if (Header.ShaderMapHashes[Index] == Hash)
		{
			return Index;
		}
	}
	return INDEX_NONE;
}

int32 FIoStoreShaderCodeArchive::FindShaderIndex(const FShaderHash& Hash)
{
	const uint32 Key = GetTypeHash(Hash);
	for (uint32 Index = ShaderHashTable.First(Key); ShaderHashTable.IsValid(Index); Index = ShaderHashTable.Next(Index))
	{
		if (Header.ShaderHashes[Index] == Hash)
		{
			return Index;
		}
	}
	return INDEX_NONE;
}

static const TCHAR* GetIoRequestBackendName(const FIoRequest& IoRequest)
{
	FIoRequestDebugInfo DebugInfo;
	FIoDispatcherInternal::GetDebugInfo(IoRequest, DebugInfo);
	return DebugInfo.BackendName ? DebugInfo.BackendName : TEXT("<None>");
}

TRefCountPtr<FRHIShader> FIoStoreShaderCodeArchive::CreateShader(int32 ShaderIndex, bool bRequired)
{
	LLM_SCOPE(ELLMTag::Shaders);

	TRACE_CPUPROFILER_EVENT_SCOPE(FIoStoreShaderCodeArchive::CreateShader);

	TRefCountPtr<FRHIShader> Shader;

	const FIoStoreShaderCodeEntry& ShaderEntry = Header.ShaderEntries[ShaderIndex];
	const int32 GroupIndex = GetGroupIndexForShader(ShaderIndex);

	// Preload shader group if it wasn't yet. This will also addref it so we can be sure it will exist.
	FGraphEventArray Dummy;
#if UE_SCA_DEBUG_PRELOADING
	FString Callsite = FString::Printf(TEXT("CreateShader %d"), ShaderIndex);
#endif
	PreloadShaderGroup(GroupIndex, Dummy
#if UE_SCA_DEBUG_PRELOADING
		, Callsite
#endif
	);

	FShaderGroupPreloadEntry* PreloadEntryPtr;
	{
		FReadScopeLock Lock(PreloadedShaderGroupsLock);
		PreloadEntryPtr = FindExistingPreloadEntry(GroupIndex);
	}
#if USE_MMAPPED_SHADERARCHIVE
	if (!PreloadEntryPtr->ShaderGroupData->MappedRegionStatus.IsOk())
#endif
	{
		// raise the prio if still ongoing
		if (!PreloadEntryPtr->ShaderGroupData->IoRequest.Status().IsCompleted())
		{
			PreloadEntryPtr->ShaderGroupData->IoRequest.UpdatePriority(IoDispatcherPriority_Max);
		}
		FGraphEventRef Event = PreloadEntryPtr->PreloadEvent;
		
		bool bMissedPreLoaded = false;
		const bool bNeededToWait = Event.IsValid() && !Event->IsComplete();
		if (bNeededToWait)
		{
			if (!bRequired)
			{
				PreloadEntryPtr = nullptr;
				ReleasePreloadEntry(GroupIndex
#if UE_SCA_DEBUG_PRELOADING
					, Callsite
#endif
				);
				return Shader;
			}
			
			bMissedPreLoaded = true;
			TRACE_CPUPROFILER_EVENT_SCOPE(BlockingShaderLoad);
			
			const double TimeStarted = FPlatformTime::Seconds();
			FTaskGraphInterface::Get().WaitUntilTaskCompletes(Event);
			const double WaitDuration = FPlatformTime::Seconds() - TimeStarted;
			
			// only complain if we spent more than 1ms waiting
			if (WaitDuration > GShaderCodeLibraryMaxShaderPreloadWaitTime)
			{
				UE_LOGF(LogShaderLibrary, Warning, "Spent %.2f ms in a blocking wait for shader preload, NumRefs: %d, FramePreloadStarted: %d, CurrentFrame: %d", WaitDuration * 1000.0, PreloadEntryPtr->NumRefs, PreloadEntryPtr->FramePreloadStarted, GFrameNumber);
			}
			CSV_CUSTOM_STAT(Shaders, PreloadShaderMiss, 1, ECsvCustomStatOp::Accumulate);
			CSV_CUSTOM_STAT(Shaders, PreloadShaderWaitTime, WaitDuration * 1000.0f, ECsvCustomStatOp::Accumulate);
		}
		
#if !UE_BUILD_SHIPPING
		const bool bSimulateFailure = !bRequired && GShaderCodeLibraryOptionalShaderFailChance > 0.0f && FMath::FRand() < GShaderCodeLibraryOptionalShaderFailChance;
#else
		const bool bSimulateFailure = false;
#endif

		// If we get here log information, but if the shader isn't required, return. Otherwise continue on and crash
		if (bSimulateFailure || UNLIKELY(PreloadEntryPtr->ShaderGroupData->IoRequest.GetResult() == nullptr))
		{
			UE_LOGF(LogShaderLibrary, Warning, "IoRequest backing buffer is not there %s. Library '%ls', shader hash '%ls' (freq=%lld), group %d IoChunkId '%ls', started preloading at frame %d, current frame is %d, preloading %ls",
				bRequired ? "when we need it, about to crash" : "and not required",
				*GetName(),
				*Header.ShaderHashes[ShaderIndex].ToString(),
				ShaderEntry.Frequency,
				GroupIndex,
				*LexToString(Header.ShaderGroupIoHashes[GroupIndex]),
				PreloadEntryPtr->FramePreloadStarted,
				GFrameNumber,
				PreloadEntryPtr->PreloadEvent.IsValid() ? (PreloadEntryPtr->PreloadEvent->IsComplete() ? TEXT("is complete") : TEXT("is NOT complete!")) : TEXT("event is NOT VALID!")
				);
			if (!bRequired)
			{
				ReleasePreloadEntry(GroupIndex
#if UE_SCA_DEBUG_PRELOADING
					, Callsite
#endif
				);
				return Shader;
			}
		}
	}
	
	const uint8* ShaderCode = nullptr;
	uint32 DataSize = 0;
#if USE_MMAPPED_SHADERARCHIVE
	if (PreloadEntryPtr->ShaderGroupData->MappedRegionStatus.IsOk())
	{
		IMappedFileRegion* MappedFileRegion = PreloadEntryPtr->ShaderGroupData->MappedRegionStatus.ValueOrDie().MappedFileRegion;
		ShaderCode = MappedFileRegion->GetMappedPtr();
		DataSize = MappedFileRegion->GetMappedSize();
	}
	else
#endif
	{
		const FIoBuffer& IoBuffer = PreloadEntryPtr->ShaderGroupData->IoRequest.GetResultOrDie();
		ShaderCode = IoBuffer.Data();
		DataSize = IoBuffer.DataSize();
		
		// Verbose logging from what IoDispatcher backend this shader chunk was loaded. If this is null, GetResultOrDie() will have already crashed.
		UE_LOGF(LogShaderLibrary, VeryVerbose, "Received shader chunk '%ls' in library '%ls' from '%ls'",
			*LexToString(Header.ShaderGroupIoHashes[GroupIndex]), *GetName(), GetIoRequestBackendName(PreloadEntryPtr->ShaderGroupData->IoRequest));
	}
	
	FMemStackBase& MemStack = FMemStack::Get();
	FMemMark Mark(MemStack);
	FIoStoreShaderGroupEntry& GroupEntry = Header.ShaderGroupEntries[GroupIndex];
	const bool IsGroupCompressed = GroupEntry.IsGroupCompressed();
	if (IsGroupCompressed)
	{
		uint8* UncompressedCode = reinterpret_cast<uint8*>(MemStack.Alloc(GroupEntry.UncompressedSize, 16));
		DecompressShadergroupWithOodleAndExtraLogging(GroupIndex, Header.ShaderGroupIoHashes[GroupIndex], GroupEntry, ShaderIndex, ShaderEntry.ShaderGroupIndex, Header.ShaderHashes[ShaderIndex], UncompressedCode, GroupEntry.UncompressedSize, ShaderCode, DataSize);
		ShaderCode = reinterpret_cast<uint8*>(UncompressedCode) + ShaderEntry.UncompressedOffsetInGroup;

#if UE_SCA_VISUALIZE_SHADER_USAGE
		for (uint32 ShaderIdxIdx = GroupEntry.ShaderIndicesOffset, StopBeforeIdxIdx = GroupEntry.ShaderIndicesOffset + GroupEntry.NumShaders; ShaderIdxIdx < StopBeforeIdxIdx; ++ShaderIdxIdx)
		{
			DebugVisualizer.MarkDecompressedForVisualization(Header.ShaderIndices[ShaderIdxIdx]);
		}
#endif // UE_SCA_VISUALIZE_SHADER_USAGE
	}
	else
	{
		ShaderCode += ShaderEntry.UncompressedOffsetInGroup;
	}
	auto MakeCreateShaderDesc = [this, IsGroupCompressed, ShaderCode, ShaderIndex, PreloadEntryPtr]() -> FRHICreateShaderDesc
	{
		if (IsGroupCompressed)
		{
			return FRHICreateShaderDesc(MakeArrayView(ShaderCode, Header.GetShaderUncompressedSize(ShaderIndex)), this);
		}
		OnShaderGroupDataOwnerCreated();
		return FRHICreateShaderDesc(
			MakeArrayView(ShaderCode, Header.GetShaderUncompressedSize(ShaderIndex)),
			[this, ShaderGroupDataSharedPtr = PreloadEntryPtr->ShaderGroupData](void*)
			{
				// ShaderGroupDataSharedPtr will be released when this lambda is destroyed
				OnShaderGroupDataOwnerReleased();
			},
			this);
	};
	FRHICreateShaderDesc CreateShaderDesc = MakeCreateShaderDesc();
	
	switch (ShaderEntry.Frequency)
	{
	case SF_Vertex: Shader = RHICreateVertexShader(CreateShaderDesc); break;
	case SF_Mesh: Shader = RHICreateMeshShader(CreateShaderDesc); break;
	case SF_Amplification: Shader = RHICreateAmplificationShader(CreateShaderDesc); break;
	case SF_Pixel: Shader = RHICreatePixelShader(CreateShaderDesc); break;
	case SF_Geometry: Shader = RHICreateGeometryShader(CreateShaderDesc); break;
	case SF_Compute: Shader = RHICreateComputeShader(CreateShaderDesc); break;
	case SF_WorkGraphRoot: Shader = RHICreateWorkGraphShader(CreateShaderDesc, SF_WorkGraphRoot); break;
	case SF_WorkGraphComputeNode: Shader = RHICreateWorkGraphShader(CreateShaderDesc, SF_WorkGraphComputeNode); break;
	case SF_RayGen: case SF_RayMiss: case SF_RayHitGroup: case SF_RayCallable:
#if RHI_RAYTRACING
		if (GRHISupportsRayTracing && GRHISupportsRayTracingShaders)
		{
			Shader = RHICreateRayTracingShader(CreateShaderDesc, ShaderEntry.GetFrequency());
		}
#endif // RHI_RAYTRACING
		break;
	default: checkNoEntry(); break;
	}
	DebugVisualizer.MarkCreatedForVisualization(ShaderIndex);

	PreloadEntryPtr = nullptr;	

	ReleasePreloadEntry(GroupIndex
#if UE_SCA_DEBUG_PRELOADING
						, Callsite
#endif
						);
	
	
	if (Shader)
	{
		INC_DWORD_STAT(STAT_Shaders_NumShadersCreated);

#if (CSV_PROFILER_STATS && !UE_BUILD_SHIPPING) 
		TCsvPersistentCustomStat<int>* CsvStatNumShadersCreated = FCsvProfiler::Get()->GetOrCreatePersistentCustomStatInt(TEXT("NumShadersCreated"), CSV_CATEGORY_INDEX(Shaders));
		CsvStatNumShadersCreated->Add(1);
#endif

		Shader->SetHash(Header.ShaderHashes[ShaderIndex]);
	}

	return Shader;
}
