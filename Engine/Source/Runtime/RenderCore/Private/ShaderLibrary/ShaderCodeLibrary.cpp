// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
ShaderCodeLibrary.cpp: Bound shader state cache implementation.
=============================================================================*/

#include "ShaderCodeLibrary.h"

#include "Algo/Replace.h"
#include "Async/ParallelFor.h"
#include "Containers/HashTable.h"
#include "Containers/Set.h"
#include "Containers/StringView.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "FileCache/FileCache.h"
#include "HAL/FileManager.h"
#include "HAL/LowLevelMemTracker.h"
#include "HAL/PlatformFileManager.h"
#include "HAL/PlatformSplash.h" // IWYU pragma: keep
#include "Hash/CityHash.h"
#include "Interfaces/IPluginManager.h"
#include "Internationalization/Regex.h"
#include "Math/UnitConversion.h"
#include "Misc/CommandLine.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/CoreDelegates.h"
#include "Misc/MessageDialog.h"
#include "Misc/Paths.h"
#include "Misc/ScopeLock.h"
#include "Misc/ScopeRWLock.h"
#include "Misc/SecureHash.h"
#include "Misc/StringBuilder.h"
#include "PipelineFileCache.h"
#include "ProfilingDebugging/AssetMetadataTrace.h"
#include "ProfilingDebugging/LoadTimeTracker.h"
#include "ProfilingDebugging/DiagnosticTable.h"
#include "RenderingThread.h"
#include "Shader.h"
#include "ShaderCodeArchive.h"
#include "ShaderCompilerCore.h"
#include "ShaderPipelineCache.h"
#include "ShaderLibrary/EditorShaderCodeArchive.h"
#include "ShaderLibrary/EditorShaderStableInfo.h"
#include "ShaderLibrary/ShaderCodeLibraryInternal.h"
#include "ShaderLibrary/ShaderCodeLibraryUtilities.h"
#include "ShaderLibrary/ShaderLibraryInstance.h"
#include "String/ParseTokens.h"
#include "IO/IoChunkId.h"
#include "IO/IoDispatcher.h"
#include "IO/IoDispatcherInternal.h"

#if WITH_EDITORONLY_DATA
#include "Interfaces/IShaderFormat.h"
#include "Interfaces/ITargetPlatform.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#endif

#if WITH_EDITOR
#include "PipelineCacheUtilities.h"
#include "UObject/Class.h"
#include "UObject/Object.h"
#include "Serialization/CompactBinarySerialization.h"
#include "Serialization/CompactBinaryWriter.h"
#include "RHIStrings.h"
#include "Cooker/CookArtifact.h"
#include "Cooker/CookArtifactReader.h"
#endif

// allow introspection (e.g. dumping the contents) for easier debugging
#define UE_SHADERLIB_WITH_INTROSPECTION			!UE_BUILD_SHIPPING

// Enabled by default for all configurations, runtime check should prevent chuck discovery in non dev builds
#define UE_SHADERLIB_SUPPORT_CHUNK_DISCOVERY	(1)

DEFINE_LOG_CATEGORY(LogShaderLibrary);

using namespace UE::ShaderLibrary::Private;

namespace UE::ShaderLibrary::Private
{
	using ESaveToDiskTarget = ShaderCodeArchive::ESaveToDiskTarget;
	using ESaveToDiskSortOrder = ShaderCodeArchive::ESaveToDiskSortOrder;

	static FDelegateHandle OnPakFileMountedDelegateHandle;
	static FDelegateHandle OnPakFileUnmountingDelegateHandle;
	static FDelegateHandle OnPluginMountedDelegateHandle;
	static FDelegateHandle OnPluginUnmountedDelegateHandle;

	static TSet<FString> PluginsToIgnoreOnMount;

	using FShaderLibraryInstancePriorityPair = TPair<int32, TUniquePtr<FShaderLibraryInstance>>;

	// [RCL] TODO 2020-11-20: Separate runtime and editor-only code (tracked as UE-103486)
	/** Descriptor used to pass the pak file information to the library as we cannot store IPakFile ref */
	struct FMountedPakFileInfo
	{
		/** Pak filename (external OS filename) */
		FString PakFilename;
		/** In-game path for the pak content */
		FString MountPoint;
		/** Chunk ID */
		int32 ChunkId;

		// this constructor is used for chunks that we have not yet possibly seen
		FMountedPakFileInfo(int32 InChunkId)
			: PakFilename(TEXT("Fake")),
			ChunkId(InChunkId)
		{
		}

#if UE_SHADERLIB_SUPPORT_CHUNK_DISCOVERY
		FMountedPakFileInfo(const FString& InMountPoint, int32 InChunkId)
			: PakFilename(TEXT("Fake")),
			MountPoint(InMountPoint),
			ChunkId(InChunkId)
		{
		}
#endif // UE_SHADERLIB_SUPPORT_CHUNK_DISCOVERY

		FMountedPakFileInfo(const IPakFile& PakFile)
			: PakFilename(PakFile.PakGetPakFilename()),
			MountPoint(PakFile.PakGetMountPoint()),
			ChunkId(PakFile.PakGetPakchunkIndex())
		{
		}

		FString ToString() const
		{
			return FString::Printf(TEXT("ChunkID:%d Root:%s File:%s"), ChunkId, *MountPoint, *PakFilename);
		}

		friend uint32 GetTypeHash(const FMountedPakFileInfo& InData)
		{
			return HashCombine(HashCombine(GetTypeHash(InData.PakFilename), GetTypeHash(InData.MountPoint)), ::GetTypeHash(InData.ChunkId));
		}

		friend bool operator==(const FMountedPakFileInfo& A, const FMountedPakFileInfo& B)
		{
			return A.PakFilename == B.PakFilename && A.MountPoint == B.MountPoint && A.ChunkId == B.ChunkId;
		}

		/** Holds a set of all known paks that can be added very early. Each library on Open will traverse that list. */
		static TSet<FMountedPakFileInfo> KnownPakFiles;

		/** Guards access to the list of known pak files*/
		static FCriticalSection KnownPakFilesAccessLock;
	};

	// At runtime, a descriptor of a named library
	struct FNamedShaderLibrary
	{
		/** A name that is passed to Open/CloseLibrary, like "Global", "ShooterGame", "MyPlugin" */
		FString	LogicalName;

		/** Shader platform */
		EShaderPlatform ShaderPlatform;

		/** Base directory for chunk 0 */
		FString BaseDirectory;

		/** Guards access to components*/
		FRWLock ComponentsMutex;

		/** Even putting aside chunking, each named library can be potentially comprised of multiple files */
		TArray<FShaderLibraryInstancePriorityPair> Components;

		FNamedShaderLibrary(const FString& InLogicalName, const EShaderPlatform InShaderPlatform, const FString& InBaseDirectory)
			: LogicalName(InLogicalName)
			, ShaderPlatform(InShaderPlatform)
			, BaseDirectory(InBaseDirectory)
		{
		}

		int32 GetNumComponents() const
		{
			return Components.Num();
		}

		/** 
		 * Returns which chunk Ids are present in this library. Monolithic libraries will return empty set
		 */
		TSet<int32> GetPresentChunks()
		{
			FRWScopeLock ReadLock(ComponentsMutex, SLT_ReadOnly);
			TSet<int32> PresentChunks;

			for (const FShaderLibraryInstancePriorityPair& Component : Components)
			{
				int32 ChunkId = Component.Value->GetChunkId();
				if (LIKELY(ChunkId != INDEX_NONE))
				{
					PresentChunks.Add(ChunkId);
				}
			}

			return MoveTemp(PresentChunks);
		}

		/** Helper struct to carry the chunk library name, ID, and priority assigned to the library instance internals. */
		struct FShaderLibraryChunkInfo
		{
			FString ChunkLibraryName;
			int32 ChunkId = INDEX_NONE;
			int32 Priority = 0;
		};

		// Transforms the chunk Id of the pak file for a named shader library component,
		// e.g. pakchunk100 and pakchunk100iad must not point to the same ID 100, so the latter one will be transformed to a higher number.
		FShaderLibraryChunkInfo GetLibraryChunkInfo(const FMountedPakFileInfo& MountInfo) const
		{
			constexpr int32 kChunkIdSuffixOffsetRange = 100000;
			constexpr int32 kChunkIdSuffixBitmask = 0xFF;

			FShaderLibraryChunkInfo ChunkInfo;
			ChunkInfo.ChunkLibraryName = GetShaderLibraryNameForChunk(LogicalName, MountInfo.ChunkId);
			ChunkInfo.ChunkId = MountInfo.ChunkId;

			// If the pakchunk ID contains a suffix such as "iad", the shader library must have been created as a proxy target file and will also contain that suffix.
			FString ChunkIdSuffix;
			uint64 ChunkIdSuffixHash = 0;
			if (FShaderLibraryNameInfo::ParsePakChunkId(FPaths::GetBaseFilename(MountInfo.PakFilename), ChunkInfo.ChunkId, ChunkIdSuffix) && !ChunkIdSuffix.IsEmpty())
			{
				// Mangle chunk ID used for internal redundancy checks, so that IAD and non-IAD chunks with the same ID don't collide (e.g. pakchunk100 and pakchunk100iad)
				ChunkInfo.ChunkLibraryName.Append(ChunkIdSuffix);
				ChunkIdSuffixHash = CityHash64(reinterpret_cast<const char*>(*ChunkIdSuffix), ChunkIdSuffix.Len() * sizeof(TCHAR));
				ChunkInfo.ChunkId += (ChunkIdSuffixHash & kChunkIdSuffixBitmask) * kChunkIdSuffixOffsetRange;

				// Lower priority of such secondary chunks
				ChunkInfo.Priority = -1;
			}

			return MoveTemp(ChunkInfo);
		}

		void OnPakFileMounted(const FMountedPakFileInfo& MountInfo, const FString& Directory)
		{
			FShaderLibraryChunkInfo ChunkInfo = GetLibraryChunkInfo(MountInfo);
			if (!GetPresentChunks().Contains(ChunkInfo.ChunkId))
			{
				// Parts of shader library might be in the UFS (e.g. .metallibs), hence we need to look for them in the appropriate directory.
				OpenShaderCode(Directory, ChunkInfo.ChunkLibraryName, ChunkInfo.ChunkId, ChunkInfo.Priority);
			}
		}

		void OnPakFileUnmounting(const FMountedPakFileInfo& MountInfo, const FString& Directory)
		{
			FShaderLibraryChunkInfo ChunkInfo = GetLibraryChunkInfo(MountInfo);
			if (GetPresentChunks().Contains(ChunkInfo.ChunkId))
			{
				CloseShaderCode(Directory, ChunkInfo.ChunkLibraryName, ChunkInfo.ChunkId);
			}
		}

		bool OpenShaderCode(const FString& ShaderCodeDir, FString const& Library, int32 ChunkId, int32 Priority = 0);
		void CloseShaderCode(const FString& ShaderCodeDir, FString const& Library, int32 ChunkId);
		void ForEachShaderLibraryWithShaderMap(const FShaderHash& Hash, TFunctionRef<void(FShaderLibraryInstance* InLibraryInstance, int32 InShaderMapIndex)> InPerShaderMapCallback);
		FShaderLibraryInstance* FindShaderLibraryForShaderMap(const FShaderHash& Hash, int32& OutShaderMapIndex, const FShaderLibraryInstance* SkipLibraryInstance = nullptr);
		FShaderLibraryInstance* FindShaderLibraryForShader(const FShaderHash& Hash, int32& OutShaderIndex);
		uint32 GetShaderCount();
#if UE_SHADERLIB_WITH_INTROSPECTION
		void DumpLibraryContents(const FString& Prefix);
#endif
	};

	static bool bUseFixForDeferredShaderLibraryDeletion = true;
	static FAutoConsoleVariableRef CVarUseFixForDeferredShaderLibraryDeletion(
		TEXT("r.ShaderCodeLibrary.UseFixForDeferredShaderLibraryDeletion"),
		bUseFixForDeferredShaderLibraryDeletion,
		TEXT("Rollback CVar for potential crash fix\n")
		TEXT(" 1: (default) New code - defers a particular deletion of a shader library to happen on the game thread to prevent FlushRenderingCommands() from being called on a background thread\n")
		TEXT(" 0: Old code - in this particular case, deletes the shader library directly instead of deferring its deletion"),
		ECVF_Default);
} // UE::ShaderLibrary::Private

TSet<UE::ShaderLibrary::Private::FMountedPakFileInfo> UE::ShaderLibrary::Private::FMountedPakFileInfo::KnownPakFiles;
FCriticalSection UE::ShaderLibrary::Private::FMountedPakFileInfo::KnownPakFilesAccessLock;

FSharedShaderMapResourceExplicitRelease OnSharedShaderMapResourceExplicitRelease;

FORCEINLINE FName ParseFNameCached(const FStringView& Src, TMap<uint32,FName>& NameCache)
{
	uint32 SrcHash = CityHash32(reinterpret_cast<const char*>(Src.GetData()), Src.Len() * sizeof(TCHAR));
	if (FName* Name = NameCache.Find(SrcHash))
	{
		return *Name;
	}
	else
	{
		return NameCache.Emplace(SrcHash, FName(Src.Len(), Src.GetData()));
	}
}

static void AppendFNameAsUTF8(FAnsiStringBuilderBase& Out, const FName& InName)
{
	if (!InName.TryAppendAnsiString(Out))
	{
		TStringBuilder<128> WideName;
		InName.AppendString(WideName);
		Out << TCHAR_TO_UTF8(WideName.ToString());
	}
}

static void AppendSanitizedFNameAsUTF8(FAnsiStringBuilderBase& Out, const FName& InName, ANSICHAR Delim)
{
	const int32 Offset = Out.Len();
	AppendFNameAsUTF8(Out, InName);
	Algo::Replace(MakeArrayView(Out).Slice(Offset, Out.Len() - Offset), Delim, ' ');
}

static void AppendSanitizedFName(FStringBuilderBase& Out, const FName& InName, TCHAR Delim)
{
	const int32 Offset = Out.Len();
	InName.AppendString(Out);
	Algo::Replace(MakeArrayView(Out).Slice(Offset, Out.Len() - Offset), Delim, TEXT(' '));
}

FString FCompactFullName::ToString() const
{
	TStringBuilder<256> RetString;
	AppendString(RetString);
	return FString(FStringView(RetString));
}

FString FCompactFullName::ToStringPathOnly() const
{
	const int32 ObjectClassAndPathCount = ObjectClassAndPath.Num();
	if (!ObjectClassAndPathCount)
	{
		return FString();
	}

	TStringBuilder<256> RetString;
	// skip the first element which is class name, and the last, which is the object name itself
	for (int32 NameIdx = 1; NameIdx < ObjectClassAndPathCount - 1; NameIdx++)
	{
		RetString << ObjectClassAndPath[NameIdx];
		if (NameIdx < ObjectClassAndPathCount - 2)
		{
			RetString << TEXT('/');
		}
	}
	return FString(FStringView(RetString));
}

void FCompactFullName::AppendString(FStringBuilderBase& Out) const
{
	const int32 ObjectClassAndPathCount = ObjectClassAndPath.Num();
	if (!ObjectClassAndPathCount)
	{
		Out << TEXT("empty");
	}
	else
	{
		for (int32 NameIdx = 0; NameIdx < ObjectClassAndPathCount; NameIdx++)
		{
			Out << ObjectClassAndPath[NameIdx];
			if (NameIdx == 0)
			{
				Out << TEXT(' ');
			}
			else if (NameIdx < ObjectClassAndPathCount - 1)
			{
				if (NameIdx == ObjectClassAndPathCount - 2)
				{
					Out << TEXT('.');
				}
				else
				{
					Out << TEXT('/');
				}
			}
		}
	}
}

void FCompactFullName::AppendString(FAnsiStringBuilderBase& Out) const
{
	const int32 ObjectClassAndPathCount = ObjectClassAndPath.Num();
	if (!ObjectClassAndPathCount)
	{
		Out << "empty";
	}
	else
	{
		for (int32 NameIdx = 0; NameIdx < ObjectClassAndPathCount; NameIdx++)
		{
			AppendFNameAsUTF8(Out, ObjectClassAndPath[NameIdx]);
			if (NameIdx == 0)
			{
				Out << ' ';
			}
			else if (NameIdx < ObjectClassAndPathCount - 1)
			{
				if (NameIdx == ObjectClassAndPathCount - 2)
				{
					Out << '.';
				}
				else
				{
					Out << '/';
				}
			}
		}
	}
}

void FCompactFullName::ParseFromString(const FStringView& InSrc)
{
	TArray<FStringView, TInlineAllocator<64>> Fields;
	// do not split by '/' as this splits the original FName into per-path components
	UE::String::ParseTokensMultiple(InSrc.TrimStartAndEnd(), {TEXT(' '), TEXT('.'), /*TEXT('/'),*/ TEXT('\t')},
		[&Fields](FStringView Field) { if (!Field.IsEmpty()) { Fields.Add(Field); } });
	if (Fields.Num() == 1 && Fields[0] == TEXTVIEW("empty"))
	{
		ObjectClassAndPath.Empty();
	}
	// fix up old format that removed the leading '/'
	else if (Fields.Num() == 3 && Fields[1][0] != TEXT('/'))
	{
		ObjectClassAndPath.Empty(3);
		ObjectClassAndPath.Emplace(Fields[0]);
		FString Fixup("/");
		Fixup += Fields[1];
		ObjectClassAndPath.Emplace(Fixup);
		ObjectClassAndPath.Emplace(Fields[2]);
	}
	else
	{
		ObjectClassAndPath.Empty(Fields.Num());
		for (const FStringView& Item : Fields)
		{
			ObjectClassAndPath.Emplace(Item);
		}
	}
}

#if WITH_EDITOR
void FCompactFullName::SetCompactFullNameFromObject(UObject* InDepObject)
{
	ObjectClassAndPath.Empty();

	UObject* DepObject = InDepObject;
	if (DepObject)
	{
		ObjectClassAndPath.Add(DepObject->GetClass()->GetFName());
		while (DepObject)
		{
			ObjectClassAndPath.Insert(DepObject->GetFName(), 1);
			DepObject = DepObject->GetOuter();
		}
	}
	else
	{
		ObjectClassAndPath.Add(FName("null"));
	}
}

FString FCompactFullName::GetPackageNameOnly() const
{
	return ObjectClassAndPath.Num() >= 2 ? ObjectClassAndPath[1].ToString() : ToStringPathOnly();
}
#endif

uint32 GetTypeHash(const FCompactFullName& A)
{
	uint32 Hash = 0;
	for (const FName& Name : A.ObjectClassAndPath)
	{
		Hash = HashCombine(Hash, GetTypeHash(Name));
	}
	return Hash;
}

void FixupUnsanitizedNames(const FString& Src, TArray<FString>& OutFields) 
{
	FString NewSrc(Src);

	int32 ParenOpen = -1;
	int32 ParenClose = -1;

	if (NewSrc.FindChar(TCHAR('('), ParenOpen) && NewSrc.FindChar(TCHAR(')'), ParenClose) && ParenOpen < ParenClose && ParenOpen >= 0 && ParenClose >= 0)
	{
		for (int32 Index = ParenOpen + 1; Index < ParenClose; Index++)
		{
			if (NewSrc[Index] == TCHAR(','))
			{
				NewSrc[Index] = ' ';
			}
		}
		OutFields.Empty();
		NewSrc.TrimStartAndEnd().ParseIntoArray(OutFields, TEXT(","), false);
		// allow formats both with and without pipeline hash
		check(OutFields.Num() == 11 || OutFields.Num() == 12);
	}
}

void FStableShaderKeyAndValue::ComputeKeyHash()
{
	KeyHash = GetTypeHash(ClassNameAndObjectPath);

	KeyHash = HashCombine(KeyHash, GetTypeHash(ShaderType));
	KeyHash = HashCombine(KeyHash, GetTypeHash(ShaderClass));
	KeyHash = HashCombine(KeyHash, GetTypeHash(MaterialDomain));
	KeyHash = HashCombine(KeyHash, GetTypeHash(FeatureLevel));

	KeyHash = HashCombine(KeyHash, GetTypeHash(QualityLevel));
	KeyHash = HashCombine(KeyHash, GetTypeHash(TargetFrequency));
	KeyHash = HashCombine(KeyHash, GetTypeHash(TargetPlatform));

	KeyHash = HashCombine(KeyHash, GetTypeHash(VFType));
	KeyHash = HashCombine(KeyHash, GetTypeHash(PermutationId));
	KeyHash = HashCombine(KeyHash, GetTypeHash(PipelineHash));
}

void FStableShaderKeyAndValue::ParseFromString(const FStringView& Src)
{
	TArray<FStringView, TInlineAllocator<12>> Fields;
	UE::String::ParseTokens(Src.TrimStartAndEnd(), TEXT(','), [&Fields](FStringView Field) { Fields.Add(Field); });

	/* disabled, should not be happening since 1H 2018
	if (Fields.Num() > 12)
	{
		// hack fix for unsanitized names, should not occur anymore.
		FixupUnsanitizedNames(Src, Fields);
	}
	*/

	// for a while, accept old .scl.csv without pipelinehash
	check(Fields.Num() == 11 || Fields.Num() == 12);

	int32 Index = 0;
	ClassNameAndObjectPath.ParseFromString(Fields[Index++]);

	ShaderType = FName(Fields[Index++]);
	ShaderClass = FName(Fields[Index++]);
	MaterialDomain = FName(Fields[Index++]);
	FeatureLevel = FName(Fields[Index++]);

	QualityLevel = FName(Fields[Index++]);
	TargetFrequency = FName(Fields[Index++]);
	TargetPlatform = FName(Fields[Index++]);

	VFType = FName(Fields[Index++]);
	PermutationId = FName(Fields[Index++]);

	OutputHash.FromString(Fields[Index++]);

	check(Index == 11);

	if (Fields.Num() == 12)
	{
		PipelineHash.FromString(Fields[Index++]);
	}
	else
	{
		PipelineHash = FShaderHash();
	}

	ComputeKeyHash();
}


void FStableShaderKeyAndValue::ParseFromStringCached(const FStringView& Src, TMap<uint32, FName>& NameCache)
{
	TArray<FStringView, TInlineAllocator<12>> Fields;
	UE::String::ParseTokens(Src.TrimStartAndEnd(), TEXT(','), [&Fields](FStringView Field) { Fields.Add(Field); });

	/* disabled, should not be happening since 1H 2018
	if (Fields.Num() > 11)
	{
		// hack fix for unsanitized names, should not occur anymore.
		FixupUnsanitizedNames(Src, Fields);
	}
	*/

	// for a while, accept old .scl.csv without pipelinehash
	check(Fields.Num() == 11 || Fields.Num() == 12);

	int32 Index = 0;
	ClassNameAndObjectPath.ParseFromString(Fields[Index++]);

	// There is a high level of uniformity on the following FNames, use
	// the local name cache to accelerate lookup
	ShaderType = ParseFNameCached(Fields[Index++], NameCache);
	ShaderClass = ParseFNameCached(Fields[Index++], NameCache);
	MaterialDomain = ParseFNameCached(Fields[Index++], NameCache);
	FeatureLevel = ParseFNameCached(Fields[Index++], NameCache);

	QualityLevel = ParseFNameCached(Fields[Index++], NameCache);
	TargetFrequency = ParseFNameCached(Fields[Index++], NameCache);
	TargetPlatform = ParseFNameCached(Fields[Index++], NameCache);

	VFType = ParseFNameCached(Fields[Index++], NameCache);
	PermutationId = ParseFNameCached(Fields[Index++], NameCache);

	OutputHash.FromString(Fields[Index++]);

	check(Index == 11);

	if (Fields.Num() == 12)
	{
		PipelineHash.FromString(Fields[Index++]);
	}
	else
	{
		PipelineHash = FShaderHash();
	}

	ComputeKeyHash();
}

FString FStableShaderKeyAndValue::ToString() const
{
	FString Result;
	ToString(Result);
	return Result;
}

void FStableShaderKeyAndValue::ToString(FString& OutResult) const
{
	TStringBuilder<384> Out;
	const TCHAR Delim = TEXT(',');

	const int32 ClassNameAndObjectPathOffset = Out.Len();
	ClassNameAndObjectPath.AppendString(Out);
	Algo::Replace(MakeArrayView(Out).Slice(ClassNameAndObjectPathOffset, Out.Len() - ClassNameAndObjectPathOffset), Delim, TEXT(' '));
	Out << Delim;

	AppendSanitizedFName(Out, ShaderType, Delim);
	Out << Delim;
	AppendSanitizedFName(Out, ShaderClass, Delim);
	Out << Delim;

	Out << MaterialDomain << Delim;
	Out << FeatureLevel << Delim;
	Out << QualityLevel << Delim;
	Out << TargetFrequency << Delim;
	Out << TargetPlatform << Delim;
	Out << VFType << Delim;
	Out << PermutationId << Delim;

	Out << OutputHash << Delim;
	Out << PipelineHash;

	OutResult = FStringView(Out);
}

void FStableShaderKeyAndValue::AppendString(FAnsiStringBuilderBase& Out) const
{
	const ANSICHAR Delim = ',';

	const int32 ClassNameAndObjectPathOffset = Out.Len();
	ClassNameAndObjectPath.AppendString(Out);
	Algo::Replace(MakeArrayView(Out).Slice(ClassNameAndObjectPathOffset, Out.Len() - ClassNameAndObjectPathOffset), Delim, ' ');
	Out << Delim;

	AppendSanitizedFNameAsUTF8(Out, ShaderType, Delim);
	Out << Delim;
	AppendSanitizedFNameAsUTF8(Out, ShaderClass, Delim);
	Out << Delim;

	AppendFNameAsUTF8(Out, MaterialDomain);
	Out << Delim;
	AppendFNameAsUTF8(Out, FeatureLevel);
	Out << Delim;
	AppendFNameAsUTF8(Out, QualityLevel);
	Out << Delim;
	AppendFNameAsUTF8(Out, TargetFrequency);
	Out << Delim;
	AppendFNameAsUTF8(Out, TargetPlatform);
	Out << Delim;
	AppendFNameAsUTF8(Out, VFType);
	Out << Delim;
	AppendFNameAsUTF8(Out, PermutationId);
	Out << Delim;

	Out << OutputHash;
	Out << Delim;
	Out << PipelineHash;
}

FString FStableShaderKeyAndValue::HeaderLine()
{
	FString Result;

	const FString Delim(",");

	Result += TEXT("ClassNameAndObjectPath");
	Result += Delim;

	Result += TEXT("ShaderType");
	Result += Delim;
	Result += TEXT("ShaderClass");
	Result += Delim;
	Result += TEXT("MaterialDomain");
	Result += Delim;
	Result += TEXT("FeatureLevel");
	Result += Delim;

	Result += TEXT("QualityLevel");
	Result += Delim;
	Result += TEXT("TargetFrequency");
	Result += Delim;
	Result += TEXT("TargetPlatform");
	Result += Delim;

	Result += TEXT("VFType");
	Result += Delim;
	Result += TEXT("Permutation");
	Result += Delim;

	Result += TEXT("OutputHash");
	Result += Delim;

	Result += TEXT("PipelineHash");

	return Result;
}

void FStableShaderKeyAndValue::SetPipelineHash(const FShaderPipeline* Pipeline)
{
	if (LIKELY(Pipeline))
	{
		// cache this?
		FShaderCodeLibraryPipeline LibraryPipeline;
		LibraryPipeline.Initialize(Pipeline);
		LibraryPipeline.GetPipelineHash(PipelineHash); 
	}
	else
	{
		PipelineHash = FShaderHash();
	}
}

#if WITH_EDITOR
void WriteToCompactBinary(FCbWriter& Writer, const FStableShaderKeyAndValue& Key,
	const TMap<FShaderHash, int32>& HashToIndex)
{
	Writer.BeginArray();
	{
		TStringBuilder<128> ClassNameAndObjectPathStr;
		Key.ClassNameAndObjectPath.AppendString(ClassNameAndObjectPathStr);
		Writer << ClassNameAndObjectPathStr;
	}
	Writer << Key.ShaderType;
	Writer << Key.ShaderClass;
	Writer << Key.MaterialDomain;
	Writer << Key.FeatureLevel;
	Writer << Key.QualityLevel;
	Writer << Key.TargetFrequency;
	Writer << Key.TargetPlatform;
	Writer << Key.VFType;
	Writer << Key.PermutationId;
	Writer << HashToIndex[Key.OutputHash];
	Writer << HashToIndex[Key.PipelineHash];
	Writer.EndArray();
}

bool LoadFromCompactBinary(FCbFieldView Field, FStableShaderKeyAndValue& Key,
	const TArray<FShaderHash>& IndexToHash)
{
	int32 NumFields = Field.AsArrayView().Num();
	if (NumFields != 12)
	{
		Key = FStableShaderKeyAndValue();
		return false;
	}
	FCbFieldViewIterator It = Field.CreateViewIterator();
	bool bOk = true;

	{
		TStringBuilder<128> ClassNameAndObjectPathStr;
		bOk = LoadFromCompactBinary(*It++, ClassNameAndObjectPathStr) && bOk;
		Key.ClassNameAndObjectPath.ParseFromString(ClassNameAndObjectPathStr.ToView());
	}

	auto LoadIndexedHashFromCompactBinary = [&IndexToHash](FCbFieldView HashField, FShaderHash& OutHash)
	{
		int32 Index;
		if (!LoadFromCompactBinary(HashField, Index))
		{
			OutHash = FShaderHash();
			return false;
		}
		if (!IndexToHash.IsValidIndex(Index))
		{
			OutHash = FShaderHash();
			return false;
		}
		OutHash = IndexToHash.GetData()[Index];
		return true;
	};

	bOk = LoadFromCompactBinary(*It++, Key.ShaderType) && bOk;
	bOk = LoadFromCompactBinary(*It++, Key.ShaderClass) && bOk;
	bOk = LoadFromCompactBinary(*It++, Key.MaterialDomain) && bOk;
	bOk = LoadFromCompactBinary(*It++, Key.FeatureLevel) && bOk;
	bOk = LoadFromCompactBinary(*It++, Key.QualityLevel) && bOk;
	bOk = LoadFromCompactBinary(*It++, Key.TargetFrequency) && bOk;
	bOk = LoadFromCompactBinary(*It++, Key.TargetPlatform) && bOk;
	bOk = LoadFromCompactBinary(*It++, Key.VFType) && bOk;
	bOk = LoadFromCompactBinary(*It++, Key.PermutationId) && bOk;
	bOk = LoadIndexedHashFromCompactBinary(*It++, Key.OutputHash) && bOk;
	bOk = LoadIndexedHashFromCompactBinary(*It++, Key.PipelineHash) && bOk;

	Key.ComputeKeyHash();
	return bOk;
}
#endif


void FShaderCodeLibraryPipeline::Initialize(const FShaderPipeline* Pipeline)
{
	check(Pipeline != nullptr);
	for (uint32 Frequency = 0u; Frequency < SF_NumGraphicsFrequencies; ++Frequency)
	{
		if (Pipeline->Shaders[Frequency].IsValid())
		{
			Shaders[Frequency] = Pipeline->Shaders[Frequency]->GetOutputHash();
		}
	}
}

void FShaderCodeLibraryPipeline::GetPipelineHash(FShaderHash& Output)
{
	uint64 Combined = 0;
	for (int32 Frequency = 0; Frequency < SF_NumGraphicsFrequencies; Frequency++)
	{
		Combined += Shaders[Frequency].Hash;
	}
	Output.Hash = Combined;
}

class FShaderLibrariesCollection
{
	/** At runtime, this is set to the valid shader platform in use. At cook time, this value is SP_NumPlatforms. */
	EShaderPlatform ShaderPlatform;

	/** At runtime, shader code collection for current shader platform */
	TMap<FString, TUniquePtr<UE::ShaderLibrary::Private::FNamedShaderLibrary>> NamedLibrariesStack;

	/** Mutex that guards the access to the above stack. */
	FRWLock NamedLibrariesMutex;

#if UE_SHADERLIB_WITH_INTROSPECTION
	IConsoleObject* DumpLibraryContentsCmd;
#endif

#if WITH_EDITOR
	FCriticalSection ShaderCodeCS;
	// At cook time, shader code collection for each shader platform
	FEditorShaderCodeArchive* EditorShaderCodeArchive[EShaderPlatform::SP_NumPlatforms];
	// At cook time, whether we saved the shader code archive via SaveShaderLibraryChunk, so we can avoid saving it again in the end.
	// [RCL] FIXME 2020-11-25: this tracking is not perfect as the code in the asset registry performs chunking by ITargetPlatform, whereas if two platforms
	// share the same shader format (e.g. Vulkan on Linux and Windows), we cannot make such a distinction. However, as of now it is a very hypothetical case 
	// as the project settings don't allow disabling chunking for a particular platform.
	TSet<int32> ChunksSaved[EShaderPlatform::SP_NumPlatforms];
	// At cook time, shader code collection for each shader platform
	FEditorShaderStableInfo* EditorShaderStableInfo[EShaderPlatform::SP_NumPlatforms];
	// Cached bit field for shader formats that require stable keys
	TBitArray<> ShaderFormatsThatNeedStableKeys;
	// At cook time, shader stats for each shader platform
	FShaderCodeStats EditorShaderCodeStats[EShaderPlatform::SP_NumPlatforms];
	// At cook time, whether the shader archive supports pipelines (only OpenGL should)
	bool EditorArchivePipelines[EShaderPlatform::SP_NumPlatforms];
	bool bIsEditorShaderCodeArchiveEmpty = true;
	bool bIsEditorShaderStableInfoEmpty = true;
#endif //WITH_EDITOR
	bool bSupportsPipelines;
	bool bNativeFormat;

	/** This function only exists because I'm not able yet to untangle editor and non-editor usage (or rather cooking and not cooking). */
	inline bool IsLibraryInitializedForRuntime() const
	{
#if WITH_EDITOR
		return ShaderPlatform != SP_NumPlatforms;
#else
		// to make it a faster check, for games assume this function is no-op
		checkf(ShaderPlatform != SP_NumPlatforms, TEXT("Shader library has not been properly initialized for a cooked game"));
		return true;
#endif
	}

private:
	// Prints a comma-separated list of all IoChunkIds that the specified shadermap will be resolved to.
	FString PrintResolvedShaderMapIoChunks(const FShaderHash& ShaderMapHash)
	{
		TSet<FIoChunkId> ShaderMapIoChunks;

		ForEachShaderLibraryWithShaderMap(ShaderMapHash, [&ShaderMapIoChunks](FShaderLibraryInstance* LibraryInstance, int32 ShaderMapIndex) {
			LibraryInstance->ResolvePackageShaderMap(ShaderMapIndex, [&ShaderMapIoChunks](const FIoChunkId& IoChunk) {
				ShaderMapIoChunks.Add(IoChunk);
			});
		});

		// Log all IoChunks to verbose log
		TStringBuilder<1024> IoChunksString;
		if (ShaderMapIoChunks.IsEmpty())
		{
			IoChunksString.Append(TEXT("None"));
		}
		else
		{
			for (const FIoChunkId& IoChunk : ShaderMapIoChunks)
			{
				if (IoChunksString.Len() > 0)
				{
					IoChunksString.Append(TEXT(", "));
				}
				IoChunksString.Append(LexToString(IoChunk));
			}
		}

		return FString(IoChunksString);
	}

public:
	static FShaderLibrariesCollection* Impl;

	FShaderLibrariesCollection(EShaderPlatform InShaderPlatform, bool bInNativeFormat)
		: ShaderPlatform(InShaderPlatform)
#if UE_SHADERLIB_WITH_INTROSPECTION
		, DumpLibraryContentsCmd(nullptr)
#endif
		, bSupportsPipelines(false)
		, bNativeFormat(bInNativeFormat)
	{
#if WITH_EDITOR
		ShaderFormatsThatNeedStableKeys.Init(false, EShaderPlatform::SP_NumPlatforms);
		FMemory::Memzero(EditorShaderCodeArchive);
		FMemory::Memzero(EditorShaderStableInfo);
		FMemory::Memzero(EditorShaderCodeStats);
		FMemory::Memzero(EditorArchivePipelines);
		for (int32 Idx = 0; Idx < UE_ARRAY_COUNT(ChunksSaved); ++Idx)
		{
			ChunksSaved[Idx].Empty();
		}
#endif

#if UE_SHADERLIB_WITH_INTROSPECTION
		DumpLibraryContentsCmd = IConsoleManager::Get().RegisterConsoleCommand(
			TEXT("r.ShaderLibrary.Dump"),
			TEXT("Dumps shader library map."),
			FConsoleCommandDelegate::CreateStatic(DumpLibraryContentsStatic),
			ECVF_Default
			);
#endif

		FCoreDelegates::ResolvePackageShaderMaps.BindRaw(this, &FShaderLibrariesCollection::ResolvePackageShaderMapsDelegate);
		FCoreDelegates::PreloadPackageShaderMaps.BindRaw(this, &FShaderLibrariesCollection::PreloadPackageShaderMapsDelegate);
		FCoreDelegates::ReleasePreloadedPackageShaderMaps.BindRaw(this, &FShaderLibrariesCollection::ReleasePreloadedPackageShaderMapsDelegate);
	}

	~FShaderLibrariesCollection()
	{
		FCoreDelegates::ResolvePackageShaderMaps.Unbind();
		FCoreDelegates::PreloadPackageShaderMaps.Unbind();
		FCoreDelegates::ReleasePreloadedPackageShaderMaps.Unbind();

#if WITH_EDITOR
		for (uint32 i = 0; i < EShaderPlatform::SP_NumPlatforms; i++)
		{
			if (EditorShaderCodeArchive[i])
			{
				delete EditorShaderCodeArchive[i];
			}
			if (EditorShaderStableInfo[i])
			{
				delete EditorShaderStableInfo[i];
			}
		}
		FMemory::Memzero(EditorShaderCodeArchive);
		FMemory::Memzero(EditorShaderStableInfo);
#endif

#if UE_SHADERLIB_WITH_INTROSPECTION
		if (DumpLibraryContentsCmd)
		{
			IConsoleManager::Get().UnregisterConsoleObject(DumpLibraryContentsCmd);
		}
#endif

		TArray<FString> LibraryNames;
		{
			FRWScopeLock ReadLock(NamedLibrariesMutex, SLT_ReadOnly);
			LibraryNames.Reserve(NamedLibrariesStack.Num());
			for (TTuple<FString, TUniquePtr<UE::ShaderLibrary::Private::FNamedShaderLibrary>>& NamedLibraryPair : NamedLibrariesStack)
			{
				LibraryNames.Add(NamedLibraryPair.Key);
			}
		}

		for (const FString& LibraryName : LibraryNames)
		{
			CloseLibrary(LibraryName);
		}
	}

	void GatherShaderMapResourceReferences(const FShaderMapResource_SharedCode* InResource, TArray<FString>& OutLibrariesWithRefToResource)
	{
		using namespace UE::ShaderLibrary::Private;

		FRWScopeLock ReadLock(NamedLibrariesMutex, SLT_ReadOnly);

		for (const TTuple<FString, TUniquePtr<UE::ShaderLibrary::Private::FNamedShaderLibrary>>& NamedLibraryPair : NamedLibrariesStack)
		{
			const FNamedShaderLibrary* Library = NamedLibraryPair.Value.Get();
			for (const FShaderLibraryInstancePriorityPair& LibraryInstance : Library->Components)
			{
				for (int32 LibraryResourceIndex = 0; LibraryResourceIndex < LibraryInstance.Value->GetNumResources(); ++LibraryResourceIndex)
				{
					if (LibraryInstance.Value->GetResource(LibraryResourceIndex).GetReference() == InResource)
					{
						OutLibrariesWithRefToResource.Add(NamedLibraryPair.Value->LogicalName);
					}
				}
			}
		}
	}

	bool OpenLibrary(FString const& Name, FString const& Directory, const bool bMonolithicOnly = false)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(OpenShaderLibrary);

		using namespace UE::ShaderLibrary::Private;

		bool bResult = false;

		TSet<int32> NewComponentIDs;
		if (IsLibraryInitializedForRuntime())
		{
			LLM_SCOPE(ELLMTag::Shaders);
			bool bAddNewNamedLibrary = false;
			FNamedShaderLibrary* Library = nullptr;
			{
				// scope of this lock should be as limited as possible - particularly, OpenShaderCode will start async work which is not a good idea to do while holding a lock
				FRWScopeLock ReadLock(NamedLibrariesMutex, SLT_ReadOnly);

				// create a named library if one didn't exist
				TUniquePtr<FNamedShaderLibrary>* LibraryPtr = NamedLibrariesStack.Find(Name);
				if (LIKELY(LibraryPtr))
				{
					Library = LibraryPtr->Get();
				}
				else
				{
					bAddNewNamedLibrary = true;
					Library = new FNamedShaderLibrary(Name, ShaderPlatform, Directory);
				}
			}
			check(Library);
			// note, that since we're out of NamedLibrariesMutex locks now, other threads may arrive at the same point and acquire the same Library pointer
			// (or create yet another new named library). In the latter case, the duplicate library will be deleted later, since we will re-check (under a lock)
			// the presence of the same name in NamedLibrariesStack.  In the former case (two threads sharing the same Library pointer), we rely on FNamedShaderLibrary::OpenShaderCode
			// implementation being thread-safe (which it is).

			// more info for better logging
			bool bOpenedAsMonolithic = false;

			// if we're able to open the library by name, it's not chunked
			if (Library->OpenShaderCode(Directory, Name, INDEX_NONE))
			{
				bResult = true;
				bOpenedAsMonolithic = true;
			}
			else if (!bMonolithicOnly) // attempt to open a chunked library
			{
				TSet<int32> PrevComponentSet = Library->GetPresentChunks();

				// Copy the known pak files under the lock, then release it before iterating.
				// OnPakFileMounted calls OpenShaderCode which performs disk I/O, so holding
				// KnownPakFilesAccessLock for the entire iteration blocks pak mount callbacks
				// (FShaderLibraryPakFileMountedCallback) that need the same lock.
				TSet<FMountedPakFileInfo> KnownPakFilesCopy;
				{
					FScopeLock KnownPakFilesLocker(&FMountedPakFileInfo::KnownPakFilesAccessLock);
					KnownPakFilesCopy = FMountedPakFileInfo::KnownPakFiles;
				}
				for (TSet<FMountedPakFileInfo>::TConstIterator Iter(KnownPakFilesCopy); Iter; ++Iter)
				{
					Library->OnPakFileMounted(*Iter, Directory);
				}

				NewComponentIDs = Library->GetPresentChunks().Difference(PrevComponentSet);
				bResult = !NewComponentIDs.IsEmpty();

#if UE_SHADERLIB_SUPPORT_CHUNK_DISCOVERY
				if (!bResult)
				{
					// Some deployment flows (e.g. Launch on) avoid pak files despite project packaging settings. 
					// In case we run under such circumstances, we need to discover the components ourselves
					if (!IsRunningWithPakFile() && ShouldLookForLooseCookedChunks())
					{
						UE_LOGF(LogShaderLibrary, Display, "Running without a pakfile/IoStore and did not find a monolithic library '%ls' - attempting disk search for its chunks", *Name);

						TArray<FString> UshaderbytecodeFiles;
						FString SearchMask = Directory / FString::Printf(TEXT("ShaderArchive-*%s*.ushaderbytecode"), *Name);
						ShaderFindFiles(UshaderbytecodeFiles, *SearchMask, true, false);
#if PLATFORM_APPLE
						// In case we are using bSharedMaterialNativeLibraries=True, attempt to load fat metallibs instead.
						if (UshaderbytecodeFiles.Num() == 0)
						{
							SearchMask = Directory / FString::Printf(TEXT("%s*.metallib"), *Name);
							ShaderFindFiles(UshaderbytecodeFiles, *SearchMask, true, false);
						}
#endif

						if (UshaderbytecodeFiles.Num() > 0)
						{
							UE_LOGF(LogShaderLibrary, Display, "   ....  found %d files", UshaderbytecodeFiles.Num());
							for (const FString& Filename : UshaderbytecodeFiles)
							{
								const TCHAR* ChunkSubstring = TEXT("_Chunk");
								const int kChunkSubstringSize = 6; // strlen(ChunkSubstring)
								int32 ChunkSuffix = Filename.Find(ChunkSubstring, ESearchCase::IgnoreCase, ESearchDir::FromEnd);
								if (ChunkSuffix != INDEX_NONE && ChunkSuffix + kChunkSubstringSize < Filename.Len())
								{
									const TCHAR* ChunkIDString = &Filename[ChunkSuffix + kChunkSubstringSize];
									int32 ChunkID = FCString::Atoi(ChunkIDString);
									if (ChunkID >= 0)
									{
										// create a fake FPakFileMountedInfo
										FMountedPakFileInfo PakFileInfo(Directory, ChunkID);
										Library->OnPakFileMounted(PakFileInfo, Directory);
									}
								}
							}

							NewComponentIDs = Library->GetPresentChunks().Difference(PrevComponentSet);
							bResult = !NewComponentIDs.IsEmpty();
						}
						else
						{
							UE_LOGF(LogShaderLibrary, Display, "   ....  not found");
						}
					}
				}
#endif // UE_SHADERLIB_SUPPORT_CHUNK_DISCOVERY
			}

			if (bResult)
			{
				if (bAddNewNamedLibrary)
				{
					// re-check that the library indeed was added right now - we can have multiple threads race to create it as described above
					FRWScopeLock WriteLock(NamedLibrariesMutex, SLT_Write);
					TUniquePtr<FNamedShaderLibrary>* LibraryPtr = NamedLibrariesStack.Find(Name);
					if (LibraryPtr == nullptr)
					{
						if (bOpenedAsMonolithic)
						{
							UE_LOGF(LogShaderLibrary, Display, "Logical shader library '%ls' has been created as a monolithic library, components %d", *Name, Library->GetNumComponents());
						}
						else
						{
							UE_LOGF(LogShaderLibrary, Display, "Logical shader library '%ls' has been created, components %d", *Name, Library->GetNumComponents());
						}
						NamedLibrariesStack.Emplace(Name, Library);
					}
					else 
					{
						// this is where concurrent work from thread(s) that lost the race to create the same library gets wasted.
						delete Library;
						Library = nullptr;
					}
				}
				else
				{
					UE_LOGF(LogShaderLibrary, Display, "Discovered new %d components for logical shader library '%ls' (total number of components is now %d)", NewComponentIDs.Num(), *Name, Library->GetNumComponents());
				}

				// Inform the pipeline cache that the state of loaded libraries has changed (unless we had to delete the duplicate)
				if (Library != nullptr)
				{
					FShaderPipelineCache::ShaderLibraryStateChanged(FShaderPipelineCache::Opened, ShaderPlatform, Name, INDEX_NONE);
					for (int32 ComponentID : NewComponentIDs)
					{
						FShaderPipelineCache::ShaderLibraryStateChanged(FShaderPipelineCache::OpenedComponent, ShaderPlatform, Name, ComponentID);
					}
				}
			}
			else
			{
				if (bAddNewNamedLibrary)
				{
					UE_LOGF(LogShaderLibrary, Verbose, "Tried to open shader library '%ls', but could not find it%ls", *Name,
						bMonolithicOnly ? TEXT(" (only tried to open it as a monolithic library).") : TEXT(" neither as a monolithic library nor as a chunked one."));

					check(Library->GetNumComponents() == 0);
					delete Library;
					Library = nullptr;
				}
				else
				{
					UE_LOGF(LogShaderLibrary, Display, "Tried to open again shader library '%ls', but could not find new components for it (existing components: %d).", *Name, Library->GetNumComponents());
				}
			}
		}

#if WITH_EDITOR
		if (!bIsEditorShaderCodeArchiveEmpty)
		{
			for (uint32 i = 0; i < EShaderPlatform::SP_NumPlatforms; i++)
			{
				FEditorShaderCodeArchive* CodeArchive = EditorShaderCodeArchive[i];
				if (CodeArchive)
				{
					CodeArchive->OpenLibrary(Name);
				}
			}
		}
		if (!bIsEditorShaderStableInfoEmpty)
		{
			for (uint32 i = 0; i < EShaderPlatform::SP_NumPlatforms; i++)
			{
				FEditorShaderStableInfo* StableArchive = EditorShaderStableInfo[i];
				if (StableArchive)
				{
					StableArchive->OpenLibrary(Name);
				}
			}
		}
#endif
		
		return bResult;
	}

	void CloseLibrary(FString const& Name)
	{
		if (IsLibraryInitializedForRuntime())
		{
			TUniquePtr<UE::ShaderLibrary::Private::FNamedShaderLibrary> RemovedLibrary = nullptr;

			{
				FRWScopeLock WriteLock(NamedLibrariesMutex, SLT_Write);
				NamedLibrariesStack.RemoveAndCopyValue(Name, RemovedLibrary);
			}

			if (RemovedLibrary)
			{
				UE_LOGF(LogShaderLibrary, Display, "Closing logical shader library '%ls' with %d components", *Name, RemovedLibrary->GetNumComponents());
				RemovedLibrary = nullptr;
			}
		}

		// Inform the pipeline cache that the state of loaded libraries has changed
		FShaderPipelineCache::ShaderLibraryStateChanged(FShaderPipelineCache::Closed, ShaderPlatform, Name, -1);

#if WITH_EDITOR
		for (uint32 i = 0; i < EShaderPlatform::SP_NumPlatforms; i++)
		{
			if (EditorShaderCodeArchive[i])
			{
				EditorShaderCodeArchive[i]->CloseLibrary(Name);
			}
			if (EditorShaderStableInfo[i])
			{
				EditorShaderStableInfo[i]->CloseLibrary(Name);
			}
			ChunksSaved[i].Empty();
		}
#endif
	}

	void PakFileStateChangeInternal(FShaderPipelineCache::ELibraryState InStateChange, TFunctionRef<void(UE::ShaderLibrary::Private::FNamedShaderLibrary* InNamedShaderLibrary)> InStateChangeCallback)
	{
		if (IsLibraryInitializedForRuntime())
		{
			LLM_SCOPE(ELLMTag::Shaders);

			// Collect library pointers under the lock, then release it before invoking the callback.
			// The callback may call OpenShaderCode which performs disk I/O, so holding NamedLibrariesMutex
			// for the entire iteration can starve writers (e.g. OpenLibrary on the GameThread) for the
			// duration of all that I/O, causing hang detection to fire.
			TArray<UE::ShaderLibrary::Private::FNamedShaderLibrary*> Libraries;
			{
				FRWScopeLock ReadLock(NamedLibrariesMutex, SLT_ReadOnly);
				Libraries.Reserve(NamedLibrariesStack.Num());
				for (TTuple<FString, TUniquePtr<UE::ShaderLibrary::Private::FNamedShaderLibrary>>& NamedLibraryPair : NamedLibrariesStack)
				{
					Libraries.Add(NamedLibraryPair.Value.Get());
				}
			}

			TArray<TUniqueFunction<void()>> ShaderLibraryStateChanges;
			for (UE::ShaderLibrary::Private::FNamedShaderLibrary* NamedShaderLibrary : Libraries)
			{
				const TSet<int32> PrevComponentSet = NamedShaderLibrary->GetPresentChunks();
				InStateChangeCallback(NamedShaderLibrary);
				const TSet<int32> NewComponentSet = NamedShaderLibrary->GetPresentChunks();
				const TSet<int32> ChangedComponentSet = NewComponentSet.Num() > PrevComponentSet.Num()
					? NewComponentSet.Difference(PrevComponentSet)
					: PrevComponentSet.Difference(NewComponentSet);
				for (int32 ComponentID : ChangedComponentSet)
				{
					ShaderLibraryStateChanges.Add([StateChange = InStateChange, ShaderPlatform = NamedShaderLibrary->ShaderPlatform, LogicalName = NamedShaderLibrary->LogicalName, ComponentID]() {
						FShaderPipelineCache::ShaderLibraryStateChanged(StateChange, ShaderPlatform, LogicalName, ComponentID);
					});
				}
			}
			for (TUniqueFunction<void()>& OnShaderLibraryStateChange : ShaderLibraryStateChanges)
			{
				OnShaderLibraryStateChange();
			}
		}
	}

	void OnPakFileMounted(const UE::ShaderLibrary::Private::FMountedPakFileInfo& MountInfo)
	{
		PakFileStateChangeInternal(FShaderPipelineCache::OpenedComponent, [&MountInfo](UE::ShaderLibrary::Private::FNamedShaderLibrary* InNamedShaderLibrary) {
			InNamedShaderLibrary->OnPakFileMounted(MountInfo, InNamedShaderLibrary->BaseDirectory);
		});
	}

	void OnPakFileUnmounting(const UE::ShaderLibrary::Private::FMountedPakFileInfo& MountInfo)
	{
		PakFileStateChangeInternal(FShaderPipelineCache::Closed, [&MountInfo](UE::ShaderLibrary::Private::FNamedShaderLibrary* InNamedShaderLibrary) {
			InNamedShaderLibrary->OnPakFileUnmounting(MountInfo, InNamedShaderLibrary->BaseDirectory);
		});
	}

	uint32 GetShaderCount(void)
	{
		FRWScopeLock ReadLock(NamedLibrariesMutex, SLT_ReadOnly);

		int32 ShaderCount = 0;
		for (TTuple<FString, TUniquePtr<UE::ShaderLibrary::Private::FNamedShaderLibrary>>& NamedLibraryPair : NamedLibrariesStack)
		{
			ShaderCount += NamedLibraryPair.Value->GetShaderCount();
		}
		return ShaderCount;
	}

#if UE_SHADERLIB_WITH_INTROSPECTION
	static void DumpLibraryContentsStatic()
	{
		if (FShaderLibrariesCollection::Impl)
		{
			FShaderLibrariesCollection::Impl->DumpLibraryContents();
		}
	}

	void DumpLibraryContents()
	{
		FRWScopeLock ReadLock(NamedLibrariesMutex, SLT_ReadOnly);

		UE_LOGF(LogShaderLibrary, Display, "==== Dumping shader library contents ====");
		UE_LOGF(LogShaderLibrary, Display, "Shader platform (EShaderPlatform) is %d", static_cast<int32>(ShaderPlatform));
		UE_LOGF(LogShaderLibrary, Display, "%d named libraries open with %d shaders total", NamedLibrariesStack.Num(), GetShaderCount());
		int32 LibraryIdx = 0;
		for (TTuple<FString, TUniquePtr<UE::ShaderLibrary::Private::FNamedShaderLibrary>>& NamedLibraryPair : NamedLibrariesStack)
		{
			UE_LOGF(LogShaderLibrary, Display, "%d: Name='%ls' Shaders %d Components %d", 
				LibraryIdx, *NamedLibraryPair.Key, NamedLibraryPair.Value->GetShaderCount(), NamedLibraryPair.Value->GetNumComponents());

			NamedLibraryPair.Value->DumpLibraryContents(TEXT("  "));

			++LibraryIdx;
		}
		UE_LOGF(LogShaderLibrary, Display, "==== End of shader library dump ====");
	}
#endif

	EShaderPlatform GetRuntimeShaderPlatform(void)
	{
		return ShaderPlatform;
	}

	void ForEachShaderLibraryWithShaderMap(const FShaderHash& Hash, TFunctionRef<void(FShaderLibraryInstance* InLibraryInstance, int32 InShaderMapIndex)> InPerShaderMapCallback)
	{
		FRWScopeLock ReadLock(NamedLibrariesMutex, SLT_ReadOnly);
		for (TTuple<FString, TUniquePtr<UE::ShaderLibrary::Private::FNamedShaderLibrary>>& NamedLibraryPair : NamedLibrariesStack)
		{
			NamedLibraryPair.Value->ForEachShaderLibraryWithShaderMap(Hash, InPerShaderMapCallback);
		}
	}

	FShaderLibraryInstance* FindShaderLibraryForShaderMap(const FShaderHash& Hash, int32& OutShaderMapIndex, const FShaderLibraryInstance* SkipLibraryInstance = nullptr)
	{
		FRWScopeLock ReadLock(NamedLibrariesMutex, SLT_ReadOnly);

		for (TTuple<FString, TUniquePtr<UE::ShaderLibrary::Private::FNamedShaderLibrary>>& NamedLibraryPair : NamedLibrariesStack)
		{
			FShaderLibraryInstance* Instance = NamedLibraryPair.Value->FindShaderLibraryForShaderMap(Hash, OutShaderMapIndex, SkipLibraryInstance);
			if (Instance)
			{
				return Instance;
			}
		}
		return nullptr;
	}

	FShaderLibraryInstance* FindShaderLibraryForShader(const FShaderHash& Hash, int32& OutShaderIndex)
	{
		FRWScopeLock ReadLock(NamedLibrariesMutex, SLT_ReadOnly);

		for (TTuple<FString, TUniquePtr<UE::ShaderLibrary::Private::FNamedShaderLibrary>>& NamedLibraryPair : NamedLibrariesStack)
		{
			FShaderLibraryInstance* Instance = NamedLibraryPair.Value->FindShaderLibraryForShader(Hash, OutShaderIndex);
			if (Instance)
			{
				return Instance;
			}
		}
		return nullptr;
	}

	TRefCountPtr<FShaderMapResource_SharedCode> LoadResource(const FShaderHash& Hash, FArchive* Ar)
	{
		int32 ShaderMapIndex = INDEX_NONE;
		FShaderLibraryInstance* LibraryInstance = FindShaderLibraryForShaderMap(Hash, ShaderMapIndex);
		if (LibraryInstance)
		{
			SCOPED_LOADTIMER(LoadShaderResource_Internal);

			TRefCountPtr<FShaderMapResource_SharedCode> Resource = LibraryInstance->GetResource(ShaderMapIndex);
			if (!Resource)
			{
				SCOPED_LOADTIMER(LoadShaderResource_AddResource);
				Resource = LibraryInstance->AddResource(ShaderMapIndex, Ar);
			}

			return Resource;
		}

		return TRefCountPtr<FShaderMapResource_SharedCode>();
	}

	TRefCountPtr<FRHIShader> CreateShader(EShaderFrequency Frequency, const FShaderHash& Hash)
	{
		int32 ShaderIndex = INDEX_NONE;
		FShaderLibraryInstance* LibraryInstance = FindShaderLibraryForShader(Hash, ShaderIndex);
		if (LibraryInstance)
		{
			TRefCountPtr<FRHIShader> Shader = LibraryInstance->GetOrCreateShader(ShaderIndex);
			check(Shader->GetFrequency() == Frequency);
			return Shader;
		}
		return TRefCountPtr<FRHIShader>();
	}

	bool PreloadShader(const FShaderHash& Hash, FArchive* Ar)
	{
		int32 ShaderIndex = INDEX_NONE;
		FShaderLibraryInstance* LibraryInstance = FindShaderLibraryForShader(Hash, ShaderIndex);
		if (LibraryInstance)
		{
			LibraryInstance->PreloadShader(ShaderIndex, Ar);
			return true;
		}
		return false;
	}

	bool ReleasePreloadedShader(const FShaderHash& Hash)
	{
		int32 ShaderIndex = INDEX_NONE;
		FShaderLibraryInstance* LibraryInstance = FindShaderLibraryForShader(Hash, ShaderIndex);
		if (LibraryInstance)
		{
			LibraryInstance->ReleasePreloadedShader(ShaderIndex);
			return true;
		}
		return false;
	}

	bool ContainsShaderCode(const FShaderHash& Hash)
	{
		int32 ShaderIndex = INDEX_NONE;
		FShaderLibraryInstance* LibraryInstance = FindShaderLibraryForShader(Hash, ShaderIndex);
		return LibraryInstance != nullptr;
	}
	
	bool ContainsShaderCode(const FShaderHash& Hash, const FString& LogicalLibraryName)
	{
		int32 ShaderIndex = INDEX_NONE;
		FShaderLibraryInstance* LibraryInstance = FindShaderLibraryForShader(Hash, ShaderIndex);
		return LibraryInstance != nullptr && LibraryInstance->Library->GetName() == LogicalLibraryName;
	}

#if WITH_EDITOR

	FString GetFormatAndPlatformName(const FName& Format)
	{
		EShaderPlatform Platform = ShaderFormatToLegacyShaderPlatform(Format);
		FName PossiblyAdjustedFormat = LegacyShaderPlatformToShaderFormat(Platform);	// Vulkan and GL switch between name variants depending on CVars 

		return PossiblyAdjustedFormat.ToString() + TEXT("-") + FDataDrivenShaderPlatformInfo::GetName(Platform).ToString();
	}

	void CleanDirectories(TArray<FName> const& ShaderFormats)
	{
		const FString IntermediatePathBase = GetShaderCodeIntermediatePath();
		for (FName const& Format : ShaderFormats)
		{
			FString ShaderIntermediateLocation = IntermediatePathBase / Format.ToString();
			FString ShaderPlatformIntermediateLocation = IntermediatePathBase / GetFormatAndPlatformName(Format);
			IFileManager::Get().DeleteDirectory(*ShaderIntermediateLocation, false, true);
			IFileManager::Get().DeleteDirectory(*ShaderPlatformIntermediateLocation, false, true);
		}
	}

	void CookShaderFormats(TArray<FShaderLibraryCooker::FShaderFormatDescriptor> const& ShaderFormats)
	{
		for (const FShaderLibraryCooker::FShaderFormatDescriptor& Descriptor : ShaderFormats)
		{
			FName const& Format = Descriptor.ShaderFormat;

			EShaderPlatform Platform = ShaderFormatToLegacyShaderPlatform(Format);
			FString FormatAndPlatformName = GetFormatAndPlatformName(Format);
			FEditorShaderCodeArchive* CodeArchive = EditorShaderCodeArchive[Platform];
			if (!CodeArchive)
			{
				bIsEditorShaderCodeArchiveEmpty = false;
				CodeArchive = new FEditorShaderCodeArchive(FName(FormatAndPlatformName));
				EditorShaderCodeArchive[Platform] = CodeArchive;
				EditorArchivePipelines[Platform] = !bNativeFormat;
			}
			check(CodeArchive);
		}
		for (const FShaderLibraryCooker::FShaderFormatDescriptor& Descriptor : ShaderFormats)
		{
			FName const& Format = Descriptor.ShaderFormat;
			bool bUseStableKeys = Descriptor.bNeedsStableKeys;

			EShaderPlatform Platform = ShaderFormatToLegacyShaderPlatform(Format);
			FName PossiblyAdjustedFormat = LegacyShaderPlatformToShaderFormat(Platform);	// Vulkan and GL switch between name variants depending on CVars 
			FEditorShaderStableInfo* StableArchive = EditorShaderStableInfo[Platform];
			if (!StableArchive && bUseStableKeys)
			{
				bIsEditorShaderStableInfoEmpty = false;
				StableArchive = new FEditorShaderStableInfo(PossiblyAdjustedFormat);
				EditorShaderStableInfo[Platform] = StableArchive;
				ShaderFormatsThatNeedStableKeys[(int)Platform] = true;
			}
		}
	}

	bool NeedsShaderStableKeys(EShaderPlatform Platform) 
	{
		if (Platform == EShaderPlatform::SP_NumPlatforms)
		{
			return ShaderFormatsThatNeedStableKeys.Find(true) != INDEX_NONE;
		}
		return (ShaderFormatsThatNeedStableKeys[(int)Platform]) != 0;
	}

	void AddShaderCode(EShaderPlatform Platform, const FShaderMapResourceCode* Code, const FShaderMapAssetPaths& AssociatedAssets)
	{
		FScopeLock ScopeLock(&ShaderCodeCS);
		checkf(Platform < UE_ARRAY_COUNT(EditorShaderCodeStats), TEXT("FShaderCodeLibrary::AddShaderCode can only be called with a valid shader platform (expected no more than %d, passed: %d)"), 
			static_cast<int32>(UE_ARRAY_COUNT(EditorShaderCodeStats)), static_cast<int32>(Platform));
		static_assert(UE_ARRAY_COUNT(EditorShaderCodeStats) == UE_ARRAY_COUNT(EditorShaderCodeArchive), "Size of EditorShaderCodeStats must match size of EditorShaderCodeArchive");

		FShaderCodeStats& CodeStats = EditorShaderCodeStats[Platform];
		FEditorShaderCodeArchive* CodeArchive = EditorShaderCodeArchive[Platform];
		checkf(CodeArchive, TEXT("EditorShaderCodeArchive for (EShaderPlatform)%d is null!"), (int32)Platform);

		CodeArchive->AddShaderCode(Code, AssociatedAssets, ShaderCodeArchive::ECookShaderLibrarySource::CurrentCook, &CodeStats);
	}

	void CopyToCompactBinaryAndClear(FCbWriter& Writer, bool& bOutHasData, bool& bOutRanOutOfRoom, int64 MaxShaderSize)
	{
		bOutRanOutOfRoom = false;

		TArray<EShaderPlatform, TInlineAllocator<10>> PlatformsToCopy;
		FScopeLock ScopeLock(&ShaderCodeCS);
		for (EShaderPlatform Platform = (EShaderPlatform)0; Platform < (EShaderPlatform)UE_ARRAY_COUNT(EditorShaderCodeStats);
			Platform = (EShaderPlatform)((int)Platform + 1))
		{
			FEditorShaderCodeArchive* CodeArchive = EditorShaderCodeArchive[Platform];
			FEditorShaderStableInfo* StableInfo = EditorShaderStableInfo[Platform];
			if ((CodeArchive && CodeArchive->HasDataToCopy()) || (StableInfo && StableInfo->HasDataToCopy()))
			{
				PlatformsToCopy.Add(Platform);
			}
		}

		if (PlatformsToCopy.IsEmpty())
		{
			bOutHasData = false;
			return;
		}
		bOutHasData = true;
		int64 RemainingSize = MaxShaderSize;
		Writer.BeginArray();
		for (EShaderPlatform Platform : PlatformsToCopy)
		{
			if (bOutRanOutOfRoom)
			{
				break;
			}
			Writer.BeginObject();
			{
				Writer << "Platform" << (uint32)Platform;
				FEditorShaderCodeArchive* CodeArchive = EditorShaderCodeArchive[Platform];
				FEditorShaderStableInfo* StableInfo = EditorShaderStableInfo[Platform];

				if ((CodeArchive && CodeArchive->HasDataToCopy()))
				{
					FSerializedShaderArchive TransferArchive;
					TArray<uint8> TransferCode;
					int64 MaxShaderSizeThisCall = RemainingSize;
					int64 MaxShaderCount = -1;
					bool bRanOutOfRoom;
					CodeArchive->CopyToArchiveAndClear(TransferArchive, TransferCode, bRanOutOfRoom, MaxShaderSizeThisCall,
						MaxShaderCount, ShaderCodeArchive::ECookShaderLibrarySource::CurrentCook);
					bOutRanOutOfRoom |= bRanOutOfRoom;
					if (bRanOutOfRoom && TransferCode.IsEmpty() && RemainingSize == MaxShaderSize)
					{
						UE_LOG(LogShaderLibrary, Error,
							TEXT("MaxShaderSize %" INT64_FMT " is too small to read even a single shader. We will ignore it and allow uncapped size, which will possibly cause an overflow in the caller."),
							MaxShaderSize);
						MaxShaderSizeThisCall = -1;
						MaxShaderCount = 1;
						TransferArchive.Empty();
						CodeArchive->CopyToArchiveAndClear(TransferArchive, TransferCode, bRanOutOfRoom,
							MaxShaderSizeThisCall, MaxShaderCount, ShaderCodeArchive::ECookShaderLibrarySource::CurrentCook);
					}
					if (!TransferArchive.IsEmpty())
					{
						RemainingSize -= TransferCode.Num();
						Writer.SetName("EditorShaderCodeArchive");
						CodeArchive->CopyToCompactBinary(Writer, TransferArchive, TransferCode);
					}

				}
				if (!bOutRanOutOfRoom && StableInfo && StableInfo->HasDataToCopy())
				{
					Writer.SetName("EditorShaderStableInfo");
					StableInfo->CopyToCompactBinary(Writer);
				}

				// ChunksSaved is not copied; it is constructed at end of cook after copying from remote workers is complete
				// EditorShaderCodeStats is not copied; it is gathered from the shadercode we send when the director receives it
				// bShaderFormatsThatNeedStableKeys is not copied; it is constructed during BeginCook and is the same on all machines
				// EditorArchivePipelines is not copied; it is constructed during BeginCook and is the same on all machines

			}
			Writer.EndObject();
		}
		Writer.EndArray();
	}

	bool AppendFromCompactBinary(FCbFieldView Field)
	{
		LLM_SCOPE(ELLMTag::Shaders);
		bool bOk = true;
		for (FCbFieldView PlatformField : Field)
		{
			uint32 PlatformInt;
			if (!LoadFromCompactBinary(PlatformField["Platform"], PlatformInt) ||
				PlatformInt >= UE_ARRAY_COUNT(EditorShaderCodeStats))
			{
				bOk = false;
				continue;
			}
			EShaderPlatform Platform = (EShaderPlatform)PlatformInt;

			FCbFieldView CodeArchiveField = PlatformField["EditorShaderCodeArchive"];
			if (CodeArchiveField.HasValue())
			{
				FEditorShaderCodeArchive* CodeArchive = EditorShaderCodeArchive[Platform];
				if (CodeArchive)
				{
					FShaderCodeStats& CodeStats = EditorShaderCodeStats[Platform];
					bOk = CodeArchive->AppendFromCompactBinary(CodeArchiveField,
						ShaderCodeArchive::ECookShaderLibrarySource::CurrentCook, &CodeStats) && bOk;
				}
				else
				{
					UE_LOGF(LogShaderLibrary, Error,
						"ShaderMapLibrary transfer received from a remote machine includes data for Platform %d, but the ShaderMapLibrary has not been initialized for that platform in the local process. "
						"The information will be ignored.", (int32)Platform);
					bOk = false;
				}
			}

			FCbFieldView StableArchiveField = PlatformField["EditorShaderStableInfo"];
			if (StableArchiveField.HasValue())
			{
				FEditorShaderStableInfo* StableArchive = EditorShaderStableInfo[Platform];
				if (StableArchive)
				{
					bOk = StableArchive->AppendFromCompactBinary(StableArchiveField) && bOk;
				}
			}
		}
		return bOk;
	}

	void AddShaderStableKeyValue(EShaderPlatform InShaderPlatform, FStableShaderKeyAndValue& StableKeyValue)
	{
		FEditorShaderStableInfo* StableArchive = EditorShaderStableInfo[InShaderPlatform];
		if (!StableArchive)
		{
			return;
		}

		FScopeLock ScopeLock(&ShaderCodeCS);

		StableKeyValue.ComputeKeyHash();
		StableArchive->AddShader(StableKeyValue, FEditorShaderStableInfo::EMergeRule::OverwriteUnmodifiedWarnModified);
	}
 
	void FinishPopulateShaderCode(const FString& ShaderCodeDir, const FString& MetaOutputDir, const TArray<FName>& ShaderFormats,
		bool bInitializeReferenceTracking)
	{
		for (FName ShaderFormatName : ShaderFormats)
		{
			EShaderPlatform SPlatform = ShaderFormatToLegacyShaderPlatform(ShaderFormatName);

			FShaderCodeStats& CodeStats = EditorShaderCodeStats[SPlatform];

			FEditorShaderCodeArchive* CodeArchive = EditorShaderCodeArchive[SPlatform];
			if (CodeArchive)
			{
				CodeArchive->FinishPopulate(ShaderCodeDir,
					ShaderCodeArchive::ECookShaderLibrarySource::PreviousIncremental, &CodeStats);
				if (bInitializeReferenceTracking)
				{
					CodeArchive->InitializeReferenceTracking();
				}
			}

			FEditorShaderStableInfo* StableArchive = EditorShaderStableInfo[SPlatform];
			if (StableArchive)
			{
				StableArchive->FinishPopulate(MetaOutputDir);
			}
		}
	}

	void UpdateOplogPackages(UE::Cook::Artifact::FUpdateOplogPackagesContext& Context, const TArray<FName>& ShaderFormats)
	{
		for (FName ShaderFormatName : ShaderFormats)
		{
			EShaderPlatform SPlatform = ShaderFormatToLegacyShaderPlatform(ShaderFormatName);

			FEditorShaderCodeArchive* CodeArchive = EditorShaderCodeArchive[SPlatform];
			if (CodeArchive)
			{
				CodeArchive->UpdateOplogPackages(Context);
			}
		}
	}

	void PruneShaderLibrary(const TArray<FName>& ShaderFormats)
	{
		for (FName ShaderFormatName : ShaderFormats)
		{
			EShaderPlatform SPlatform = ShaderFormatToLegacyShaderPlatform(ShaderFormatName);

			FEditorShaderCodeArchive* CodeArchive = EditorShaderCodeArchive[SPlatform];
			if (CodeArchive)
			{
				CodeArchive->Prune();
			}
		}
	}

	bool SaveShaderCode(const FString& ShaderCodeDir, const FString& MetaOutputDir, const TArray<FName>& ShaderFormats,
		TArray<FString>& OutSCLCSVPath, ESaveToDiskSortOrder SortOrder)
	{
		using namespace UE::ShaderLibrary::Private;
		bool bAllOk = ShaderFormats.Num() > 0;

		FScopeLock ScopeLock(&ShaderCodeCS);

		for (const FName ShaderFormatName : ShaderFormats)
		{
			EShaderPlatform SPlatform = ShaderFormatToLegacyShaderPlatform(ShaderFormatName);

			FEditorShaderCodeArchive* CodeArchive = EditorShaderCodeArchive[SPlatform];
			if (CodeArchive)
			{
				// If we saved the shader code while generating the chunk, do not save a single consolidated library as it should not be used and
				// will only bloat the build.
				// Still save the full asset info for debugging
				if (ChunksSaved[SPlatform].Num() == 0)
				{
					TUniquePtr<FEditorShaderCodeArchive> ReferencedAssetsArchive(CodeArchive->CreateArchiveFromAssetsReferencedByStaging());
					if (!ReferencedAssetsArchive->IsEmpty())
					{
						// always save shaders in our format even if the platform will use native one. This is needed for iterative cooks (Launch On et al)
						// to reload previously cooked shaders
						bool bOk = ReferencedAssetsArchive->SaveToDisk(ShaderCodeDir, MetaOutputDir,
							ESaveToDiskTarget::Staging, SortOrder);

						bool bShouldWriteInNativeFormat = bOk && bNativeFormat && ReferencedAssetsArchive->SupportsShaderArchives();
						if (bShouldWriteInNativeFormat)
						{
							bOk = ReferencedAssetsArchive->PackageNativeShaderLibrary(ShaderCodeDir) && bOk;
						}

						if (bOk)
						{
							ReferencedAssetsArchive->DumpStatsAndDebugInfo();
						}
						bAllOk = bAllOk && bOk;
					}
				}
				// save UnreferencedAssets for IncrementalCook
				TUniquePtr<FEditorShaderCodeArchive> UnreferencedAssetsArchive(CodeArchive->CreateArchiveFromAssetsOnlyReferencedByOplog());
				if (!UnreferencedAssetsArchive->IsEmpty())
				{
					bAllOk = UnreferencedAssetsArchive->SaveToDisk(ShaderCodeDir, MetaOutputDir,
						ESaveToDiskTarget::CookCache, ESaveToDiskSortOrder::ShaderHash) && bAllOk;
				}
			}
			FEditorShaderStableInfo* StableArchive = EditorShaderStableInfo[SPlatform];
			if (StableArchive)
			{
				// Stable shader info is not saved per-chunk (it is not needed at runtime), so save it always
				FString SCLCSVPath;
				bAllOk = StableArchive->SaveToDisk(MetaOutputDir, SCLCSVPath) && bAllOk;
				
				// Only add output files if they were actually written to disk (if there were no shaders in the library it is not a failure).
				if (!SCLCSVPath.IsEmpty())
				{
					OutSCLCSVPath.Add(SCLCSVPath);
				}
			}
		}

		return bAllOk;
	}

	bool SaveShaderCodeChunk(int32 ChunkId, const TSet<FName>& InPackagesInChunk, const TArray<FName>& ShaderFormats,
		const FString& SandboxDestinationPath, const FString& SandboxMetadataPath,
		ShaderCodeArchive::ESaveToDiskSortOrder SortOrder, TArray<FString>& OutChunkFilenames)
	{
		using namespace UE::ShaderLibrary::Private;

		bool bOk = ShaderFormats.Num() > 0;

		FScopeLock ScopeLock(&ShaderCodeCS);

		for (const FName ShaderFormatName : ShaderFormats)
		{
			EShaderPlatform SPlatform = ShaderFormatToLegacyShaderPlatform(ShaderFormatName);

			// we may get duplicate calls for the same Chunk Id because the cooker sometimes calls asset registry SaveManifests twice.
			if (ChunksSaved[SPlatform].Contains(ChunkId))
			{
				continue;
			}

			FEditorShaderCodeArchive* OrginalCodeArchive = EditorShaderCodeArchive[SPlatform];
			if (!OrginalCodeArchive)
			{
				bOk = false;
				break;
			}

			TUniquePtr<FEditorShaderCodeArchive> PerChunkArchive = OrginalCodeArchive->CreateChunk(ChunkId, InPackagesInChunk);
			if (!PerChunkArchive)
			{
				bOk = false;
				break;
			}

			// skip saving if no shaders are actually stored
			if (!PerChunkArchive->IsEmpty())
			{

				// always save shaders in our format even if the platform will use native one. This is needed for iterative cooks (Launch On et al)
				// to reload previously cooked shaders
				bOk = PerChunkArchive->SaveToDisk(SandboxDestinationPath, SandboxMetadataPath,
					ESaveToDiskTarget::Staging, SortOrder, &OutChunkFilenames) && bOk;

				bool bShouldWriteInNativeFormat = bOk && bNativeFormat && PerChunkArchive->SupportsShaderArchives();
				if (bShouldWriteInNativeFormat)
				{
					bOk = PerChunkArchive->PackageNativeShaderLibrary(SandboxDestinationPath, &OutChunkFilenames) && bOk;
				}

				if (bOk)
				{
					PerChunkArchive->DumpStatsAndDebugInfo();
					ChunksSaved[SPlatform].Add(ChunkId);
				}
			}
		}

		return bOk;
	}

	void DeleteShaderCodeChunks(const FString& SandboxDestinationPath, const FString& SandboxMetadataPath)
	{
		TArray<FString> ExistingFiles;
		IFileManager::Get().FindFiles(ExistingFiles, *SandboxDestinationPath);
		for (FString& FileName : ExistingFiles)
		{
			FileName = FPaths::Combine(SandboxDestinationPath, FileName);
		}
		if (SandboxMetadataPath.Len() > 0)
		{
			FString MetadataShaderPath = SandboxMetadataPath / TEXT("../ShaderLibrarySource");
			TArray<FString> ExistingFilesMetadata;
			IFileManager::Get().FindFiles(ExistingFilesMetadata, *MetadataShaderPath);
			for (FString& FileName : ExistingFilesMetadata)
			{
				FileName = FPaths::Combine(MetadataShaderPath, FileName);
			}
			ExistingFiles.Append(ExistingFilesMetadata);
		}
		if (ExistingFiles.IsEmpty())
		{
			return;
		}

		FRegexPattern ShaderFileForChunkPattern(ShaderFileForChunkPatternStr);
		for (FString& ExistingFileName : ExistingFiles)
		{
			// Delete all the per-chunk files, we haven't written any of those yet in this cook, but do not
			// delete .ushaderbytecode files for other libraries such as the GlobalShaderLibrary which is
			// written at beginning of cook.
			FString LeafName(FPathViews::GetPathLeaf(ExistingFileName));
			FRegexMatcher ChunkFileMatcher(ShaderFileForChunkPattern, LeafName);
			if (ChunkFileMatcher.FindNext())
			{
				IFileManager::Get().Delete(*ExistingFileName,
					true /* RequireExists */, false /* EvenReadOnly */, true /* Quiet */);
			}
		}
	}

	/**
	 * Returns true if the specified shader type is referenced by any asset for staging,
	 * and optionally reports referencers.
	 */
	bool IsShaderTypeReferencedByStaging(int32 PlatformId, uint64 InShaderTypeHash,
		TArray<FName>* OutAssetNames = nullptr, int32 MaxNumAssetNames = 1)
	{
		bool bOutIsReferencedByStaging = false;
		if (OutAssetNames)
		{
			OutAssetNames->Empty();
		}
		if (FEditorShaderCodeArchive* ShaderArchive = this->EditorShaderCodeArchive[PlatformId])
		{
			ShaderArchive->ForEachAssetReferencingShaderType(
				InShaderTypeHash,
				[&bOutIsReferencedByStaging, OutAssetNames, MaxNumAssetNames]
				(FName AssetName, const FShaderMapAssetAssociations::FAssociatedAssetData& AssetData) -> bool
				{
					if (AssetData.bReferencedByStaging)
					{
						bOutIsReferencedByStaging = true;
						if (OutAssetNames)
						{
							OutAssetNames->Add(AssetName);
							if (MaxNumAssetNames >= 1 && OutAssetNames->Num() >= MaxNumAssetNames)
							{
								return false; // stop iterating
							}
						}
						else
						{
							return false; // stop iterating
						}
					}
					return true; // keep iterating
				}
			);
		}
		return bOutIsReferencedByStaging;
	};

	void DumpShaderTypeStats(const FString& DebugInfoDir, const FString& MetaDataDir,
		TFunctionRef<void(FName AssetName, bool& bImplemented, bool& bOutReferenced, bool& bOutRecooked)> GetCookStatus)
	{
		for (int32 PlatformId = 0; PlatformId < EShaderPlatform::SP_NumPlatforms; ++PlatformId)
		{
			const FShaderCodeStats& PlatformCodeStats = EditorShaderCodeStats[PlatformId];
			if (PlatformCodeStats.NumShaderMaps > 0)
			{
				const FName PlatformFName = FGenericDataDrivenShaderPlatformInfo::GetName((EShaderPlatform)PlatformId);
				FString PlatformName = PlatformFName.ToString();
				FString Filename = FPaths::Combine(*DebugInfoDir, *PlatformName, TEXT("ShaderTypeStats.csv"));
				TMap<const FShaderType*, FShaderCodeStats::FPerTypeStats> CountersByTypeName;
				for (const TPair<uint64, FShaderCodeStats::FPerTypeStats>& Pair : PlatformCodeStats.ShaderTypeStats)
				{
					if (const FShaderType* ShaderType = FindShaderTypeByName(FHashedName(Pair.Key)))
					{
						CountersByTypeName.Add(ShaderType, Pair.Value);
					}
					else
					{
						// This error needs to be suppressed for data used by ShaderMaps or Shaders
						// that are not referenced by any assets that have bReferencedByStaging.
						TArray<FName> AssetNames;
						bool bReferencedbyStaging = IsShaderTypeReferencedByStaging(PlatformId, Pair.Key,
							&AssetNames, 1 /* MaxNumAssetNames */);
						if (bReferencedbyStaging)
						{
							FString CookStatus = TEXT("It is unknown whether this asset was incrementally skipped in the current cook.");
							if (!AssetNames.IsEmpty())
							{
								bool bImplemented = false;
								bool bReferenced = false;
								bool bRecooked = false;
								GetCookStatus(AssetNames[0], bImplemented, bReferenced, bRecooked);
								if (bImplemented)
								{
									if (!bReferenced)
									{
										CookStatus = TEXT("This asset was not referenced in the current cook; ")
											TEXT("there is a Cooker bug that reported it to the ShaderCodeLibrary as referenced by staging.");
									}
									else if (bRecooked)
									{
										CookStatus = TEXT("This asset was referenced and recooked in the current cook; ")
											TEXT("there is a Cooker or ShaderCodeLibrary bug that failed to remove its now-stale previously cooked shader that uses the removed or renamed ShaderType.");
									}
									else
									{
										CookStatus = TEXT("This asset was referenced but was incrementally skipped in the current cook; ")
											TEXT("the change that removed or renamed the ShaderType is missing a version bump that triggers the recook of the asset.");
									}
								}
							}

							checkf(false,
								TEXT("Shader library contains a reference to an FShaderType with hash %" UINT64_FMT " which does not exist.\n")
								TEXT("Removed or renamed ShaderTypes are supposed to be impossible to stage; recooks should not be able ")
								TEXT("to reference them and incremental cooks are supposed to recook assets that use them.\n")
								TEXT("This shadertype however is referenced by an asset which is being staged: \"%s\". %s"),
								Pair.Key, AssetNames.IsEmpty() ? TEXT("<Unknown>") : *AssetNames[0].ToString(), *CookStatus);
						}
					}
				}
				
				// sort first by the type of shadertype, then by the name (so all global, material, etc. shaders are grouped together in the CSV)
				struct FShaderTypeSort
				{
					bool operator()(const FShaderType& A, const FShaderType& B) const
					{
						if (A.GetTypeForDynamicCast() == B.GetTypeForDynamicCast())
						{
							return FCString::Strcmp(A.GetName(), B.GetName()) < 0;
						}
						return A.GetTypeForDynamicCast() < B.GetTypeForDynamicCast();
					}
				};
				CountersByTypeName.KeySort(FShaderTypeSort());

				{
					TUniquePtr<FArchive> ShaderTypeStatsCsv(IFileManager::Get().CreateFileWriter(*Filename));
					if (ShaderTypeStatsCsv)
					{
						FDiagnosticTableWriterCSV StatWriter(ShaderTypeStatsCsv.Get());
						StatWriter.AddColumn(TEXT("ShaderTypeName"));
						StatWriter.AddColumn(TEXT("ShaderType"));
						StatWriter.AddColumn(TEXT("Frequency"));
						StatWriter.AddColumn(TEXT("NumShaders"));
						StatWriter.AddColumn(TEXT("ShadersSize"));
						StatWriter.CycleRow();

						for (const TPair<const FShaderType*, FShaderCodeStats::FPerTypeStats>& Pair : CountersByTypeName)
						{
							StatWriter.AddColumn(Pair.Key->GetName());
							StatWriter.AddColumn(*LexToString(Pair.Key->GetTypeForDynamicCast()));
							StatWriter.AddColumn(GetShaderFrequencyString(Pair.Key->GetFrequency()));
							StatWriter.AddColumn(TEXT("%d"), Pair.Value.NumShaders);
							StatWriter.AddColumn(TEXT("%lld"), Pair.Value.ShadersSize);
							StatWriter.CycleRow();
						}
					}
					else
					{
						UE_LOGF(LogShaderLibrary, Warning, "Failed to create shader type stats file '%ls'", *Filename);
					}
				}

				// Add a copy of the type stats to the build metadata as well, potentially overriding name by cvar
				const FName ShaderFormat = FDataDrivenShaderPlatformInfo::GetShaderFormat((EShaderPlatform)PlatformId);
				FString StatsFileName = FString::Printf(TEXT("ShaderTypeStats-%s"), *PlatformName);
				GetShaderFileNameOverride(StatsFileName, TEXT("r.Shaders.ShaderTypeStatsFileNameOverride"), ShaderFormat, PlatformFName);
				FString MetaPath = FString::Printf(TEXT("%s/%s.csv"), *MetaDataDir, *StatsFileName);

				if (IFileManager::Get().Copy(*MetaPath, *Filename) != COPY_OK)
				{
					UE_LOGF(LogShaderLibrary, Warning, "Failed to copy from '%ls' to '%ls'", *Filename, *MetaPath);
				}
			}
		}
	}
#endif // WITH_EDITOR

	void ResolvePackageShaderMapsDelegate(TArrayView<const FShaderHash> ShaderMapHashes, FCoreDelegates::FIoChunkIdResolvedFunc IoChunkIdResolvedFunc)
	{
		for (const FShaderHash& ShaderMapHash : ShaderMapHashes)
		{
			// When resolving shadermaps to all of their IoChunks, we have to iterate through all shader library instances that contain such shadermap.
			// Otherwise, the order of mounted (and unmounted) library instances might break the availability of shaders.
			ForEachShaderLibraryWithShaderMap(ShaderMapHash, [&IoChunkIdResolvedFunc](FShaderLibraryInstance* LibraryInstance, int32 ShaderMapIndex) {
				LibraryInstance->ResolvePackageShaderMap(ShaderMapIndex, IoChunkIdResolvedFunc);
			});

			UE_LOGF(LogShaderLibrary, VeryVerbose, "Shadermap '%ls' resolved to IoChunks: %ls", *LexToString(ShaderMapHash), *PrintResolvedShaderMapIoChunks(ShaderMapHash));
		}
	}

	void PreloadPackageShaderMapsDelegate(TArrayView<const FShaderHash> ShaderMapHashes, FCoreDelegates::FAttachShaderReadRequestFunc AttachShaderReadRequestFunc)
	{
		LLM_SCOPE(ELLMTag::Shaders);
		for (const FShaderHash& ShaderMapHash : ShaderMapHashes)
		{
			int32 ShaderMapIndex;
			FShaderLibraryInstance* LibraryInstance = FindShaderLibraryForShaderMap(ShaderMapHash, ShaderMapIndex);
			if (LibraryInstance)
			{
				LibraryInstance->PreloadPackageShaderMap(ShaderMapIndex, AttachShaderReadRequestFunc);
			}
		}
	}

	void ReleasePreloadedPackageShaderMapsDelegate(TArrayView<const FShaderHash> ShaderMapHashes)
	{
		for (const FShaderHash& ShaderMapHash : ShaderMapHashes)
		{
			int32 ShaderMapIndex;
			FShaderLibraryInstance* LibraryInstance = FindShaderLibraryForShaderMap(ShaderMapHash, ShaderMapIndex);
			if (LibraryInstance)
			{
				LibraryInstance->ReleasePreloadedPackageShaderMap(ShaderMapIndex);
			}
		}
	}

	template<typename F>
	void IterateNamedShaderLibrariesSafe(F&& Func)
	{
		FRWScopeLock NamedReadLock(NamedLibrariesMutex, SLT_ReadOnly);
		for (auto& [LogicalName, NamedShaderLibrary] : NamedLibrariesStack)
		{
			Invoke(Forward<F>(Func), LogicalName, *NamedShaderLibrary);
		}
	}

	bool MoveShaderMapResourceOwnership(FShaderMapResource_SharedCode* Resource)
	{
		check(Resource);

		// Find new shader library for this resource.
		int32 NewShaderMapIndex = INDEX_NONE;
		if (FShaderLibraryInstance* NewLibraryInstance = FindShaderLibraryForShaderMap(Resource->GetShaderMapHash(), NewShaderMapIndex, Resource->LibraryInstance))
		{
			Resource->AssignToNewShaderLibrary(NewLibraryInstance, NewShaderMapIndex);
			return true;
		}
		return false;
	}
};

static FSharedShaderCodeRequest OnSharedShaderCodeRequest;

FShaderLibrariesCollection* FShaderLibrariesCollection::Impl = nullptr;

static void FShaderCodeLibraryPluginMountedCallback(IPlugin& Plugin)
{
	TRACE_CPUPROFILER_EVENT_SCOPE_STR("FShaderCodeLibraryPluginMountedCallback");
	if (FApp::CanEverRender() && UE::ShaderLibrary::Private::PluginsToIgnoreOnMount.Remove(Plugin.GetName()) == 0)
	{
		FShaderCodeLibrary::OpenPluginShaderLibrary(Plugin);
	}
}

static void FShaderCodeLibraryPluginUnmountedCallback(IPlugin& Plugin)
{
	class FShaderCodeLibraryCleanup : public FDeferredCleanupInterface
	{
	public:
		FShaderCodeLibraryCleanup(const FString& Name)
			: Name(Name)
		{
		}

		virtual ~FShaderCodeLibraryCleanup()
		{
			FShaderCodeLibrary::CloseLibrary(Name);
		}

	private:
		FString Name;
	};

	if (Plugin.CanContainContent() && Plugin.IsEnabled())
	{
		// unload any shader libraries that may exist in this plugin
		BeginCleanup(new FShaderCodeLibraryCleanup(Plugin.GetName()));
	}
}

static void FShaderLibraryPakFileMountedCallback(const IPakFile& PakFile)
{
	using namespace UE::ShaderLibrary::Private;

	UE_LOGF(LogShaderLibrary, Verbose, "ShaderCodeLibraryPakFileMountedCallback: PakFile '%ls' (chunk index %d, root '%ls') mounted", *PakFile.PakGetPakFilename(), PakFile.PakGetPakchunkIndex(), *PakFile.PakGetMountPoint());

	FMountedPakFileInfo PakFileInfo(PakFile);
	{
		FScopeLock PakFilesLocker(&FMountedPakFileInfo::KnownPakFilesAccessLock);
		FMountedPakFileInfo::KnownPakFiles.Add(PakFileInfo);
	}

	// If shaderlibrary has not yet been initialized, report the chunk as pending
	if (FShaderLibrariesCollection::Impl)
	{
		FShaderLibrariesCollection::Impl->OnPakFileMounted(PakFileInfo);
	}
	else
	{
		UE_LOGF(LogShaderLibrary, Verbose, "ShaderCodeLibraryPakFileMountedCallback: pending pak file info (%ls)", *PakFileInfo.ToString());
	}
}

static void FShaderLibraryPakFileUnmountingCallback(const IPakFile& PakFile)
{
	using namespace UE::ShaderLibrary::Private;

    UE_LOGF(LogShaderLibrary, Verbose, "ShaderCodeLibraryPakFileUnmountingCallback: PakFile '%ls' (chunk index %d, root '%ls') unmounting", *PakFile.PakGetPakFilename(), PakFile.PakGetPakchunkIndex(), *PakFile.PakGetMountPoint());

	FMountedPakFileInfo PakFileInfo(PakFile);
	{
		FScopeLock PakFilesLocker(&FMountedPakFileInfo::KnownPakFilesAccessLock);
		FMountedPakFileInfo::KnownPakFiles.Remove(PakFileInfo);
	}

	// If shaderlibrary has not yet been initialized, report the chunk as pending
	if (FShaderLibrariesCollection::Impl)
	{
		FShaderLibrariesCollection::Impl->OnPakFileUnmounting(PakFileInfo);
	}
	else
	{
		UE_LOGF(LogShaderLibrary, Verbose, "ShaderCodeLibraryPakFileUnmountedCallback: pending pak file info (%ls)", *PakFileInfo.ToString());
	}
}

void FShaderCodeLibrary::PreInit()
{
	// add a callback for opening later chunks
	UE::ShaderLibrary::Private::OnPakFileMountedDelegateHandle = FCoreDelegates::GetOnPakFileMounted2().AddStatic(&FShaderLibraryPakFileMountedCallback);
	UE::ShaderLibrary::Private::OnPakFileUnmountingDelegateHandle = FCoreDelegates::GetOnPakFileUnmounting().AddStatic(&FShaderLibraryPakFileUnmountingCallback);
}

void FShaderCodeLibrary::InitForRuntime(EShaderPlatform ShaderPlatform)
{
	if (FShaderLibrariesCollection::Impl != nullptr)
	{
		//cooked, can't change shader platform on the fly
		check(FShaderLibrariesCollection::Impl->GetRuntimeShaderPlatform() == ShaderPlatform);
		return;
	}

	// Cannot be enabled by the server, pointless if we can't ever render and not compatible with cook-on-the-fly
	bool bArchive = false;
	GConfig->GetBool(TEXT("/Script/UnrealEd.ProjectPackagingSettings"), TEXT("bShareMaterialShaderCode"), bArchive, GGameIni);

	// We cannot enable native shader libraries when running with NullRHI, so for consistency all libraries (both native and non-native) are disabled if FApp::CanEverRender() == false
	bool bEnable = !FPlatformProperties::IsServerOnly() && FApp::CanEverRender() && bArchive;
#if !UE_BUILD_SHIPPING
	const bool bCookOnTheFly = IsRunningCookOnTheFly(); 
	bEnable &= !bCookOnTheFly;
#endif

	if (bEnable)
	{
		FShaderLibrariesCollection::Impl = new FShaderLibrariesCollection(ShaderPlatform, false);
		if (FShaderLibrariesCollection::Impl->OpenLibrary(TEXT("Global"), FPaths::ProjectContentDir()))
		{
			UE::ShaderLibrary::Private::OnPluginMountedDelegateHandle = IPluginManager::Get().OnNewPluginMounted().AddStatic(&FShaderCodeLibraryPluginMountedCallback);
			UE::ShaderLibrary::Private::OnPluginUnmountedDelegateHandle = IPluginManager::Get().OnPluginUnmounted().AddStatic(&FShaderCodeLibraryPluginUnmountedCallback);
		
#if (!UE_BUILD_SHIPPING && !UE_BUILD_TEST)	// test builds are supposed to be closer to Shipping than to Development, and as such not have development features
			// support shared cooked builds by also opening the shared cooked build shader code file
			FShaderLibrariesCollection::Impl->OpenLibrary(TEXT("Global_SC"), FPaths::ProjectContentDir());
#endif

			// mount shader library from the plugins as they may also have global shaders
			auto Plugins = IPluginManager::Get().GetEnabledPluginsWithContent();
			ParallelFor(Plugins.Num(), [&](int32 Index)
			{
				FShaderCodeLibrary::OpenPluginShaderLibrary(*Plugins[Index]);
			});
		}
		else
		{
			Shutdown();
#if !WITH_EDITOR
			if (FPlatformProperties::SupportsWindowedMode())
			{
				FPlatformSplash::Hide();

				UE_LOGF(LogShaderLibrary, Error, "Failed to initialize ShaderCodeLibrary required by the project because part of the Global shader library is missing from %ls.", *FPaths::ProjectContentDir());

				FText LocalizedMsg = FText::Format(NSLOCTEXT("MessageDialog", "MissingGlobalShaderLibraryFiles_Body", "Game files required to initialize the global shader library are missing from:\n\n{0}\n\nPlease make sure the game is installed correctly."), FText::FromString(FPaths::ConvertRelativePathToFull(FPaths::ProjectContentDir())));
				FPlatformMisc::MessageBoxExt(EAppMsgType::Ok, *LocalizedMsg.ToString(), *NSLOCTEXT("MessageDialog", "MissingGlobalShaderLibraryFiles_Title", "Missing game files").ToString());
			}
			else
			{
                FMessageDialog::Open(EAppMsgType::Ok, NSLOCTEXT("MessageDialog", "MissingGlobalShaderLibraryFilesClient_Body", "Game files required to initialize the global shader and cooked content are most likely missing. Refer to Engine log for details."));
				UE_LOGF(LogShaderLibrary, Fatal, "Failed to initialize ShaderCodeLibrary required by the project because part of the Global shader library is missing from %ls.", *FPaths::ProjectContentDir());
			}
			FPlatformMisc::RequestExit(true, TEXT("FShaderCodeLibrary::InitForRuntime"));
#endif // !WITH_EDITOR	
		}
	}
}

void FShaderCodeLibrary::Shutdown()
{
	if (UE::ShaderLibrary::Private::OnPakFileMountedDelegateHandle.IsValid())
	{
		FCoreDelegates::GetOnPakFileMounted2().Remove(UE::ShaderLibrary::Private::OnPakFileMountedDelegateHandle);
		UE::ShaderLibrary::Private::OnPakFileMountedDelegateHandle.Reset();
	}
	if (UE::ShaderLibrary::Private::OnPakFileUnmountingDelegateHandle.IsValid())
	{
		FCoreDelegates::GetOnPakFileUnmounting().Remove(UE::ShaderLibrary::Private::OnPakFileUnmountingDelegateHandle);
		UE::ShaderLibrary::Private::OnPakFileUnmountingDelegateHandle.Reset();
	}
	if (UE::ShaderLibrary::Private::OnPluginMountedDelegateHandle.IsValid())
	{
		IPluginManager::Get().OnNewPluginMounted().Remove(UE::ShaderLibrary::Private::OnPluginMountedDelegateHandle);
		UE::ShaderLibrary::Private::OnPluginMountedDelegateHandle.Reset();
	}
	if (UE::ShaderLibrary::Private::OnPluginUnmountedDelegateHandle.IsValid())
	{
		IPluginManager::Get().OnPluginUnmounted().Remove(UE::ShaderLibrary::Private::OnPluginUnmountedDelegateHandle);
		UE::ShaderLibrary::Private::OnPluginUnmountedDelegateHandle.Reset();
	}

	UE::ShaderLibrary::Private::PluginsToIgnoreOnMount.Empty();

	if (FShaderLibrariesCollection::Impl)
	{
		delete FShaderLibrariesCollection::Impl;
		FShaderLibrariesCollection::Impl = nullptr;
	}

	FScopeLock PakFilesLocker(&UE::ShaderLibrary::Private::FMountedPakFileInfo::KnownPakFilesAccessLock);
	UE::ShaderLibrary::Private::FMountedPakFileInfo::KnownPakFiles.Empty();
}

bool FShaderCodeLibrary::IsEnabled()
{
	return FShaderLibrariesCollection::Impl != nullptr;
}

bool FShaderCodeLibrary::AreShaderMapsPreloadedAtLoadTime()
{
	return GPreloadShaderMaps > 0;
}

bool FShaderCodeLibrary::ContainsShaderCode(const FShaderHash& Hash)
{
	if (FShaderLibrariesCollection::Impl)
	{
		return FShaderLibrariesCollection::Impl->ContainsShaderCode(Hash);
	}
	return false;
}

bool FShaderCodeLibrary::ContainsShaderCode(const FShaderHash& Hash, const FString& LogicalLibraryName)
{
	if (FShaderLibrariesCollection::Impl)
	{
		return FShaderLibrariesCollection::Impl->ContainsShaderCode(Hash, LogicalLibraryName);
	}
	return false;
}

TRefCountPtr<FShaderMapResource> FShaderCodeLibrary::LoadResource(const FShaderHash& Hash, FArchive* Ar)
{
	if (FShaderLibrariesCollection::Impl)
	{
		SCOPED_LOADTIMER(FShaderCodeLibrary_LoadResource);
		OnSharedShaderCodeRequest.Broadcast(Hash, Ar);
		return TRefCountPtr<FShaderMapResource>(FShaderLibrariesCollection::Impl->LoadResource(Hash, Ar));
	}
	return TRefCountPtr<FShaderMapResource>();
}

bool FShaderCodeLibrary::PreloadShader(const FShaderHash& Hash, FArchive* Ar)
{
	if (FShaderLibrariesCollection::Impl)
	{
		OnSharedShaderCodeRequest.Broadcast(Hash, Ar);
		return FShaderLibrariesCollection::Impl->PreloadShader(Hash, Ar);
	}
	return false;
}

bool FShaderCodeLibrary::ReleasePreloadedShader(const FShaderHash& Hash)
{
	if (FShaderLibrariesCollection::Impl)
	{
		return FShaderLibrariesCollection::Impl->ReleasePreloadedShader(Hash);
	}
	return false;
}

FVertexShaderRHIRef FShaderCodeLibrary::CreateVertexShader(EShaderPlatform Platform, const FShaderHash& Hash)
{
	if (FShaderLibrariesCollection::Impl)
	{
		return FVertexShaderRHIRef(FShaderLibrariesCollection::Impl->CreateShader(SF_Vertex, Hash));
	}
	return nullptr;
}

FPixelShaderRHIRef FShaderCodeLibrary::CreatePixelShader(EShaderPlatform Platform, const FShaderHash& Hash)
{
	if (FShaderLibrariesCollection::Impl)
	{
		return FPixelShaderRHIRef(FShaderLibrariesCollection::Impl->CreateShader(SF_Pixel, Hash));
	}
	return nullptr;
}

FGeometryShaderRHIRef FShaderCodeLibrary::CreateGeometryShader(EShaderPlatform Platform, const FShaderHash& Hash)
{
	if (FShaderLibrariesCollection::Impl)
	{
		return FGeometryShaderRHIRef(FShaderLibrariesCollection::Impl->CreateShader(SF_Geometry, Hash));
	}
	return nullptr;
}

FComputeShaderRHIRef FShaderCodeLibrary::CreateComputeShader(EShaderPlatform Platform, const FShaderHash& Hash)
{
	if (FShaderLibrariesCollection::Impl)
	{
		return FComputeShaderRHIRef(FShaderLibrariesCollection::Impl->CreateShader(SF_Compute, Hash));
	}
	return nullptr;
}

FMeshShaderRHIRef FShaderCodeLibrary::CreateMeshShader(EShaderPlatform Platform, const FShaderHash& Hash)
{
	if (FShaderLibrariesCollection::Impl)
	{
		return FMeshShaderRHIRef(FShaderLibrariesCollection::Impl->CreateShader(SF_Mesh, Hash));
	}
	return nullptr;
}

FAmplificationShaderRHIRef FShaderCodeLibrary::CreateAmplificationShader(EShaderPlatform Platform, const FShaderHash& Hash)
{
	if (FShaderLibrariesCollection::Impl)
	{
		return FAmplificationShaderRHIRef(FShaderLibrariesCollection::Impl->CreateShader(SF_Amplification, Hash));
	}
	return nullptr;
}

FRayTracingShaderRHIRef FShaderCodeLibrary::CreateRayTracingShader(EShaderPlatform Platform, const FShaderHash& Hash, EShaderFrequency Frequency)
{
	if (FShaderLibrariesCollection::Impl)
	{
		check(Frequency >= SF_RayGen && Frequency <= SF_RayCallable);
		return FRayTracingShaderRHIRef(FShaderLibrariesCollection::Impl->CreateShader(Frequency, Hash));
	}
	return nullptr;
}

uint32 FShaderCodeLibrary::GetShaderCount(void)
{
	uint32 Num = 0;
	if (FShaderLibrariesCollection::Impl)
	{
		Num = FShaderLibrariesCollection::Impl->GetShaderCount();
	}
	return Num;
}

EShaderPlatform FShaderCodeLibrary::GetRuntimeShaderPlatform(void)
{
	EShaderPlatform Platform = SP_NumPlatforms;
	if (FShaderLibrariesCollection::Impl)
	{
		Platform = FShaderLibrariesCollection::Impl->GetRuntimeShaderPlatform();
	}
	return Platform;
}

void FShaderCodeLibrary::AddKnownChunkIDs(const int32* IDs, const int32 NumChunkIDs)
{
	using namespace UE::ShaderLibrary::Private;

	checkf(IDs, TEXT("Invalid pointer to chunk IDs passed"));
	UE_LOGF(LogShaderLibrary, Display, "AddKnownChunkIDs: adding %d chunk IDs", NumChunkIDs);

	for (int32 IdxChunkId = 0; IdxChunkId < NumChunkIDs; ++IdxChunkId)
	{
		FMountedPakFileInfo PakFileInfo(IDs[IdxChunkId]);
		{
			FScopeLock PakFilesLocker(&FMountedPakFileInfo::KnownPakFilesAccessLock);
			FMountedPakFileInfo::KnownPakFiles.Add(PakFileInfo);
		}

		// if shaderlibrary has not yet been initialized, add the chunk as pending
		if (FShaderLibrariesCollection::Impl)
		{
			FShaderLibrariesCollection::Impl->OnPakFileMounted(PakFileInfo);
		}
		else
		{
			UE_LOGF(LogShaderLibrary, Display, "AddKnownChunkIDs: pending pak file info (%ls)", *PakFileInfo.ToString());
		}
	}
}

bool FShaderCodeLibrary::OpenLibrary(FString const& Name, FString const& Directory, bool bMonolithicOnly)
{
	bool bResult = false;
	if (FShaderLibrariesCollection::Impl)
	{
		bResult = FShaderLibrariesCollection::Impl->OpenLibrary(Name, Directory, bMonolithicOnly);
	}
	return bResult;
}

void FShaderCodeLibrary::CloseLibrary(FString const& Name)
{
	if (FShaderLibrariesCollection::Impl)
	{
		FShaderLibrariesCollection::Impl->CloseLibrary(Name);
	}
}

#if WITH_EDITOR
// for now a lot of FShaderLibraryCooker code is aliased with the runtime code, but this will be refactored (UE-103486)
void FShaderLibraryCooker::InitForCooking(bool bNativeFormat, UE::Cook::ICookArtifactReader* InCookArtifactReader)
{
	CookArtifactReader = InCookArtifactReader;
	FShaderLibrariesCollection::Impl = new FShaderLibrariesCollection(SP_NumPlatforms, bNativeFormat);
}

void FShaderLibraryCooker::Shutdown()
{
	if (FShaderLibrariesCollection::Impl)
	{
		delete FShaderLibrariesCollection::Impl;
		FShaderLibrariesCollection::Impl = nullptr;
	}
}

void FShaderLibraryCooker::CleanDirectories(TArray<FName> const& ShaderFormats)
{
	if (FShaderLibrariesCollection::Impl)
	{
		FShaderLibrariesCollection::Impl->CleanDirectories(ShaderFormats);
	}
}

bool FShaderLibraryCooker::BeginCookingLibrary(FString const& Name)
{
	// for now this is aliased with the runtime code, but this will be refactored (UE-103486)
	bool bResult = false;
	if (FShaderLibrariesCollection::Impl)
	{
		bResult = FShaderLibrariesCollection::Impl->OpenLibrary(Name, TEXT(""));
	}
	return bResult;
}

void FShaderLibraryCooker::EndCookingLibrary(FString const& Name)
{
	// for now this is aliased with the runtime code, but this will be refactored (UE-103486)
	if (FShaderLibrariesCollection::Impl)
	{
		FShaderLibrariesCollection::Impl->CloseLibrary(Name);
	}
}

bool FShaderLibraryCooker::IsShaderLibraryEnabled()
{
	return FShaderLibrariesCollection::Impl != nullptr;
}

void FShaderLibraryCooker::CookShaderFormats(TArray<FShaderFormatDescriptor> const& ShaderFormats)
{
	if (FShaderLibrariesCollection::Impl)
	{
		FShaderLibrariesCollection::Impl->CookShaderFormats(ShaderFormats);
	}
}

bool FShaderLibraryCooker::AddShaderCode(EShaderPlatform ShaderPlatform, const FShaderMapResourceCode* Code, const FShaderMapAssetPaths& AssociatedAssets)
{
	if (FShaderLibrariesCollection::Impl)
	{
		FShaderLibrariesCollection::Impl->AddShaderCode(ShaderPlatform, Code, AssociatedAssets);
		return true;
	}
	return false;
}

void FShaderLibraryCooker::CopyToCompactBinaryAndClear(FCbWriter& Writer, bool& bOutHasData,
	bool& bOutRanOutOfRoom, int64 MaxShaderSize)
{
	if (FShaderLibrariesCollection::Impl)
	{
		FShaderLibrariesCollection::Impl->CopyToCompactBinaryAndClear(Writer, bOutHasData,
			bOutRanOutOfRoom, MaxShaderSize);
	}
}

bool FShaderLibraryCooker::AppendFromCompactBinary(FCbFieldView Field)
{
	if (FShaderLibrariesCollection::Impl)
	{
		return FShaderLibrariesCollection::Impl->AppendFromCompactBinary(Field);
	}
	return false;
}

bool FShaderLibraryCooker::NeedsShaderStableKeys(EShaderPlatform ShaderPlatform)
{
	if (FShaderLibrariesCollection::Impl)
	{
		return FShaderLibrariesCollection::Impl->NeedsShaderStableKeys(ShaderPlatform);
	}
	return false;
}

void FShaderLibraryCooker::AddShaderStableKeyValue(EShaderPlatform ShaderPlatform, FStableShaderKeyAndValue& StableKeyValue)
{
	if (FShaderLibrariesCollection::Impl)
	{
		FShaderLibrariesCollection::Impl->AddShaderStableKeyValue(ShaderPlatform, StableKeyValue);
	}
}

void FShaderLibraryCooker::DumpShaderTypeStats(const FString& DebugInfoDir, const FString& MetaDataDir,
	TFunctionRef<void(FName AssetName, bool& bImplemented, bool& bOutReferenced, bool& bOutRecooked)> GetCookStatus)
{
	if (FShaderLibrariesCollection::Impl)
	{
		FShaderLibrariesCollection::Impl->DumpShaderTypeStats(DebugInfoDir, MetaDataDir, GetCookStatus);
	}
}

bool FShaderLibraryCooker::CreatePatchLibrary(TArray<FString> const& OldMetaDataDirs, FString const& NewMetaDataDir, FString const& OutDir, bool bNativeFormat, bool /*bNeedsDeterministicOrder*/)
{
	TMap<FName, TSet<FString>> FormatLibraryMap;
	TArray<FString> LibraryFiles;
	ShaderFindFiles(LibraryFiles, *(NewMetaDataDir / TEXT("ShaderLibrarySource")), *ShaderExtension);
	
	for (FString const& Path : LibraryFiles)
	{
		FString Name = FPaths::GetBaseFilename(Path);
		if (Name.RemoveFromStart(TEXT("ShaderArchive-")))
		{
			TArray<FString> Components;
			if (Name.ParseIntoArray(Components, TEXT("-")) == 3)
			{
				FName Format(*Components[1]);
				FName Platform(*Components[2]);
				FString FormatAndPlatform = Format.ToString() + TEXT("-") + Platform.ToString();

				TSet<FString>& Libraries = FormatLibraryMap.FindOrAdd(FName(FormatAndPlatform));
				Libraries.Add(Components[0]);
			}
		}
	}
	
	bool bOK = true;
	for (auto const& Entry : FormatLibraryMap)
	{
		for (auto const& Library : Entry.Value)
		{
			bOK |= FEditorShaderCodeArchive::CreatePatchLibrary(Entry.Key, Library, OldMetaDataDirs, NewMetaDataDir, OutDir, bNativeFormat);
		}
	}
	return bOK;
}

void FShaderLibraryCooker::FinishPopulateShaderLibrary(const ITargetPlatform* TargetPlatform, FString const& Name,
	FString const& SandboxDestinationPath, FString const& SandboxMetadataPath, bool bInitializeReferenceTracking)
{
	const FString& ShaderCodeDir = SandboxDestinationPath;
	const FString& MetaDataPath = SandboxMetadataPath;

	checkf(FShaderLibrariesCollection::Impl != nullptr, TEXT("FShaderLibraryCooker was not initialized properly"));
	checkf(TargetPlatform, TEXT("A valid TargetPlatform is expected"));

	// note that shader formats can be shared across the target platforms
	TArray<FName> ShaderFormats;
	TargetPlatform->GetAllTargetedShaderFormats(ShaderFormats);
	if (ShaderFormats.Num() > 0)
	{
		FShaderLibrariesCollection::Impl->FinishPopulateShaderCode(ShaderCodeDir, MetaDataPath, ShaderFormats, bInitializeReferenceTracking);
	}
}

void FShaderLibraryCooker::UpdateOplogPackages(UE::Cook::Artifact::FUpdateOplogPackagesContext& Context)
{
	TArray<FName> ShaderFormats;
	Context.GetTargetPlatform()->GetAllTargetedShaderFormats(ShaderFormats);
	if (ShaderFormats.Num() > 0)
	{
		FShaderLibrariesCollection::Impl->UpdateOplogPackages(Context, ShaderFormats);
	}
}

void FShaderLibraryCooker::PruneShaderLibrary(const ITargetPlatform* TargetPlatform, const FString& Name)
{
	TArray<FName> ShaderFormats;
	TargetPlatform->GetAllTargetedShaderFormats(ShaderFormats);
	if (ShaderFormats.Num() > 0)
	{
		FShaderLibrariesCollection::Impl->PruneShaderLibrary(ShaderFormats);
	}
}

bool FShaderLibraryCooker::MergeShaderCodeArchive(const TArray<FString>& CookedMetadataDirs, const FString& OutputDir, TArray<FString>& OutWrittenFiles)
{
	using namespace UE::ShaderLibrary::Private;

	OutWrittenFiles.Empty();

	static const FRegexPattern ShaderArchivePattern(ShaderArchivePatternStr);
	static const FRegexPattern StableInfoPattern(StableInfoPatternStr);

	// Map of file name to the archive we are unioning
	TMap<FString, TPair<FEditorShaderCodeArchive, FShaderCodeStats>> ShaderCodeArchives;
	TMap<FString, FEditorShaderStableInfo> ShaderStableInfos;

	for (const FString& MetadataDir : CookedMetadataDirs)
	{
		const FString ShaderCodeDir = MetadataDir / TEXT("ShaderLibrarySource");
		const FString ShaderStableInfoDir = MetadataDir / TEXT("PipelineCaches");

		TArray<FString> ShaderBytecodeFiles;
		ShaderFindFiles(ShaderBytecodeFiles, *ShaderCodeDir, *ShaderExtension);
		for (const FString& ByteCodeFile : ShaderBytecodeFiles)
		{
			if (ShaderCodeArchives.Contains(ByteCodeFile))
			{
				ShaderCodeArchives[ByteCodeFile].Key.AddShaderCodeLibraryFromDirectory(ShaderCodeDir,
					ShaderCodeArchive::ECookShaderLibrarySource::CurrentCook, &ShaderCodeArchives[ByteCodeFile].Value);
			}
			else
			{
				FRegexMatcher FindShaderFormat(ShaderArchivePattern, ByteCodeFile);
				if (ensureMsgf(FindShaderFormat.FindNext(), TEXT("Unable to parse out shader format from %s"), *ByteCodeFile))
				{
					const FName ShaderFormat(*FindShaderFormat.GetCaptureGroup(2));
					FEditorShaderCodeArchive NewCodeArchive(ShaderFormat);
					FShaderCodeStats NewCodeStats;
					NewCodeArchive.OpenLibrary(FindShaderFormat.GetCaptureGroup(1));
					NewCodeArchive.AddShaderCodeLibraryFromDirectory(ShaderCodeDir,
						ShaderCodeArchive::ECookShaderLibrarySource::CurrentCook, &NewCodeStats);

					ShaderCodeArchives.FindOrAdd(ByteCodeFile, { MoveTemp(NewCodeArchive), MoveTemp(NewCodeStats) });
				}
			}
		}

		TArray<FString> StableInfoFiles;
		ShaderFindFiles(StableInfoFiles, *ShaderStableInfoDir, *StableExtension);
		for (const FString& StableInfoFile : StableInfoFiles)
		{
			if (ShaderStableInfos.Contains(StableInfoFile))
			{
				ShaderStableInfos[StableInfoFile].AddShaderCodeLibraryFromDirectory(ShaderStableInfoDir,
					FEditorShaderStableInfo::EMergeRule::OverwriteUnmodifiedWarnModified);
			}
			else
			{
				FRegexMatcher FindShaderInfo(StableInfoPattern, StableInfoFile);
				if (ensureMsgf(FindShaderInfo.FindNext(), TEXT("Unable to parse out shader format from %s"), *StableInfoFile))
				{
					const FName ShaderFormat(*FindShaderInfo.GetCaptureGroup(2));
					FEditorShaderStableInfo NewStableInfo(ShaderFormat);
					NewStableInfo.OpenLibrary(FindShaderInfo.GetCaptureGroup(1));
					NewStableInfo.AddShaderCodeLibraryFromDirectory(ShaderStableInfoDir,
						FEditorShaderStableInfo::EMergeRule::OverwriteUnmodifiedWarnModified);

					ShaderStableInfos.FindOrAdd(StableInfoFile, MoveTemp(NewStableInfo));
				}
			}
		}
	}

	const FString OutShaderCodeDir = OutputDir / TEXT("ShaderLibrarySource");
	const FString OutContentDir = OutputDir / TEXT("../Content");
	const FString OutShaderStableInfoDir = OutputDir / TEXT("PipelineCaches");

	bool bSuccess = true;
	for (TPair<FString, TPair<FEditorShaderCodeArchive, FShaderCodeStats>>& ShaderCodeArchivePair : ShaderCodeArchives)
	{
		TArray<FString> CreatedFiles;
		if (ShaderCodeArchivePair.Value.Key.SaveToDisk(OutShaderCodeDir, TEXT(""),
			ESaveToDiskTarget::Staging, ESaveToDiskSortOrder::PackageLoad, &CreatedFiles))
		{
			OutWrittenFiles.Append(MoveTemp(CreatedFiles));
		}
		else
		{
			UE_LOGF(LogShaderLibrary, Error, "Failed to save %ls", *ShaderCodeArchivePair.Key);
			bSuccess = false;
		}
		CreatedFiles.Reset();
		if (ShaderCodeArchivePair.Value.Key.SaveToDisk(OutContentDir, TEXT(""),
			ESaveToDiskTarget::Staging, ESaveToDiskSortOrder::PackageLoad, &CreatedFiles))
		{
			OutWrittenFiles.Append(MoveTemp(CreatedFiles));
		}
		else
		{
			UE_LOGF(LogShaderLibrary, Error, "Failed to save to Content Dir %ls", *ShaderCodeArchivePair.Key);
			bSuccess = false;
		}
	}

	for (TPair<FString, FEditorShaderStableInfo>& ShaderStableInfoPair : ShaderStableInfos)
	{
		FString WrittenFile;
		if (ShaderStableInfoPair.Value.SaveToDisk(OutShaderStableInfoDir, WrittenFile))
		{
			OutWrittenFiles.Add(MoveTemp(WrittenFile));
		}
		else
		{
			UE_LOGF(LogShaderLibrary, Error, "Failed to save %ls", *ShaderStableInfoPair.Key);
			bSuccess = false;
		}
	}

	return bSuccess;
}

static ShaderCodeArchive::ESaveToDiskSortOrder GetSortOrderForPackagingSystem(UE::Cook::EPackagingSystem PackagingSystem)
{
	using namespace ShaderCodeArchive;

	// We prefer ShaderHash sorting for determinism, but with loose (or .pak file packaging of loose shaderfiles),
	// the ordering from PackageLoad during the cook is important for performance.
	return PackagingSystem != UE::Cook::EPackagingSystem::Loose ?
		ESaveToDiskSortOrder::ShaderHash : ESaveToDiskSortOrder::PackageLoad;
}

bool FShaderLibraryCooker::SaveShaderLibraryWithoutChunking(const ITargetPlatform* TargetPlatform, FString const& Name,
	FString const& SandboxDestinationPath, FString const& SandboxMetadataPath, TArray<FString>& PlatformSCLCSVPaths,
	UE::Cook::EPackagingSystem PackagingSystem, FString& OutErrorMessage, bool& bOutHasData)
{
	checkf(TargetPlatform, TEXT("A valid TargetPlatform is expected"));

	// note that shader formats can be shared across the target platforms
	TArray<FName> ShaderFormats;
	TargetPlatform->GetAllTargetedShaderFormats(ShaderFormats);
	if (ShaderFormats.IsEmpty())
	{
		bOutHasData = false;
		return true;
	}
	bOutHasData = true;

	ShaderCodeArchive::ESaveToDiskSortOrder SortOrder = GetSortOrderForPackagingSystem(PackagingSystem);

	const FString& ShaderCodeDir = SandboxDestinationPath;
	const FString& MetaDataPath = SandboxMetadataPath;

	checkf(FShaderLibrariesCollection::Impl != nullptr, TEXT("FShaderLibraryCooker was not initialized properly"));

	const bool bSaved = FShaderLibrariesCollection::Impl->SaveShaderCode(ShaderCodeDir, MetaDataPath, ShaderFormats,
		PlatformSCLCSVPaths, SortOrder);
	if (UNLIKELY(!bSaved))
	{
		OutErrorMessage = FString::Printf(TEXT("Saving shared material shader code library failed for %s."),
			*TargetPlatform->PlatformName());
		return false;
	}

	return true;
}

bool FShaderLibraryCooker::SaveShaderLibraryChunk(int32 ChunkId, const TSet<FName>& InPackagesInChunk,
	const ITargetPlatform* TargetPlatform, const FString& SandboxDestinationPath, const FString& SandboxMetadataPath,
	UE::Cook::EPackagingSystem PackagingSystem, TArray<FString>& OutChunkFilenames, bool& bOutHasData)
{
	checkf(TargetPlatform, TEXT("A valid TargetPlatform is expected"));

	TArray<FName> ShaderFormats;
	TargetPlatform->GetAllTargetedShaderFormats(ShaderFormats);
	if (ShaderFormats.IsEmpty())
	{
		bOutHasData = false;
		return true;
	}
	bOutHasData = true;

	ShaderCodeArchive::ESaveToDiskSortOrder SortOrder = GetSortOrderForPackagingSystem(PackagingSystem);

	checkf(FShaderLibrariesCollection::Impl != nullptr, TEXT("FShaderLibraryCooker was not initialized properly"));
	return FShaderLibrariesCollection::Impl->SaveShaderCodeChunk(ChunkId, InPackagesInChunk, ShaderFormats,
		SandboxDestinationPath, SandboxMetadataPath, SortOrder, OutChunkFilenames);
}

void FShaderLibraryCooker::DeleteShaderLibraryChunks(const ITargetPlatform* TargetPlatform,
	const FString& SandboxDestinationPath, const FString& SandboxMetadataPath)
{
	// The destinationpaths are specific to the TargetPlatform, so we are free to delete files in
	// them without worrying about deleting files for other TargetPlatforms
	FShaderLibrariesCollection::Impl->DeleteShaderCodeChunks(SandboxDestinationPath, SandboxMetadataPath);
}

#endif// WITH_EDITOR

void FShaderCodeLibrary::SafeAssignHash(FRHIShader* InShader, const FShaderHash& Hash)
{
	if (InShader)
	{
		InShader->SetHash(Hash);
	}
}

FDelegateHandle FShaderCodeLibrary::RegisterSharedShaderCodeRequestDelegate_Handle(const FSharedShaderCodeRequest::FDelegate& Delegate)
{
	return OnSharedShaderCodeRequest.Add(Delegate);
}

void FShaderCodeLibrary::UnregisterSharedShaderCodeRequestDelegate_Handle(FDelegateHandle Handle)
{
	OnSharedShaderCodeRequest.Remove(Handle);
}

void FShaderCodeLibrary::DontOpenPluginShaderLibraryOnMount(const FString& PluginName)
{
	check(IsInGameThread());
	if (FApp::CanEverRender())
	{
		UE::ShaderLibrary::Private::PluginsToIgnoreOnMount.Add(PluginName);
	}
}

void FShaderCodeLibrary::OpenPluginShaderLibrary(IPlugin& Plugin, bool bMonolithicOnly)
{
	if (Plugin.CanContainContent() && Plugin.IsEnabled() && FApp::CanEverRender())
	{
		// load any shader libraries that may exist in this plugin
		if (!bMonolithicOnly)
		{
			// Chunked libraries in plugins that are not built in (i.e. ones that were cooked separately as DLC) are not supported atm. This is because the main game can be cooked without chunks (-fastcook), but still needs
			// to load the same plugins, so it would not know which ChunkIDs to try.
			// Plugins that are built-in can be chunked, but their shaders don't go into a separate library (cooker doesn't separate that atm), everything goes into main project's library.
			UE_LOGF(LogShaderLibrary, Verbose, "Opening a chunked shader library for plugin '%ls' is ignored. Chunked libraries for plugins are not supported.", *Plugin.GetName());
		}
		bMonolithicOnly = true;
		FShaderCodeLibrary::OpenLibrary(Plugin.GetName(), Plugin.GetContentDir(), bMonolithicOnly);
	}
}

bool FShaderCodeLibraryInternal::MoveShaderMapResourceOwnership(FShaderMapResource_SharedCode* Resource)
{
	return FShaderLibrariesCollection::Impl->MoveShaderMapResourceOwnership(Resource);
}

// Singleton to defer the release of ShaderLibraryInstance components to the end of a frame.
class FDeferredShaderLibraryInstanceDeleter
{
	FCriticalSection Lock;
	FDelegateHandle OnEndFrameDelegateHandle;
	TArray<FShaderLibraryInstancePriorityPair> ComponentsEnqueuedForRelease;

	void OnEndFrame()
	{
		checkf(IsInGameThread(), TEXT("Shader library instances must be released on the game thread"));
		FScopeLock ScopeLock(&Lock);

		// Reset the container. This will flush the delete operations.
		ComponentsEnqueuedForRelease.Reset();

		// Remove OnEndFrame delegate handle to avoid being invoked in further frames
		if (OnEndFrameDelegateHandle.IsValid())
		{
			if (FCoreDelegates::OnEndFrame.Remove(OnEndFrameDelegateHandle))
			{
				OnEndFrameDelegateHandle.Reset();
			}
			else
			{
				UE_LOGF(LogShaderLibrary, Warning, "Failed to remove `OnEndFrame` delegate to release shader library instances");
			}
		}
	}

	void RegisterDelegate()
	{
		if (!OnEndFrameDelegateHandle.IsValid())
		{
			OnEndFrameDelegateHandle = FCoreDelegates::OnEndFrame.AddRaw(this, &FDeferredShaderLibraryInstanceDeleter::OnEndFrame);
		}
	}

	FDeferredShaderLibraryInstanceDeleter() = default;

public:
	FDeferredShaderLibraryInstanceDeleter(const FDeferredShaderLibraryInstanceDeleter&) = delete;
	FDeferredShaderLibraryInstanceDeleter& operator = (const FDeferredShaderLibraryInstanceDeleter&) = delete;

	void EnqueueRelease(FShaderLibraryInstancePriorityPair&& Component)
	{
		FScopeLock ScopeLock(&Lock);
		RegisterDelegate();
		ComponentsEnqueuedForRelease.Add(MoveTemp(Component));
	}

	static FDeferredShaderLibraryInstanceDeleter& Get()
	{
		static FDeferredShaderLibraryInstanceDeleter Instance;
		return Instance;
	}
};

// FNamedShaderLibrary methods

static void LogShaderLibraryInstanceState(const FShaderLibraryInstance* InLibraryInstance, const TCHAR* InLibraryName, const TCHAR* InStateInfo)
{
	UE_LOGF(LogShaderLibrary, Display, "Cooked Context: %ls %ls Shader Library %ls",
		InStateInfo, InLibraryInstance->Library->IsNativeLibrary() ? TEXT("Native Shared") : TEXT("Shared"), InLibraryName);
}

static FString PrintComponentChunkIds(const TConstArrayView<FShaderLibraryInstancePriorityPair>& InComponents)
{
	TStringBuilder<1024> OutString;
	for (const FShaderLibraryInstancePriorityPair& Component : InComponents)
	{
		if (OutString.Len() > 0)
		{
			OutString.Append(TEXT(", "));
		}
		OutString.Appendf(TEXT("%d"), Component.Value->GetChunkId());
	}
	return FString(OutString);
}

// At runtime, open shader code collection for specified shader platform. Returns true if new code was opened
bool UE::ShaderLibrary::Private::FNamedShaderLibrary::OpenShaderCode(const FString& ShaderCodeDir, FString const& Library, int32 ChunkId, int32 Priority)
{
	LLM_SCOPE(ELLMTag::Shaders);
	// check if any of the components has this content
	{
		FRWScopeLock ReadLock(ComponentsMutex, SLT_ReadOnly);
		for (const FShaderLibraryInstancePriorityPair& Component : Components)
		{
			if (Component.Value->HasContentFrom(ShaderCodeDir, Library))
			{
				// in this context, "false" means "no new library was opened"
				ensureMsgf(Component.Key == Priority,
					TEXT("Priority mismatch of shader library '%s' between new request (%d) and existing instance (%d)"),
					*Library, Priority, Component.Key);
				return false;
			}
		}
	}

	// Any shader library instance with a lower than default priority is expected to have its shadermaps resolved before requesting shaders to be created.
	// This is the case for shader libraries from on-demand containers since all IoChunks associated with their shadermaps need to be resolved before any CreateShader() requests are permitted.
	const bool bRequireResolvedShaderMaps = Priority < 0;
	FShaderLibraryInstance* LibraryInstance = FShaderLibraryInstance::Create(ShaderPlatform, ShaderCodeDir, Library, ChunkId, bRequireResolvedShaderMaps);
	if (LibraryInstance == nullptr)
	{
		UE_LOGF(LogShaderLibrary, Verbose, "Cooked Context: No Shared Shader Library for: %ls and native library not supported.", *Library);
		// PVS reports "The function was exited without releasing the 'LibraryInstance' pointer. A memory leak is possible", which is totally bogus here
		return false;
	}

	FRWScopeLock WriteLock(ComponentsMutex, SLT_Write);

	// re-check that no one has added the same library while we were creating it. If so, delete ours
	for (const FShaderLibraryInstancePriorityPair& Component : Components)
	{
		if (Component.Value->HasContentFrom(ShaderCodeDir, Library))
		{
			// in this context, "false" means "no new library was opened"
			if (bUseFixForDeferredShaderLibraryDeletion)
			{
				// Defer deletion of the library instance to the game thread - the FShaderLibraryInstance destructor calls FlushRenderingCommands(), which must not run on a background thread.
				FShaderLibraryInstancePriorityPair Pair(Priority, LibraryInstance);
				FDeferredShaderLibraryInstanceDeleter::Get().EnqueueRelease(MoveTemp(Pair));
			}
			else
			{
				delete LibraryInstance;
			}
			return false;
		}
	}

	LogShaderLibraryInstanceState(LibraryInstance, *Library, TEXT("Loaded"));

	// Insertion sort of new component by their priority.
	// This is mandatory because some shaders can be grouped across different IoChunks, some of which can be packaged into an IAD and a non-IAD container respectively.
	// When such shaders are requested from systems that don't handle deferred shader downloads (unlike IAD asset),
	// we need to ensure that shader is loaded from the non-IAD container, which is why they are assigned a higher priority.
	const int32 InsertIndex = Components.IndexOfByPredicate([Priority](const FShaderLibraryInstancePriorityPair& Entry) -> bool { return Priority > Entry.Key; });
	if (InsertIndex == INDEX_NONE)
	{
		Components.Emplace(FShaderLibraryInstancePriorityPair(Priority, LibraryInstance));
	}
	else
	{
		Components.Insert(FShaderLibraryInstancePriorityPair(Priority, LibraryInstance), InsertIndex);
	}

	// Log new set of component IDs if verbose logging is enabled
	UE_LOGF(LogShaderLibrary, Verbose, "OpenShaderCode('%ls'): Components=(%ls)", *Library, *PrintComponentChunkIds(Components));

	return true;
}

void UE::ShaderLibrary::Private::FNamedShaderLibrary::CloseShaderCode(const FString& ShaderCodeDir, FString const& Library, int32 ChunkId)
{
	LLM_SCOPE(ELLMTag::Shaders);

	FRWScopeLock WriteLock(ComponentsMutex, SLT_Write);
	for (int32 Index = 0; Index < Components.Num(); ++Index)
	{
		if (const FShaderLibraryInstance* LibraryInstance = Components[Index].Value.Get())
		{
			if (LibraryInstance->Library->GetName() == Library && LibraryInstance->GetChunkId() == ChunkId)
			{
				LogShaderLibraryInstanceState(LibraryInstance, *Library, TEXT("Released"));

				// Give the FRHIShaderLibrary an opportunity to do cleanup before we unmount the pak file at the end of the frame.
				LibraryInstance->Library->OnCloseShaderCode();
				// Defer the actual deletion of the shader library instance to the end of a frame, since some RHI resources might still hold on to their shaders in the current frame.
				FDeferredShaderLibraryInstanceDeleter::Get().EnqueueRelease(MoveTemp(Components[Index]));

				// Remove component. Ensure we preserve the order since non-IAD components need to be iterated before IAD components.
				Components.RemoveAt(Index);

				// Log new set of component IDs if verbose logging is enabled
				UE_LOGF(LogShaderLibrary, Verbose, "CloseShaderCode('%ls'): Components=(%ls)", *Library, *PrintComponentChunkIds(Components));

				return;
			}
		}
	}
}

void UE::ShaderLibrary::Private::FNamedShaderLibrary::ForEachShaderLibraryWithShaderMap(const FShaderHash& Hash, TFunctionRef<void(FShaderLibraryInstance* InLibraryInstance, int32 InShaderMapIndex)> InPerShaderMapCallback)
{
	FRWScopeLock ReadLock(ComponentsMutex, SLT_ReadOnly);

	for (const FShaderLibraryInstancePriorityPair& Instance : Components)
	{
		const int32 ShaderMapIndex = Instance.Value->Library->FindShaderMapIndex(Hash);
		if (ShaderMapIndex != INDEX_NONE)
		{
			InPerShaderMapCallback(Instance.Value.Get(), ShaderMapIndex);
		}
	}
}

FShaderLibraryInstance* UE::ShaderLibrary::Private::FNamedShaderLibrary::FindShaderLibraryForShaderMap(const FShaderHash& Hash, int32& OutShaderMapIndex, const FShaderLibraryInstance* SkipLibraryInstance)
{
	FRWScopeLock ReadLock(ComponentsMutex, SLT_ReadOnly);

	for (const FShaderLibraryInstancePriorityPair& Instance : Components)
	{
		if (Instance.Value.Get() != SkipLibraryInstance)
		{
			const int32 ShaderMapIndex = Instance.Value->Library->FindShaderMapIndex(Hash);
			if (ShaderMapIndex != INDEX_NONE)
			{
				OutShaderMapIndex = ShaderMapIndex;
				return Instance.Value.Get();
			}
		}
	}
	return nullptr;
}

FShaderLibraryInstance* UE::ShaderLibrary::Private::FNamedShaderLibrary::FindShaderLibraryForShader(const FShaderHash& Hash, int32& OutShaderIndex)
{
	FRWScopeLock ReadLock(ComponentsMutex, SLT_ReadOnly);

	for (const FShaderLibraryInstancePriorityPair& Instance : Components)
	{
		const int32 ShaderIndex = Instance.Value->Library->FindShaderIndex(Hash);
		if (ShaderIndex != INDEX_NONE)
		{
			OutShaderIndex = ShaderIndex;
			return Instance.Value.Get();
		}
	}
	return nullptr;
}

uint32 UE::ShaderLibrary::Private::FNamedShaderLibrary::GetShaderCount(void)
{
	FRWScopeLock ReadLock(ComponentsMutex, SLT_ReadOnly);

	int ShaderCount = 0;
	for (const FShaderLibraryInstancePriorityPair& Instance : Components)
	{
		ShaderCount += Instance.Value->Library->GetNumShaders();
	}
	return ShaderCount;
}

#if UE_SHADERLIB_WITH_INTROSPECTION
void UE::ShaderLibrary::Private::FNamedShaderLibrary::DumpLibraryContents(const FString& Prefix)
{
	FRWScopeLock ReadLock(ComponentsMutex, SLT_ReadOnly);

	int32 ComponentIdx = 0;
	for (const FShaderLibraryInstancePriorityPair& Instance : Components)
	{
		UE_LOGF(LogShaderLibrary, Display, "%lsComponent %d: Native=%ls Shaders: %d Name: %ls",
			*Prefix, ComponentIdx, Instance.Value->Library->IsNativeLibrary() ? TEXT("yes") : TEXT("no"), Instance.Value->GetNumShaders(), *Instance.Value->Library->GetName() );
		++ComponentIdx;
	}
}
#endif

FAutoConsoleCommandWithArgsAndOutputDevice GListShaderLibrariesCmd(
	TEXT("ListShaderLibraries"),
	TEXT("Spits out a csv table containing stats of all shader libraries"),
	FConsoleCommandWithArgsAndOutputDeviceDelegate::CreateStatic(
		[](const TArray<FString>& Params, FOutputDevice& Out)
		{
			auto Iter = [&](const FString& LogicalName, UE::ShaderLibrary::Private::FNamedShaderLibrary& NamedShaderLibrary)
			{
				FRWScopeLock ComponentReadLock(NamedShaderLibrary.ComponentsMutex, SLT_ReadOnly);
				for (const FShaderLibraryInstancePriorityPair& ShaderLibraryPair : NamedShaderLibrary.Components)
				{
					FShaderLibraryInstance* ShaderLibrary = ShaderLibraryPair.Value.Get();
					const FString& Name = ShaderLibrary->Library->GetName();
					const FString OwnerName = ShaderLibrary->Library->GetOwnerName().ToString();
					uint32 Id = ShaderLibrary->Library->GetId();
					int32 NumShaders = ShaderLibrary->GetNumShaders();
					int32 NumShaderMaps = ShaderLibrary->GetNumResources();
					uint32 LibrarySize = ShaderLibrary->GetSizeBytes();
					uint32 RHILibrarySize = ShaderLibrary->Library->GetSizeBytes();
					uint32 MapsSize = ShaderLibrary->GetShaderMapsSizeBytes();

					Out.Logf(TEXT("%s,%s,%s,%x,%d,%d,%.3f,%.3f,%.3f"),
						*Name,
						*LogicalName,
						*OwnerName,
						Id,
						NumShaders,
						NumShaderMaps,
						LibrarySize / 1024.f,
						RHILibrarySize / 1024.f,
						MapsSize / 1024.f
					);
				}
			};

			if (FShaderLibrariesCollection* Collection = FShaderLibrariesCollection::Impl)
			{
				Out.Logf(TEXT("ShaderLibraryName,LogicalName,OwnerName,Id,NumShaders,NumShaderMaps,LibrarySizeKb,RHILibrarySizeKb,ShaderMapsSizeKb"));
				Collection->IterateNamedShaderLibrariesSafe(Iter);
			}
			else
			{
				UE_LOGF(LogShaderLibrary, Warning, "ShaderLibrariesCollection is not available.");
			}
		}));
