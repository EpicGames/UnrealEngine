// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ShaderCodeLibraryUtilities.cpp: Shared utility functions for the shader code library
=============================================================================*/

#include "ShaderCodeLibraryUtilities.h"

#include "HAL/IConsoleManager.h"
#include "HAL/PlatformFileManager.h"
#include "IO/IoDispatcher.h"
#include "IO/IoDispatcherInternal.h"
#include "Misc/Paths.h"
#include "ShaderCompilerCore.h" // for GetBuildMachineArtifactBasePath()

#if WITH_EDITOR
#include "Cooker/CookArtifactReader.h"
#endif

namespace UE::ShaderLibrary::Private
{
	using ESaveToDiskTarget = ShaderCodeArchive::ESaveToDiskTarget;

	const uint32 GShaderCodeArchiveVersion = 3;

	const FString ShaderExtension = ".ushaderbytecode";
	const FString CookCacheShaderExtension = ".ucacheshaderbytecode";
	const FString ShaderAssetInfoExtension = ".assetinfo.json";
	const FString ShaderTypeInfoExtension = ".stinfo";
	const FString StableExtension = ".shk";
	const FString PipelineExtension = ".ushaderpipelines";

	static const FString CookCacheSuffix = "_CookCache";

	// ShaderCode(_CookCache)-LibraryName-Platform.u(cache)shaderbytecode
	const FString ShaderCodePatternStr = "^ShaderCode(?:_CookCache)?\\-([\\w-]+)\\-([^-]+-[^-]+)\\.(?:ushaderbytecode|ucacheshaderbytecode)$";
	const FString ShaderArchivePatternStr = "^ShaderArchive(?:_CookCache)?\\-([\\w-]+)\\-([^-]+-[^-]+)\\.(?:ushaderbytecode|ucacheshaderbytecode)$";
	const FString StableInfoPatternStr = "^ShaderStableInfo\\-([\\w-]+)\\-([^-]+)\\.shk$";
	const FString ShaderFileForChunkPatternStr = "Shader.*_Chunk[0-9]+-.*\\.(ushaderbytecode|assetinfo.json|stinfo|ushaderpipelines)";

	//@todo-lh - Rename to r.ShaderCodeLibrary.PrintExtendedStats for consistency
	int32 GProduceExtendedStats = 1;
	static FAutoConsoleVariableRef CVarShaderLibraryProduceExtendedStats(
		TEXT("r.ShaderLibrary.PrintExtendedStats"),
		GProduceExtendedStats,
		TEXT("if != 0, shader library will produce extended stats, including the textual representation"),
		ECVF_Default
	);

	int32 GShaderCodeLibrarySeparateLoadingCache = 0;
	static FAutoConsoleVariableRef CVarShaderCodeLibrarySeparateLoadingCache(
		TEXT("r.ShaderCodeLibrary.SeparateLoadingCache"),
		GShaderCodeLibrarySeparateLoadingCache,
		TEXT("if > 0, each shader code library has it's own loading cache."),
		ECVF_Default
	);

	int32 GPreloadShaderMaps = 1;
	static FAutoConsoleVariableRef CVarShaderCodeLibraryPreloadShaderMaps(
		TEXT("r.ShaderCodeLibrary.PreloadShaderMaps"),
		GPreloadShaderMaps,
		TEXT("If > 0, shader maps will be preloaded at package/resource load time."),
		ECVF_Default
	);

	bool GShaderMapResourceRef = false;
	static FAutoConsoleVariableRef CVarShaderMapResourceRef(
		TEXT("r.ShaderCodeLibrary.ShaderMapResourceRef"),
		GShaderMapResourceRef,
		TEXT("Track reference to the shader group for different shadermaps.\n")
		TEXT("Normally used when dynamic shader preloading is enable to make sure we dont unload a shader group shared by two different shadermaps"),
		ECVF_Default
	);

#if WITH_EDITOR
	UE::Cook::ICookArtifactReader* CookArtifactReader = nullptr;
#endif // WITH_EDITOR

	FString GetShaderCodeIntermediatePath()
	{
		return FPaths::Combine(FPaths::ProjectSavedDir(), TEXT("Shaders"));
	}

	FString GetCodeArchiveFilename(const FString& BaseDir, const FString& LibraryName, FName Platform, ESaveToDiskTarget Target)
	{
		switch (Target)
		{
		case ESaveToDiskTarget::Staging:
			return BaseDir / FString::Printf(TEXT("ShaderArchive-%s-"), *LibraryName) + Platform.ToString() + ShaderExtension;
		case ESaveToDiskTarget::CookCache:
			return BaseDir / FString::Printf(TEXT("ShaderArchive%s-%s-"), *CookCacheSuffix, *LibraryName) + Platform.ToString() + CookCacheShaderExtension;
		default:
			checkNoEntry();
			return FString();
		}
	}

	FString GetStableInfoArchiveFilename(const FString& BaseDir, const FString& LibraryName, FName Platform)
	{
		return BaseDir / FString::Printf(TEXT("ShaderStableInfo-%s-"), *LibraryName) + Platform.ToString() + StableExtension;
	}

	FString GetShaderCodeFilename(const FString& BaseDir, const FString& LibraryName, FName Platform, ESaveToDiskTarget Target)
	{
		switch (Target)
		{
		case ESaveToDiskTarget::Staging:
			return BaseDir / FString::Printf(TEXT("ShaderCode-%s-"), *LibraryName) + Platform.ToString() + ShaderExtension;
		case ESaveToDiskTarget::CookCache:
			return BaseDir / FString::Printf(TEXT("ShaderCode%s-%s-"), *CookCacheSuffix, *LibraryName) + Platform.ToString() + CookCacheShaderExtension;
		default:
			checkNoEntry();
			return FString();
		}
	}

	FString GetShaderAssetInfoFilename(const FString& BaseDir, const FString& LibraryName, FName Platform, ESaveToDiskTarget Target)
	{
		switch (Target)
		{
		case ESaveToDiskTarget::Staging:
			return BaseDir / FString::Printf(TEXT("ShaderAssetInfo-%s-"), *LibraryName) + Platform.ToString() + ShaderAssetInfoExtension;
		case ESaveToDiskTarget::CookCache:
			return BaseDir / FString::Printf(TEXT("ShaderAssetInfo%s-%s-"), *CookCacheSuffix, *LibraryName) + Platform.ToString() + ShaderAssetInfoExtension;
		default:
			checkNoEntry();
			return FString();
		}
	}

	FString GetShaderTypeInfoFilename(const FString& BaseDir, const FString& LibraryName, FName Platform, ESaveToDiskTarget Target)
	{
		switch (Target)
		{
		case ESaveToDiskTarget::Staging:
			return BaseDir / FString::Printf(TEXT("ShaderTypeInfo-%s-"), *LibraryName) + Platform.ToString() + ShaderTypeInfoExtension;
		case ESaveToDiskTarget::CookCache:
			return BaseDir / FString::Printf(TEXT("ShaderTypeInfo%s-%s-"), *CookCacheSuffix, *LibraryName) + Platform.ToString() + ShaderTypeInfoExtension;
		default:
			checkNoEntry();
			return FString();
		}
	}

	FString GetShaderDebugFolder(const FString& BaseDir, const FString& LibraryName, FName Platform)
	{
		return BaseDir / FString::Printf(TEXT("ShaderDebug-%s-"), *LibraryName) + Platform.ToString();
	}

	/** Helper function shared between the cooker and runtime */
	FString GetShaderLibraryNameForChunk(FString const& BaseName, int32 ChunkId)
	{
		if (ChunkId == INDEX_NONE)
		{
			return BaseName;
		}
		return FString::Printf(TEXT("%s_Chunk%d"), *BaseName, ChunkId);
	}

	FArchive* CreateShaderFileReader(const TCHAR* Filename)
	{
#if WITH_EDITOR
		if (CookArtifactReader)
		{
			return CookArtifactReader->CreateFileReader(Filename);
		}
#endif
		return IFileManager::Get().CreateFileReader(Filename);
	}

	void ShaderFindFiles(TArray<FString>& FoundFiles, const TCHAR* Directory, const TCHAR* FileExtension)
	{
#if WITH_EDITOR
		if (CookArtifactReader)
		{
			return CookArtifactReader->FindFiles(FoundFiles, Directory, FileExtension);
		}
#endif
		return IFileManager::Get().FindFiles(FoundFiles, Directory, FileExtension);
	}

	void ShaderFindFiles(TArray<FString>& Result, const TCHAR* Filename, bool Files, bool Directories)
	{
#if WITH_EDITOR
		if (CookArtifactReader)
		{
			return CookArtifactReader->FindFiles(Result, Filename, Files, Directories);
		}
#endif
		return IFileManager::Get().FindFiles(Result, Filename, Files, Directories);
	}

	bool IsRunningWithPakFile()
	{
		static const bool bRunningWithPakFile =
			(FPlatformFileManager::Get().FindPlatformFile(TEXT("PakFile")) != nullptr);
		return bRunningWithPakFile;
	}

	bool IsRunningWithIoStore()
	{
		static const bool bRunningWithIoStore =
			FIoDispatcher::IsInitialized() && FIoDispatcherInternal::HasPackageData();
		return bRunningWithIoStore;
	}

	bool IsRunningWithZenStore()
	{
		static const bool bRunningWithZenStore =
			FPlatformFileManager::Get().FindPlatformFile(TEXT("StorageServer")) != nullptr;
		return bRunningWithZenStore;
	}

	bool ShouldLookForLooseCookedChunks()
	{
		return IsRunningWithZenStore() || !IsRunningWithIoStore();
	}

} // UE::ShaderLibrary::Private
