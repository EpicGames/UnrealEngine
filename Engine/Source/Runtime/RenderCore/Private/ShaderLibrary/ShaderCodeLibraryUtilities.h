// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ShaderCodeLibraryUtilities.h: Shared utility functions for the shader code library
=============================================================================*/

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Serialization/Archive.h"
#include "UObject/NameTypes.h"
#include "ShaderLibrary/ShaderCodeArchiveInternal.h"

namespace UE::Cook { class ICookArtifactReader; }

namespace UE::ShaderLibrary::Private
{
	// Compile time switch to enable the new feature of pruning stale shader data from incremental cooks.
	// Remove this switch once the feature is stable enough not to need an off switch.
	constexpr bool bIncrementalCookPruningEnabled = true;

	// Shader library constants:
	extern const uint32 GShaderCodeArchiveVersion;

	extern const FString ShaderExtension;
	extern const FString CookCacheShaderExtension;
	extern const FString ShaderAssetInfoExtension;
	extern const FString ShaderTypeInfoExtension;
	extern const FString StableExtension;
	extern const FString PipelineExtension;

	extern const FString ShaderCodePatternStr;
	extern const FString ShaderArchivePatternStr;
	extern const FString StableInfoPatternStr;
	extern const FString ShaderFileForChunkPatternStr;

	// Shader library CVars:
	extern int32 GProduceExtendedStats;
	extern int32 GShaderCodeLibrarySeparateLoadingCache;
	extern int32 GPreloadShaderMaps;
	extern bool GShaderMapResourceRef;
	extern bool GUploadShaderDebugArtifacts;

	// Shader library intermediates:
#if WITH_EDITOR
	extern UE::Cook::ICookArtifactReader* CookArtifactReader;
#endif // WITH_EDITOR

	// Shader library utility functions:
	FString GetShaderCodeIntermediatePath();
	FString GetCodeArchiveFilename(const FString& BaseDir, const FString& LibraryName, FName Platform, ShaderCodeArchive::ESaveToDiskTarget Target);
	FString GetStableInfoArchiveFilename(const FString& BaseDir, const FString& LibraryName, FName Platform);
	FString GetShaderCodeFilename(const FString& BaseDir, const FString& LibraryName, FName Platform, ShaderCodeArchive::ESaveToDiskTarget Target);
	FString GetShaderAssetInfoFilename(const FString& BaseDir, const FString& LibraryName, FName Platform, ShaderCodeArchive::ESaveToDiskTarget Target);
	FString GetShaderTypeInfoFilename(const FString& BaseDir, const FString& LibraryName, FName Platform, ShaderCodeArchive::ESaveToDiskTarget Target);
	FString GetShaderDebugFolder(const FString& BaseDir, const FString& LibraryName, FName Platform);

	/** Helper function shared between the cooker and runtime */
	FString GetShaderLibraryNameForChunk(FString const& BaseName, int32 ChunkId);

	FArchive* CreateShaderFileReader(const TCHAR* Filename);
	void ShaderFindFiles(TArray<FString>& FoundFiles, const TCHAR* Directory, const TCHAR* FileExtension);
	void ShaderFindFiles(TArray<FString>& Result, const TCHAR* Filename, bool Files, bool Directories);

	bool IsRunningWithPakFile();
	bool IsRunningWithIoStore();
	bool IsRunningWithZenStore();
	bool ShouldLookForLooseCookedChunks();

} // UE::ShaderLibrary::Private
