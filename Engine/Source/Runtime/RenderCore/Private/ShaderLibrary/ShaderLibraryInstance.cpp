// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ShaderLibraryInstance.cpp: 
=============================================================================*/

#include "ShaderLibraryInstance.h"

#include "HAL/PlatformFileManager.h"
#include "Misc/ScopeRWLock.h"
#include "ProfilingDebugging/AssetMetadataTrace.h"
#include "ProfilingDebugging/LoadTimeTracker.h"
#include "ProfilingDebugging/DiagnosticTable.h"
#include "RHIShaderLibrary.h"
#include "ShaderCodeArchive.h"
#include "ShaderLibrary/ShaderCodeLibraryInternal.h"
#include "ShaderLibrary/ShaderCodeLibraryUtilities.h"
#include "UObject/NameTypes.h"


FShaderLibraryInstance* FShaderLibraryInstance::Create(EShaderPlatform InShaderPlatform, const FString& ShaderCodeDir, FString const& InLibraryName, int32 InChunkId, bool bInRequireResolvedShaderMaps)
{
	using namespace UE::ShaderLibrary::Private;

	LLM_SCOPE(ELLMTag::Shaders);
	UE_TRACE_METADATA_SCOPE_ASSET_FNAME(FName(*InLibraryName), FName("ShaderLibraryInstance"), FName(*TStringBuilder<520>(InPlace, ShaderCodeDir, TEXT("/"), InLibraryName)));

	FRHIShaderLibraryRef Library;
	FString ShaderCodeDirectory;
	if (RHISupportsNativeShaderLibraries(InShaderPlatform))
	{
		Library = RHICreateShaderLibrary(InShaderPlatform, ShaderCodeDir, InLibraryName);
		ShaderCodeDirectory = ShaderCodeDir;
	}

	if (!Library && UE::ShaderLibrary::Private::IsRunningWithIoStore())
	{
		Library = FIoStoreShaderCodeArchive::Create(InShaderPlatform, InLibraryName, FIoDispatcher::Get());
		ShaderCodeDirectory.Empty();	// paths don't matter for IoStore-based libraries
	}

	// Shader library as a ushaderbytecode file is no longer an option for distribution. Some cases (a build using loose files) still require its support though.
	if (!Library && UE::ShaderLibrary::Private::ShouldLookForLooseCookedChunks())
	{
		const FName PlatformName = FDataDrivenShaderPlatformInfo::GetName(InShaderPlatform);
		const FName ShaderFormatName = LegacyShaderPlatformToShaderFormat(InShaderPlatform);
		FString ShaderFormatAndPlatform = ShaderFormatName.ToString() + TEXT("-") + PlatformName.ToString();

		const FString DestFilePath = GetCodeArchiveFilename(ShaderCodeDir, InLibraryName, FName(ShaderFormatAndPlatform), ShaderCodeArchive::ESaveToDiskTarget::Staging);
		TUniquePtr<FArchive> Ar(CreateShaderFileReader(*DestFilePath));
		if (Ar)
		{
			if (FSerializedShaderArchive::SerializeHeaderVersion(*Ar))
			{
				Library = FShaderCodeArchive::Create(InShaderPlatform, *Ar, DestFilePath, ShaderCodeDir, InLibraryName);
				if (Library)
				{
					ShaderCodeDirectory = ShaderCodeDir;

					bool ShaderCodeLibrarySeparateLoadingCacheCommandLineOverride = FParse::Param(FCommandLine::Get(), TEXT("ShaderCodeLibrarySeparateLoadingCache"));;
					if (GShaderCodeLibrarySeparateLoadingCache || ShaderCodeLibrarySeparateLoadingCacheCommandLineOverride)
					{
						TArray<TArray<FString>> FilesToMakeUnique;
						FilesToMakeUnique.AddDefaulted(1);
						FilesToMakeUnique[0].Add(DestFilePath);
						FPlatformFileManager::Get().GetPlatformFile().MakeUniquePakFilesForTheseFiles(FilesToMakeUnique);
					}
				}
			}
		}
	}

	if (!Library)
	{
		return nullptr;
	}

	FShaderLibraryInstance* Instance = new FShaderLibraryInstance();
	Instance->Library = Library;
	Instance->ShaderCodeDirectory = ShaderCodeDirectory;
	Instance->ChunkId = InChunkId;
	Instance->bRequireResolvedShaderMaps = bInRequireResolvedShaderMaps;

	const int32 NumResources = Library->GetNumShaderMaps();
	Instance->Resources.AddZeroed(NumResources);

	INC_DWORD_STAT_BY(STAT_Shaders_ShaderResourceMemory, Instance->GetSizeBytes());

	return Instance;
}

FShaderLibraryInstance::~FShaderLibraryInstance()
{
	// Ensure RHI shaders are no longer used before attempting to move them to a new shader library instance
	checkf(IsInGameThread(), TEXT("Shader library closure is expected to happen only on the game thread, at the \'top\' of the pipeline"));
	FlushRenderingCommands();

	// release RHI resources on all of the resources (note: this has nothing to do with their own lifetime and refcount)
	for (FShaderMapResource_SharedCode* Resource : Resources)
	{
		if (Resource)
		{
			const int32 NumRefs = Resource->GetNumRefs();
			if (NumRefs > 0)
			{
				// Try to find new shader library instance to take ownership
				if (!FShaderCodeLibraryInternal::MoveShaderMapResourceOwnership(Resource))
				{
					UE_LOGF(LogShaderLibrary, Warning, "Failed to move shadermap resource '%ls' (ShaderMapIndex=%d, Owner='%ls') to a new library instance, but its resource is still referenced (Refcount=%d).",
						*LexToString(Resource->GetShaderMapHash()), Resource->ShaderMapIndex, *Resource->GetOwnerName().ToString(), NumRefs);

					// If no new shader library was found, something is still holding on to this resource which it shouldn't.
					// If the reference was leaked, we'll only see the warning above. Otherwise, we'll see another warning or failure later on.
					// Either way, this resource must be released or it could attempt to create shaders from a library that's being destroyed.
					BeginReleaseResource(Resource);
				}
			}
			else
			{
				BeginReleaseResource(Resource);
			}
		}
	}

	// if rendering thread is active, the actual teardown may happen later, so flush the rendering commands here.
	// this will also flush pending deletes
	FlushRenderingCommands();

	Library->Teardown();
	DEC_DWORD_STAT_BY(STAT_Shaders_ShaderResourceMemory, GetSizeBytes());
}

uint32 FShaderLibraryInstance::GetSizeBytes()
{
	uint32 ShaderBucketsSize = 0;
	for (int32 IdxBucket = 0, NumBuckets = UE_ARRAY_COUNT(RHIShaders); IdxBucket < NumBuckets; ++IdxBucket)
	{
		FRWScopeLock Locker(ShaderLocks[IdxBucket], SLT_ReadOnly);
		ShaderBucketsSize += RHIShaders[IdxBucket].GetAllocatedSize();
	}
	return sizeof(*this) + ShaderBucketsSize + Resources.GetAllocatedSize();
}

uint32 FShaderLibraryInstance::GetShaderMapsSizeBytes()
{
	uint32 ShaderMapsSize = 0;
	FRWScopeLock Locker(ResourceLock, SLT_ReadOnly);
	for (FShaderMapResource_SharedCode* Resource : Resources)
	{
		if (Resource)
		{
			ShaderMapsSize += Resource->GetSizeBytes();
		}
	}
	return ShaderMapsSize;
}

int32 FShaderLibraryInstance::GetNumShadersForShaderMap(int32 ShaderMapIndex) const
{
	return Library->GetNumShadersForShaderMap(ShaderMapIndex);
}

void FShaderLibraryInstance::PreloadShader(int32 ShaderIndex, FArchive* Ar)
{
	SCOPED_LOADTIMER(FShaderLibraryInstance_PreloadShader);
	FGraphEventArray PreloadCompletionEvents;
	Library->PreloadShader(ShaderIndex, PreloadCompletionEvents);
	if (Ar && PreloadCompletionEvents.Num() > 0)
	{
		FExternalReadCallback ExternalReadCallback = [this, PreloadCompletionEvents = MoveTemp(PreloadCompletionEvents)](double ReaminingTime)
			{
				return this->OnExternalReadCallback(PreloadCompletionEvents, ReaminingTime);
			};
		Ar->AttachExternalReadDependency(ExternalReadCallback);
	}
}

void FShaderLibraryInstance::PreloadShader(int32 ShaderIndex, FGraphEventArray& OutCompletionEvents)
{
	LLM_SCOPE(ELLMTag::Shaders);
	SCOPED_LOADTIMER(FShaderLibraryInstance_PreloadShader);

	const int32 BucketIndex = ShaderIndex % NumShaderLocks;

	// Don't preload if we already have the shader or we already preloaded it.
	FRWScopeLock Locker(ShaderLocks[BucketIndex], SLT_Write);
	FCachedRHIShader& Shader = RHIShaders[BucketIndex].FindOrAdd(ShaderIndex, FCachedRHIShader());
	if (Shader.PreloadingState == EPreloadingState::NotPreloaded)
	{
		Library->PreloadShader(ShaderIndex, OutCompletionEvents);
		Shader.PreloadingState = EPreloadingState::Preloaded;
	}
	else if (Shader.PreloadingState == EPreloadingState::Preloaded)
	{
		// Check if still preloading, which will get the completion events if they are still outstanding.
		Library->IsPreloading(ShaderIndex, OutCompletionEvents);
	}
}

void FShaderLibraryInstance::ReleasePreloadedShader(int32 ShaderIndex)
{
	SCOPED_LOADTIMER(FShaderLibraryInstance_PreloadShader);
	Library->ReleasePreloadedShader(ShaderIndex);
}

TRefCountPtr<FShaderMapResource_SharedCode> FShaderLibraryInstance::GetResource(int32 ShaderMapIndex)
{
	FRWScopeLock Locker(ResourceLock, SLT_ReadOnly);
	return Resources[ShaderMapIndex];
}

bool FShaderLibraryInstance::IsShaderMapResourceResolved(int32 ShaderMapIndex) const
{
	// If this library instance loads its shaders on-demand, then we expect that all IoChunks for this shader map have already been resolved.
	// Otherwise, this will fail to create shaders when the chunks are requested to be installed too late.
	return !bRequireResolvedShaderMaps || Library->IsShaderMapResolved(ShaderMapIndex);
}

TRefCountPtr<FShaderMapResource_SharedCode> FShaderLibraryInstance::AddResource(int32 ShaderMapIndex, FArchive* Ar)
{
	if (!IsShaderMapResourceResolved(ShaderMapIndex))
	{
		return nullptr;
	}

	TRefCountPtr<FShaderMapResource_SharedCode> OutResource;
	bool bPreload = false;
	{
		FRWScopeLock Locker(ResourceLock, SLT_Write);
		FShaderMapResource_SharedCode* PrevResource = Resources[ShaderMapIndex];
		if (!PrevResource)
		{
			Resources[ShaderMapIndex] = new FShaderMapResource_SharedCode(this, ShaderMapIndex);
			OutResource = Resources[ShaderMapIndex];
			bPreload = !GRHILazyShaderCodeLoading && UE::ShaderLibrary::Private::GPreloadShaderMaps;
		}
		else
		{
			OutResource = PrevResource;
		}
	}

	if (bPreload)
	{
		SCOPED_LOADTIMER(FShaderLibraryInstance_PreloadShaderMap);
		FGraphEventArray PreloadCompletionEvents;
		OutResource->bEntireShaderMapPreloaded = Library->PreloadShaderMap(ShaderMapIndex, PreloadCompletionEvents);
		if (Ar && PreloadCompletionEvents.Num() > 0)
		{
			FExternalReadCallback ExternalReadCallback = [this, PreloadCompletionEvents = MoveTemp(PreloadCompletionEvents)](double ReaminingTime)
				{
					return this->OnExternalReadCallback(PreloadCompletionEvents, ReaminingTime);
				};
			Ar->AttachExternalReadDependency(ExternalReadCallback);
		}
	}

	return OutResource;
}

bool FShaderLibraryInstance::TryInsertResource(FShaderMapResource_SharedCode* InResource)
{
	check(InResource);
	check(InResource->LibraryInstance == this);

	// Check if this entry is already occupied
	{
		FRWScopeLock Locker(ResourceLock, SLT_ReadOnly);
		check(InResource->ShaderMapIndex < Resources.Num());
		if (Resources[InResource->ShaderMapIndex] != nullptr)
		{
			return false;
		}
	}

	// Insert resource entry
	{
		FRWScopeLock Locker(ResourceLock, SLT_Write);
		// Check again since the slot might have been lost between the previous read-lock and this write-lock.
		if (Resources[InResource->ShaderMapIndex] != nullptr)
		{
			return false;
		}
		Resources[InResource->ShaderMapIndex] = InResource;
	}

	return true;
}

bool FShaderLibraryInstance::TryRemoveResource(FShaderMapResource_SharedCode* Resource, bool bForceRemoval)
{
	if (Resource->GetNumRefs() == 0 || bForceRemoval)
	{
		FRWScopeLock Locker(ResourceLock, SLT_Write);

		// If this resource is not registered in this library instance, it must have been transferred from another library for a slot that was already occupied
		const int32 ShaderMapIndex = Resource->ShaderMapIndex;
		if (Resources[ShaderMapIndex] == Resource)
		{
			Resources[ShaderMapIndex] = nullptr;
			return true;
		}
	}

	// Another thread found the resource after ref-count was decremented to zero
	return false;
}

TRefCountPtr<FRHIShader> FShaderLibraryInstance::GetOrCreateShader(int32 ShaderIndex, bool bRequired)
{
	LLM_SCOPE(ELLMTag::Shaders);

	const int32 BucketIndex = ShaderIndex % NumShaderLocks;
	TRefCountPtr<FRHIShader> RHIShader;
	{
		FRWScopeLock Locker(ShaderLocks[BucketIndex], SLT_ReadOnly);
		FCachedRHIShader* ShaderPtr = RHIShaders[BucketIndex].Find(ShaderIndex);
		if (ShaderPtr)
		{
			RHIShader = ShaderPtr->RHIShader;
		}
	}
	if (!RHIShader)
	{
		// We're going to create the shader now. Disallow preloading from this point on.
		{
			FRWScopeLock Locker(ShaderLocks[BucketIndex], SLT_Write);
			FCachedRHIShader& Shader = RHIShaders[BucketIndex].FindOrAdd(ShaderIndex, FCachedRHIShader());
			if (Shader.PreloadingState == EPreloadingState::NotPreloaded)
			{
				Shader.PreloadingState = EPreloadingState::CannotPreload;
			}
		}

		RHIShader = Library->CreateShader(ShaderIndex, bRequired);

		if (RHIShader)
		{
			FRWScopeLock Locker(ShaderLocks[BucketIndex], SLT_Write);
			FCachedRHIShader& Shader = RHIShaders[BucketIndex].FindOrAdd(ShaderIndex, FCachedRHIShader());
			if (LIKELY(Shader.RHIShader == nullptr))
			{
				Shader.RHIShader = RHIShader;

				if (Shader.PreloadingState == EPreloadingState::Preloaded)
				{
					ReleasePreloadedShader(ShaderIndex);
					Shader.PreloadingState = EPreloadingState::PreloadedAndCreated;
				}
			}
			else
			{
				RHIShader = Shader.RHIShader;
			}
		}
	}

	return RHIShader;
}

void FShaderLibraryInstance::ReleaseShader(int32 ShaderIndex)
{
	const int32 BucketIndex = ShaderIndex % NumShaderLocks;
	FRWScopeLock Locker(ShaderLocks[BucketIndex], SLT_Write);
	FCachedRHIShader* ShaderPtr = RHIShaders[BucketIndex].Find(ShaderIndex);
	FRHIShader* Shader = ShaderPtr ? ShaderPtr->RHIShader.GetReference() : nullptr;
	if (Shader)
	{
		// The library instance is holding one ref
		// External caller of this method must be holding a ref as well, so there must be at least 2 refs
		// If those are the only 2 refs, we release the ref held by the library instance, to allow the shader to be destroyed once caller releases its ref
		const uint32 NumRefs = Shader->GetRefCount();
		check(NumRefs > 1u);
		if (NumRefs == 2u)
		{
			RHIShaders[BucketIndex].Remove(ShaderIndex);
		}
	}
}

void FShaderLibraryInstance::ReleasePreloadedShaderIfNecessary(int32 ShaderIndex)
{
	const int32 BucketIndex = ShaderIndex % NumShaderLocks;
	FRWScopeLock Locker(ShaderLocks[BucketIndex], SLT_Write);
	FCachedRHIShader* ShaderPtr = RHIShaders[BucketIndex].Find(ShaderIndex);
	if (ShaderPtr && ShaderPtr->PreloadingState == EPreloadingState::Preloaded)
	{
		// We should only be here when we preload the shader but don't end up actually creating it.
		check(!ShaderPtr->RHIShader.IsValid());
		Library->ReleasePreloadedShader(ShaderIndex);
		RHIShaders[BucketIndex].Remove(ShaderIndex);
	}
}

void FShaderLibraryInstance::ResolvePackageShaderMap(int32 ShaderMapIndex, FCoreDelegates::FIoChunkIdResolvedFunc IoChunkIdResolvedFunc) const
{
	Library->ResolveShaderMap(ShaderMapIndex, IoChunkIdResolvedFunc);
}

void FShaderLibraryInstance::PreloadPackageShaderMap(int32 ShaderMapIndex, FCoreDelegates::FAttachShaderReadRequestFunc AttachShaderReadRequestFunc)
{
	if (!IsShaderMapResourceResolved(ShaderMapIndex))
	{
		return;
	}

	FRWScopeLock Locker(ResourceLock, SLT_Write);
	FShaderMapResource_SharedCode*& Resource = Resources[ShaderMapIndex];
	if (!Resource)
	{
		Resource = new FShaderMapResource_SharedCode(this, ShaderMapIndex);
		if (UE::ShaderLibrary::Private::GPreloadShaderMaps)
		{
			Resource->bEntireShaderMapPreloaded = Library->PreloadShaderMap(ShaderMapIndex, AttachShaderReadRequestFunc);
		}
		BeginInitResource(Resource);
	}
	Resource->AddRef();
}

void FShaderLibraryInstance::ReleasePreloadedPackageShaderMap(int32 ShaderMapIndex)
{
	FShaderMapResource_SharedCode* Resource = nullptr;
	{
		FRWScopeLock Locker(ResourceLock, SLT_Write);
		Resource = Resources[ShaderMapIndex];
	}

	if (Resource)
	{
		Resource->Release();
	}
}

bool FShaderLibraryInstance::HasContentFrom(const FString& ShaderCodeDir, FString const& InLibraryName) const
{
	if (Library->GetName() == InLibraryName)
	{
		// IoStore-based libraries don't care about the directory, so name collision alone is enough to say yes.
		return ShaderCodeDirectory.IsEmpty() || (ShaderCodeDirectory == ShaderCodeDir);
	}

	return false;
}

FShaderLibraryInstance::FShaderLibraryInstance()
{
	// dummy
}

bool FShaderLibraryInstance::OnExternalReadCallback(const FGraphEventArray& Events, double RemainingTime)
{
	if (Events.Num())
	{
		if (RemainingTime < 0.0)
		{
			for (const FGraphEventRef& Event : Events)
			{
				if (!Event->IsComplete()) return false;
			}
			return true;
		}
		FTaskGraphInterface::Get().WaitUntilTasksComplete(Events);
	}
	return true;
}

