// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ShaderLibraryInstance.h: 
=============================================================================*/

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "HAL/CriticalSection.h"
#include "RHIFwd.h"
#include "RHIShaderPlatform.h"
#include "Serialization/Archive.h"
#include "Shader.h"
#include "ShaderLibrary/ShaderMapResourceSharedCode.h"


class FShaderLibraryInstance
{
public:
	static FShaderLibraryInstance* Create(EShaderPlatform InShaderPlatform, const FString& ShaderCodeDir, FString const& InLibraryName, int32 InChunkId, bool bInRequireResolvedShaderMaps = false);

	~FShaderLibraryInstance();

	FORCEINLINE EShaderPlatform GetPlatform() const { return Library->GetPlatform(); }
	FORCEINLINE const int32 GetNumResources() const { return Resources.Num(); }
	FORCEINLINE const int32 GetNumShaders() const { return Library->GetNumShaders(); }

	uint32 GetSizeBytes();

	uint32 GetShaderMapsSizeBytes();

	int32 GetNumShadersForShaderMap(int32 ShaderMapIndex) const;

	void PreloadShader(int32 ShaderIndex, FArchive* Ar);

	void PreloadShader(int32 ShaderIndex, FGraphEventArray& OutCompletionEvents);

	void ReleasePreloadedShader(int32 ShaderIndex);

	TRefCountPtr<FShaderMapResource_SharedCode> GetResource(int32 ShaderMapIndex);

	TRefCountPtr<FShaderMapResource_SharedCode> AddResource(int32 ShaderMapIndex, FArchive* Ar);

	/** Tries to insert the specified shadermap resource into this library instance. */
	bool TryInsertResource(FShaderMapResource_SharedCode* Resource);

	bool TryRemoveResource(FShaderMapResource_SharedCode* Resource, bool bForceRemoval = false);

	TRefCountPtr<FRHIShader> GetOrCreateShader(int32 ShaderIndex, bool bRequired = true);

	void ReleaseShader(int32 ShaderIndex);

	void ReleasePreloadedShaderIfNecessary(int32 ShaderIndex);

	void ResolvePackageShaderMap(int32 ShaderMapIndex, FCoreDelegates::FIoChunkIdResolvedFunc IoChunkIdResolvedFunc) const;

	void PreloadPackageShaderMap(int32 ShaderMapIndex, FCoreDelegates::FAttachShaderReadRequestFunc AttachShaderReadRequestFunc);

	void ReleasePreloadedPackageShaderMap(int32 ShaderMapIndex);

	bool HasContentFrom(const FString& ShaderCodeDir, FString const& InLibraryName) const;

	/** Returns chunk id for this instance, if it is a part of a chunked named library. INDEX_NONE is returned for a monolithic lib. */
	int32 GetChunkId() const { return ChunkId; }

	/** Returns whether the specified shadermap in this library instance is considered resolved. Always true for shader libraries that represent non-IAD containers. */
	bool IsShaderMapResourceResolved(int32 ShaderMapIndex) const;

public:
	FRHIShaderLibraryRef Library;

private:
	static const int32 NumShaderLocks = 32;

	FShaderLibraryInstance();

	bool OnExternalReadCallback(const FGraphEventArray& Events, double RemainingTime);

	enum class EPreloadingState : uint8
	{
		NotPreloaded = 0, // Not preloaded.
		Preloaded = 1, // Preloaded but RHI shader not yet created (preloaded memory is still allocated).
		PreloadedAndCreated = 2, // Preloaded and RHI shader created (preloaded memory has been freed).
		CannotPreload = 3 // Not preloaded and cannot preload because the RHI shader is now being created.
	};

	struct FCachedRHIShader
	{
		EPreloadingState PreloadingState = EPreloadingState::NotPreloaded;
		TRefCountPtr<FRHIShader> RHIShader;
	};

	/** Number of shaders can be pretty large (several hundred thousands). Do not allocate memory for them upfront, but instead store them in a map.
		There's number of maps to reduce the lock contention. */
	TMap<int32, FCachedRHIShader> RHIShaders[NumShaderLocks];

	TArray<FShaderMapResource_SharedCode*> Resources;

	/** Locks that guard access to particular shader buckets. */
	FRWLock ShaderLocks[NumShaderLocks];
	FRWLock ResourceLock;

	/** Folder the library was created from (doesn't matter for IoStore-based libraries) */
	FString ShaderCodeDirectory;

	/** Chunk ID of this library, if any (INDEX_NONE if it is monolithic) */
	int32 ChunkId = INDEX_NONE;

	/** Specifies whether shadermaps must be fully resolved and ready to use when loading shadermap resources.
	    By default, resources can be loaded even when their shadermaps have not be resolved yet. */
	bool bRequireResolvedShaderMaps = false;
};
