// Copyright Epic Games, Inc. All Rights Reserved.

#include "ShaderMapAssetAssociation.h"

#if WITH_EDITORONLY_DATA

void FShaderMapAssetAssociations::ReserveAssets(int32 NumAssets)
{
	ShaderMapToAssets.Reserve(NumAssets);
	AssetToShaderMaps.Reserve(NumAssets);
}

FShaderMapAssetAssociations::FAssociatedAssetData& FShaderMapAssetAssociations::FindOrAddAsset(FName Asset, const FShaderHash& ShaderMap)
{
	FAssociatedAssetData& Result = AssetToShaderMaps.FindOrAdd(Asset);
	FShaderMapAssetPaths& Assets = ShaderMapToAssets.FindOrAdd(ShaderMap);
	bool bAlreadyExists = false;
	Assets.Add(Asset, &bAlreadyExists);
	if (!bAlreadyExists)
	{
		Result.ShaderMaps.AddUnique(ShaderMap);
	}
	check(AssetToShaderMaps.IsEmpty() == ShaderMapToAssets.IsEmpty());
	return Result;
}

void FShaderMapAssetAssociations::RemoveAsset(FName Asset)
{
	FAssociatedAssetData Removed;
	if (AssetToShaderMaps.RemoveAndCopyValue(Asset, Removed))
	{
		for (const FShaderHash& ShaderMap : Removed.ShaderMaps)
		{
			FShaderMapAssetPaths* AssetsUsingShader = ShaderMapToAssets.Find(ShaderMap);
			if (AssetsUsingShader)
			{
				AssetsUsingShader->Remove(Asset);
				if (AssetsUsingShader->IsEmpty())
				{
					ShaderMapToAssets.Remove(ShaderMap);
				}
			}
		}
	}
	check(AssetToShaderMaps.IsEmpty() == ShaderMapToAssets.IsEmpty());
}

void FShaderMapAssetAssociations::SortForSaving()
{
	ShaderMapToAssets.KeySort(
		[](const FShaderHash& A, const FShaderHash& B)
		{
			return GetTypeHash(A) < GetTypeHash(B);
		});
	for (TPair<FShaderHash, FShaderMapAssetPaths>& Pair : ShaderMapToAssets)
	{
		Pair.Value.Sort(FNameLexicalLess());
	}
}

void FShaderMapAssetAssociations::Append(const FShaderMapAssetAssociations& InOtherAssociation)
{
	//@todo-lh - This can be optimized slightly, but let's keep it straightforward for now
	for (const TPair<FName, FAssociatedAssetData>& AssetData : InOtherAssociation.ViewAssets())
	{
		for (const FShaderHash& ShaderMapHash : AssetData.Value.ShaderMaps)
		{
			this->FindOrAddAsset(AssetData.Key, ShaderMapHash);
		}
	}
}

#endif // WITH_EDITORONLY_DATA
