// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ShaderMapResourceSharedCode.cpp: 
=============================================================================*/

#include "ShaderMapResourceSharedCode.h"

#include "ProfilingDebugging/AssetMetadataTrace.h"
#include "ProfilingDebugging/LoadTimeTracker.h"
#include "ProfilingDebugging/DiagnosticTable.h"
#include "ShaderCodeLibrary.h"
#include "ShaderLibrary/ShaderCodeLibraryUtilities.h"
#include "ShaderLibrary/ShaderLibraryInstance.h"


FShaderMapResource_SharedCode::FShaderMapResource_SharedCode(FShaderLibraryInstance* InLibraryInstance, int32 InShaderMapIndex)
	: FShaderMapResource(InLibraryInstance->GetPlatform(), InLibraryInstance->GetNumShadersForShaderMap(InShaderMapIndex))
	, LibraryInstance(InLibraryInstance)
	, ShaderMapIndex(InShaderMapIndex)
	, bEntireShaderMapPreloaded(false)
{
	if (UE::ShaderLibrary::Private::GShaderMapResourceRef)
	{
		TArray<int32> ShaderGroupIndexes;
		const int32 NumShaders = GetNumShaders();
		for (int32 i = 0; i < NumShaders; ++i)
		{
			const int32 LibraryShaderIndex = LibraryInstance->Library->GetShaderIndex(ShaderMapIndex, i);
			const int32 ShaderGroupIndex = LibraryInstance->Library->GetGroupIndexForShader(LibraryShaderIndex);
			ShaderGroupIndexes.AddUnique(ShaderGroupIndex);
		}

		for (int32 ShaderGroupIndex : ShaderGroupIndexes)
		{
			LibraryInstance->Library->AddRefPreloadedShaderGroup(ShaderGroupIndex);
		}
	}
}

FShaderMapResource_SharedCode::~FShaderMapResource_SharedCode()
{
	// dummy
}

FShaderHash FShaderMapResource_SharedCode::GetShaderHash(int32 ShaderIndex)
{
	return LibraryInstance->Library->GetShaderHash(ShaderMapIndex, ShaderIndex);
}

FRHIShader* FShaderMapResource_SharedCode::CreateRHIShaderOrCrash(int32 ShaderIndex, bool bRequired)
{
	SCOPED_LOADTIMER(FShaderMapResource_SharedCode_InitRHI);

	if (UNLIKELY(LibraryInstance == nullptr))
	{
		if(bRequired)
		{
			UE_LOGF(LogShaders, Fatal, "FShaderMapResource_SharedCode::CreateRHIShaderOrCrash: LibraryInstance is null");
		}
		return nullptr;
	}

	const int32 LibraryShaderIndex = LibraryInstance->Library->GetShaderIndex(ShaderMapIndex, ShaderIndex);
	TRefCountPtr<FRHIShader> CreatedShader = LibraryInstance->GetOrCreateShader(LibraryShaderIndex, bRequired);
	if (UNLIKELY(CreatedShader == nullptr))
	{
		if (bRequired)
		{
			UE_LOGF(LogShaders, Fatal, "FShaderMapResource_SharedCode::InitRHI is unable to create a shader");
		}

		return nullptr;
	}

	CreatedShader->AddRef();
	return CreatedShader;
}

uint32 FShaderMapResource_SharedCode::GetShaderSizeBytes(int32 ShaderIndex) const
{
	const int32 LibraryShaderIndex = LibraryInstance->Library->GetShaderIndex(ShaderMapIndex, ShaderIndex);
	return LibraryInstance->Library->GetShaderSizeBytes(LibraryShaderIndex);
}

FShaderHash FShaderMapResource_SharedCode::GetShaderMapHash() const
{
	return LibraryInstance->Library->GetShaderMapHash(ShaderMapIndex);
}

bool FShaderMapResource_SharedCode::IsResolved() const
{
	return LibraryInstance && LibraryInstance->IsShaderMapResourceResolved(ShaderMapIndex);
}

void FShaderMapResource_SharedCode::AssignToNewShaderLibrary(FShaderLibraryInstance* NewLibraryInstance, int32 NewShaderMapIndex)
{
	const int32 OldShaderMapIndex = this->ShaderMapIndex;
	FShaderLibraryInstance* OldLibraryInstance = this->LibraryInstance;

	if (NewLibraryInstance == OldLibraryInstance)
	{
		checkf(NewShaderMapIndex == OldShaderMapIndex, TEXT("Shader library assignment to shadermap resource is unchanged, but shadermap indices don't match: Old(%d) != New(%d)"), OldShaderMapIndex, NewShaderMapIndex);
		return;
	}

	// Release references to old RHI shaders in resource
	ReleaseRHIShadersFromLibrary();

	// Remove shadermap entry from old library
	OldLibraryInstance->TryRemoveResource(this, /*bForceRemoval:*/ true);

	// Set reference to new shader library in resource
	this->LibraryInstance = NewLibraryInstance;
	this->ShaderMapIndex = NewShaderMapIndex;

	// Try to insert the new resource. If the specified slot is already occupied, the resource is considered detached from the library and we skip removing it in FShaderLibraryInstance::TryRemoveResource().
	// Such resources will only live for as long as they are still in use and the shader library does not keep it cached as another resource has taken that spot.
	const bool bIsResourceAttached = NewLibraryInstance->TryInsertResource(this);

	UE_LOGF(LogShaderLibrary, Verbose, "Moved shadermap '%ls' (Refcount=%d, Owner='%ls') from '%ls' (Index=%d) to '%ls' (Index=%d) with %hs resource"
		, *LexToString(this->GetShaderMapHash())
		, this->GetNumRefs()
		, *this->GetOwnerName().ToString()
		, *OldLibraryInstance->Library->GetName()
		, OldShaderMapIndex
		, *NewLibraryInstance->Library->GetName()
		, NewShaderMapIndex
		, bIsResourceAttached ? "attached" : "detached"
	);
}

void FShaderMapResource_SharedCode::ReleasePreloadedShaderCode(int32 ShaderIndex)
{
	SCOPED_LOADTIMER(FShaderMapResource_SharedCode_InitRHI);	// part of shader initialization in a way

	if (bEntireShaderMapPreloaded)
	{
		const int32 LibraryShaderIndex = LibraryInstance->Library->GetShaderIndex(ShaderMapIndex, ShaderIndex);
		LibraryInstance->Library->ReleasePreloadedShader(LibraryShaderIndex);
	}
}

int32 FShaderMapResource_SharedCode::GetLibraryShaderIndex(int32 ShaderIndex) const
{
	return LibraryInstance->Library->GetShaderIndex(ShaderMapIndex, ShaderIndex);
}

void FShaderMapResource_SharedCode::PreloadShader(int32 ShaderIndex, FGraphEventArray& OutCompletionEvents)
{
	// Don't preload if we already preloaded the full shader map, or we already created the RHI shader.
	if (bEntireShaderMapPreloaded || HasShader(ShaderIndex))
	{
		return;
	}

	const int32 LibraryShaderIndex = LibraryInstance->Library->GetShaderIndex(ShaderMapIndex, ShaderIndex);
	LibraryInstance->PreloadShader(LibraryShaderIndex, OutCompletionEvents);
}

void FShaderMapResource_SharedCode::PreloadShaderMap(FGraphEventArray& OutCompletionEvents)
{
	TArray<int32> ShaderIndices;
	LibraryInstance->Library->GetAllShaderIndices(ShaderMapIndex, ShaderIndices);

	for (int32 ShaderIndex : ShaderIndices)
	{
		LibraryInstance->PreloadShader(ShaderIndex, OutCompletionEvents);
	}
}

void FShaderMapResource_SharedCode::ReleaseRHIShadersFromLibrary()
{
	if (LibraryInstance && ensureMsgf(LibraryInstance->Library, TEXT("LibraryInstance->Library pointer is expected to be valid as long as library's FShaderMapResource are alive.")))
	{
		const int32 NumShaders = GetNumShaders();
		TArray<int32> ShaderGroupIndexes;

		for (int32 i = 0; i < NumShaders; ++i)
		{
			const int32 LibraryShaderIndex = LibraryInstance->Library->GetShaderIndex(ShaderMapIndex, i);
			if (HasShader(i))
			{
				LibraryInstance->ReleaseShader(LibraryShaderIndex);
			}
			else if (bEntireShaderMapPreloaded)
			{
				// Release the preloaded memory if the entire shader map was preloaded, but not created yet.
				LibraryInstance->Library->ReleasePreloadedShader(LibraryShaderIndex);
			}
			else
			{
				// Release preloaded memory if we individually preloaded that shader.
				LibraryInstance->ReleasePreloadedShaderIfNecessary(LibraryShaderIndex);
			}

			if (UE::ShaderLibrary::Private::GShaderMapResourceRef)
			{
				const int32 ShaderGroupIndex = LibraryInstance->Library->GetGroupIndexForShader(LibraryShaderIndex);
				ShaderGroupIndexes.AddUnique(ShaderGroupIndex);
			}
		}

		if (UE::ShaderLibrary::Private::GShaderMapResourceRef)
		{
			for (int32 ShaderGroupIndex : ShaderGroupIndexes)
			{
				LibraryInstance->Library->ReleasePreloadedShaderGroup(ShaderGroupIndex);
			}
		}
	}

	bEntireShaderMapPreloaded = false;

	FShaderMapResource::ReleaseRHI();
}

void FShaderMapResource_SharedCode::ReleaseRHI()
{
	ReleaseRHIShadersFromLibrary();

	if (GetNumRefs() > 0)
	{
		// Only print the shadermap index, not the hash since we cannot access it from the library anymore
		ensureMsgf(false, TEXT("FShaderMapResource_SharedCode::ReleaseRHI is still referenced (Refcount=%d, Owner='%s')"), GetNumRefs(), *GetOwnerName().ToString());
		UE_LOGF(LogShaderLibrary, Warning, "FShaderMapResource_SharedCode::ReleaseRHI is still referenced (Refcount=%d, ShaderMapHash='%ls', ShaderMapIndex=%d, Library='%ls', Owner='%ls')"
			, GetNumRefs()
			, LibraryInstance ? *LexToString(GetShaderMapHash()) : TEXT("<Null>")
			, ShaderMapIndex
			, LibraryInstance ? *LibraryInstance->Library->GetName() : TEXT("<Null>")
			, *GetOwnerName().ToString()
		);

		// Invoke delegate to notify shader map resource needs to be forced released
		OnSharedShaderMapResourceExplicitRelease.ExecuteIfBound(this);
	}

	// on assumption that we aren't going to get resurrected
	LibraryInstance = nullptr;
}

bool FShaderMapResource_SharedCode::TryRelease()
{
	if (LibraryInstance && LibraryInstance->TryRemoveResource(this))
	{
		return true;
	}

	return false;
}

FString FShaderMapResource_SharedCode::GetFriendlyName() const
{
	return LibraryInstance->Library->GetName();
}

int32 FShaderMapResource_SharedCode::GetGroupIndexForShader(int32 ShaderIndex) const
{
	const int32 LibraryShaderIndex = LibraryInstance->Library->GetShaderIndex(ShaderMapIndex, ShaderIndex);
	const int32 ShaderGroupIndex = LibraryInstance->Library->GetGroupIndexForShader(LibraryShaderIndex);
	return ShaderGroupIndex;
}

int32 FShaderMapResource_SharedCode::GetLibraryId() const
{
	return LibraryInstance->Library->GetLibraryId();
}

