// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ShaderMapResourceSharedCode.h: 
=============================================================================*/

#pragma once

#include "Shader.h"

class FShaderMapResource_SharedCode final : public FShaderMapResource
{
	void ReleaseRHIShadersFromLibrary();

public:
	FShaderMapResource_SharedCode(class FShaderLibraryInstance* InLibraryInstance, int32 InShaderMapIndex);
	virtual ~FShaderMapResource_SharedCode();

	// FRenderResource interface.
	virtual void ReleaseResource() override
	{
		FShaderMapResource::ReleaseResource();
		ensureMsgf(!bEntireShaderMapPreloaded && !LibraryInstance, TEXT("FShaderMapResource_SharedCode::ReleaseRHI() was not called on a shadermap resource owned by %s"), *GetOwnerName().ToString());
	}
	virtual void ReleaseRHI() override;

	// FShaderMapResource interface
	virtual FShaderHash GetShaderHash(int32 ShaderIndex) override;
	virtual FRHIShader* CreateRHIShaderOrCrash(int32 ShaderIndex, bool bRequired) override;
	virtual void ReleasePreloadedShaderCode(int32 ShaderIndex) override;
	virtual void PreloadShader(int32 ShaderIndex, FGraphEventArray& OutCompletionEvents) override;
	virtual void PreloadShaderMap(FGraphEventArray& OutCompletionEvents) override;
	virtual bool TryRelease() override;
	virtual uint32 GetSizeBytes() const override { return sizeof(*this) + GetAllocatedSize(); }
	virtual FString GetFriendlyName() const override;
	virtual int32 GetGroupIndexForShader(int32 ShaderIndex) const override;
	virtual int32 GetLibraryId() const override;
	virtual int32 GetLibraryShaderIndex(int32 ShaderIndex) const override;
	virtual uint32 GetShaderSizeBytes(int32 ShaderIndex) const override;
	virtual FShaderHash GetShaderMapHash() const override;
	virtual bool IsResolved() const override;

	// Assigns this resource to the new shader library with its new shadermap index. See FShaderLibrariesCollection::MoveShaderMapResourceOwnership().
	void AssignToNewShaderLibrary(FShaderLibraryInstance* NewLibraryInstance, int32 NewShaderMapIndex);

	class FShaderLibraryInstance* LibraryInstance;
	int32 ShaderMapIndex;
	bool bEntireShaderMapPreloaded;
};
