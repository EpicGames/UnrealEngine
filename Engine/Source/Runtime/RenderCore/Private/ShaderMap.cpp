// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	Shader.cpp: Shader implementation.
=============================================================================*/

#include "Shader.h"

#include "DataDrivenShaderPlatformInfo.h"
#include "HAL/LowLevelMemTracker.h"
#include "Misc/App.h"
#include "Misc/CoreMisc.h"
#include "Misc/ScopeLock.h"
#include "Misc/StringBuilder.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "ShaderCodeLibrary.h"
#include "ShaderCore.h"
#include "ShaderSerialization.h"
#include "UObject/RenderingObjectVersion.h"
#include "VertexFactory.h"

static EShaderPermutationFlags GetCurrentShaderPermutationFlags()
{
	FPlatformTypeLayoutParameters LayoutParams;
	LayoutParams.InitializeForCurrent();
	return GetShaderPermutationFlags(LayoutParams);
}

#ifndef ALLOW_SHADERMAP_TRACKING
#define ALLOW_SHADERMAP_TRACKING 0
#endif

#if ALLOW_SHADERMAP_TRACKING
static TArray<FShaderMapBase*> GAllShaderMaps;
static FCriticalSection GAllShaderMapsGuard;

TAutoConsoleVariable<bool> CVarEnableShaderMapTracking(
	TEXT("r.TrackShaderMaps"),
	WITH_EDITOR,
	TEXT("Enables the tracking of every shadermap instantiated. Required to run ListShaderMaps command."),
	ECVF_ReadOnly);

FAutoConsoleCommandWithArgsAndOutputDevice GListShaderCmd(
	TEXT("ListShaders"),
	TEXT("Spits out a csv table containing stats of all shaders"),
	FConsoleCommandWithArgsAndOutputDeviceDelegate::CreateStatic(
		[](const TArray<FString>& Params, FOutputDevice& Out)
		{
#if !UE_BUILD_SHIPPING
			if (!CVarEnableShaderMapTracking.GetValueOnGameThread())
			{
				UE_LOGF(LogShaders, Warning, "Enable r.TrackShaderMaps in ini config to enable the functionality");
				return;
			}
			Out.Logf(TEXT("FriendlyName,OwnerName,ShaderMapHash,VertexFactoryName,ShaderTypeName,ShaderPermutation,UncompressedSizeKb,Hash,UsedForRenderingRHIPtr"));
			FScopeLock SMAccess(&GAllShaderMapsGuard);
			for (FShaderMapBase* ShaderMap : GAllShaderMaps)
			{
				TStringBuilder<FShaderHash::GetStringLen()+1> ShaderMapHash; // +1 for null terminator (appended when calling ToString)
				if (ShaderMap->GetResource())
				{
					ShaderMapHash << ShaderMap->GetResource()->GetShaderMapHash();
				}

				FShaderListReport Report;
				Report.BindLambda(
					[&Out, &ShaderMapHash](
						const TCHAR* FriendlyName,
						const TCHAR* OwnerName,
						const TCHAR* VertexFactory,
						const TCHAR* ShaderType,
						uint32 Permutation,
						uint32 Bytes,
						const TCHAR* Hash,
						void* Ptr)
					{
						Out.Logf(TEXT("%s,%s,%s,%s,%s,%d,%.2f,%s,%p"), FriendlyName, OwnerName, ShaderMapHash.ToString(), VertexFactory, ShaderType, Permutation, Bytes / 1024.f, Hash, Ptr);
					}
				);
				ShaderMap->DumpShaderList(Report);
			}
#endif // !UE_BUILD_SHIPPING
		}));

FAutoConsoleCommandWithArgsAndOutputDevice GListShaderMapsCmd(
	TEXT("ListShaderMaps"),
	TEXT("Spits out a csv table containing stats of all shadermaps"),
	FConsoleCommandWithArgsAndOutputDeviceDelegate::CreateStatic(
		[](const TArray<FString>& Params, FOutputDevice& Out)
		{
			if (!CVarEnableShaderMapTracking.GetValueOnGameThread())
			{
				UE_LOGF(LogShaders, Warning, "Enable r.TrackShaderMaps in ini config to enable the functionality");
				return;
			}
			Out.Logf(TEXT("ShaderMapName,OwnerName,NumShaders,NumPipelines,SizeKb,bUsedForRendering"));
			FScopeLock SMAccess(&GAllShaderMapsGuard);
			for (FShaderMapBase* ShaderMap : GAllShaderMaps)
			{
				FString FriendlyName = "Unknown";
				FString OwnerName = "Unknown";
				uint32 CodeSize = ShaderMap->GetFrozenContentSize();
				bool bUseForRendering = false;

				if (const FShaderMapResource* Resource = ShaderMap->GetResource())
				{
					FriendlyName = Resource->GetFriendlyName();
					OwnerName = Resource->GetOwnerName().ToString();
					CodeSize += Resource->GetSizeBytes();
					bUseForRendering = Resource->ContainsAtLeastOneRHIShaderCreated();
				}

				TMap<FHashedName, TShaderRef<FShader>> Shaders;
				TArray<FShaderPipelineRef> Pipelines;
				if (ShaderMap->GetContent())
				{
					ShaderMap->GetShaderList(Shaders);
					ShaderMap->GetShaderPipelineList(Pipelines);
				}

				// Editor doesn't have the size baked, so grab it from shaders themselves.
				if (CodeSize == 0)
				{
					for (auto& [Hash, Shader] : Shaders)
					{
						CodeSize += Shader->GetCodeSize();
					}
				}

				Out.Logf(TEXT("%s,%s,%d,%d,%.3f,%s"),
					*FriendlyName,
					*OwnerName,
					Shaders.Num(),
					Pipelines.Num(),
					CodeSize / 1024.f,
					bUseForRendering ? TEXT("YES") : TEXT("NO")
				);
			}
		}));
#endif // ALLOW_SHADERMAP_TRACKING

FShaderMapBase::FShaderMapBase()
	: PointerTable(nullptr)
	, NumFrozenShaders(0u)
{
	PermutationFlags = GetCurrentShaderPermutationFlags();

#if ALLOW_SHADERMAP_TRACKING
	if (CVarEnableShaderMapTracking.GetValueOnAnyThread())
	{
		LLM_SCOPE_BYNAME(TEXT("Debug/ShaderMapsTracking"));
		FScopeLock SMAccess(&GAllShaderMapsGuard);
		GAllShaderMaps.Add(this);
	}
#endif // ALLOW_SHADERMAP_TRACKING
}

FShaderMapBase::~FShaderMapBase()
{
	DestroyContent();
	if (PointerTable)
	{
		delete PointerTable;
	}

#if ALLOW_SHADERMAP_TRACKING
	if (CVarEnableShaderMapTracking.GetValueOnAnyThread())
	{
		FScopeLock SMAccess(&GAllShaderMapsGuard);
		GAllShaderMaps.RemoveSingleSwap(this, EAllowShrinking::No);
	}
#endif // ALLOW_SHADERMAP_TRACKING
}

FShaderMapResourceCode* FShaderMapBase::GetResourceCode()
{
	if (!Code)
	{
		Code = new FShaderMapResourceCode();
	}
	return Code;
}

void FShaderMapBase::AssignContent(TMemoryImageObject<FShaderMapContent> InContent)
{
	check(!Content.Object);
	check(!PointerTable);
	const FTypeLayoutDesc& ExpectedTypeDesc = GetContentTypeDesc();
	checkf(*InContent.TypeDesc == ExpectedTypeDesc, TEXT("FShaderMapBase expected content of type %s, got %s"), ExpectedTypeDesc.Name, InContent.TypeDesc->Name);

	Content = InContent;
	PointerTable = CreatePointerTable();

	PostFinalizeContent();
}

void FShaderMapBase::AssignCopy(const FShaderMapBase& Source)
{
	check(!PointerTable);
	check(!Code);
	check(Source.Content.Object);

	if (Source.Content.FrozenSize == 0u)
	{
		PointerTable = CreatePointerTable();
		Content = TMemoryImageObject<FShaderMapContent>(FreezeMemoryImageObject(Source.Content.Object, *Source.Content.TypeDesc, PointerTable));
	}
	else
	{
		PointerTable = Source.PointerTable->Clone();
		Content.TypeDesc = Source.Content.TypeDesc;
		Content.FrozenSize = Source.Content.FrozenSize;
		Content.Object = static_cast<FShaderMapContent*>(FMemory::Malloc(Content.FrozenSize));
		FMemory::Memcpy(Content.Object, Source.Content.Object, Content.FrozenSize);
	}

	NumFrozenShaders = Content.Object->GetNumShaders();
	INC_DWORD_STAT_BY(STAT_Shaders_ShaderMemory, Content.FrozenSize);
	INC_DWORD_STAT_BY(STAT_Shaders_NumShadersLoaded, NumFrozenShaders);

#if CSV_PROFILER_STATS
	static TCsvPersistentCustomStat<float>* CsvStatShaderMemoryMB = FCsvProfiler::Get()->GetOrCreatePersistentCustomStatFloat(TEXT("ShaderMemoryMB"), CSV_CATEGORY_INDEX(Shaders));
	static TCsvPersistentCustomStat<int>* CsvStatNumShadersLoaded = FCsvProfiler::Get()->GetOrCreatePersistentCustomStatInt(TEXT("NumShadersLoaded"), CSV_CATEGORY_INDEX(Shaders));

	CsvStatShaderMemoryMB->Add(float(Content.FrozenSize) / (1024.0f * 1024.f));
	CsvStatNumShadersLoaded->Add(NumFrozenShaders);
#endif // CSV_PROFILER_STATS

	Code = new FShaderMapResourceCode(*Source.Code);
	InitResource();
}

void FShaderMapBase::InitResource()
{
	Resource.SafeRelease();
	if (Code)
	{
		Code->Finalize();
		Resource = new FShaderMapResource_InlineCode(GetShaderPlatform(), Code);
		BeginInitResource(Resource);
	}
	PostFinalizeContent();
}

void FShaderMapBase::FinalizeContent()
{
	if (Content.Freeze(PointerTable))
	{
		NumFrozenShaders = Content.Object->GetNumShaders();
		INC_DWORD_STAT_BY(STAT_Shaders_ShaderMemory, Content.FrozenSize);
		INC_DWORD_STAT_BY(STAT_Shaders_NumShadersLoaded, NumFrozenShaders);

#if CSV_PROFILER_STATS
		static TCsvPersistentCustomStat<float>* CsvStatShaderMemoryMB = FCsvProfiler::Get()->GetOrCreatePersistentCustomStatFloat(TEXT("ShaderMemoryMB"), CSV_CATEGORY_INDEX(Shaders));
		static TCsvPersistentCustomStat<int>* CsvStatNumShadersLoaded = FCsvProfiler::Get()->GetOrCreatePersistentCustomStatInt(TEXT("NumShadersLoaded"), CSV_CATEGORY_INDEX(Shaders));

		CsvStatShaderMemoryMB->Add(float(Content.FrozenSize) / (1024.0f * 1024.f));
		CsvStatNumShadersLoaded->Add(NumFrozenShaders);
#endif // CSV_PROFILER_STATS
	}
	InitResource();
}

void FShaderMapBase::UnfreezeContent()
{
	DEC_DWORD_STAT_BY(STAT_Shaders_ShaderMemory, Content.FrozenSize);
	DEC_DWORD_STAT_BY(STAT_Shaders_NumShadersLoaded, NumFrozenShaders);

#if CSV_PROFILER_STATS
	static TCsvPersistentCustomStat<float>* CsvStatShaderMemoryMB = FCsvProfiler::Get()->GetOrCreatePersistentCustomStatFloat(TEXT("ShaderMemoryMB"), CSV_CATEGORY_INDEX(Shaders));
	static TCsvPersistentCustomStat<int>* CsvStatNumShadersLoaded = FCsvProfiler::Get()->GetOrCreatePersistentCustomStatInt(TEXT("NumShadersLoaded"), CSV_CATEGORY_INDEX(Shaders));

	CsvStatShaderMemoryMB->Sub(float(Content.FrozenSize) / (1024.0f * 1024.f));
	CsvStatNumShadersLoaded->Sub(NumFrozenShaders);
#endif // CSV_PROFILER_STATS

	Content.Unfreeze(PointerTable);
	NumFrozenShaders = 0u;
}

#define CHECK_SHADERMAP_DEPENDENCIES (WITH_EDITOR || !(UE_BUILD_SHIPPING || UE_BUILD_TEST))

bool FShaderMapBase::Serialize(FShaderSerializeContext& Ctx)
{
	FArchive& Ar = Ctx.GetMainArchive();
	LLM_SCOPE(ELLMTag::Shaders);
	EShaderPlatform ShaderPlatform = EShaderPlatform::SP_NumPlatforms;
	if (Ar.IsSaving())
	{
		check(Content.Object);
		Content.Object->Validate(*this);

		{
			TUniquePtr<FShaderMapPointerTable> SavePointerTable(CreatePointerTable());

			FMemoryImage MemoryImage;
			MemoryImage.PrevPointerTable = PointerTable;
			MemoryImage.PointerTable = SavePointerTable.Get();
			MemoryImage.TargetLayoutParameters.InitializeForArchive(Ar);

			FMemoryImageWriter Writer(MemoryImage);
			Writer.WriteRootObject(Content.Object, *Content.TypeDesc);

			FMemoryImageResult MemoryImageResult;
			MemoryImage.Flatten(MemoryImageResult, true);

			MemoryImageResult.SaveToArchive(Ar);
		}

		bool bShareCode = false;
#if WITH_EDITOR
		bShareCode = FShaderLibraryCooker::IsShaderLibraryEnabled() && Ar.IsCooking();
#endif // WITH_EDITOR
		Ar << bShareCode;
#if WITH_EDITOR

		// Serialize a copy of ShaderPlatform directly into the archive
		// This will allow us to correctly deserialize the stream, even if we're not able to load the frozen content
		ShaderPlatform = GetShaderPlatform();
		FName ShaderPlatformName = FDataDrivenShaderPlatformInfo::GetName(ShaderPlatform);
		Ar << ShaderPlatformName;

		if (bShareCode)
		{
			FShaderHash ResourceHash = Code->ResourceHash;
			Ar << ResourceHash;
			FShaderLibraryCooker::AddShaderCode(ShaderPlatform, Code, GetAssociatedAssets());
		}
		else
#endif // WITH_EDITOR
		{
			Code->Serialize(Ctx);
		}
	}
	else
	{
		check(!PointerTable);
		PointerTable = CreatePointerTable();

		FPlatformTypeLayoutParameters LayoutParameters;
		FMemoryImageObject LoadedContent = FMemoryImageResult::LoadFromArchive(Ar, GetContentTypeDesc(), PointerTable, LayoutParameters);
		PermutationFlags = GetShaderPermutationFlags(LayoutParameters);

		bool bShareCode = false;
		Ar << bShareCode;

		FName ShaderPlatformName;
		Ar << ShaderPlatformName;

		ShaderPlatform = FDataDrivenShaderPlatformInfo::GetShaderPlatformFromName(ShaderPlatformName);

		if (bShareCode)
		{
			FShaderHash ResourceHash;
			Ar << ResourceHash;
			Resource = FShaderCodeLibrary::LoadResource(ResourceHash, &Ar);
			if (!Resource)
			{
				// do not warn when running -nullrhi (the resource cannot be created as the shader library will not be uninitialized),
				// also do not warn for shader platforms other than current (if the game targets more than one RHI)
				if (FApp::CanEverRender() && ShaderPlatform == GMaxRHIShaderPlatform)
				{
					UE_LOGF(LogShaders, Error, "Missing shader resource for hash '%ls' for shader platform '%ls' in the shader library while serializing asset %ls", *ResourceHash.ToString(),
						*LexToString(ShaderPlatform),
						*Ctx.SerializingAsset.ToString());
				}
			}
		}
		else
		{
			Code = new FShaderMapResourceCode();
			Code->Serialize(Ctx);
			Resource = new FShaderMapResource_InlineCode(ShaderPlatform, Code);
		}

		if (LoadedContent.Object && Resource)
		{
			Content = TMemoryImageObject<FShaderMapContent>(LoadedContent);

			// Possible we've loaded/converted unfrozen content, make sure it's frozen for the current platform before trying to render anything
			if (Content.FrozenSize == 0u)
			{
				Content.Freeze(PointerTable);
			}
			PostFinalizeContent();

			NumFrozenShaders = Content.Object->GetNumShaders();
			INC_DWORD_STAT_BY(STAT_Shaders_ShaderMemory, Content.FrozenSize);
			INC_DWORD_STAT_BY(STAT_Shaders_NumShadersLoaded, NumFrozenShaders);

#if CSV_PROFILER_STATS
			static TCsvPersistentCustomStat<float>* CsvStatShaderMemoryMB = FCsvProfiler::Get()->GetOrCreatePersistentCustomStatFloat(TEXT("ShaderMemoryMB"), CSV_CATEGORY_INDEX(Shaders));
			static TCsvPersistentCustomStat<int>* CsvStatNumShadersLoaded = FCsvProfiler::Get()->GetOrCreatePersistentCustomStatInt(TEXT("NumShadersLoaded"), CSV_CATEGORY_INDEX(Shaders));

			CsvStatShaderMemoryMB->Add(float(Content.FrozenSize) / (1024.0f * 1024.f));
			CsvStatNumShadersLoaded->Add(NumFrozenShaders);
#endif // CSV_PROFILER_STATS

			BeginInitResource(Resource);
			INC_DWORD_STAT_BY(STAT_Shaders_ShaderResourceMemory, Resource->GetSizeBytes());
		}
		else
		{
			// Missing either content and/or resource
			// In either case, shader map has failed to load
			LoadedContent.Destroy(PointerTable);
			Resource.SafeRelease();
		}
	}

#if WITH_EDITOR
	bool bNotify = Ar.IsCooking() || GIsEditor;
	if (bNotify)
	{
		check(ShaderPlatform != SP_NumPlatforms); // ShaderPlatform should have been initialized above in either the saving or loading blocks
		const FName ShaderFormat = LegacyShaderPlatformToShaderFormat(ShaderPlatform);
		if (ShaderFormat != NAME_None)
		{
			if (Code)
			{
				Code->NotifyShadersCompiled(ShaderFormat);
			}
		}
	}
#endif

	return (bool)Content.Object;
}

#if WITH_EDITORONLY_DATA
TArray<FGenericShaderStat> FShaderMapBase::GetShaderStatistics(FShaderType* ShaderType) const
{
	TArray<FGenericShaderStat> ShaderStatistics;
	FShader* Shader = GetContent()->GetShader(ShaderType);
	if (Shader)
	{
		ShaderStatistics = GetShaderStatistics(*Shader);
	}
	return ShaderStatistics;
}

TArray<FGenericShaderStat> FShaderMapBase::GetShaderStatistics(FShader& Shader) const
{
	TArray<FGenericShaderStat> ShaderStatistics;

	if (Code)
	{
		const int32 ShaderIndex = Code->FindShaderIndex(Shader.GetOutputHash());
		if (Code->ShaderEditorOnlyDataEntries.IsValidIndex(ShaderIndex))
		{
			for (const FGenericShaderStat& Stat : Code->ShaderEditorOnlyDataEntries[ShaderIndex].ShaderStatistics)
			{
				if (!EnumHasAnyFlags(Stat.Flags, FGenericShaderStat::EFlags::Hidden))
				{
					ShaderStatistics.Add(Stat);
				}
			}
		}
	}

	return ShaderStatistics;
}
#endif // WITH_EDITORONLY_DATA

FString FShaderMapBase::ToString() const
{
	TStringBuilder<32000> String;
	{
		FMemoryToStringContext Context;
		Context.PrevPointerTable = PointerTable;
		Context.String = &String;

		FPlatformTypeLayoutParameters LayoutParams;
		LayoutParams.InitializeForCurrent();

		Content.TypeDesc->ToStringFunc(Content.Object, *Content.TypeDesc, LayoutParams, Context);
	}

	if (Code)
	{
		Code->ToString(String);
	}

	return FString(String);
}

void FShaderMapBase::DestroyContent()
{
	DEC_DWORD_STAT_BY(STAT_Shaders_ShaderMemory, Content.FrozenSize);
	DEC_DWORD_STAT_BY(STAT_Shaders_NumShadersLoaded, NumFrozenShaders);

#if CSV_PROFILER_STATS
	static TCsvPersistentCustomStat<float>* CsvStatShaderMemoryMB = FCsvProfiler::Get()->GetOrCreatePersistentCustomStatFloat(TEXT("ShaderMemoryMB"), CSV_CATEGORY_INDEX(Shaders));
	static TCsvPersistentCustomStat<int>* CsvStatNumShadersLoaded = FCsvProfiler::Get()->GetOrCreatePersistentCustomStatInt(TEXT("NumShadersLoaded"), CSV_CATEGORY_INDEX(Shaders));

	CsvStatShaderMemoryMB->Sub(float(Content.FrozenSize) / (1024.0f * 1024.f));
	CsvStatNumShadersLoaded->Sub(NumFrozenShaders);
#endif // CSV_PROFILER_STATS

	Content.Destroy(PointerTable);
	NumFrozenShaders = 0u;
}

static uint16 MakeShaderHash(const FHashedName& TypeName, int32 PermutationId)
{
	return (uint16)CityHash128to64({ TypeName.GetHash(), (uint64)PermutationId });
}

FShaderMapContent::FShaderMapContent(EShaderPlatform InPlatform)
	: ShaderHash(128u), ShaderPlatformName(FDataDrivenShaderPlatformInfo::GetName(InPlatform))
{}

FShaderMapContent::~FShaderMapContent()
{
	Empty();
}

EShaderPlatform FShaderMapContent::GetShaderPlatform() const
{
	return FDataDrivenShaderPlatformInfo::GetShaderPlatformFromName(ShaderPlatformName);
}

FShader* FShaderMapContent::GetShader(const FHashedName& TypeName, int32 PermutationId) const
{
	// TRACE_CPUPROFILER_EVENT_SCOPE(FShaderMapContent::GetShader); -- this function is called too frequently, so don't add the scope by default

	const uint16 Hash = MakeShaderHash(TypeName, PermutationId);
	const FHashedName* RESTRICT LocalShaderTypes = ShaderTypes.GetData();
	const int32* RESTRICT LocalShaderPermutations = ShaderPermutations.GetData();
	const uint32* RESTRICT LocalNextHashIndices = ShaderHash.GetNextIndices();
	const uint32 NumShaders = Shaders.Num();

	for (uint32 Index = ShaderHash.First(Hash); ShaderHash.IsValid(Index); Index = LocalNextHashIndices[Index])
	{
		checkSlow(Index < NumShaders);
		if (LocalShaderTypes[Index] == TypeName && LocalShaderPermutations[Index] == PermutationId)
		{
			return Shaders[Index].GetChecked();
		}
	}

	return nullptr;
}

void FShaderMapContent::AddShader(const FHashedName& TypeName, int32 PermutationId, FShader* Shader)
{
	check(!Shader->IsFrozen());
	checkSlow(!HasShader(TypeName, PermutationId));

	const uint16 Hash = MakeShaderHash(TypeName, PermutationId);
	const int32 Index = Shaders.Add(Shader);
	ShaderTypes.Add(TypeName);
	ShaderPermutations.Add(PermutationId);
	check(ShaderTypes.Num() == Shaders.Num());
	check(ShaderPermutations.Num() == Shaders.Num());
	ShaderHash.Add(Hash, Index);
}

FShader* FShaderMapContent::FindOrAddShader(const FHashedName& TypeName, int32 PermutationId, FShader* Shader)
{
	check(!Shader->IsFrozen());

	const uint16 Hash = MakeShaderHash(TypeName, PermutationId);
	for (uint32 Index = ShaderHash.First(Hash); ShaderHash.IsValid(Index); Index = ShaderHash.Next(Index))
	{
		if (ShaderTypes[Index] == TypeName && ShaderPermutations[Index] == PermutationId)
		{
			DeleteObjectFromLayout(Shader);
			return Shaders[Index].GetChecked();
		}
	}

	const int32 Index = Shaders.Add(Shader);
	ShaderHash.Add(Hash, Index);
	ShaderTypes.Add(TypeName);
	ShaderPermutations.Add(PermutationId);
	check(ShaderTypes.Num() == Shaders.Num());
	check(ShaderPermutations.Num() == Shaders.Num());
	return Shader;
}

void FShaderMapContent::AddShaderPipeline(FShaderPipeline* Pipeline)
{
	checkSlow(!HasShaderPipeline(Pipeline->TypeName));
	const int32 Index = Algo::LowerBoundBy(ShaderPipelines, Pipeline->TypeName, FProjectShaderPipelineToKey());
	ShaderPipelines.Insert(Pipeline, Index);
}

FShaderPipeline* FShaderMapContent::FindOrAddShaderPipeline(FShaderPipeline* Pipeline)
{
	const int32 Index = Algo::LowerBoundBy(ShaderPipelines, Pipeline->TypeName, FProjectShaderPipelineToKey());
	if (Index < ShaderPipelines.Num())
	{
		FShaderPipeline* PrevShaderPipeline = ShaderPipelines[Index];
		if (PrevShaderPipeline->TypeName == Pipeline->TypeName)
		{
			delete Pipeline;
			return PrevShaderPipeline;
		}
	}

	ShaderPipelines.Insert(Pipeline, Index);
	return Pipeline;
}

/**
 * Removes the shader of the given type from the shader map
 * @param Type Shader type to remove the entry for
 */
void FShaderMapContent::RemoveShaderTypePermutaion(const FHashedName& TypeName, int32 PermutationId)
{
	const uint16 Hash = MakeShaderHash(TypeName, PermutationId);

	for (uint32 Index = ShaderHash.First(Hash); ShaderHash.IsValid(Index); Index = ShaderHash.Next(Index))
	{
		FShader* Shader = Shaders[Index].GetChecked();
		if (ShaderTypes[Index] == TypeName && ShaderPermutations[Index] == PermutationId)
		{
			DeleteObjectFromLayout(Shader);

			// Replace the shader we're removing with the last shader in the list
			Shaders.RemoveAtSwap(Index, 1, EAllowShrinking::No);
			ShaderTypes.RemoveAtSwap(Index, 1, EAllowShrinking::No);
			ShaderPermutations.RemoveAtSwap(Index, 1, EAllowShrinking::No);
			check(ShaderTypes.Num() == Shaders.Num());
			check(ShaderPermutations.Num() == Shaders.Num());
			ShaderHash.Remove(Hash, Index);

			// SwapIndex is the old index of the shader at the end of the list, that's now been moved to replace the current shader
			const int32 SwapIndex = Shaders.Num();
			if (Index != SwapIndex)
			{
				// We need to update the hash table to reflect shader previously at SwapIndex being moved to Index
				// Here we construct the hash from values at Index, since type/permutation have already been moved
				const uint16 SwapHash = MakeShaderHash(ShaderTypes[Index], ShaderPermutations[Index]);
				ShaderHash.Remove(SwapHash, SwapIndex);
				ShaderHash.Add(SwapHash, Index);
			}

			break;
		}
	}
}

void FShaderMapContent::RemoveShaderPipelineType(const FShaderPipelineType* ShaderPipelineType)
{
	const int32 Index = Algo::BinarySearchBy(ShaderPipelines, ShaderPipelineType->GetHashedName(), FProjectShaderPipelineToKey());
	if (Index != INDEX_NONE)
	{
		FShaderPipeline* Pipeline = ShaderPipelines[Index];
		delete Pipeline;
		ShaderPipelines.RemoveAt(Index, EAllowShrinking::No);
	}
}

#if !UE_BUILD_SHIPPING

static void RetrieveShaderInfo(uint32& OutShaderCodeSizeBytes, FRHIShader*& OutRHIShader, FString& OutHash, FShaderMapResource* Resource, FShader* Shader)
{
	if (Resource)
	{
		if (Resource->HasShader(Shader->GetResourceIndex()))
		{
			OutRHIShader = Resource->GetShader(Shader->GetResourceIndex());
		}
		const uint32 Index = Shader->GetResourceIndex();
		OutShaderCodeSizeBytes = Resource->GetShaderSizeBytes(Index);
		OutHash = Resource->GetShaderHash(Index).ToString();
	}
}

void FShaderMapContent::DumpShaderList(const FShaderMapBase& InShaderMap, FShaderListReport& Out) const
{
	FString FriendlyName;
	FString OwnerName;
	FShaderMapResource* Resource = InShaderMap.GetResource();
	if (Resource)
	{
		FriendlyName = Resource->GetFriendlyName();
		OwnerName = Resource->GetOwnerName().ToString();
	}
	for (int32 ShaderIndex = 0; ShaderIndex < Shaders.Num(); ++ShaderIndex)
	{
		FShader* Shader = Shaders[ShaderIndex].GetChecked();
		uint32 ShaderCodeSizeBytes = 0;
		FRHIShader* RHIShader = nullptr;
		FString Hash;
		RetrieveShaderInfo(ShaderCodeSizeBytes, RHIShader, Hash, Resource, Shader);

		Out.ExecuteIfBound(
			*FriendlyName, 
			*OwnerName,
			Shader->GetVertexFactoryType(InShaderMap.GetPointerTable()) ? Shader->GetVertexFactoryType(InShaderMap.GetPointerTable())->GetName() : TEXT(""),
			Shader->GetType(InShaderMap.GetPointerTable()) ? Shader->GetType(InShaderMap.GetPointerTable())->GetName() : TEXT(""),
			ShaderPermutations[ShaderIndex],
			ShaderCodeSizeBytes,
			*Hash,
			RHIShader
		);
	}	

	for (const FShaderPipeline* ShaderPipeline : ShaderPipelines)
	{
		for (uint32 Frequency = 0u; Frequency < SF_NumGraphicsFrequencies; ++Frequency)
		{
			FShader* Shader = ShaderPipeline->Shaders[Frequency].Get();
			if (Shader)
			{
				uint32 ShaderCodeSizeBytes = 0;
				FRHIShader* RHIShader = nullptr;
				FString Hash;

				RetrieveShaderInfo(ShaderCodeSizeBytes, RHIShader, Hash, Resource, Shader);

				Out.ExecuteIfBound(
					*FriendlyName, 
					*OwnerName,
					Shader->GetVertexFactoryType(InShaderMap.GetPointerTable()) ? Shader->GetVertexFactoryType(InShaderMap.GetPointerTable())->GetName() : TEXT(""),
					Shader->GetType(InShaderMap.GetPointerTable()) ? Shader->GetType(InShaderMap.GetPointerTable())->GetName() : TEXT(""),
					ShaderPipeline->PermutationIds[Frequency],
					ShaderCodeSizeBytes,
					*Hash,
					RHIShader
				);
			}
		}
	}
}
#endif // !UE_BUILD_SHIPPING

void FShaderMapContent::GetShaderList(const FShaderMapBase& InShaderMap, uint64 InMaterialShaderMapHash, TMap<FShaderId, TShaderRef<FShader>>& OutShaders) const
{
	for (int32 ShaderIndex = 0; ShaderIndex < Shaders.Num(); ++ShaderIndex)
	{
		FShader* Shader = Shaders[ShaderIndex].GetChecked();
		const FShaderId ShaderId(
			Shader->GetType(InShaderMap.GetPointerTable()),
			InMaterialShaderMapHash,
			FHashedName(),
			Shader->GetVertexFactoryType(InShaderMap.GetPointerTable()),
			ShaderPermutations[ShaderIndex],
			GetShaderPlatform());

		OutShaders.Add(ShaderId, TShaderRef<FShader>(Shader, InShaderMap));
	}

	for (const FShaderPipeline* ShaderPipeline : ShaderPipelines)
	{
		for (uint32 Frequency = 0u; Frequency < SF_NumGraphicsFrequencies; ++Frequency)
		{
			FShader* Shader = ShaderPipeline->Shaders[Frequency].Get();
			if (Shader)
			{
				const FShaderId ShaderId(
					Shader->GetType(InShaderMap.GetPointerTable()),
					InMaterialShaderMapHash,
					ShaderPipeline->TypeName,
					Shader->GetVertexFactoryType(InShaderMap.GetPointerTable()),
					ShaderPipeline->PermutationIds[Frequency],
					GetShaderPlatform());
				OutShaders.Add(ShaderId, TShaderRef<FShader>(Shader, InShaderMap));
			}
		}
	}
}

void FShaderMapContent::GetShaderList(const FShaderMapBase& InShaderMap, TMap<FHashedName, TShaderRef<FShader>>& OutShaders) const
{
	for (int32 ShaderIndex = 0; ShaderIndex < Shaders.Num(); ++ShaderIndex)
	{
		FShader* Shader = Shaders[ShaderIndex].Get();
		if (ensure(Shader))
		{
			OutShaders.Add(ShaderTypes[ShaderIndex], TShaderRef<FShader>(Shader, InShaderMap));
		}
	}

	for (const FShaderPipeline* ShaderPipeline : ShaderPipelines)
	{
		for (const TShaderRef<FShader>& Shader : ShaderPipeline->GetShaders(InShaderMap))
		{
			OutShaders.Add(Shader.GetType()->GetHashedName(), Shader);
		}
	}
}

void FShaderMapContent::GetShaderPipelineList(const FShaderMapBase& InShaderMap, TArray<FShaderPipelineRef>& OutShaderPipelines, FShaderPipeline::EFilter Filter) const
{
	const EShaderPlatform ShaderPlatform = GetShaderPlatform();
	for (FShaderPipeline* Pipeline : ShaderPipelines)
	{
		const FShaderPipelineType* PipelineType = FShaderPipelineType::GetShaderPipelineTypeByName(Pipeline->TypeName);
		if (PipelineType->ShouldOptimizeUnusedOutputs(ShaderPlatform) && Filter == FShaderPipeline::EOnlyShared)
		{
			continue;
		}
		else if (!PipelineType->ShouldOptimizeUnusedOutputs(ShaderPlatform) && Filter == FShaderPipeline::EOnlyUnique)
		{
			continue;
		}
		OutShaderPipelines.Add(FShaderPipelineRef(Pipeline, InShaderMap));
	}
}

void FShaderMapContent::Validate(const FShaderMapBase& InShaderMap) const
{
	for (const FShader* Shader : Shaders)
	{
		checkf(Shader->GetResourceIndex() != INDEX_NONE, TEXT("Missing resource for %s"), Shader->GetType(InShaderMap.GetPointerTable())->GetName());
	}

	/*for(FShaderPipeline* Pipeline : ShaderPipelines)
	{
		for(const TShaderRef<FShader>& Shader : Pipeline->GetShaders(InPtrTable))
		{
			checkf(Shader.GetResource(), TEXT("Missing resource for %s"), Shader.GetType()->GetName());
		}
	}*/
}

#if WITH_EDITOR
static bool CheckOutdatedShaderType(EShaderPlatform Platform, const TShaderRef<FShader>& Shader, TArray<const FShaderType*>& OutdatedShaderTypes, TArray<const FVertexFactoryType*>& OutdatedFactoryTypes)
{
	const FShaderType* Type = Shader.GetType();
	const bool bOutdatedShader = Type->GetSourceHash(Platform) != Shader->GetHash();

	const FVertexFactoryType* VFType = Shader.GetVertexFactoryType();
	const bool bOutdatedVertexFactory = VFType && VFType->GetSourceHash(Platform) != Shader->GetVertexFactoryHash();

	if (bOutdatedShader)
	{
		OutdatedShaderTypes.AddUnique(Type);
	}
	if (bOutdatedVertexFactory)
	{
		OutdatedFactoryTypes.AddUnique(VFType);
	}

	return bOutdatedShader || bOutdatedVertexFactory;
}

void FShaderMapContent::GetOutdatedTypes(const FShaderMapBase& InShaderMap, TArray<const FShaderType*>& OutdatedShaderTypes, TArray<const FShaderPipelineType*>& OutdatedShaderPipelineTypes, TArray<const FVertexFactoryType*>& OutdatedFactoryTypes) const
{
	for (FShader* Shader : Shaders)
	{
		CheckOutdatedShaderType(GetShaderPlatform(), TShaderRef<FShader>(Shader, InShaderMap), OutdatedShaderTypes, OutdatedFactoryTypes);
	}

	for (const FShaderPipeline* Pipeline : ShaderPipelines)
	{
		for (const TShaderRef<FShader>& Shader : Pipeline->GetShaders(InShaderMap))
		{
			if (CheckOutdatedShaderType(GetShaderPlatform(), Shader, OutdatedShaderTypes, OutdatedFactoryTypes))
			{
				const FShaderPipelineType* PipelineType = FShaderPipelineType::GetShaderPipelineTypeByName(Pipeline->TypeName);
				check(PipelineType);
				OutdatedShaderPipelineTypes.AddUnique(PipelineType);
			}
		}
	}
}

void FShaderMapContent::SaveShaderStableKeys(const FShaderMapBase& InShaderMap, EShaderPlatform TargetShaderPlatform, const struct FStableShaderKeyAndValue& SaveKeyVal)
{
	for (int32 ShaderIndex = 0; ShaderIndex < Shaders.Num(); ++ShaderIndex)
	{
		const int32 PermutationId = ShaderPermutations[ShaderIndex];
		Shaders[ShaderIndex]->SaveShaderStableKeys(InShaderMap.GetPointerTable(), TargetShaderPlatform, PermutationId, SaveKeyVal);
	}

	for (const FShaderPipeline* Pipeline : ShaderPipelines)
	{
		Pipeline->SaveShaderStableKeys(InShaderMap.GetPointerTable(), TargetShaderPlatform, SaveKeyVal);
	}
}

uint32 FShaderMapContent::GetMaxTextureSamplersShaderMap(const FShaderMapBase& InShaderMap) const
{
	uint32 MaxTextureSamplers = 0;

	for (FShader* Shader : Shaders)
	{
		if (ensure(Shader))
		{
			MaxTextureSamplers = FMath::Max(MaxTextureSamplers, Shader->GetNumTextureSamplers());
		}
	}

	for (FShaderPipeline* Pipeline : ShaderPipelines)
	{
		for (const TShaderRef<FShader>& Shader : Pipeline->GetShaders(InShaderMap))
		{
			MaxTextureSamplers = FMath::Max(MaxTextureSamplers, Shader->GetNumTextureSamplers());
		}
	}

	return MaxTextureSamplers;
}
#endif // WITH_EDITOR

uint32 FShaderMapContent::GetNumShaders() const
{
	uint32 NumShaders = Shaders.Num();
	for (FShaderPipeline* Pipeline : ShaderPipelines)
	{
		NumShaders += Pipeline->GetNumShaders();
	}
	return NumShaders;
}

uint32 FShaderMapContent::GetMaxNumInstructionsForShader(const FShaderMapBase& InShaderMap, FShaderType* ShaderType) const
{
	uint32 MaxNumInstructions = 0;
	FShader* Shader = GetShader(ShaderType);
	if (Shader)
	{
		MaxNumInstructions = FMath::Max(MaxNumInstructions, Shader->GetNumInstructions());
	}

	for (FShaderPipeline* Pipeline : ShaderPipelines)
	{
		FShader* PipelineShader = Pipeline->GetShader(ShaderType->GetFrequency());
		if (PipelineShader)
		{
			const FShaderType* PipelineShaderType = PipelineShader->GetType(InShaderMap.GetPointerTable());
			if (PipelineShaderType &&
				(PipelineShaderType == ShaderType))
			{
				MaxNumInstructions = FMath::Max(MaxNumInstructions, PipelineShader->GetNumInstructions());
			}
		}
	}

	return MaxNumInstructions;
}

#if WITH_EDITORONLY_DATA
TArray<FGenericShaderStat> FShaderMapContent::GetShaderStatistics(const FShaderMapBase& InShaderMap, FShaderType* ShaderType) const
{
	TArray<FGenericShaderStat> ShaderStatistics;

	FShader* Shader = GetShader(ShaderType);
	if (Shader)
	{
		ShaderStatistics = InShaderMap.GetShaderStatistics(*Shader);
	}

	return ShaderStatistics;
}
#endif // WITH_EDITORONLY_DATA

struct FSortedShaderEntry
{
	FHashedName TypeName;
	int32 PermutationId;
	int32 Index;

	friend bool operator<(const FSortedShaderEntry& Lhs, const FSortedShaderEntry& Rhs)
	{
		if (Lhs.TypeName != Rhs.TypeName)
		{
			return Lhs.TypeName < Rhs.TypeName;
		}
		return Lhs.PermutationId < Rhs.PermutationId;
	}
};

void FShaderMapContent::Finalize(const FShaderMapResourceCode* Code)
{
	check(Code);

	for (FShader* Shader : Shaders)
	{
		Shader->Finalize(Code);
	}

	for (FShaderPipeline* Pipeline : ShaderPipelines)
	{
		Pipeline->Finalize(Code);
	}

	// Sort the shaders by type/permutation, so they are consistently ordered
	TArray<FSortedShaderEntry> SortedEntries;
	SortedEntries.Empty(Shaders.Num());
	for (int32 ShaderIndex = 0; ShaderIndex < Shaders.Num(); ++ShaderIndex)
	{
		FSortedShaderEntry& Entry = SortedEntries.AddDefaulted_GetRef();
		Entry.TypeName = ShaderTypes[ShaderIndex];
		Entry.PermutationId = ShaderPermutations[ShaderIndex];
		Entry.Index = ShaderIndex;
	}
	SortedEntries.Sort();

	// Choose a good hash size based on the number of shaders we have
	const uint32 HashSize = FMath::RoundUpToPowerOfTwo(FMath::Max((Shaders.Num() * 3) / 2, 1));
	FMemoryImageHashTable NewShaderHash(HashSize, Shaders.Num());
	TMemoryImageArray<TMemoryImagePtr<FShader>> NewShaders;
	NewShaders.Empty(Shaders.Num());
	ShaderTypes.Empty(Shaders.Num());
	ShaderPermutations.Empty(Shaders.Num());

	for (int32 SortedIndex = 0; SortedIndex < SortedEntries.Num(); ++SortedIndex)
	{
		const FSortedShaderEntry& SortedEntry = SortedEntries[SortedIndex];

		const uint16 Key = MakeShaderHash(SortedEntry.TypeName, SortedEntry.PermutationId);
		NewShaders.Add(Shaders[SortedEntry.Index]);
		ShaderTypes.Add(SortedEntry.TypeName);
		ShaderPermutations.Add(SortedEntry.PermutationId);
		NewShaderHash.Add(Key, SortedIndex);
	}

	Shaders = MoveTemp(NewShaders);
	ShaderHash = MoveTemp(NewShaderHash);
}

void FShaderMapContent::Empty()
{
	EmptyShaderPipelines();
	for (int32 i = 0; i < Shaders.Num(); ++i)
	{
		TMemoryImagePtr<FShader>& Shader = Shaders[i];
		Shader.SafeDelete();
	}
	Shaders.Empty();
	ShaderTypes.Empty();
	ShaderPermutations.Empty();
	ShaderHash.Clear();
}

void FShaderMapContent::EmptyShaderPipelines()
{
	for (TMemoryImagePtr<FShaderPipeline>& Pipeline : ShaderPipelines)
	{
		Pipeline.SafeDelete();
	}
	ShaderPipelines.Empty();
}
