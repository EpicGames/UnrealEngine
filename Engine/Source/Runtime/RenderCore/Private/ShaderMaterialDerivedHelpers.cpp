// Copyright Epic Games, Inc. All Rights Reserved.

#include "RHIFeatureLevel.h"
#include "RHIFwd.h" // IWYU pragma: keep
#include "ShaderMaterial.h"


FShaderGlobalDefines FetchRuntimeShaderGlobalDefines(EShaderPlatform TargetPlatform)
{
	FShaderGlobalDefines Ret = {};
	return Ret;
}


FShaderMaterialDerivedDefines RENDERCORE_API CalculateDerivedMaterialParameters(
	const FShaderMaterialPropertyDefines& Mat,
	const FShaderLightmapPropertyDefines& Lightmap,
	const FShaderGlobalDefines& SrcGlobal,
	const FShaderCompilerDefines& Compiler,
	ERHIFeatureLevel::Type FEATURE_LEVEL)
{
	FShaderMaterialDerivedDefines Dst = {};

	// Translucent materials need to compute fogging in the forward shading pass
	// Materials that read from scene color skip getting fogged, because the contents of the scene color lookup have already been fogged
	// This is not foolproof, as any additional color the material adds will then not be fogged correctly
	Dst.TRANSLUCENCY_NEEDS_BASEPASS_FOGGING = (Mat.MATERIAL_ENABLE_TRANSLUCENCY_FOGGING && Mat.MATERIALBLENDING_ANY_TRANSLUCENT && !Mat.MATERIAL_USES_SCENE_COLOR_COPY);

	// With forward shading, fog always needs to be computed in the base pass to work correctly with MSAA
	Dst.OPAQUE_NEEDS_BASEPASS_FOGGING = (!Mat.MATERIALBLENDING_ANY_TRANSLUCENT && SrcGlobal.FORWARD_SHADING);

	Dst.NEEDS_BASEPASS_VERTEX_FOGGING = ((Dst.TRANSLUCENCY_NEEDS_BASEPASS_FOGGING && !Mat.MATERIAL_COMPUTE_FOG_PER_PIXEL) || (Dst.OPAQUE_NEEDS_BASEPASS_FOGGING && SrcGlobal.PROJECT_VERTEX_FOGGING_FOR_OPAQUE));
	Dst.NEEDS_BASEPASS_PIXEL_FOGGING = ((Dst.TRANSLUCENCY_NEEDS_BASEPASS_FOGGING && Mat.MATERIAL_COMPUTE_FOG_PER_PIXEL) || (Dst.OPAQUE_NEEDS_BASEPASS_FOGGING && !SrcGlobal.PROJECT_VERTEX_FOGGING_FOR_OPAQUE));

	// Volumetric fog interpolated per vertex gives very poor results, always sample the volumetric fog texture per-pixel
	// Opaque materials in the deferred renderer get volumetric fog applied in a deferred fog pass
	Dst.NEEDS_BASEPASS_PIXEL_VOLUMETRIC_FOGGING = (Mat.MATERIALBLENDING_ANY_TRANSLUCENT || SrcGlobal.FORWARD_SHADING);

	// need to change this for mobile vs PC, and get rid of the #undefs
	Dst.NEEDS_LIGHTMAP_COORDINATE = (Lightmap.HQ_TEXTURE_LIGHTMAP || Lightmap.LQ_TEXTURE_LIGHTMAP);

	// this logic differs from the actual defines due to confusing #undefs. NEEDS_LIGHTMAP is only allowed to be true if PRECOMPUTED_IRRADIANCE_VOLUME_LIGHTING
	// is false because otherwise GetLightMapCoordinates() will not be defined causing a compiler error.
	Dst.NEEDS_LIGHTMAP = (Dst.NEEDS_LIGHTMAP_COORDINATE) && !Lightmap.PRECOMPUTED_IRRADIANCE_VOLUME_LIGHTING;

	Dst.USES_GBUFFER = (FEATURE_LEVEL >= ERHIFeatureLevel::SM4_REMOVED && (Mat.MATERIALBLENDING_SOLID || Mat.MATERIALBLENDING_MASKED) && !SrcGlobal.FORWARD_SHADING);

	// Only some shader models actually need custom data.
	Dst.WRITES_CUSTOMDATA_TO_GBUFFER = (Dst.USES_GBUFFER && (Mat.MATERIAL_SHADINGMODEL_SUBSURFACE || Mat.MATERIAL_SHADINGMODEL_PREINTEGRATED_SKIN || Mat.MATERIAL_SHADINGMODEL_SUBSURFACE_PROFILE || Mat.MATERIAL_SHADINGMODEL_CLEAR_COAT || Mat.MATERIAL_SHADINGMODEL_TWOSIDED_FOLIAGE || Mat.MATERIAL_SHADINGMODEL_HAIR || Mat.MATERIAL_SHADINGMODEL_CLOTH || Mat.MATERIAL_SHADINGMODEL_EYE || Mat.MATERIAL_SHADINGMODEL_TOON));

	// Based on GetPrecomputedShadowMasks()
	// Note: WRITES_PRECSHADOWFACTOR_TO_GBUFFER is currently disabled because we use the precomputed shadow factor GBuffer outside of STATICLIGHTING_TEXTUREMASK to store UseSingleSampleShadowFromStationaryLights
	Dst.GBUFFER_HAS_PRECSHADOWFACTOR = (Dst.USES_GBUFFER && SrcGlobal.ALLOW_STATIC_LIGHTING);
	Dst.WRITES_PRECSHADOWFACTOR_ZERO = (!(Lightmap.STATICLIGHTING_TEXTUREMASK && Lightmap.STATICLIGHTING_SIGNEDDISTANCEFIELD) && (Lightmap.HQ_TEXTURE_LIGHTMAP || Lightmap.LQ_TEXTURE_LIGHTMAP));
	Dst.WRITES_PRECSHADOWFACTOR_TO_GBUFFER = (Dst.GBUFFER_HAS_PRECSHADOWFACTOR && !Dst.WRITES_PRECSHADOWFACTOR_ZERO);

	// If a primitive has static lighting, we assume it is not moving. If it is, it will be rerendered in an extra renderpass.
	Dst.SUPPORTS_WRITING_VELOCITY_TO_BASE_PASS = (FEATURE_LEVEL >= ERHIFeatureLevel::SM4_REMOVED && (Mat.MATERIALBLENDING_SOLID || Mat.MATERIALBLENDING_MASKED));
	Dst.WRITES_VELOCITY_TO_GBUFFER = (Dst.SUPPORTS_WRITING_VELOCITY_TO_BASE_PASS || Dst.USES_GBUFFER) &&
									 SrcGlobal.GBUFFER_HAS_VELOCITY &&
									 (!Mat.COMPUTE_SHADED || SrcGlobal.USES_BASE_PASS_VELOCITY || Mat.USES_WORLD_POSITION_OFFSET || Mat.USES_FIRST_PERSON_INTERPOLATION);

	Dst.TRANSLUCENCY_ANY_PERVERTEX_LIGHTING = (Mat.TRANSLUCENCY_LIGHTING_VOLUMETRIC_PERVERTEX_NONDIRECTIONAL || Mat.TRANSLUCENCY_LIGHTING_VOLUMETRIC_PERVERTEX_DIRECTIONAL);
	Dst.TRANSLUCENCY_ANY_VOLUMETRIC = (Mat.TRANSLUCENCY_LIGHTING_VOLUMETRIC_NONDIRECTIONAL || Mat.TRANSLUCENCY_LIGHTING_VOLUMETRIC_DIRECTIONAL || Dst.TRANSLUCENCY_ANY_PERVERTEX_LIGHTING);
	Dst.TRANSLUCENCY_PERVERTEX_LIGHTING_VOLUME = (!SrcGlobal.FORWARD_SHADING && (Mat.TRANSLUCENCY_LIGHTING_VOLUMETRIC_PERVERTEX_DIRECTIONAL || Mat.TRANSLUCENCY_LIGHTING_VOLUMETRIC_PERVERTEX_NONDIRECTIONAL));
	Dst.TRANSLUCENCY_PERVERTEX_FORWARD_SHADING = (SrcGlobal.FORWARD_SHADING && (Mat.TRANSLUCENCY_LIGHTING_VOLUMETRIC_PERVERTEX_DIRECTIONAL || Mat.TRANSLUCENCY_LIGHTING_VOLUMETRIC_PERVERTEX_NONDIRECTIONAL));

	Dst.COMPILE_SHADERS_FOR_DEVELOPMENT = SrcGlobal.COMPILE_SHADERS_FOR_DEVELOPMENT_ALLOWED && Mat.bAllowDevelopmentShaderCompile;

	Dst.USE_DEVELOPMENT_SHADERS = (Dst.COMPILE_SHADERS_FOR_DEVELOPMENT && Compiler.PLATFORM_SUPPORTS_DEVELOPMENT_SHADERS);


	Dst.PLATFORM_SUPPORTS_EDITOR_SHADERS = !Compiler.ESDEFERRED_PROFILE;

	Dst.USE_EDITOR_SHADERS = (Dst.PLATFORM_SUPPORTS_EDITOR_SHADERS && Dst.USE_DEVELOPMENT_SHADERS);

	//Materials also have to opt in to these features.
	Dst.USE_EDITOR_COMPOSITING = (Dst.USE_EDITOR_SHADERS && Mat.EDITOR_PRIMITIVE_MATERIAL);
	Dst.MATERIALBLENDING_ANY_TRANSLUCENT = (Mat.MATERIALBLENDING_TRANSLUCENT || Mat.MATERIALBLENDING_ADDITIVE || Mat.MATERIALBLENDING_MODULATE
		|| Mat.SUBSTRATE_BLENDING_TRANSLUCENT_GREYTRANSMITTANCE || Mat.SUBSTRATE_BLENDING_TRANSLUCENT_COLOREDTRANSMITTANCE || Mat.SUBSTRATE_BLENDING_COLOREDTRANSMITTANCEONLY);

	Dst.IS_MESHPARTICLE_FACTORY = (Lightmap.PARTICLE_MESH_FACTORY || Lightmap.NIAGARA_MESH_FACTORY);

	Dst.SUPPORTS_PIXEL_COVERAGE = (FEATURE_LEVEL >= ERHIFeatureLevel::SM5 && !Compiler.COMPILER_GLSL);

	Dst.FORCE_FULLY_ROUGH = (Mat.MATERIAL_FULLY_ROUGH);
	Dst.EDITOR_ALPHA2COVERAGE = (Dst.USE_EDITOR_COMPOSITING && Dst.SUPPORTS_PIXEL_COVERAGE);
	Dst.POST_PROCESS_SUBSURFACE = ((Mat.MATERIAL_SHADINGMODEL_SUBSURFACE_PROFILE || Mat.MATERIAL_SHADINGMODEL_EYE) && Dst.USES_GBUFFER);

	// matching the logic in: bool FMaterialResource::IsDualBlendingEnabled(EShaderPlatform Platform) const
	Dst.THIN_TRANSLUCENT_USE_DUAL_BLEND = Mat.MATERIAL_SHADINGMODEL_THIN_TRANSLUCENT && SrcGlobal.bSupportsDualBlending;

	// matching the logic in BasePassPixelShader.usf
	Dst.SUBSTRATE_ENABLED = Mat.SUBSTRATE_ENABLED | Mat.MATERIAL_IS_SUBSTRATE;
	Dst.SUBSTRATE_GBUFFER_FORMAT = Mat.SUBSTRATE_GBUFFER_FORMAT;
	Dst.SHADER_SUBSTRATE_TRANSLUCENT_ENABLED = Dst.SUBSTRATE_ENABLED && Dst.MATERIALBLENDING_ANY_TRANSLUCENT;
	// As of today, all translucent Substrate material forces dual source color blending. SUBSTRATE_TODO: optimize out dual color blending when not needed or requested
	Dst.MATERIAL_WORKS_WITH_DUAL_SOURCE_COLOR_BLENDING = (Mat.MATERIAL_SHADINGMODEL_THIN_TRANSLUCENT || Mat.SUBSTRATE_BLENDING_TRANSLUCENT_COLOREDTRANSMITTANCE);

	// matching the logic in BasePassPixelShader.usf
	Dst.OIT_ENABLED = (FEATURE_LEVEL >= ERHIFeatureLevel::SM5) && Mat.PROJECT_OIT && (Mat.MATERIALBLENDING_TRANSLUCENT || Mat.MATERIALBLENDING_ADDITIVE || Dst.MATERIAL_WORKS_WITH_DUAL_SOURCE_COLOR_BLENDING);

	// There are 4 different ways of setting MRTs depending on which .usf file is the base shader, which sets defines which includes a different include file
	// which may or may not override the current defines. Here is my best attempt at figuring out the logic.
	if (Mat.IS_VIRTUAL_TEXTURE_MATERIAL)
	{
		//if (0)
		{
			if (Mat.OUT_BASECOLOR)
			{
				Dst.PIXELSHADEROUTPUT_MRT0 = 1;
			}
			else if (Mat.OUT_BASECOLOR_NORMAL_ROUGHNESS)
			{
				Dst.PIXELSHADEROUTPUT_MRT0 = 1;
				Dst.PIXELSHADEROUTPUT_MRT1 = 1;
			}
			else if (Mat.OUT_BASECOLOR_NORMAL_SPECULAR)
			{
				Dst.PIXELSHADEROUTPUT_MRT0 = 1;
				Dst.PIXELSHADEROUTPUT_MRT1 = 1;
				Dst.PIXELSHADEROUTPUT_MRT2 = 1;
			}
			else if (Mat.OUT_MASK4)
			{
				Dst.PIXELSHADEROUTPUT_MRT0 = 1;
				Dst.PIXELSHADEROUTPUT_MRT1 = 1;
			}
			else if (Mat.OUT_WORLDHEIGHT)
			{
				Dst.PIXELSHADEROUTPUT_MRT0 = 1;
			}
			else if (Mat.OUT_DISPLACEMENT)
			{
				Dst.PIXELSHADEROUTPUT_MRT0 = 1;
			}
		}
	}
	else if (Mat.IS_DECAL)
	{
		//if (0)
		{
			Dst.PIXELSHADEROUTPUT_MRT0 = Mat.DECAL_RENDERTARGET_COUNT > 0;
			Dst.PIXELSHADEROUTPUT_MRT1 = Mat.DECAL_RENDERTARGET_COUNT > 1;
			Dst.PIXELSHADEROUTPUT_MRT2 = Mat.DECAL_RENDERTARGET_COUNT > 2;
			Dst.PIXELSHADEROUTPUT_MRT3 = Mat.DECAL_RENDERTARGET_COUNT > 3;
			Dst.PIXELSHADEROUTPUT_MRT4 = Mat.DECAL_RENDERTARGET_COUNT > 4;
		}
	}
	else if (Mat.IS_BASE_PASS)
	{
		Dst.PIXELSHADEROUTPUT_BASEPASS = 1;
		if (Dst.USES_GBUFFER)
		{
			Dst.PIXELSHADEROUTPUT_MRT0 = (!SrcGlobal.SELECTIVE_BASEPASS_OUTPUTS || Dst.NEEDS_BASEPASS_VERTEX_FOGGING || Mat.USES_EMISSIVE_COLOR || SrcGlobal.ALLOW_STATIC_LIGHTING || Mat.MATERIAL_SHADINGMODEL_SINGLELAYERWATER);
			Dst.PIXELSHADEROUTPUT_MRT1 = ((!SrcGlobal.SELECTIVE_BASEPASS_OUTPUTS || !Mat.MATERIAL_SHADINGMODEL_UNLIT));
			Dst.PIXELSHADEROUTPUT_MRT2 = ((!SrcGlobal.SELECTIVE_BASEPASS_OUTPUTS || !Mat.MATERIAL_SHADINGMODEL_UNLIT));
			Dst.PIXELSHADEROUTPUT_MRT3 = ((!SrcGlobal.SELECTIVE_BASEPASS_OUTPUTS || !Mat.MATERIAL_SHADINGMODEL_UNLIT));
			if (SrcGlobal.GBUFFER_HAS_VELOCITY || SrcGlobal.GBUFFER_HAS_TANGENT)
			{
				Dst.PIXELSHADEROUTPUT_MRT4 = Dst.WRITES_VELOCITY_TO_GBUFFER || SrcGlobal.GBUFFER_HAS_TANGENT;
				Dst.PIXELSHADEROUTPUT_MRT5 = (!SrcGlobal.SELECTIVE_BASEPASS_OUTPUTS || Dst.WRITES_CUSTOMDATA_TO_GBUFFER);
				Dst.PIXELSHADEROUTPUT_MRT6 = (Dst.GBUFFER_HAS_PRECSHADOWFACTOR && (!SrcGlobal.SELECTIVE_BASEPASS_OUTPUTS || (Dst.WRITES_PRECSHADOWFACTOR_TO_GBUFFER && !Mat.MATERIAL_SHADINGMODEL_UNLIT)));
			}
			else
			{
				Dst.PIXELSHADEROUTPUT_MRT4 = (!SrcGlobal.SELECTIVE_BASEPASS_OUTPUTS || Dst.WRITES_CUSTOMDATA_TO_GBUFFER);
				Dst.PIXELSHADEROUTPUT_MRT5 = (Dst.GBUFFER_HAS_PRECSHADOWFACTOR && (!SrcGlobal.SELECTIVE_BASEPASS_OUTPUTS || (Dst.WRITES_PRECSHADOWFACTOR_TO_GBUFFER && !Mat.MATERIAL_SHADINGMODEL_UNLIT)));
			}
		}
		else
		{
			Dst.PIXELSHADEROUTPUT_MRT0 = true;
			// we also need MRT for thin translucency due to dual blending if we are not on the fallback path
			Dst.PIXELSHADEROUTPUT_MRT1 = (Dst.WRITES_VELOCITY_TO_GBUFFER || (Mat.DUAL_SOURCE_COLOR_BLENDING_ENABLED && Dst.MATERIAL_WORKS_WITH_DUAL_SOURCE_COLOR_BLENDING));
		}
	}
	else
	{
		// Shouldn't be any other cases using PIXELSHADEROUTPUT_MRTX
	}

	Dst.PIXELSHADEROUTPUT_A2C = ((Dst.EDITOR_ALPHA2COVERAGE) != 0);
	Dst.PIXELSHADEROUTPUT_COVERAGE = (Mat.MATERIALBLENDING_MASKED_USING_COVERAGE && !SrcGlobal.EARLY_Z_PASS_ONLY_MATERIAL_MASKING);
	return Dst;
}


