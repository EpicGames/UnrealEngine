// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ShaderParameterMetadata.cpp: Shader parameter metadata implementations.
=============================================================================*/

#include "ShaderParameterMetadata.h"

#include "DataDrivenShaderPlatformInfo.h"
#include "Interfaces/IPluginManager.h"
#include "Misc/CommandLine.h"
#include "RenderCore.h"
#include "RHIUniformBufferLayoutInitializer.h"
#include "RHIShaderBindingLayout.h"
#include "Serialization/MemoryHasher.h"
#include "Serialization/ShaderKeyGenerator.h"
#include "ShaderCore.h"
#include "ShaderCompilerCore.h"
#include "ShaderParameters.h"
#include "ShaderParameterMacros.h"
#include "HLSLReservedSpaces.h"

bool SupportShaderPrecisionModifier(EShaderPlatform Platform)
{
	return IsMobilePlatform(Platform);
}

FUniformBufferStaticSlotRegistrar::FUniformBufferStaticSlotRegistrar(const TCHAR* InName)
{
	FUniformBufferStaticSlotRegistry::Get().RegisterSlot(InName);
}

FUniformBufferStaticSlotRegistry& FUniformBufferStaticSlotRegistry::Get()
{
	static FUniformBufferStaticSlotRegistry Registry;
	return Registry;
}

void FUniformBufferStaticSlotRegistry::RegisterSlot(FName SlotName)
{
	// Multiple definitions with the same name resolve to the same slot.
	const FUniformBufferStaticSlot Slot = FindSlotByName(SlotName);

	if (!IsUniformBufferStaticSlotValid(Slot))
	{
		SlotNames.Emplace(SlotName);
	}
}

#define VALIDATE_UNIFORM_BUFFER_UNIQUE_NAME (!UE_BUILD_SHIPPING && !UE_BUILD_TEST)

#if VALIDATE_UNIFORM_BUFFER_UNIQUE_NAME
TMap<FName, FName> & GetGlobalShaderVariableToStructMap()
{
	static TMap<FName, FName> GlobalShaderVariableToStructMap;
	return GlobalShaderVariableToStructMap;
}
#endif

TMap<uint32, FShaderParametersMetadata*> & GetLayoutHashStructMap()
{
	static TMap<uint32, FShaderParametersMetadata*> GLayoutHashStructMap;
	return GLayoutHashStructMap;
}

TLinkedList<FShaderParametersMetadata*>*& FShaderParametersMetadata::GetStructList()
{
	static TLinkedList<FShaderParametersMetadata*>* GUniformStructList = nullptr;
	return GUniformStructList;
}

TMap<FHashedName, FShaderParametersMetadata*>& FShaderParametersMetadata::GetNameStructMap()
{
	static TMap<FHashedName, FShaderParametersMetadata*> NameStructMap;
	return NameStructMap;
}

#if WITH_EDITOR
TMap<FString, FShaderParametersMetadata*>& FShaderParametersMetadata::GetStringStructMap()
{
	static TMap<FString, FShaderParametersMetadata*> StringStructMap;
	return StringStructMap;
}
#endif  // WITH_EDITOR

void FShaderParametersMetadata::FMember::GenerateShaderParameterType(
	FString& Result,
	bool bSupportsPrecisionModifier,
	EUniformBufferBaseType BaseType,
	EShaderPrecisionModifier::Type PrecisionModifier,
	uint32 NumRows,
	uint32 NumColumns)
{
	switch (BaseType)
	{
	case UBMT_INT32:   Result = TEXT("int"); break;
	case UBMT_UINT32:  Result = TEXT("uint"); break;
	case UBMT_FLOAT32:
		if (PrecisionModifier == EShaderPrecisionModifier::Float || !bSupportsPrecisionModifier)
		{
			Result = TEXT("float");
		}
		else if (PrecisionModifier == EShaderPrecisionModifier::Half)
		{
			Result = TEXT("half");
		}
		else if (PrecisionModifier == EShaderPrecisionModifier::Fixed)
		{
			Result = TEXT("fixed");
		}
		break;
	case UBMT_DESCRIPTOR_RANGE:
		Result = TEXT("FDescriptorRange");
		break;
	default:
		UE_LOGF(LogShaders, Fatal, "Unrecognized uniform buffer struct member base type.");
	};

	// Generate the type dimensions for vectors and matrices.
	if (NumRows > 1)
	{
		Result = FString::Printf(TEXT("%s%ux%u"), *Result, NumRows, NumColumns);
	}
	else if (NumColumns > 1)
	{
		Result = FString::Printf(TEXT("%s%u"), *Result, NumColumns);
	}
}

void FShaderParametersMetadata::FMember::GenerateShaderParameterType(FString& Result, bool bSupportsPrecisionModifier) const
{
	GenerateShaderParameterType(Result, bSupportsPrecisionModifier, GetBaseType(), GetPrecision(), GetNumRows(), GetNumColumns());
}

void FShaderParametersMetadata::FMember::GenerateShaderParameterType(FString& Result, EShaderPlatform ShaderPlatform) const
{
	GenerateShaderParameterType(Result, SupportShaderPrecisionModifier(ShaderPlatform));
}

FShaderParametersMetadata* FindUniformBufferStructByName(const TCHAR* StructName)
{
	return FindUniformBufferStructByFName(FName(StructName, FNAME_Find));
}

FShaderParametersMetadata* FindUniformBufferStructByFName(FName StructName)
{
	return FShaderParametersMetadata::GetNameStructMap().FindRef(StructName);
}

FShaderParametersMetadata* FindUniformBufferStructByLayoutHash(uint32 Hash)
{
	return GetLayoutHashStructMap().FindRef(Hash);
}

void BuildShaderBindingLayout(TConstArrayView<FShaderParametersMetadata*> UniformBuffers, EShaderBindingLayoutFlags BaseShaderBindingLayoutFlags, FShaderBindingLayoutContainer& OutShaderBindingLayoutContainer)
{
	// only build the bindless version for now
	FShaderBindingLayoutContainer::EBindingType BindingType = FShaderBindingLayoutContainer::EBindingType::Bindless;
	
	FShaderBindingLayout ShaderBindingLayout;

	EShaderBindingLayoutFlags ShaderBindingLayoutFlags = BaseShaderBindingLayoutFlags;
	//if (BindingType == FShaderBindingLayoutContainer::EBindingType::Bindless)
	{
		ShaderBindingLayoutFlags |= EShaderBindingLayoutFlags::BindlessResources | EShaderBindingLayoutFlags::BindlessSamplers;
	}

	bool bBindlessResources = EnumHasAllFlags(ShaderBindingLayoutFlags, EShaderBindingLayoutFlags::BindlessResources);
	bool bBindlessSamplers = EnumHasAllFlags(ShaderBindingLayoutFlags, EShaderBindingLayoutFlags::BindlessSamplers);

	TArray<FRHIUniformBufferShaderBindingLayout> UniformBufferEntries;
	UniformBufferEntries.Reserve(UniformBuffers.Num());

	int32 CurrentCBVResourceIndex = 0;
	int32 CurrentBaseSRVResourceIndex = bBindlessResources ? -1 : 0;
	int32 CurrentBaseUAVResourceIndex = bBindlessResources ? -1 : 0;
	int32 CurrentBaseSamplerResourceIndex = bBindlessSamplers ? -1 : 0;
	for (FShaderParametersMetadata* ShaderParametersMetaData : UniformBuffers)
	{
		const FRHIUniformBufferLayout& UniformBufferLayout = ShaderParametersMetaData->GetLayout();
		check(UniformBufferLayout.ConstantBufferSize > 0);

		FRHIUniformBufferShaderBindingLayout& NewEntry = UniformBufferEntries.AddDefaulted_GetRef();
		NewEntry.LayoutName = UniformBufferLayout.Name;
		NewEntry.RegisterSpace = UE_HLSL_SPACE_STATIC_SHADER_BINDINGS;
		NewEntry.CBVResourceIndex = CurrentCBVResourceIndex;
		NewEntry.BaseSRVResourceIndex = CurrentBaseSRVResourceIndex;
		NewEntry.BaseUAVResourceIndex = CurrentBaseUAVResourceIndex;
		NewEntry.BaseSamplerResourceIndex = CurrentBaseSamplerResourceIndex;

		// Increment counters		
		CurrentCBVResourceIndex++;

		// Don't need to increment resources if using bindless because all data is in the CBV
		if (!bBindlessSamplers)
		{
			// TODO use layout to find out all the samplers
			check(false);
		}
		if (!bBindlessResources)
		{
			// TODO use layout to find out all the resources and uavs
			check(false);
		}
	}

	ShaderBindingLayout.RHILayout = FRHIShaderBindingLayout(ShaderBindingLayoutFlags, UniformBufferEntries);

#if WITH_EDITOR
	// Also create the uniform buffer decleration shared ansi strings used during shader gen
	for (uint8 UniformBufferIndex = 0; UniformBufferIndex < UniformBuffers.Num(); ++UniformBufferIndex)	
	{
		FShaderParametersMetadata* ShaderParametersMetaData = UniformBuffers[UniformBufferIndex];
		const FString& LayoutName = ShaderParametersMetaData->GetLayout().Name;
		const FRHIUniformBufferShaderBindingLayout* UniformBufferSBLayout = ShaderBindingLayout.RHILayout.FindEntry(LayoutName);
		check(UniformBufferSBLayout);

		FString UniformBufferName(ShaderParametersMetaData->GetShaderVariableName());
		FString NewDeclaration = UE::ShaderParameters::CreateUniformBufferShaderDeclaration(*UniformBufferName, *ShaderParametersMetaData, UniformBufferSBLayout);
		check(!NewDeclaration.IsEmpty());
			
		// Cache preprocessor friendly copy of uniform buffer declaration
		TArray<ANSICHAR>* NewDeclarationAnsi = new TArray<ANSICHAR>;
		ShaderConvertAndStripComments(NewDeclaration, *NewDeclarationAnsi);
		FThreadSafeSharedAnsiStringPtr UniformBufferDeclarationAnsi = MakeShareable(NewDeclarationAnsi);

		ShaderBindingLayout.SetUniformBufferDeclarationAnsiPtr(ShaderParametersMetaData, UniformBufferDeclarationAnsi);
	}
#endif //WITH_EDITOR

	OutShaderBindingLayoutContainer.SetLayout(BindingType, ShaderBindingLayout);
}

static TArray<const FShaderParametersMetadataRegistration*>* GShaderParametersMetadataRegistrationInstances = nullptr;
TArray<const FShaderParametersMetadataRegistration*>& FShaderParametersMetadataRegistration::GetInstances()
{
	if (GShaderParametersMetadataRegistrationInstances == nullptr)
	{
		GShaderParametersMetadataRegistrationInstances = new TArray<const FShaderParametersMetadataRegistration*>();
	}
	return *GShaderParametersMetadataRegistrationInstances;
}

void FShaderParametersMetadataRegistration::CommitAll()
{
	for (const auto* Instance : GetInstances())
	{
		Instance->LazyShaderParametersMetadataAccessor();
	}
	GetInstances().Empty();
}

bool FShaderParametersMetadataRegistration::IsReadyForRegistration()
{
	return FCommandLine::IsInitialized() /* If cmd is not ready yet, then it's too early for the plugin manager */ 
		&& IPluginManager::Get().GetLastCompletedLoadingPhase() >= ELoadingPhase::PostConfigInit;
}

const TCHAR* const kShaderParameterMacroNames[] = {
	TEXT(""), // UBMT_INVALID,

	// Invalid type when trying to use bool, to have explicit error message to programmer on why
	// they shouldn't use bool in shader parameter structures.
	TEXT("SHADER_PARAMETER"), // UBMT_BOOL,

	// Parameter types.
	TEXT("SHADER_PARAMETER"), // UBMT_INT32,
	TEXT("SHADER_PARAMETER"), // UBMT_UINT32,
	TEXT("SHADER_PARAMETER"), // UBMT_FLOAT32,

	// RHI resources not tracked by render graph.
	TEXT("SHADER_PARAMETER_TEXTURE"), // UBMT_TEXTURE,
	TEXT("SHADER_PARAMETER_SRV"), // UBMT_SRV,
	TEXT("SHADER_PARAMETER_UAV"), // UBMT_UAV,
	TEXT("SHADER_PARAMETER_SAMPLER"), // UBMT_SAMPLER,

	// Resources tracked by render graph.
	TEXT("SHADER_PARAMETER_RDG_TEXTURE"), // UBMT_RDG_TEXTURE,
	TEXT("RDG_TEXTURE_ACCESS"), // UBMT_RDG_TEXTURE_ACCESS,
	TEXT("RDG_TEXTURE_ACCESS_ARRAY"), // UBMT_RDG_TEXTURE_ACCESS,
	TEXT("SHADER_PARAMETER_RDG_TEXTURE_SRV"), // UBMT_RDG_TEXTURE_SRV,
	TEXT("SHADER_PARAMETER_RDG_TEXTURE_NON_PIXEL_SRV"), // UBMT_RDG_TEXTURE_NON_PIXEL_SRV,
	TEXT("SHADER_PARAMETER_RDG_TEXTURE_UAV"), // UBMT_RDG_TEXTURE_UAV,
	TEXT("RDG_BUFFER_ACCESS"), // UBMT_RDG_BUFFER_ACCESS,
	TEXT("RDG_BUFFER_ACCESS_ARRAY"), // UBMT_RDG_BUFFER_ACCESS_ARRAY,
	TEXT("SHADER_PARAMETER_RDG_BUFFER_SRV"), // UBMT_RDG_BUFFER_SRV,
	TEXT("SHADER_PARAMETER_RDG_BUFFER_UAV"), // UBMT_RDG_BUFFER_UAV,
	TEXT("SHADER_PARAMETER_RDG_UNIFORM_BUFFER"), // UBMT_RDG_UNIFORM_BUFFER,

	// Nested structure.
	TEXT("SHADER_PARAMETER_STRUCT"), // UBMT_NESTED_STRUCT,

	// Structure that is nested on C++ side, but included on shader side.
	TEXT("SHADER_PARAMETER_STRUCT_INCLUDE"), // UBMT_INCLUDED_STRUCT,

	// GPU Indirection reference of struct, like is currently named Uniform buffer.
	TEXT("SHADER_PARAMETER_STRUCT_REF"), // UBMT_REFERENCED_STRUCT,

	// Structure dedicated to setup render targets for a rasterizer pass.
	TEXT("RENDER_TARGET_BINDING_SLOTS"), // UBMT_RENDER_TARGET_BINDING_SLOTS,

	TEXT("RESOURCE_COLLECTION"), // UBMT_RESOURCE_COLLECTION,
	TEXT("DESCRIPTOR_RANGE"), // UBMT_DESCRIPTOR_RANGE,
};

static_assert(UE_ARRAY_COUNT(kShaderParameterMacroNames) == int32(EUniformBufferBaseType_Num), "Shader parameter enum does not match name macro name array.");

/** Returns the name of the macro that should be used for a given shader parameter base type. */
const TCHAR* GetShaderParameterMacroName(EUniformBufferBaseType ShaderParameterBaseType)
{
	check(ShaderParameterBaseType != UBMT_INVALID);
	return kShaderParameterMacroNames[int32(ShaderParameterBaseType)];
}

EShaderCodeResourceBindingType ParseShaderResourceBindingType(const TCHAR* ShaderType)
{
	const bool bIsRasterizerOrderedResource = FCString::Strncmp(ShaderType, TEXT("RasterizerOrdered"), 17) == 0;
	const bool bIsRWResource = FCString::Strncmp(ShaderType, TEXT("RW"), 2) == 0 || bIsRasterizerOrderedResource;
	const TCHAR* ComparedShaderType = ShaderType + (bIsRWResource ? 2 : 0);

	int32 ShaderTypeLength = 0;
	while (TCHAR c = ComparedShaderType[ShaderTypeLength])
	{
		if (c == ' ' || c == '<')
			break;
		ShaderTypeLength++;
	}

	EShaderCodeResourceBindingType BindingType = EShaderCodeResourceBindingType::Invalid;
	if (!bIsRWResource && ShaderTypeLength == 12 && FCString::Strncmp(ComparedShaderType, TEXT("SamplerState"), ShaderTypeLength) == 0)
	{
		BindingType = EShaderCodeResourceBindingType::SamplerState;
	}
	else if (!bIsRWResource && ShaderTypeLength == 22 && FCString::Strncmp(ComparedShaderType, TEXT("SamplerComparisonState"), ShaderTypeLength) == 0)
	{
		BindingType = EShaderCodeResourceBindingType::SamplerState;
	}
	else if (ShaderTypeLength == 9 && FCString::Strncmp(ComparedShaderType, TEXT("Texture2D"), ShaderTypeLength) == 0)
	{
		BindingType = bIsRWResource ? EShaderCodeResourceBindingType::RWTexture2D : EShaderCodeResourceBindingType::Texture2D;
	}
	else if (ShaderTypeLength == 15 && FCString::Strncmp(ComparedShaderType, TEXT("TextureExternal"), ShaderTypeLength) == 0)
	{
		BindingType = bIsRWResource ? EShaderCodeResourceBindingType::RWTexture2D : EShaderCodeResourceBindingType::Texture2D;
	}
	else if (ShaderTypeLength == 14 && FCString::Strncmp(ComparedShaderType, TEXT("Texture2DArray"), ShaderTypeLength) == 0)
	{
		BindingType = bIsRWResource ? EShaderCodeResourceBindingType::RWTexture2DArray : EShaderCodeResourceBindingType::Texture2DArray;
	}
	else if (ShaderTypeLength == 18 && FCString::Strncmp(ComparedShaderType, TEXT("Texture2DMultiView"), ShaderTypeLength) == 0)
	{
		BindingType = bIsRWResource ? EShaderCodeResourceBindingType::RWTexture2DMultiView : EShaderCodeResourceBindingType::Texture2DMultiView;
	}
	else if (!bIsRWResource && ShaderTypeLength == 11 && FCString::Strncmp(ComparedShaderType, TEXT("Texture2DMS"), ShaderTypeLength) == 0)
	{
		BindingType = EShaderCodeResourceBindingType::Texture2DMS;
	}
	else if (ShaderTypeLength == 9 && FCString::Strncmp(ComparedShaderType, TEXT("Texture3D"), ShaderTypeLength) == 0)
	{
		BindingType = bIsRWResource ? EShaderCodeResourceBindingType::RWTexture3D : EShaderCodeResourceBindingType::Texture3D;
	}
	else if (ShaderTypeLength == 11 && FCString::Strncmp(ComparedShaderType, TEXT("TextureCube"), ShaderTypeLength) == 0)
	{
		BindingType = bIsRWResource ? EShaderCodeResourceBindingType::RWTextureCube : EShaderCodeResourceBindingType::TextureCube;
	}
	else if (!bIsRWResource && ShaderTypeLength == 16 && FCString::Strncmp(ComparedShaderType, TEXT("TextureCubeArray"), ShaderTypeLength) == 0)
	{
		BindingType = EShaderCodeResourceBindingType::TextureCubeArray;
	}
	else if (ShaderTypeLength == 15 && FCString::Strncmp(ComparedShaderType, TEXT("TextureMetadata"), ShaderTypeLength) == 0)
	{
		BindingType = bIsRWResource ? EShaderCodeResourceBindingType::RWTextureMetadata : EShaderCodeResourceBindingType::TextureMetadata;
	}
	else if (ShaderTypeLength == 6 && FCString::Strncmp(ComparedShaderType, TEXT("Buffer"), ShaderTypeLength) == 0)
	{
		BindingType = bIsRWResource ? EShaderCodeResourceBindingType::RWBuffer : EShaderCodeResourceBindingType::Buffer;
	}
	else if (ShaderTypeLength == 16 && FCString::Strncmp(ComparedShaderType, TEXT("StructuredBuffer"), ShaderTypeLength) == 0)
	{
		BindingType = bIsRWResource ? EShaderCodeResourceBindingType::RWStructuredBuffer : EShaderCodeResourceBindingType::StructuredBuffer;
	}
	else if (ShaderTypeLength == 17 && FCString::Strncmp(ComparedShaderType, TEXT("ByteAddressBuffer"), ShaderTypeLength) == 0)
	{
		BindingType = bIsRWResource ? EShaderCodeResourceBindingType::RWByteAddressBuffer : EShaderCodeResourceBindingType::ByteAddressBuffer;
	}
	else if (!bIsRWResource && ShaderTypeLength == 31 && FCString::Strncmp(ComparedShaderType, TEXT("RaytracingAccelerationStructure"), ShaderTypeLength) == 0)
	{
		BindingType = EShaderCodeResourceBindingType::RaytracingAccelerationStructure;
	}
	else if (!bIsRWResource && ShaderTypeLength == 19 && FCString::Strncmp(ComparedShaderType, TEXT("FResourceCollection"), ShaderTypeLength) == 0)
	{
		BindingType = EShaderCodeResourceBindingType::ResourceCollection;
	}
	else if (!bIsRWResource && ShaderTypeLength == 23 && FCString::Strncmp(ComparedShaderType, TEXT("FIndirectVirtualTexture"), ShaderTypeLength) == 0)
	{
		BindingType = EShaderCodeResourceBindingType::Buffer;
	}
	else if (!bIsRWResource && ShaderTypeLength == 16 && FCString::Strncmp(ComparedShaderType, TEXT("FDescriptorRange"), ShaderTypeLength) == 0)
	{
		BindingType = EShaderCodeResourceBindingType::DescriptorRange;
	}
	else if (bIsRasterizerOrderedResource)
	{
		BindingType = EShaderCodeResourceBindingType::RasterizerOrderedTexture2D;
	}
	return BindingType;
}

class FUniformBufferMemberAndOffset
{
public:
	FUniformBufferMemberAndOffset(const FShaderParametersMetadata& InContainingStruct, const FShaderParametersMetadata::FMember& InMember, int32 InStructOffset) :
		ContainingStruct(InContainingStruct),
		Member(InMember),
		StructOffset(InStructOffset)
	{}

	const FShaderParametersMetadata& ContainingStruct;
	const FShaderParametersMetadata::FMember& Member;
	int32 StructOffset;
};

FShaderParametersMetadata::FShaderParametersMetadata(
	EUseCase InUseCase,
	EUniformBufferBindingFlags InBindingFlags,
	const TCHAR* InLayoutName,
	const TCHAR* InStructTypeName,
	const TCHAR* InShaderVariableName,
	const TCHAR* InStaticSlotName,
	const ANSICHAR* InFileName,
	const int32 InFileLine,
	uint32 InSize,
	const TArray<FMember>& InMembers,
	bool bForceCompleteInitialization,
	FRHIUniformBufferLayoutInitializer* OutLayoutInitializer,
	EUsageFlags InUsageFlags)
	: LayoutName(InLayoutName)
	, StructTypeName(InStructTypeName)
	, ShaderVariableName(InShaderVariableName)
	, StaticSlotName(InStaticSlotName)
	, ShaderVariableHashedName(InShaderVariableName)
	, FileName(InFileName)
	, FileLine(InFileLine)
	, Size(InSize)
	, UseCase(InUseCase)
	, BindingFlags(InBindingFlags)
	, UsageFlags(InUsageFlags)
	, Members(InMembers)
	, GlobalListLink(this)
{
	checkf(UseCase == EUseCase::UniformBuffer || !EnumHasAnyFlags(BindingFlags, EUniformBufferBindingFlags::Static), TEXT("Only uniform buffers can utilize the global binding flag."));

	check(StructTypeName);
	if (UseCase == EUseCase::ShaderParameterStruct)
	{
		checkf(!StaticSlotName, TEXT("Only uniform buffers can be tagged with a static slot."));

		check(ShaderVariableName == nullptr);
	}
	else
	{
		check(ShaderVariableName);
	}
	
	// must ensure the global classes are full initialized before us
	//	so that they destruct after us
	#if VALIDATE_UNIFORM_BUFFER_UNIQUE_NAME
	GetGlobalShaderVariableToStructMap();
	#endif
	GetLayoutHashStructMap();
	GetStructList();
	GetNameStructMap();

	if (UseCase == EUseCase::UniformBuffer && !bForceCompleteInitialization)
	{
		// Register this uniform buffer struct in global list.
		GlobalListLink.LinkHead(GetStructList());

		FName StructTypeFName(StructTypeName);
		// Verify that during FName creation there's no case conversion
		checkSlow(FCString::Strcmp(StructTypeName, *StructTypeFName.GetPlainNameString()) == 0);
		GetNameStructMap().Add(ShaderVariableHashedName, this);


#if VALIDATE_UNIFORM_BUFFER_UNIQUE_NAME
		FName ShaderVariableFName(ShaderVariableName);

		// Verify that the global variable name is unique so that we can disambiguate when reflecting from shader source.
		if (FName* StructFName = GetGlobalShaderVariableToStructMap().Find(ShaderVariableFName))
		{
			checkf(
				false,
				TEXT("Found duplicate Uniform Buffer shader variable name %s defined by struct %s. Previous definition ")
				TEXT("found on struct %s. Uniform buffer shader names must be unique to support name-based reflection of ")
				TEXT("shader source files."),
				ShaderVariableName,
				StructTypeName,
				*StructFName->GetPlainNameString());
		}

		GetGlobalShaderVariableToStructMap().Add(ShaderVariableFName, StructTypeFName);
#endif
	}
	else
	{
#if WITH_EDITOR
		InitializeUniformBufferDeclaration();
#endif

		// We cannot initialize the layout during global initialization, since we have to walk nested struct members.
		// Structs created during global initialization will have bRegisterForAutoBinding==false, and are initialized during startup.
		// Structs created at runtime with bRegisterForAutoBinding==true can be initialized now.
		InitializeLayout(OutLayoutInitializer);
	}
}

FShaderParametersMetadata::~FShaderParametersMetadata()
{
	// FShaderParametersMetadata are objects at file scope
	//	this destructor can run at process exit
	// it touches globals like GlobalShaderVariableToStructMap and GLayoutHashStructMap
	//	must ensure they are not already be destructed

	if (GlobalListLink.IsLinked())
	{
		check(UseCase == EUseCase::UniformBuffer);
		GlobalListLink.Unlink();
		GetNameStructMap().Remove(ShaderVariableHashedName);

#if VALIDATE_UNIFORM_BUFFER_UNIQUE_NAME
		GetGlobalShaderVariableToStructMap().Remove(FName(ShaderVariableName, FNAME_Find));
#endif

		if (IsLayoutInitialized())
		{
			GetLayoutHashStructMap().Remove(GetLayout().GetHash());
		}
	}
}


void FShaderParametersMetadata::InitializeAllUniformBufferStructs()
{
#if WITH_EDITOR
	TMap<FString, FShaderParametersMetadata*>& StringStructMap = FShaderParametersMetadata::GetStringStructMap();
#endif // WITH_EDITOR

	for (TLinkedList<FShaderParametersMetadata*>::TIterator StructIt(FShaderParametersMetadata::GetStructList()); StructIt; StructIt.Next())
	{
#if WITH_EDITOR
		if (!StructIt->IsUniformBufferDeclarationInitialized())
		{
			StructIt->InitializeUniformBufferDeclaration();
		}

		StringStructMap.Add(StructIt->GetShaderVariableName(), *StructIt);
#endif // WITH_EDITOR

		if (!StructIt->IsLayoutInitialized())
		{
			StructIt->InitializeLayout();
		}
	}
}

#if WITH_EDITOR

void FShaderParametersMetadata::AppendKeyString(FString& OutKeyString) const
{
	FShaderKeyGenerator KeyGen(OutKeyString);
	Append(KeyGen);
}

void FShaderParametersMetadata::Append(FShaderKeyGenerator& KeyGen) const
{
	KeyGen.AppendDebugText(TEXT("SPM_"));
	KeyGen.Append(LayoutSignature);
}

#endif // WITH_EDITOR

void FShaderParametersMetadata::InitializeLayout(FRHIUniformBufferLayoutInitializer* OutLayoutInitializer)
{
	check(!IsLayoutInitialized());

	FRHIUniformBufferLayoutInitializer LocalLayoutInitializer(LayoutName);
	FRHIUniformBufferLayoutInitializer& LayoutInitializer = OutLayoutInitializer ? *OutLayoutInitializer : LocalLayoutInitializer;
	LayoutInitializer.ConstantBufferSize = Size;

	if (EnumHasAnyFlags(UsageFlags, EUsageFlags::UniformView))
	{
		EnumAddFlags(LayoutInitializer.Flags, ERHIUniformBufferFlags::UniformView);
	}

	if (EnumHasAnyFlags(UsageFlags, EUsageFlags::NoEmulatedUniformBuffer|EUsageFlags::UniformView))
	{
		EnumAddFlags(LayoutInitializer.Flags, ERHIUniformBufferFlags::NoEmulatedUniformBuffer);
	}

	if (EnumHasAnyFlags(UsageFlags, EUsageFlags::NeedsReflectedMembers))
	{
		EnumAddFlags(LayoutInitializer.Flags, ERHIUniformBufferFlags::NeedsReflectedMembers);
	}

	if (EnumHasAnyFlags(UsageFlags, EUsageFlags::BindlessAccessible))
	{
		EnumAddFlags(LayoutInitializer.Flags, ERHIUniformBufferFlags::BindlessAccessible);
	}
	
	if (StaticSlotName)
	{
		checkf(EnumHasAnyFlags(BindingFlags, EUniformBufferBindingFlags::Static), TEXT("Uniform buffer of type '%s' and shader name '%s' attempted to reference static slot '%s', but the binding model does not contain 'Global'."),
			StructTypeName, ShaderVariableName, StaticSlotName);

		checkf(UseCase == EUseCase::UniformBuffer,
			TEXT("Attempted to assign static slot %s to uniform buffer %s. Static slots are only supported for compile-time uniform buffers."),
			ShaderVariableName, StaticSlotName);

		const FUniformBufferStaticSlot StaticSlot = FUniformBufferStaticSlotRegistry::Get().FindSlotByName(StaticSlotName);

		checkf(IsUniformBufferStaticSlotValid(StaticSlot),
			TEXT("Uniform buffer of type '%s' and shader name '%s' attempted to reference static slot '%s', but the slot could not be found in the registry."),
			StructTypeName, ShaderVariableName, StaticSlotName);

		LayoutInitializer.StaticSlot = StaticSlot;
		LayoutInitializer.BindingFlags = BindingFlags;
	}
	else
	{
		checkf(!EnumHasAnyFlags(BindingFlags, EUniformBufferBindingFlags::Static), TEXT("Uniform buffer of type '%s' and shader name '%s' has no binding slot specified, but the binding model specifies 'Global'."),
			StructTypeName, ShaderVariableName);
	}

	TArray<FUniformBufferMemberAndOffset> MemberStack;
	MemberStack.Reserve(Members.Num());
	for (int32 MemberIndex = 0; MemberIndex < Members.Num(); MemberIndex++)
	{
		MemberStack.Push(FUniformBufferMemberAndOffset(*this, Members[MemberIndex], 0));
	}

	/** Uniform buffer references are only allowed in shader parameter structures that may be used as a root shader parameter
	 * structure.
	 */
	const bool bAllowUniformBufferReferences = UseCase == EUseCase::ShaderParameterStruct;

	/** Resource array are currently only supported for shader parameter structures. */
	const bool bAllowResourceArrays = UseCase == EUseCase::ShaderParameterStruct;

	/** Allow all use cases that inline a structure within another. Data driven are not known to inline structures. */
	const bool bAllowStructureInlining = UseCase == EUseCase::ShaderParameterStruct || UseCase == EUseCase::UniformBuffer;

	bool bHasNonGraphOutputs = false;

	for (int32 i = 0; i < MemberStack.Num(); ++i)
	{
		const FShaderParametersMetadata& CurrentStruct = MemberStack[i].ContainingStruct;
		const FMember& CurrentMember = MemberStack[i].Member;

		EUniformBufferBaseType BaseType = CurrentMember.GetBaseType();
		const uint32 ArraySize = CurrentMember.GetNumElements();
		const FShaderParametersMetadata* ChildStruct = CurrentMember.GetStructMetadata();
		const TCHAR* ShaderType = CurrentMember.GetShaderType();

		const bool bIsArray = ArraySize > 0;
		const bool bIsRHIResource = IsShaderParameterTypeReadOnlyRHIResource(BaseType);
		const bool bIsRDGResource = IsRDGResourceReferenceShaderParameterType(BaseType);
		const bool bIsVariableNativeType = CurrentMember.IsVariableNativeType();

		bHasNonGraphOutputs |= (BaseType == UBMT_UAV);

		if (DO_CHECK)
		{
			auto GetMemberErrorPrefix = [&]()
			{
				return FString::Printf(
					TEXT("%s(%i): Shader parameter %s::%s error:\n"),
					ANSI_TO_TCHAR(CurrentStruct.GetFileName()),
					CurrentMember.GetFileLine(),
					CurrentStruct.GetStructTypeName(),
					CurrentMember.GetName());
			};

			if (BaseType == UBMT_BOOL)
			{
				UE_LOGF(LogRendererCore, Fatal,
					"%lsbool are actually illegal in shader parameter structure, " "because bool type in HLSL means using scalar register to store binary information. " "Boolean information should always be packed explicitly in bitfield to reduce memory footprint, " "and use HLSL comparison operators to translate into clean SGPR, to have minimal VGPR footprint.", *GetMemberErrorPrefix());
			}

			if (BaseType == UBMT_REFERENCED_STRUCT || BaseType == UBMT_RDG_UNIFORM_BUFFER)
			{
				check(ChildStruct);

				if (!bAllowUniformBufferReferences)
				{
					UE_LOGF(LogRendererCore, Fatal, "%lsShader parameter struct reference can only be done in shader parameter structs.", *GetMemberErrorPrefix());
				}
			}
			else if (BaseType == UBMT_NESTED_STRUCT || BaseType == UBMT_INCLUDED_STRUCT)
			{
				check(ChildStruct);

				if (!bAllowStructureInlining)
				{
					UE_LOGF(LogRendererCore, Fatal, "%lsShader parameter struct is not known inline other structures.", *GetMemberErrorPrefix());
				}
				else if (ChildStruct->GetUseCase() != EUseCase::ShaderParameterStruct && UseCase == EUseCase::ShaderParameterStruct)
				{
					UE_LOGF(LogRendererCore, Fatal, "%lsCan only nest or include shader parameter struct define with BEGIN_SHADER_PARAMETER_STRUCT(), but %ls is not.", *GetMemberErrorPrefix(), ChildStruct->GetStructTypeName());
				}
			}

			if (UseCase != EUseCase::ShaderParameterStruct && IsShaderParameterTypeIgnoredByRHI(BaseType))
			{
				UE_LOGF(LogRendererCore, Fatal, "Shader parameter %ls is not allowed in a uniform buffer.", *GetMemberErrorPrefix());
			}

			const bool bTypeCanBeArray = (bAllowResourceArrays && (bIsRHIResource || bIsRDGResource)) || bIsVariableNativeType || BaseType == UBMT_NESTED_STRUCT;
			if (bIsArray && !bTypeCanBeArray)
			{
				UE_LOGF(LogRendererCore, Fatal, "%lsNot allowed to be an array.", *GetMemberErrorPrefix());
			}

			// Validate the shader binding type.
			if (bIsRHIResource || (bIsRDGResource && !IsRDGResourceAccessType(BaseType) && BaseType != UBMT_RDG_UNIFORM_BUFFER))
			{
				EShaderCodeResourceBindingType BindingType = ParseShaderResourceBindingType(ShaderType);

				if (BindingType == EShaderCodeResourceBindingType::Invalid)
				{
					UE_LOGF(LogRendererCore, Warning, "%lsUnknown shader parameter type %ls.", *GetMemberErrorPrefix(), ShaderType);
				}

				bool bIsValidBindingType = false;
				if (BindingType == EShaderCodeResourceBindingType::SamplerState)
				{
					bIsValidBindingType = (BaseType == UBMT_SAMPLER);
				}
				else if (
					BindingType == EShaderCodeResourceBindingType::Texture2D ||
					BindingType == EShaderCodeResourceBindingType::Texture2DArray ||
					BindingType == EShaderCodeResourceBindingType::Texture2DMultiView ||
					BindingType == EShaderCodeResourceBindingType::Texture2DMS ||
					BindingType == EShaderCodeResourceBindingType::Texture3D ||
					BindingType == EShaderCodeResourceBindingType::TextureCube ||
					BindingType == EShaderCodeResourceBindingType::TextureCubeArray)
				{
					bIsValidBindingType = (
						BaseType == UBMT_TEXTURE ||
						BaseType == UBMT_SRV ||
						BaseType == UBMT_RDG_TEXTURE ||
						BaseType == UBMT_RDG_TEXTURE_SRV ||
						BaseType == UBMT_RDG_TEXTURE_NON_PIXEL_SRV);
				}
				else if (BindingType == EShaderCodeResourceBindingType::TextureMetadata)
				{
					bIsValidBindingType = (
						BaseType == UBMT_SRV ||
						BaseType == UBMT_RDG_TEXTURE_SRV ||
						BaseType == UBMT_RDG_TEXTURE_NON_PIXEL_SRV);
				}
				else if (
					BindingType == EShaderCodeResourceBindingType::Buffer ||
					BindingType == EShaderCodeResourceBindingType::StructuredBuffer ||
					BindingType == EShaderCodeResourceBindingType::ByteAddressBuffer)
				{
					bIsValidBindingType = (
						BaseType == UBMT_SRV ||
						BaseType == UBMT_RDG_BUFFER_SRV);
				}
				else if (BindingType == EShaderCodeResourceBindingType::RaytracingAccelerationStructure)
				{
					bIsValidBindingType = (
						BaseType == UBMT_SRV ||
						BaseType == UBMT_RDG_BUFFER_SRV);
				}
				else if (
					BindingType == EShaderCodeResourceBindingType::RWTexture2D ||
					BindingType == EShaderCodeResourceBindingType::RWTexture2DArray ||
					BindingType == EShaderCodeResourceBindingType::RWTexture2DMultiView ||
					BindingType == EShaderCodeResourceBindingType::RWTexture3D ||
					BindingType == EShaderCodeResourceBindingType::RWTextureCube ||
					BindingType == EShaderCodeResourceBindingType::RWTextureMetadata ||
					BindingType == EShaderCodeResourceBindingType::RasterizerOrderedTexture2D)
				{
					bIsValidBindingType = (
						BaseType == UBMT_UAV ||
						BaseType == UBMT_RDG_TEXTURE_UAV);
				}
				else if (
					BindingType == EShaderCodeResourceBindingType::RWBuffer ||
					BindingType == EShaderCodeResourceBindingType::RWStructuredBuffer ||
					BindingType == EShaderCodeResourceBindingType::RWByteAddressBuffer)
				{
					bIsValidBindingType = (
						BaseType == UBMT_UAV ||
						BaseType == UBMT_RDG_BUFFER_UAV);
				}
				else if (BindingType == EShaderCodeResourceBindingType::ResourceCollection)
				{
					bIsValidBindingType = BaseType == UBMT_RESOURCE_COLLECTION;
				}
				else if (BindingType == EShaderCodeResourceBindingType::DescriptorRange)
				{
					bIsValidBindingType = BaseType == UBMT_DESCRIPTOR_RANGE;
				}
				else
				{
					//unimplemented();
				}

				if (!bIsValidBindingType)
				{
					const TCHAR* CurrentMacroName = GetShaderParameterMacroName(BaseType);
					UE_LOGF(
						LogRendererCore, Warning,
						"%lsUnable to bind a shader parameter type %ls with a %ls.",
						*GetMemberErrorPrefix(), ShaderType, CurrentMacroName);
				}
			}
		}

		if (bHasNonGraphOutputs)
		{
			EnumAddFlags(LayoutInitializer.Flags, ERHIUniformBufferFlags::HasNonGraphOutputs);
		}

		if (IsShaderParameterTypeForUniformBufferLayout(BaseType))
		{
			for (uint32 ArrayElementId = 0; ArrayElementId < (bIsArray ? ArraySize : 1u); ArrayElementId++)
			{
				const uint32 AbsoluteMemberOffset = CurrentMember.GetOffset() + MemberStack[i].StructOffset + ArrayElementId * SHADER_PARAMETER_POINTER_ALIGNMENT;
				check(AbsoluteMemberOffset < (1u << (sizeof(FRHIUniformBufferResourceInitializer::MemberOffset) * 8)));
				const FRHIUniformBufferResourceInitializer ResourceParameter{ uint16(AbsoluteMemberOffset), BaseType };

				LayoutInitializer.Resources.Add(ResourceParameter);

				if (IsRDGTextureReferenceShaderParameterType(BaseType) || BaseType == UBMT_RENDER_TARGET_BINDING_SLOTS)
				{
					LayoutInitializer.GraphResources.Add(ResourceParameter);
					LayoutInitializer.GraphTextures.Add(ResourceParameter);

					if (BaseType == UBMT_RENDER_TARGET_BINDING_SLOTS)
					{
						checkf(!LayoutInitializer.HasRenderTargets(), TEXT("Shader parameter struct %s has multiple render target binding slots."), GetStructTypeName());
						LayoutInitializer.RenderTargetsOffset = ResourceParameter.MemberOffset;
					}
				}
				else if (IsRDGBufferReferenceShaderParameterType(BaseType))
				{
					LayoutInitializer.GraphResources.Add(ResourceParameter);
					LayoutInitializer.GraphBuffers.Add(ResourceParameter);
				}
				else if (BaseType == UBMT_RDG_UNIFORM_BUFFER)
				{
					LayoutInitializer.GraphResources.Add(ResourceParameter);
					LayoutInitializer.GraphUniformBuffers.Add(ResourceParameter);
				}
				else if (BaseType == UBMT_REFERENCED_STRUCT)
				{
					LayoutInitializer.UniformBuffers.Add(ResourceParameter);
				}
			}
		}

		if (ChildStruct && BaseType != UBMT_REFERENCED_STRUCT && BaseType != UBMT_RDG_UNIFORM_BUFFER)
		{
			for (uint32 ArrayElementId = 0; ArrayElementId < (bIsArray ? ArraySize : 1u); ArrayElementId++)
			{
				int32 AbsoluteStructOffset = CurrentMember.GetOffset() + MemberStack[i].StructOffset + ArrayElementId * ChildStruct->GetSize();

				for (int32 StructMemberIndex = 0; StructMemberIndex < ChildStruct->Members.Num(); StructMemberIndex++)
				{
					const FMember& StructMember = ChildStruct->Members[StructMemberIndex];
					MemberStack.Insert(FUniformBufferMemberAndOffset(*ChildStruct, StructMember, AbsoluteStructOffset), i + 1 + StructMemberIndex);
				}
			}
		}
	}

	const auto ByMemberOffset = [](
		const FRHIUniformBufferResourceInitializer& A,
		const FRHIUniformBufferResourceInitializer& B)
	{
		return A.MemberOffset < B.MemberOffset;
	};

	const auto ByTypeThenMemberOffset = [](
		const FRHIUniformBufferResourceInitializer& A,
		const FRHIUniformBufferResourceInitializer& B)
	{
		if (A.MemberType == B.MemberType)
		{
			return A.MemberOffset < B.MemberOffset;
		}
		return A.MemberType < B.MemberType;
	};

	LayoutInitializer.Resources.Sort(ByMemberOffset);
	LayoutInitializer.GraphResources.Sort(ByMemberOffset);
	LayoutInitializer.GraphTextures.Sort(ByTypeThenMemberOffset);
	LayoutInitializer.GraphBuffers.Sort(ByTypeThenMemberOffset);
	LayoutInitializer.GraphUniformBuffers.Sort(ByMemberOffset);
	LayoutInitializer.UniformBuffers.Sort(ByMemberOffset);

	// Compute the hash of the RHI layout.
	LayoutInitializer.ComputeHash();
	
	// Fast runtime-compatible hash about the entire layout of the structure.
	{
		TMemoryHasher<FXxHash64Builder, FXxHash64> FastHasher;
		HashLayout(FastHasher);
		LayoutHash = (uint32)FastHasher.Finalize().Hash; // note: decimating 64 bit hash to 32 bits to avoid data format changes & API deprecation
	}

	if (UseCase == EUseCase::UniformBuffer)
	{
		GetLayoutHashStructMap().Emplace(LayoutInitializer.GetHash(), this);
	}

	Layout = RHICreateUniformBufferLayout(LayoutInitializer);

#if WITH_EDITOR
	// second stronger hash for use in DDC keys
	FMemoryHasherBlake3 Hasher;
	HashLayout(Hasher);
	LayoutSignature = Hasher.Finalize();
#endif
}


void FShaderParametersMetadata::GetNestedStructs(TArray<const FShaderParametersMetadata*>& OutNestedStructs) const
{
	for (int32 i = 0; i < Members.Num(); ++i)
	{
		const FMember& CurrentMember = Members[i];

		const FShaderParametersMetadata* MemberStruct = CurrentMember.GetStructMetadata();

		if (MemberStruct)
		{
			OutNestedStructs.Add(MemberStruct);
			MemberStruct->GetNestedStructs(OutNestedStructs);
		}
	}
}

#if WITH_EDITOR

// Actual longest token is 78 characters -- these are a few human written variable names concatenated together, and unlikely to go much higher.
typedef TStringBuilder<2048> FResourceTableEntryString;

// Function returns count to allocate for MemberNameBuffer and ResourceIndex specifies the number of Entries to allocate, if MemberNameBuffer is nullptr
static uint32 CacheResourceTableEntriesRecursive(
	const TArray<FShaderParametersMetadata::FMember>& Members, const TCHAR* UniformBufferName, FResourceTableEntryString& Prefix, uint16& ResourceIndex, TCHAR*& MemberNameBuffer, TArray<FUniformResourceEntry>* Entries)
{
	uint32 MemberNameBufferCount = 0;
	uint32 OriginalPrefixLength = Prefix.Len();

	uint32 UniformBufferNameLength = FCString::Strlen(UniformBufferName);
	check(UniformBufferNameLength <= UINT8_MAX);

	for (const FShaderParametersMetadata::FMember& Member : Members)
	{
		const EUniformBufferBaseType BaseType = Member.GetBaseType();
		const uint32 NumElements = Member.GetNumElements();

		if (IsShaderParameterTypeForUniformBufferLayout(BaseType))
		{
			// Format:  "%s%s"
			Prefix.Append(Member.GetName());
			uint32 StringStorageCount = Prefix.Len() + 1;
			if (MemberNameBuffer)
			{
				FCString::Strncpy(MemberNameBuffer, Prefix.ToString(), StringStorageCount);

				check(Entries);
				Entries->Add({
					MemberNameBuffer,
					(uint8)UniformBufferNameLength,
					BaseType,
					ResourceIndex
					});

				MemberNameBuffer += StringStorageCount;
			}

			MemberNameBufferCount += StringStorageCount;
			ResourceIndex++;
			Prefix.RemoveSuffix(Prefix.Len() - OriginalPrefixLength);
		}
		else if (BaseType == UBMT_NESTED_STRUCT && NumElements == 0)
		{
			check(Member.GetStructMetadata());

			// Format:  "%s%s_"
			Prefix.Append(Member.GetName());
			Prefix.AppendChar('_');
			MemberNameBufferCount += CacheResourceTableEntriesRecursive(Member.GetStructMetadata()->GetMembers(), UniformBufferName, Prefix, ResourceIndex, MemberNameBuffer, Entries);
			Prefix.RemoveSuffix(Prefix.Len() - OriginalPrefixLength);
		}
		else if (BaseType == UBMT_NESTED_STRUCT && NumElements > 0)
		{
			// Format:  %s%s_%u_
			Prefix.Append(Member.GetName());
			Prefix.AppendChar('_');
			uint32 PrefixLengthMinusArrayIndex = Prefix.Len();

			for (uint32 ArrayElementId = 0; ArrayElementId < NumElements; ArrayElementId++)
			{
				check(Member.GetStructMetadata());
				Prefix.Appendf(TEXT("%u"), ArrayElementId);
				Prefix.AppendChar('_');
				MemberNameBufferCount += CacheResourceTableEntriesRecursive(Member.GetStructMetadata()->GetMembers(), UniformBufferName, Prefix, ResourceIndex, MemberNameBuffer, Entries);

				Prefix.RemoveSuffix(Prefix.Len() - PrefixLengthMinusArrayIndex);
			}

			Prefix.RemoveSuffix(Prefix.Len() - OriginalPrefixLength);
		}
		else if (BaseType == UBMT_INCLUDED_STRUCT)
		{
			check(Member.GetStructMetadata());
			check(NumElements == 0);
			MemberNameBufferCount += CacheResourceTableEntriesRecursive(Member.GetStructMetadata()->GetMembers(), UniformBufferName, Prefix, ResourceIndex, MemberNameBuffer, Entries);
		}
	}

	return MemberNameBufferCount;
}

void FShaderParametersMetadata::InitializeUniformBufferDeclaration()
{
	if (UseCase != EUseCase::ShaderParameterStruct)
	{
		FString* NewDeclaration = new FString(UE::ShaderParameters::CreateUniformBufferShaderDeclaration(ShaderVariableName, *this, nullptr));
		check(!NewDeclaration->IsEmpty());

		UniformBufferDeclaration = MakeShareable(NewDeclaration);

		// Cache preprocessor friendly copy of uniform buffer declaration
		TArray<ANSICHAR>* NewDeclarationAnsi = new TArray<ANSICHAR>;
		ShaderConvertAndStripComments(*NewDeclaration, *NewDeclarationAnsi);
		UniformBufferDeclarationAnsi = MakeShareable(NewDeclarationAnsi);

		// Generate ResourceTableCache and MemberNameBuffer
		FResourceTableEntryString Prefix;
		Prefix.Append(ShaderVariableName);
		Prefix.AppendChar('_');

		// First pass to determine number of entries and name buffer count, by passing nullptr for Buffer
		uint16 ResourceIndex = 0;
		TCHAR* Buffer = nullptr;
		uint32 BufferCount = CacheResourceTableEntriesRecursive(Members, ShaderVariableName, Prefix, ResourceIndex, Buffer, nullptr);

		// Then generate the entries and name buffer
		TArray<TCHAR>* MemberNameBufferAllocation = new TArray<TCHAR>();
		MemberNameBufferAllocation->SetNumZeroed(BufferCount);
		ResourceTableCache.Reserve(ResourceIndex);

		ResourceIndex = 0;
		Buffer = MemberNameBufferAllocation->GetData();
		CacheResourceTableEntriesRecursive(Members, ShaderVariableName, Prefix, ResourceIndex, Buffer, &ResourceTableCache);

		MemberNameBuffer = MakeShareable(MemberNameBufferAllocation);

		// Cache strings for uniform buffer generated path and include
		UniformBufferPath = FString::Printf(TEXT("/Engine/Generated/UniformBuffers/%s.ush"), ShaderVariableName);
		UniformBufferInclude = FString::Printf(TEXT("#include \"/Engine/Generated/UniformBuffers/%s.ush\"") HLSL_LINE_TERMINATOR, ShaderVariableName);

		// Cache some frequently used hashes
		UniformBufferPathHash = GetTypeHash(UniformBufferPath);
		ShaderVariableNameHash = GetTypeHash(FStringView(ShaderVariableName));
	}
}

void FShaderParametersMetadata::AddResourceTableEntries(FShaderResourceTableMap& ResourceTableMap, TMap<FString, FUniformBufferEntry>& UniformBufferMap) const
{
	ResourceTableMap.Resources.Append(ResourceTableCache);
	
	FUniformBufferEntry UniformBufferEntry;
	UniformBufferEntry.StaticSlotName = StaticSlotName;
	UniformBufferEntry.MemberNameBuffer = MemberNameBuffer;
	UniformBufferEntry.LayoutHash = IsLayoutInitialized() ? GetLayout().GetHash() : 0;
	UniformBufferEntry.BindingFlags = BindingFlags;

	if (EnumHasAnyFlags(UsageFlags, EUsageFlags::NoEmulatedUniformBuffer | EUsageFlags::UniformView))
	{
		EnumAddFlags(UniformBufferEntry.Flags, ERHIUniformBufferFlags::NoEmulatedUniformBuffer);
	}
	if (EnumHasAnyFlags(UsageFlags, EUsageFlags::NeedsReflectedMembers))
	{
		EnumAddFlags(UniformBufferEntry.Flags, ERHIUniformBufferFlags::NeedsReflectedMembers);
	}

	UniformBufferMap.AddByHash(ShaderVariableNameHash, ShaderVariableName, UniformBufferEntry);
}

#endif // WITH_EDITOR

void FShaderParametersMetadata::FindMemberFromOffset(uint16 MemberOffset, const FShaderParametersMetadata** OutContainingStruct, const FShaderParametersMetadata::FMember** OutMember, int32* ArrayElementId, FString* NamePrefix) const
{
	check(MemberOffset < GetSize());

	for (const FMember& Member : Members)
	{
		EUniformBufferBaseType BaseType = Member.GetBaseType();
		uint32 NumElements = Member.GetNumElements();

		if ((BaseType == UBMT_NESTED_STRUCT && NumElements == 0) || BaseType == UBMT_INCLUDED_STRUCT)
		{
			const FShaderParametersMetadata* SubStruct = Member.GetStructMetadata();
			if (MemberOffset < (Member.GetOffset() + SubStruct->GetSize()))
			{
				if (NamePrefix)
				{
					*NamePrefix = FString::Printf(TEXT("%s%s::"), **NamePrefix, Member.GetName());
				}

				return SubStruct->FindMemberFromOffset(MemberOffset - Member.GetOffset(), OutContainingStruct, OutMember, ArrayElementId, NamePrefix);
			}
		}
		else if (BaseType == UBMT_NESTED_STRUCT && NumElements > 0)
		{
			const FShaderParametersMetadata* SubStruct = Member.GetStructMetadata();
			uint32 StructSize = SubStruct->GetSize();
			
			uint16 ArrayStartOffset = Member.GetOffset();
			uint16 ArrayEndOffset = ArrayStartOffset + SubStruct->GetSize() * NumElements;
			
			if (MemberOffset >= ArrayStartOffset && MemberOffset < ArrayEndOffset)
			{
				uint32 MemberOffsetInArray = MemberOffset - ArrayStartOffset;
				check((MemberOffsetInArray % StructSize) == 0);

				uint32 MemberPosInStructArray = MemberOffsetInArray / StructSize;
				uint32 MemberOffsetInStructElement = MemberOffsetInArray - MemberPosInStructArray * StructSize;

				if (NamePrefix)
				{
					*NamePrefix = FString::Printf(TEXT("%s%s[%u]::"), **NamePrefix, Member.GetName(), MemberPosInStructArray);
				}

				return SubStruct->FindMemberFromOffset(MemberOffsetInStructElement, OutContainingStruct, OutMember, ArrayElementId, NamePrefix);
			}
		}
		else if (NumElements > 0 && (
			IsShaderParameterTypeReadOnlyRHIResource(BaseType) ||
			IsRDGResourceReferenceShaderParameterType(BaseType)))
		{
			uint16 ArrayStartOffset = Member.GetOffset();
			uint16 ArrayEndOffset = ArrayStartOffset + SHADER_PARAMETER_POINTER_ALIGNMENT * NumElements;

			if (MemberOffset >= ArrayStartOffset && MemberOffset < ArrayEndOffset)
			{
				check((MemberOffset % SHADER_PARAMETER_POINTER_ALIGNMENT) == 0);
				*OutContainingStruct = this;
				*OutMember = &Member;
				*ArrayElementId = (MemberOffset - ArrayStartOffset) / SHADER_PARAMETER_POINTER_ALIGNMENT;
				return;
			}
		}
		else if (Member.GetOffset() == MemberOffset)
		{
			*OutContainingStruct = this;
			*OutMember = &Member;
			*ArrayElementId = 0;
			return;
		}
	}

	checkf(0, TEXT("Looks like this offset is invalid."));
}

FString FShaderParametersMetadata::GetFullMemberCodeName(uint16 MemberOffset) const
{
	const FShaderParametersMetadata* MemberContainingStruct = nullptr;
	const FShaderParametersMetadata::FMember* Member = nullptr;
	int32 ArrayElementId = 0;
	FString NamePrefix;
	FindMemberFromOffset(MemberOffset, &MemberContainingStruct, &Member, &ArrayElementId, &NamePrefix);

	FString MemberName = FString::Printf(TEXT("%s%s"), *NamePrefix, Member->GetName());
	if (Member->GetNumElements() > 0)
	{
		MemberName = FString::Printf(TEXT("%s%s[%d]"), *NamePrefix, Member->GetName(), ArrayElementId);
	}

	return MemberName;
}
