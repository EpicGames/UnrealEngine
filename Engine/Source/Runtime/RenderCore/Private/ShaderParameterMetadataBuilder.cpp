// Copyright Epic Games, Inc. All Rights Reserved.

#include "ShaderParameterMetadataBuilder.h"

void FShaderParametersMetadataBuilder::AddReferencedStruct(
	const TCHAR* Name,
	const FShaderParametersMetadata* StructMetadata,
	EShaderPrecisionModifier::Type Precision
)
{
	NextMemberOffset = Align(NextMemberOffset, SHADER_PARAMETER_STRUCT_ALIGNMENT);

	Members.Emplace(
		Name,
		StructMetadata->GetStructTypeName(),
		__LINE__,
		NextMemberOffset,
		UBMT_REFERENCED_STRUCT,
		Precision,
		1,
		1,
		0,
		StructMetadata
	);

	NextMemberOffset += Align(StructMetadata->GetSize(), SHADER_PARAMETER_STRUCT_ALIGNMENT);
}

void FShaderParametersMetadataBuilder::AddIncludedStruct(
	const FShaderParametersMetadata* StructMetadata,
	EShaderPrecisionModifier::Type Precision
)
{
	NextMemberOffset = Align(NextMemberOffset, SHADER_PARAMETER_STRUCT_ALIGNMENT);

	Members.Emplace(
		TEXT(""),
		StructMetadata->GetStructTypeName(),
		__LINE__,
		NextMemberOffset,
		UBMT_INCLUDED_STRUCT,
		Precision,
		1,
		1,
		0,
		StructMetadata
	);

	NextMemberOffset += Align(StructMetadata->GetSize(), SHADER_PARAMETER_STRUCT_ALIGNMENT);
}

uint32 FShaderParametersMetadataBuilder::AddNestedStruct(
	const TCHAR* Name,
	const FShaderParametersMetadata* StructMetadata,
	EShaderPrecisionModifier::Type Precision /* = EShaderPrecisionModifier::Float */
)
{
	NextMemberOffset = Align(NextMemberOffset, SHADER_PARAMETER_STRUCT_ALIGNMENT);
	const uint32 ThisMemberOffset = NextMemberOffset;

	Members.Emplace(
		Name,
		TEXT(""),
		__LINE__,
		NextMemberOffset,
		UBMT_NESTED_STRUCT,
		Precision,
		1,
		1,
		0,
		StructMetadata
	);

	NextMemberOffset += Align(StructMetadata->GetSize(), SHADER_PARAMETER_STRUCT_ALIGNMENT);
	return ThisMemberOffset;
}

void FShaderParametersMetadataBuilder::AddBufferSRV(
	const TCHAR* Name,
	const TCHAR* ShaderType,
	EShaderPrecisionModifier::Type Precision /* = EShaderPrecisionModifier::Float */
)
{
	NextMemberOffset = Align(NextMemberOffset, SHADER_PARAMETER_POINTER_ALIGNMENT);

	Members.Emplace(
		Name,
		ShaderType,
		__LINE__,
		NextMemberOffset,
		UBMT_SRV,
		Precision,
		TShaderResourceParameterTypeInfo<FRHIShaderResourceView*>::NumRows,
		TShaderResourceParameterTypeInfo<FRHIShaderResourceView*>::NumColumns,
		TShaderResourceParameterTypeInfo<FRHIShaderResourceView*>::NumElements,
		TShaderResourceParameterTypeInfo<FRHIShaderResourceView*>::GetStructMetadata()
	);

	NextMemberOffset += SHADER_PARAMETER_POINTER_ALIGNMENT;
}

void FShaderParametersMetadataBuilder::AddBufferUAV(
	const TCHAR* Name,
	const TCHAR* ShaderType,
	EShaderPrecisionModifier::Type Precision /* = EShaderPrecisionModifier::Float */
)
{
	NextMemberOffset = Align(NextMemberOffset, SHADER_PARAMETER_POINTER_ALIGNMENT);

	Members.Emplace(
		Name,
		ShaderType,
		__LINE__,
		NextMemberOffset,
		UBMT_UAV,
		Precision,
		TShaderResourceParameterTypeInfo<FRHIUnorderedAccessView*>::NumRows,
		TShaderResourceParameterTypeInfo<FRHIUnorderedAccessView*>::NumColumns,
		TShaderResourceParameterTypeInfo<FRHIUnorderedAccessView*>::NumElements,
		TShaderResourceParameterTypeInfo<FRHIUnorderedAccessView*>::GetStructMetadata()
	);

	NextMemberOffset += SHADER_PARAMETER_POINTER_ALIGNMENT;
}

void FShaderParametersMetadataBuilder::AddRDGBufferSRV(
	const TCHAR* Name,
	const TCHAR* ShaderType,
	EShaderPrecisionModifier::Type Precision /* = EShaderPrecisionModifier::Float */
	)
{
	NextMemberOffset = Align(NextMemberOffset, SHADER_PARAMETER_POINTER_ALIGNMENT);

	Members.Emplace(
		Name,
		ShaderType,
		__LINE__,
		NextMemberOffset,
		UBMT_RDG_BUFFER_SRV,
		Precision,
 		TShaderResourceParameterTypeInfo<FRDGBufferSRV*>::NumRows,
 		TShaderResourceParameterTypeInfo<FRDGBufferSRV*>::NumColumns,
 		TShaderResourceParameterTypeInfo<FRDGBufferSRV*>::NumElements,
 		TShaderResourceParameterTypeInfo<FRDGBufferSRV*>::GetStructMetadata()
	);

	NextMemberOffset += SHADER_PARAMETER_POINTER_ALIGNMENT;
}

void FShaderParametersMetadataBuilder::AddRDGBufferUAV(
	const TCHAR* Name,
	const TCHAR* ShaderType,
	EShaderPrecisionModifier::Type Precision /* = EShaderPrecisionModifier::Float */
	)
{
	NextMemberOffset = Align(NextMemberOffset, SHADER_PARAMETER_POINTER_ALIGNMENT);

	Members.Emplace(
		Name,
		ShaderType,
		__LINE__,
		NextMemberOffset,
		UBMT_RDG_BUFFER_UAV,
		Precision,
 		TShaderResourceParameterTypeInfo<FRDGBufferUAV*>::NumRows,
 		TShaderResourceParameterTypeInfo<FRDGBufferUAV*>::NumColumns,
 		TShaderResourceParameterTypeInfo<FRDGBufferUAV*>::NumElements,
 		TShaderResourceParameterTypeInfo<FRDGBufferUAV*>::GetStructMetadata()
		);

	NextMemberOffset += SHADER_PARAMETER_POINTER_ALIGNMENT;
}

void FShaderParametersMetadataBuilder::AddTexture(
	const TCHAR* Name,
	const TCHAR* ShaderType,
	EShaderPrecisionModifier::Type Precision)
{
	NextMemberOffset = Align(NextMemberOffset, SHADER_PARAMETER_POINTER_ALIGNMENT);

	Members.Emplace(
		Name,
		ShaderType,
		__LINE__,
		NextMemberOffset,
		UBMT_TEXTURE,
		Precision,
		TShaderResourceParameterTypeInfo<FRHITexture*>::NumRows,
		TShaderResourceParameterTypeInfo<FRHITexture*>::NumColumns,
		TShaderResourceParameterTypeInfo<FRHITexture*>::NumElements,
		TShaderResourceParameterTypeInfo<FRHITexture*>::GetStructMetadata()
	);

	NextMemberOffset += SHADER_PARAMETER_POINTER_ALIGNMENT;
}

void FShaderParametersMetadataBuilder::AddRDGTexture(
	const TCHAR* Name,
	const TCHAR* ShaderType,
	EShaderPrecisionModifier::Type Precision /* = EShaderPrecisionModifier::Float */
)
{
	NextMemberOffset = Align(NextMemberOffset, SHADER_PARAMETER_POINTER_ALIGNMENT);

	Members.Emplace(
		Name,
		ShaderType,
		__LINE__,
		NextMemberOffset,
		UBMT_RDG_TEXTURE,
		Precision,
		TShaderResourceParameterTypeInfo<FRDGTexture*>::NumRows,
		TShaderResourceParameterTypeInfo<FRDGTexture*>::NumColumns,
		TShaderResourceParameterTypeInfo<FRDGTexture*>::NumElements,
		TShaderResourceParameterTypeInfo<FRDGTexture*>::GetStructMetadata()
	);

	NextMemberOffset += SHADER_PARAMETER_POINTER_ALIGNMENT;
}

void FShaderParametersMetadataBuilder::AddSampler(const TCHAR* Name, const TCHAR* ShaderType, EShaderPrecisionModifier::Type Precision)
{
	NextMemberOffset = Align(NextMemberOffset, SHADER_PARAMETER_POINTER_ALIGNMENT);

	Members.Emplace(
		Name,
		ShaderType,
		__LINE__,
		NextMemberOffset,
		UBMT_SAMPLER,
		Precision,
		TShaderResourceParameterTypeInfo<FRHISamplerState*>::NumRows,
		TShaderResourceParameterTypeInfo<FRHISamplerState*>::NumColumns,
		TShaderResourceParameterTypeInfo<FRHISamplerState*>::NumElements,
		TShaderResourceParameterTypeInfo<FRHISamplerState*>::GetStructMetadata()
	);

	NextMemberOffset += SHADER_PARAMETER_POINTER_ALIGNMENT;
}

void FShaderParametersMetadataBuilder::AddRenderTargetBindingSlots(const TCHAR* Name, const TCHAR* ShaderType, EShaderPrecisionModifier::Type Precision)
{
	NextMemberOffset = Align(NextMemberOffset, FRenderTargetBindingSlots::FTypeInfo::Alignment);

	Members.Emplace(
		Name,
		ShaderType,
		__LINE__,
		NextMemberOffset,
		UBMT_RENDER_TARGET_BINDING_SLOTS,
		Precision,
		FRenderTargetBindingSlots::FTypeInfo::NumRows,
		FRenderTargetBindingSlots::FTypeInfo::NumColumns,
		FRenderTargetBindingSlots::FTypeInfo::NumElements,
		FRenderTargetBindingSlots::FTypeInfo::GetStructMetadata()
	);

	NextMemberOffset += sizeof(FRenderTargetBindingSlots::FTypeInfo::TAlignedType);
}

FShaderParametersMetadata* FShaderParametersMetadataBuilder::Build(
	FShaderParametersMetadata::EUseCase UseCase,
	const TCHAR* ShaderParameterName
	)
{
	const uint32 StructSize = Align(NextMemberOffset, SHADER_PARAMETER_STRUCT_ALIGNMENT);

	FShaderParametersMetadata* ShaderParameterMetadata = new FShaderParametersMetadata(
		UseCase,
		EUniformBufferBindingFlags::Shader,
		ShaderParameterName,
		ShaderParameterName,
		nullptr,
		nullptr,
		__FILE__,
		__LINE__,
		StructSize,
		Members
		);

	return ShaderParameterMetadata;
}

FShaderParametersMetadata* FShaderParametersMetadataBuilder::Build(
	FShaderParametersMetadata::EUseCase InUseCase,
	EUniformBufferBindingFlags InBindingFlags,
	const TCHAR* InLayoutName,
	const TCHAR* InStructTypeName,
	const TCHAR* InShaderVariableName,
	const TCHAR* InStaticSlotName,
	const ANSICHAR* InFileName,
	const int32 InFileLine,
	bool bForceCompleteInitialization,
	FRHIUniformBufferLayoutInitializer* OutLayoutInitializer,
	FShaderParametersMetadata::EUsageFlags InUsageFlags
)
{
	const uint32 StructSize = Align(NextMemberOffset, SHADER_PARAMETER_STRUCT_ALIGNMENT);
	
	FShaderParametersMetadata* ShaderParameterMetadata = new FShaderParametersMetadata(
		InUseCase,
		InBindingFlags,
		InLayoutName,
		InStructTypeName,
		InShaderVariableName,
		InStaticSlotName,
		InFileName,
		InFileLine,
		StructSize,
		Members,
		bForceCompleteInitialization,
		OutLayoutInitializer,
		InUsageFlags
	);

	return ShaderParameterMetadata;
}
