// Copyright Epic Games, Inc. All Rights Reserved.

#include "ShaderParameterParser.h"
#include "Containers/UnrealString.h"
#include "ShaderCompilerCore.h"
#include "String/RemoveFrom.h"
#include "Misc/StringBuilder.h"

inline FStringView StripTemplateFromType(const FStringView& Input)
{
	FStringView UntemplatedType = FStringView(Input);
	if (int32 Index = Input.Find(TEXT("<")); Index != INDEX_NONE)
	{
		// Remove the template argument but don't forget to clean up the type name
		const int32 NumChars = Input.Len() - Index;
		UntemplatedType = Input.LeftChop(NumChars).TrimEnd();
	}

	return UntemplatedType;
}

template<typename TParameterFunction>
static void IterateShaderParameterMembersInternal(
	const FShaderParametersMetadata& ParametersMetadata,
	uint16 ByteOffset,
	TStringBuilder<1024>& ShaderBindingNameBuilder,
	TParameterFunction Lambda)
{
	for (const FShaderParametersMetadata::FMember& Member : ParametersMetadata.GetMembers())
	{
		EUniformBufferBaseType BaseType = Member.GetBaseType();
		const uint16 MemberOffset = ByteOffset + uint16(Member.GetOffset());
		const uint32 NumElements = Member.GetNumElements();

		int32 MemberNameLength = FCString::Strlen(Member.GetName());

		if (BaseType == UBMT_INCLUDED_STRUCT)
		{
			check(NumElements == 0);
			const FShaderParametersMetadata& NewParametersMetadata = *Member.GetStructMetadata();
			IterateShaderParameterMembersInternal(NewParametersMetadata, MemberOffset, ShaderBindingNameBuilder, Lambda);
		}
		else if (BaseType == UBMT_NESTED_STRUCT && NumElements == 0)
		{
			ShaderBindingNameBuilder.Append(Member.GetName());
			ShaderBindingNameBuilder.Append(TEXT("_"));

			const FShaderParametersMetadata& NewParametersMetadata = *Member.GetStructMetadata();
			IterateShaderParameterMembersInternal(NewParametersMetadata, MemberOffset, ShaderBindingNameBuilder, Lambda);

			ShaderBindingNameBuilder.RemoveSuffix(MemberNameLength + 1);
		}
		else if (BaseType == UBMT_NESTED_STRUCT && NumElements > 0)
		{
			ShaderBindingNameBuilder.Append(Member.GetName());
			ShaderBindingNameBuilder.Append(TEXT("_"));

			const FShaderParametersMetadata& NewParametersMetadata = *Member.GetStructMetadata();
			for (uint32 ArrayElementId = 0; ArrayElementId < NumElements; ArrayElementId++)
			{
				FString ArrayElementIdString;
				ArrayElementIdString.AppendInt(ArrayElementId);
				int32 ArrayElementIdLength = ArrayElementIdString.Len();

				ShaderBindingNameBuilder.Append(ArrayElementIdString);
				ShaderBindingNameBuilder.Append(TEXT("_"));

				uint16 NewStructOffset = MemberOffset + ArrayElementId * NewParametersMetadata.GetSize();
				IterateShaderParameterMembersInternal(NewParametersMetadata, NewStructOffset, ShaderBindingNameBuilder, Lambda);

				ShaderBindingNameBuilder.RemoveSuffix(ArrayElementIdLength + 1);
			}

			ShaderBindingNameBuilder.RemoveSuffix(MemberNameLength + 1);
		}
		else
		{
			const bool bParametersAreExpanded = NumElements > 0 && (IsShaderParameterTypeRHIResource(BaseType) || IsRDGResourceReferenceShaderParameterType(BaseType));
			if (bParametersAreExpanded)
			{
				const uint16 ElementSize = SHADER_PARAMETER_POINTER_ALIGNMENT;

				for (uint32 Index = 0; Index < NumElements; Index++)
				{
					const FString RealBindingName = FString::Printf(TEXT("%s_%d"), Member.GetName(), Index);

					ShaderBindingNameBuilder.Append(RealBindingName);
					Lambda(ParametersMetadata, Member, *ShaderBindingNameBuilder, MemberOffset + Index * ElementSize);
					ShaderBindingNameBuilder.RemoveSuffix(RealBindingName.Len());
				}
			}
			else
			{
				ShaderBindingNameBuilder.Append(Member.GetName());
				Lambda(ParametersMetadata, Member, *ShaderBindingNameBuilder, MemberOffset);
				ShaderBindingNameBuilder.RemoveSuffix(MemberNameLength);
			}
		}
	}
}

template<typename TParameterFunction>
static void IterateShaderParameterMembers(const FShaderParametersMetadata& ShaderParametersMetadata, TParameterFunction Lambda)
{
	FShaderParametersMetadata::EUseCase UseCase = ShaderParametersMetadata.GetUseCase();

	TStringBuilder<1024> ShaderBindingNameBuilder;
	if (UseCase == FShaderParametersMetadata::EUseCase::UniformBuffer || UseCase == FShaderParametersMetadata::EUseCase::DataDrivenUniformBuffer)
	{
		ShaderBindingNameBuilder.Append(ShaderParametersMetadata.GetShaderVariableName());
		ShaderBindingNameBuilder.Append(TEXT("_"));
	}

	IterateShaderParameterMembersInternal(
		ShaderParametersMetadata, /* ByteOffset = */ 0, ShaderBindingNameBuilder, Lambda);
}

static void AddNoteToDisplayShaderParameterMemberOnCppSide(
	const FShaderCompilerInput& CompilerInput,
	const FShaderParameterParser::FParsedShaderParameter& ParsedParameter,
	FShaderCompilerOutput& CompilerOutput)
{
	const FShaderParametersMetadata* MemberContainingStruct = nullptr;
	const FShaderParametersMetadata::FMember* Member = nullptr;
	{
		int32 ArrayElementId = 0;
		FString NamePrefix;
		CompilerInput.RootParametersStructure->FindMemberFromOffset(ParsedParameter.ConstantBufferOffset, &MemberContainingStruct, &Member, &ArrayElementId, &NamePrefix);
	}

	FString CppCodeName = CompilerInput.RootParametersStructure->GetFullMemberCodeName(ParsedParameter.ConstantBufferOffset);

	FShaderCompilerError Error;
	Error.StrippedErrorMessage = FString::Printf(
		TEXT("Note: Definition of %s"),
		*CppCodeName);
	Error.ErrorVirtualFilePath = ANSI_TO_TCHAR(MemberContainingStruct->GetFileName());
	Error.ErrorLineString = FString::FromInt(Member->GetFileLine());

	CompilerOutput.Errors.Add(Error);
}

FShaderParameterParser::~FShaderParameterParser() = default;

FShaderParameterParser::FShaderParameterParser(const FPlatformConfiguration& InPlatformConfiguration)
	: PlatformConfiguration(InPlatformConfiguration)
{
}

static const FStringView s_AllSRVTypes[] =
{
	TEXTVIEW("Texture1D"),
	TEXTVIEW("Texture1DArray"),
	TEXTVIEW("Texture2D"),
	TEXTVIEW("Texture2DArray"),
	TEXTVIEW("Texture2DMS"),
	TEXTVIEW("Texture2DMSArray"),
	TEXTVIEW("Texture3D"),
	TEXTVIEW("TextureCube"),
	TEXTVIEW("TextureCubeArray"),

	TEXTVIEW("Buffer"),
	TEXTVIEW("ByteAddressBuffer"),
	TEXTVIEW("StructuredBuffer"),
	TEXTVIEW("RaytracingAccelerationStructure"),
	TEXTVIEW("FResourceCollection"),
};

static const FStringView s_AllUAVTypes[] =
{
	TEXTVIEW("AppendStructuredBuffer"),
	TEXTVIEW("RWBuffer"),
	TEXTVIEW("RWByteAddressBuffer"),
	TEXTVIEW("RWStructuredBuffer"),
	TEXTVIEW("RWTexture1D"),
	TEXTVIEW("RWTexture1DArray"),
	TEXTVIEW("RWTexture2D"),
	TEXTVIEW("RWTexture2DArray"),
	TEXTVIEW("RWTexture3D"),
	TEXTVIEW("RasterizerOrderedTexture2D"),
};

static const FStringView s_AllSamplerTypes[] =
{
	TEXTVIEW("SamplerState"),
	TEXTVIEW("SamplerComparisonState"),
};

EShaderParameterType FShaderParameterParser::ParseParameterType(FStringView InType)
{
	TConstArrayView<FStringView> AllSamplerTypes(s_AllSamplerTypes);
	TConstArrayView<FStringView> AllSRVTypes(s_AllSRVTypes);
	TConstArrayView<FStringView> AllUAVTypes(s_AllUAVTypes);

	if (AllSamplerTypes.Contains(InType))
	{
		return EShaderParameterType::Sampler;
	}

	FStringView UntemplatedType = StripTemplateFromType(InType);

	if (AllSRVTypes.Contains(UntemplatedType) || PlatformConfiguration.ExtraSRVTypes.Contains(UntemplatedType))
	{
		return EShaderParameterType::SRV;
	}

	if (AllUAVTypes.Contains(UntemplatedType) || PlatformConfiguration.ExtraUAVTypes.Contains(UntemplatedType))
	{
		return EShaderParameterType::UAV;
	}

	if (UntemplatedType == TEXTVIEW("ConstantBuffer"))
	{
		return EShaderParameterType::UniformBuffer;
	}

	return EShaderParameterType::LooseData;
}

EShaderParameterType FShaderParameterParser::ParseAndRemoveBindlessParameterPrefix(FStringView& InName)
{
	const FStringView OriginalName = InName;

	if (InName = UE::String::RemoveFromStart(InName, FStringView(kBindlessSRVPrefix)); InName != OriginalName)
	{
		return EShaderParameterType::BindlessSRV;
	}

	if (InName = UE::String::RemoveFromStart(InName, FStringView(kBindlessUAVPrefix)); InName != OriginalName)
	{
		return EShaderParameterType::BindlessUAV;
	}

	if (InName = UE::String::RemoveFromStart(InName, FStringView(kBindlessSamplerPrefix)); InName != OriginalName)
	{
		return EShaderParameterType::BindlessSampler;
	}

	return EShaderParameterType::LooseData;
}

EShaderParameterType FShaderParameterParser::ParseAndRemoveBindlessParameterPrefix(FString& InName)
{
	FStringView Name(InName);
	const EShaderParameterType ParameterType = ParseAndRemoveBindlessParameterPrefix(Name);
	InName = FString(Name);

	return ParameterType;
}

bool FShaderParameterParser::RemoveBindlessParameterPrefix(FString& InName)
{
	return InName.RemoveFromStart(kBindlessSRVPrefix)
		|| InName.RemoveFromStart(kBindlessUAVPrefix)
		|| InName.RemoveFromStart(kBindlessSamplerPrefix);
}

FStringView FShaderParameterParser::GetBindlessParameterPrefix(EShaderParameterType InShaderParameterType)
{
	switch (InShaderParameterType)
	{
	case EShaderParameterType::BindlessSampler: return kBindlessSamplerPrefix;
	case EShaderParameterType::BindlessSRV:     return kBindlessSRVPrefix;
	case EShaderParameterType::BindlessUAV:     return kBindlessUAVPrefix;
	}

	return FStringView();
}

static bool IsBindlessHeap(FStringView VariableName)
{
	return VariableName.StartsWith(FShaderParameterParser::kBindlessSRVArrayPrefix)
		|| VariableName.StartsWith(FShaderParameterParser::kBindlessUAVArrayPrefix)
		|| VariableName.StartsWith(FShaderParameterParser::kBindlessCBVArrayPrefix)
		|| VariableName.StartsWith(FShaderParameterParser::kBindlessSamplerArrayPrefix);
}

bool FShaderParameterParser::ParseParameters(
	const FShaderParametersMetadata* RootParametersStructure,
	TArray<FShaderCompilerError>& OutErrors)
{
	const FStringView ShaderSource(OriginalParsedShader);

	if (RootParametersStructure)
	{
		// Reserves the number of parameters up front.
		ParsedParameters.Reserve(RootParametersStructure->GetSize() / sizeof(int32));

		IterateShaderParameterMembers(
			*RootParametersStructure,
			[&](const FShaderParametersMetadata& ParametersMetadata,
				const FShaderParametersMetadata::FMember& Member,
				const TCHAR* ShaderBindingName,
				uint16 ByteOffset)
			{
				FParsedShaderParameter ParsedParameter;
				EnumAddFlags(ParsedParameter.Flags, EParsedShaderParameterFlags::WasInRootParameters);
				ParsedParameter.ConstantBufferOffset = ByteOffset;
				ParsedParameter.BaseType = Member.GetBaseType();
				ParsedParameter.PrecisionModifier = Member.GetPrecision();
				ParsedParameter.NumRows = Member.GetNumRows();
				ParsedParameter.NumColumns = Member.GetNumColumns();
				if (Member.IsVariableNativeType())
				{
					ParsedParameter.MemberSize = Member.GetMemberSize();
				}
				else if (Member.GetBaseType() == UBMT_DESCRIPTOR_RANGE)
				{
					ParsedParameter.MemberSize = sizeof(uint32) * 2;
				}
				else
				{
					ParsedParameter.MemberSize = 0;
				}
				ParsedParameters.Add(ShaderBindingName, ParsedParameter);
			});
	}

	bool bSuccess = true;

	// Browse the code for global shader parameter, Save their type and erase them white spaces.
	{
		enum class EState
		{
			// When to look for something to scan.
			Scanning,

			// When going to next ; in the global scope and reset.
			GoToNextSemicolonAndReset,

			// Parsing what might be a type of the parameter.
			ParsingPotentialType,
			ParsingPotentialTypeTemplateArguments,
			FinishedPotentialType,

			// Parsing what might be a name of the parameter.
			ParsingPotentialName,
			FinishedPotentialName,

			// Parsing what looks like array of the parameter.
			ParsingPotentialArraySize,
			FinishedArraySize,

			// Found a parameter, just finish to it's semi colon.
			FoundParameter,
		};

		const int32 ShaderSourceLen = ShaderSource.Len();

		int32 CurrentPragmaLineOffset = -1;
		int32 CurrentLineOffset = 0;

		int32 TypeQualifierStartPos = -1;
		int32 TypeStartPos = -1;
		int32 TypeEndPos = -1;
		int32 NameStartPos = -1;
		int32 NameEndPos = -1;
		int32 ArrayStartPos = -1;
		int32 ArrayEndPos = -1;
		int32 ScopeIndent = 0;
		bool bGloballyCoherent = false;

		EState State = EState::Scanning;
		bool bGoToNextLine = false;
		bool bGoToCommentClose = false;
		bool bGoToNextCloseParen = false;

		auto ResetState = [&]()
		{
			TypeQualifierStartPos = -1;
			TypeStartPos = -1;
			TypeEndPos = -1;
			NameStartPos = -1;
			NameEndPos = -1;
			ArrayStartPos = -1;
			ArrayEndPos = -1;
			bGloballyCoherent = false;
			State = EState::Scanning;
		};

		auto EmitError = [&](const FString& ErrorMessage)
		{
			FShaderCompilerError Error;
			Error.StrippedErrorMessage = ErrorMessage;
			ExtractFileAndLine(CurrentPragmaLineOffset, CurrentLineOffset, Error.ErrorVirtualFilePath, Error.ErrorLineString);
			OutErrors.Add(Error);
			bSuccess = false;
		};

		auto EmitUnpextectedHLSLSyntaxError = [&]()
		{
			EmitError(TEXT("Unexpected syntax when parsing shader parameters from shader code."));
			State = EState::GoToNextSemicolonAndReset;
		};

		for (int32 Cursor = 0; Cursor < ShaderSourceLen; Cursor++)
		{
			const TCHAR Char = ShaderSource[Cursor];

			auto FoundShaderParameter = [&]()
			{
				check(Char == ';');
				check(TypeStartPos != -1);
				check(TypeEndPos != -1);
				check(NameStartPos != -1);
				check(NameEndPos != -1);

				const FStringView Type = ShaderSource.Mid(TypeStartPos, TypeEndPos - TypeStartPos + 1);
				FStringView Name = ShaderSource.Mid(NameStartPos, NameEndPos - NameStartPos + 1);

				FStringView Leftovers = ShaderSource.Mid(NameEndPos + 1, (Cursor - 1) - (NameEndPos + 1) + 1);

				const EShaderParameterType ParsedBindlessType = ParseAndRemoveBindlessParameterPrefix(Name);
				const EShaderParameterType ParsedParameterType = ParseParameterType(Type);

				EBindlessConversionType BindlessConversionType = EBindlessConversionType::None;
				EShaderParameterType ParsedConstantBufferType = ParsedBindlessType;

				if (bBindlessEnabled && ParsedBindlessType == EShaderParameterType::LooseData)
				{
					if (bBindlessEnabled)
					{
						if (ParsedParameterType == EShaderParameterType::SRV)
						{
							BindlessConversionType = EBindlessConversionType::SRV;
							ParsedConstantBufferType = EShaderParameterType::BindlessSRV;
						}
						else if (ParsedParameterType == EShaderParameterType::UAV)
						{
							BindlessConversionType = EBindlessConversionType::UAV;
							ParsedConstantBufferType = EShaderParameterType::BindlessUAV;
						}
						else if (ParsedParameterType == EShaderParameterType::Sampler)
						{
							BindlessConversionType = EBindlessConversionType::Sampler;
							ParsedConstantBufferType = EShaderParameterType::BindlessSampler;
						}
					}

					if (BindlessConversionType != EBindlessConversionType::None && Leftovers.Contains(TEXT("register")))
					{
						// avoid rewriting hardcoded register assignments
						BindlessConversionType = EBindlessConversionType::None;
						ParsedConstantBufferType = ParsedParameterType;
					}
				}

				FStringView StrippedTypeStringView = StripTemplateFromType(Type);
				FString StrippedTypeString(StrippedTypeStringView);
				EShaderCodeResourceBindingType TypeDecl = ParseShaderResourceBindingType(*StrippedTypeString);

				FParsedShaderParameter ParsedParameter;

				EShaderParameterType RootCBParameterType = EShaderParameterType::Num;
				bool bUpdateParsedParameters = false;

 				const FString ParsedParameterKey(Name);
				if (ParsedParameters.Contains(ParsedParameterKey))
				{
					if (ParsedParameters.FindChecked(ParsedParameterKey).IsFound())
					{
						// If it has already been found, it means it is duplicated. Do nothing and let the shader compiler throw the error.
					}
					else
					{
						// Update the parsed parameters
						bUpdateParsedParameters = true;
						ParsedParameter = ParsedParameters.FindChecked(ParsedParameterKey);

						// Erase the parameter to move it into the root constant buffer.
						if (bMoveGlobalsToRootCB && EnumHasAnyFlags(ParsedParameter.Flags, EParsedShaderParameterFlags::WasInRootParameters))
						{
							const bool bMoveToRootCB =
								ParsedParameter.BaseType == UBMT_INT32 ||
								ParsedParameter.BaseType == UBMT_UINT32 ||
								ParsedParameter.BaseType == UBMT_FLOAT32 ||
								ParsedParameter.BaseType == UBMT_DESCRIPTOR_RANGE ||
								ParsedBindlessType != EShaderParameterType::LooseData ||
								BindlessConversionType != EBindlessConversionType::None;

							if (bMoveToRootCB)
							{
								RootCBParameterType = ParsedConstantBufferType;
							}
						}
					}
				}
				else
				{
					// Update the parsed parameters to still have file and line number.
					bUpdateParsedParameters = true;
				}

				// Update 
				if (bUpdateParsedParameters)
				{
					ParsedParameter.ParsedName = Name;
					ParsedParameter.ParsedType = Type;
					ParsedParameter.ParsedPragmaLineOffset = CurrentPragmaLineOffset;
					ParsedParameter.ParsedLineOffset = CurrentLineOffset;
					ParsedParameter.ParsedCharOffsetStart = TypeQualifierStartPos != -1 ? TypeQualifierStartPos : TypeStartPos;
					ParsedParameter.ParsedCharOffsetEnd = Cursor;
					ParsedParameter.BindlessConversionType = BindlessConversionType;
					ParsedParameter.RootCBParameterType = RootCBParameterType;
					ParsedParameter.ParsedTypeDecl = TypeDecl;

					if (ArrayStartPos != -1 && ArrayEndPos != -1)
					{
						ParsedParameter.ParsedArraySize = ShaderSource.Mid(ArrayStartPos + 1, ArrayEndPos - ArrayStartPos - 1);
					}

					if (bGloballyCoherent)
					{
						EnumAddFlags(ParsedParameter.Flags, EParsedShaderParameterFlags::GloballyCoherent);
					}
					if (!EnumHasAnyFlags(ParsedParameter.Flags, EParsedShaderParameterFlags::WasInRootParameters) && ParsedParameterType == EShaderParameterType::LooseData && ParsedBindlessType == EShaderParameterType::LooseData)
					{
						if (!EnumHasAnyFlags(PlatformConfiguration.Flags, EShaderParameterParserConfigurationFlags::BindlessUsesArrays) || !IsBindlessHeap(Name))
						{
							EnumAddFlags(ParsedParameter.Flags, EParsedShaderParameterFlags::IsGlobalParameter);
						}
					}

					ParsedParameters.Add(ParsedParameterKey, ParsedParameter);
				}

				ResetState();
			};

			const bool bIsWhiteSpace = Char == ' ' || Char == '\t' || Char == '\r' || Char == '\n';
			const bool bIsLetter = (Char >= 'a' && Char <= 'z') || (Char >= 'A' && Char <= 'Z');
			const bool bIsNumber = Char >= '0' && Char <= '9';

			const TCHAR* UpComing = ShaderSource.GetData() + Cursor;
			const int32 RemainingSize = ShaderSourceLen - Cursor;

			CurrentLineOffset += Char == '\n';

			// Go to the next line if this is a preprocessor macro.
			if (bGoToNextLine)
			{
				if (Char == '\n')
				{
					bGoToNextLine = false;
				}
				continue;
			}
			else if (bGoToCommentClose)
			{
				if (Char == '*' && UpComing[1] == '/')
				{
					Cursor++;
					bGoToCommentClose = false;
				}
				continue;
			}
			else if (bGoToNextCloseParen)
			{
				if (Char == ')')
				{
					Cursor++;
					bGoToNextCloseParen = false;
				}
				continue;
			}
			else if (Char == '#')
			{
				if (RemainingSize > 6 && FCString::Strncmp(UpComing, TEXT("#line "), 6) == 0)
				{
					CurrentPragmaLineOffset = Cursor;
					CurrentLineOffset = -1; // that will be incremented to 0 when reaching the \n at the end of the #line
				}

				bGoToNextLine = true;
				continue;
			}
			else if (Char == '_' && RemainingSize > 7 && FCString::Strncmp(UpComing, TEXT("_Pragma"), 7) == 0)
			{
				bGoToNextCloseParen = true;
				continue;
			}

			// If within a scope, just carry on until outside the scope.
			if (ScopeIndent > 0 || Char == '{')
			{
				if (Char == '{')
				{
					ScopeIndent++;
				}
				else if (Char == '}')
				{
					ScopeIndent--;
					if (ScopeIndent == 0)
					{
						ResetState();
					}
				}
				continue;
			}

			if (State == EState::Scanning)
			{
				if (bIsLetter)
				{
					static const TCHAR* KeywordTable[] =
					{
						TEXT("const"),
						TEXT("globallycoherent"),
						TEXT("enum"),
						TEXT("class"),
						TEXT("struct"),
						TEXT("static"),
					};
					static int32 KeywordTableSize[] = { 5, 16, 4, 5, 6, 6 };

					int32 RecognisedKeywordId = -1;
					for (int32 KeywordId = 0; KeywordId < UE_ARRAY_COUNT(KeywordTable); KeywordId++)
					{
						const TCHAR* Keyword = KeywordTable[KeywordId];
						const int32 KeywordSize = KeywordTableSize[KeywordId];

						if (RemainingSize > KeywordSize)
						{
							TCHAR KeywordEndTestChar = UpComing[KeywordSize];

							if ((KeywordEndTestChar == ' ' || KeywordEndTestChar == '\r' || KeywordEndTestChar == '\n' || KeywordEndTestChar == '\t') &&
								FCString::Strncmp(UpComing, Keyword, KeywordSize) == 0)
							{
								RecognisedKeywordId = KeywordId;
								break;
							}
						}
					}

					if (RecognisedKeywordId == -1)
					{
						// Might have found beginning of the type of a parameter.
						State = EState::ParsingPotentialType;
						TypeStartPos = Cursor;
					}
					else if (RecognisedKeywordId == 0)
					{
						// Ignore the const keywords, but still parse given it might still be a shader parameter.
						if (TypeQualifierStartPos == -1)
						{
							// If the parameter is erased, we also have to erase *all* 'const'-qualifiers, e.g. "const int Foo" or "const const int Foo".
							TypeQualifierStartPos = Cursor;
						}
						Cursor += KeywordTableSize[RecognisedKeywordId];
					}
					else if (RecognisedKeywordId == 1)
					{
						// Mark that we got the globallycoherent keyword and keep moving to the next set of qualifiers
						bGloballyCoherent = true;
						if (TypeQualifierStartPos == -1)
						{
							// If the parameter is erased, we also have to erase *all* qualifiers, e.g. "const int Foo" or "const const int Foo".
							TypeQualifierStartPos = Cursor;
						}
						Cursor += KeywordTableSize[RecognisedKeywordId];
					}
					else
					{
						// Purposefully ignore enum, class, struct, static
						State = EState::GoToNextSemicolonAndReset;
					}
				}
				else if (bIsWhiteSpace)
				{
					// Keep parsing void.
				}
				else if (Char == ';')
				{
					// Looks like redundant semicolon, just ignore and keep scanning.
				}
				else if (Char == '/')
				{
					if (UpComing[1] == '/')
					{
						bGoToNextLine = true;
						continue;
					}

					if (UpComing[1] == '*')
					{
						bGoToCommentClose = true;
						continue;
					}
				}
				else
				{
					// No idea what this is, just go to next semi colon.
					State = EState::GoToNextSemicolonAndReset;
				}
			}
			else if (State == EState::GoToNextSemicolonAndReset)
			{
				// If need to go to next global semicolon and reach it. Resume browsing.
				if (Char == ';')
				{
					ResetState();
				}
			}
			else if (State == EState::ParsingPotentialType)
			{
				// Found character legal for a type...
				if (bIsLetter ||
					bIsNumber ||
					Char == '_')
				{
					// Keep browsing what might be type of the parameter.
				}
				else if (Char == ':')
				{
					// Handle :: in type names
					if (UpComing[1] == ':')
					{
						// next loop iteration takes us to the next ':', so go past that
						Cursor++;
					}
				}
				else if (Char == '<')
				{
					// Found what looks like the beginning of template argument that is legal on resource types for Instance Texture2D< float >
					State = EState::ParsingPotentialTypeTemplateArguments;
				}
				else if (bIsWhiteSpace)
				{
					// Might have found a type.
					State = EState::FinishedPotentialType;
					TypeEndPos = Cursor - 1;
				}
				else
				{
					// Found unexpected character in the type.
					State = EState::GoToNextSemicolonAndReset;
				}
			}
			else if (State == EState::ParsingPotentialTypeTemplateArguments)
			{
				// Found character legal for a template argument...
				if (bIsLetter ||
					bIsNumber ||
					Char == '_')
				{
					// Keep browsing what might be type of the parameter.
				}
				else if (Char == ':')
				{
					// Handle :: in type names
					if (UpComing[1] == ':')
					{
						// next loop iteration takes us to the next ':', so go past that
						Cursor++;
					}
				}
				else if (bIsWhiteSpace || Char == ',')
				{
					// Spaces and comas are legal within argument of the template arguments.
				}
				else if (Char == '>')
				{
					// Might have found a type with template argument.
					State = EState::FinishedPotentialType;
					TypeEndPos = Cursor;
				}
				else
				{
					// Found unexpected character in the type.
					State = EState::GoToNextSemicolonAndReset;
				}
			}
			else if (State == EState::FinishedPotentialType)
			{
				if (bIsLetter)
				{
					// Might have found beginning of the name of a parameter.
					State = EState::ParsingPotentialName;
					NameStartPos = Cursor;
				}
				else if (Char == '<')
				{
					// Might have found beginning of a template argument for the type, that was separate by a whitespace from type. For instance Texture2D <float>
					State = EState::ParsingPotentialTypeTemplateArguments;
				}
				else if (bIsWhiteSpace)
				{
					// Keep parsing void.
				}
				else
				{
					// No idea what this is, just go to next semi colon.
					State = EState::GoToNextSemicolonAndReset;
				}
			}
			else if (State == EState::ParsingPotentialName)
			{
				// Found character legal for a name...
				if (bIsLetter ||
					bIsNumber ||
					Char == '_')
				{
					// Keep browsing what might be name of the parameter.
				}
				else if (Char == ':' || Char == '=')
				{
					// Found a parameter with syntax:
					// uint MyParameter : <whatever>;
					// uint MyParameter = <DefaultValue>;
					NameEndPos = Cursor - 1;
					State = EState::FoundParameter;
				}
				else if (Char == ';')
				{
					// Found a parameter with syntax:
					// uint MyParameter;
					NameEndPos = Cursor - 1;
					FoundShaderParameter();
				}
				else if (Char == '[')
				{
					// Syntax:
					//  uint MyArray[
					NameEndPos = Cursor - 1;
					ArrayStartPos = Cursor;
					State = EState::ParsingPotentialArraySize;
				}
				else if (bIsWhiteSpace)
				{
					// Might have found a name.
					// uint MyParameter <Still need to know what is after>;
					NameEndPos = Cursor - 1;
					State = EState::FinishedPotentialName;
				}
				else
				{
					// Found unexpected character in the name.
					// syntax:
					// uint MyFunction(<Don't care what is after>
					State = EState::GoToNextSemicolonAndReset;
				}
			}
			else if (State == EState::FinishedPotentialName || State == EState::FinishedArraySize)
			{
				if (Char == ';')
				{
					// Found a parameter with syntax:
					// uint MyParameter <a bit of OK stuf>;
					FoundShaderParameter();
				}
				else if (Char == ':')
				{
					// Found a parameter with syntax:
					// uint MyParameter <a bit of OK stuf> : <Ignore all this crap>;
					State = EState::FoundParameter;
				}
				else if (Char == '=')
				{
					// Found syntax that doesn't make any sens:
					// uint MyParameter <a bit of OK stuf> = <Ignore all this crap>;
					State = EState::FoundParameter;
					// TDOO: should error out that this is useless.
				}
				else if (Char == '[')
				{
					if (State == EState::FinishedPotentialName)
					{
						// Syntax:
						//  uint MyArray [
						ArrayStartPos = Cursor;
						State = EState::ParsingPotentialArraySize;
					}
					else
					{
						EmitError(TEXT("Shader parameters can only support one dimensional array"));
					}
				}
				else if (bIsWhiteSpace)
				{
					// Keep parsing void.
				}
				else
				{
					// Found unexpected stuff.
					State = EState::GoToNextSemicolonAndReset;
				}
			}
			else if (State == EState::ParsingPotentialArraySize)
			{
				if (Char == ']')
				{
					ArrayEndPos = Cursor;
					State = EState::FinishedArraySize;
				}
				else if (Char == ';')
				{
					EmitUnpextectedHLSLSyntaxError();
				}
				else
				{
					// Keep going through the array size that might be a complex expression.
				}
			}
			else if (State == EState::FoundParameter)
			{
				if (Char == ';')
				{
					FoundShaderParameter();
				}
				else
				{
					// Cary on skipping all crap we don't care about shader parameter until we find it's semi colon.
				}
			}
			else
			{
				unimplemented();
			}
		} // for (int32 Cursor = 0; Cursor < ShaderSourceLen; Cursor++)
	}

	return bSuccess;
}

void FShaderParameterParser::RemoveMovingParametersFromSource(FString& PreprocessedShaderSource)
{
	for (TPair<FString, FParsedShaderParameter>& Itr : ParsedParameters)
	{
		const FParsedShaderParameter& ParsedParameter = Itr.Value;

		// If this parameter is going to be in the root constant buffer
		if (ParsedParameter.RootCBParameterType != EShaderParameterType::Num &&
			// but it's not being converted to bindless
			ParsedParameter.BindlessConversionType == EBindlessConversionType::None &&
			ParsedParameter.ParsedCharOffsetStart != INDEX_NONE)
		{
			// then erase this shader parameter conserving the same line numbers.
			for (int32 j = ParsedParameter.ParsedCharOffsetStart; j <= ParsedParameter.ParsedCharOffsetEnd; j++)
			{
				if (PreprocessedShaderSource[j] != '\r' && PreprocessedShaderSource[j] != '\n')
				{
					PreprocessedShaderSource[j] = ' ';
				}
			}
		}
	}
}

static FStringView GetBindlessParameterPrefix(EBindlessConversionType InConversionType)
{
	switch (InConversionType)
	{
	case EBindlessConversionType::SRV:     return FShaderParameterParser::kBindlessSRVPrefix;
	case EBindlessConversionType::UAV:     return FShaderParameterParser::kBindlessUAVPrefix;
	case EBindlessConversionType::Sampler: return FShaderParameterParser::kBindlessSamplerPrefix;
	}
	return FStringView();
}

static FStringView GetBindlessArrayHeapPrefix(EBindlessConversionType InConversionType)
{
	switch (InConversionType)
	{
	case EBindlessConversionType::SRV:     return FShaderParameterParser::kBindlessSRVArrayPrefix;
	case EBindlessConversionType::UAV:     return FShaderParameterParser::kBindlessUAVArrayPrefix;
	case EBindlessConversionType::Sampler: return FShaderParameterParser::kBindlessSamplerArrayPrefix;
	}
	return FStringView();
}

FString FShaderParameterParser::GenerateBindlessParameterDeclaration(const FParsedShaderParameter& ParsedParameter) const
{
	const bool bIsSampler = (ParsedParameter.BindlessConversionType == EBindlessConversionType::Sampler);
	const bool bIsUAV = (ParsedParameter.BindlessConversionType == EBindlessConversionType::UAV);
	const FStringView Name = ParsedParameter.ParsedName;
	const FStringView Type = ParsedParameter.ParsedType;
	const TCHAR* StorageClass = EnumHasAnyFlags(ParsedParameter.Flags, EParsedShaderParameterFlags::GloballyCoherent) ? TEXT("globallycoherent ") : TEXT("");
	const FStringView IndexPrefix = ::GetBindlessParameterPrefix(ParsedParameter.BindlessConversionType);

	TStringBuilder<64> IndexString;
	IndexString << IndexPrefix << Name;

	TStringBuilder<512> Result;

	// If we weren't going to be added to a root constant buffer, that means we need to declare our index before we declare our getter.
	if (ParsedParameter.RootCBParameterType == EShaderParameterType::Num)
	{
		// e.g. `uint BindlessResource_##Name;`
		Result << TEXT("uint ") << IndexString << TEXT("; ");
	}

	// Add the typedef to keep return types shortened
	//  `typedef Type SafeType##Name;`

	TStringBuilder<64> TypedefName;
	TypedefName << TEXT("SafeType") << Name;

	Result << TEXT("typedef ") << Type << TEXT(" ") << TypedefName << TEXT(";");

	// Full type to use for return types. Makes sure globallycoherent is used where needed. Should be using the typedef name.
	TStringBuilder<64> FullType;
	FullType << StorageClass << TypedefName;

	TStringBuilder<64> HeapName;

	if (EnumHasAnyFlags(PlatformConfiguration.Flags, EShaderParameterParserConfigurationFlags::BindlessUsesArrays))
	{
		const FStringView HeapPrefix = GetBindlessArrayHeapPrefix(ParsedParameter.BindlessConversionType);

		HeapName << HeapPrefix << TypedefName;

		// Declare a heap for the RewriteType
		// e.g. `SafeType##Name ResourceDescriptorHeap_SafeType##Name[];`
		Result << FullType << TEXT(" ") << HeapName << TEXT("[]; ");
		// :todo-jn: specify the descriptor set and binding directly in source instead of patching SPIRV
	}

	const FString BindlessAccess = PlatformConfiguration.GenerateBindlessAccess(ParsedParameter.BindlessConversionType, FullType, HeapName, IndexString);
	const TCHAR* FunctionPrefix = bIsSampler ? TEXT("GetBindlessSampler") : bIsUAV ? TEXT("GetBindlessUAV") : TEXT("GetBindlessSRV");

	// e.g. `Type GetBindlessSRV##Name() { return GetSRVFromHeap(Type, BindlessSRV_##Name); } static const Type Name = GetBindlessSRV##Name()`
	// e.g. `Type GetBindlessUAV##Name() { return GetUAVFromHeap(Type, BindlessUAV_##Name); } static const Type Name = GetBindlessUAV##Name()`
	// or   `Type GetBindlessSampler##Name() { return GetSamplerFromHeap(Type, BindlessSampler_##Name); } static const Type Name = GetBindlessSampler##Name()`
	Result << FullType << TEXT(" ") << FunctionPrefix << Name << TEXT("() { return ") << BindlessAccess << TEXT("; } ");
	Result << TEXT("static const ") << FullType << TEXT(" ") << Name << TEXT(" = ") << FunctionPrefix << Name << TEXT("();");

	return FString(Result);
}

void FShaderParameterParser::ApplyBindlessModifications(FString& PreprocessedShaderSource)
{
	if (bBindlessEnabled)
	{
		// Array of modifications to do on PreprocessedShaderSource
		struct FShaderCodeModifications
		{
			int32 CharOffsetStart;
			int32 CharOffsetEnd;
			FString Replace;
		};
		TArray<FShaderCodeModifications> Modifications;
		Modifications.Reserve(ParsedParameters.Num());

		const bool bReplaceGlobals = EnumHasAnyFlags(PlatformConfiguration.Flags, EShaderParameterParserConfigurationFlags::ReplaceGlobals);

		for (const TPair<FString, FParsedShaderParameter>& Itr : ParsedParameters)
		{
			const FParsedShaderParameter& ParsedParameter = Itr.Value;

			if (!ParsedParameter.IsFound())
			{
				continue;
			}

			if (ParsedParameter.BindlessConversionType != EBindlessConversionType::None)
			{
				FShaderCodeModifications Modif;
				Modif.CharOffsetStart = ParsedParameter.ParsedCharOffsetStart;
				Modif.CharOffsetEnd = ParsedParameter.ParsedCharOffsetEnd + 1;
				Modif.Replace = GenerateBindlessParameterDeclaration(ParsedParameter);

				Modifications.Add(Modif);
			}
			else if (bReplaceGlobals)
			{
				if (EnumHasAnyFlags(ParsedParameter.Flags, EParsedShaderParameterFlags::IsGlobalParameter))
				{
					FShaderCodeModifications Modif;
					Modif.CharOffsetStart = ParsedParameter.ParsedCharOffsetStart;
					Modif.CharOffsetEnd = ParsedParameter.ParsedCharOffsetEnd + 1;

					const int32 NumChars = Modif.CharOffsetEnd - Modif.CharOffsetStart;
					const FStringView OriginalString(&OriginalParsedShader[Modif.CharOffsetStart], NumChars);
					Modif.Replace = PlatformConfiguration.ReplaceGlobal(OriginalString, ParsedParameter.ParsedName);

					if (Modif.Replace != OriginalString)
					{
						Modifications.Add(Modif);
					}
				}
			}
		}

		// Apply all modifications
		if (Modifications.Num() > 0)
		{
			// Sort all the modifications in order
			Modifications.Sort(
				[](const FShaderCodeModifications& ModifA, const FShaderCodeModifications& ModifB)
				{
					return  ModifA.CharOffsetStart < ModifB.CharOffsetStart;
				}
			);

			// Find out the size of the shader code after all modifications
			int32 NewShaderCodeSize = PreprocessedShaderSource.Len();
			for (const FShaderCodeModifications& Modif : Modifications)
			{
				// Count the number of line return \n in CharOffsetStart -> CharOffsetEnd to ensure the line number remain unchanged.
				check(!Modif.Replace.Contains(TEXT("\n")));
				//int32 ReplacedCarriagedReturn = 0;
				for (int32 CharPos = Modif.CharOffsetStart; CharPos < Modif.CharOffsetEnd; CharPos++)
				{
					ensure(PreprocessedShaderSource[CharPos] != '\n');
				}

				NewShaderCodeSize += Modif.Replace.Len() - (Modif.CharOffsetEnd - Modif.CharOffsetStart); // + ReplacedCarriagedReturn;
			}

			// Splice all the code and modifications together
			FString NewShaderCode;
			NewShaderCode.Reserve(NewShaderCodeSize);

			int32 CurrentCodePos = 0;
			for (const FShaderCodeModifications& Modif : Modifications)
			{
				check(CurrentCodePos <= Modif.CharOffsetStart);
				NewShaderCode += PreprocessedShaderSource.Mid(CurrentCodePos, Modif.CharOffsetStart - CurrentCodePos);
				NewShaderCode += Modif.Replace;
				CurrentCodePos = Modif.CharOffsetEnd;
			}
			check(CurrentCodePos <= PreprocessedShaderSource.Len());
			NewShaderCode += PreprocessedShaderSource.Mid(CurrentCodePos, PreprocessedShaderSource.Len() - CurrentCodePos);

			// Commit all modifications to caller
			PreprocessedShaderSource = NewShaderCode;

			bModifiedShader = true;
		}
	}
}

static const TCHAR* GetConstantSwizzle(uint16 ByteOffset)
{
	switch (ByteOffset % 16)
	{
	default: unimplemented();
	case 0:  return TEXT("");
	case 4:  return TEXT(".y");
	case 8:  return TEXT(".z");
	case 12: return TEXT(".w");
	}
}

bool FShaderParameterParser::MoveShaderParametersToRootConstantBuffer(
	const FShaderParametersMetadata* RootParametersStructure,
	EShaderFrequency ShaderFrequency,
	FString& PreprocessedShaderSource)
{
	bool bSuccess = true;

	// Generate the root cbuffer content.
	if (RootParametersStructure && bMoveGlobalsToRootCB)
	{
		TStringBuilder<0> ConstantBufferMembers;
		bool bNeedDescriptorRangeType = false;

		IterateShaderParameterMembers(
			*RootParametersStructure,
			[&](const FShaderParametersMetadata& ParametersMetadata,
				const FShaderParametersMetadata::FMember& Member,
				const TCHAR* ShaderBindingName,
				uint16 ByteOffset)
		{
			FParsedShaderParameter* ParsedParameter = ParsedParameters.Find(ShaderBindingName);
			if (ParsedParameter && ParsedParameter->IsFound() && ParsedParameter->RootCBParameterType != EShaderParameterType::Num)
			{
				const uint32 ConstantRegister = ByteOffset / 16;
				const TCHAR* ConstantSwizzle = GetConstantSwizzle(ByteOffset);

				// Detect if we need to add the typedef before the constant buffer.
				if (ParsedParameter->BaseType == UBMT_DESCRIPTOR_RANGE)
				{
					bNeedDescriptorRangeType = true;
				}

#define SVARG(N) N.Len(), N.GetData()
				if (IsParameterBindless(ParsedParameter->RootCBParameterType))
				{
					const FStringView Prefix = GetBindlessParameterPrefix(ParsedParameter->RootCBParameterType);

					ConstantBufferMembers.Appendf(
						TEXT("uint %.*s%.*s : packoffset(c%d%s);\n"),
						SVARG(Prefix),
						SVARG(ParsedParameter->ParsedName),
						ConstantRegister,
						ConstantSwizzle
					);
				}
				else if (ParsedParameter->RootCBParameterType == EShaderParameterType::LooseData)
				{
					if (!ParsedParameter->ParsedArraySize.IsEmpty())
					{
						ConstantBufferMembers.Appendf(
							TEXT("%.*s %s[%.*s] : packoffset(c%d%s);\n"),
							SVARG(ParsedParameter->ParsedType),
							ShaderBindingName,
							SVARG(ParsedParameter->ParsedArraySize),
							ConstantRegister,
							ConstantSwizzle
						);
					}
					else
					{
						ConstantBufferMembers.Appendf(
							TEXT("%.*s %s : packoffset(c%d%s);\n"),
							SVARG(ParsedParameter->ParsedType),
							ShaderBindingName,
							ConstantRegister,
							ConstantSwizzle
						);
					}
				}
#undef SVARG
			}
		});

		if (ConstantBufferMembers.Len() > 0)
		{
			TStringBuilder<0> ConstantBufferCode;

			// Hack-ish approach to make sure the descriptor range type is defined before the constant buffer.
			if (bNeedDescriptorRangeType)
			{
				ConstantBufferCode.Append(TEXT("typedef uint2 FDescriptorRange;\n"));
			}

			ConstantBufferCode << PlatformConfiguration.StableConstantBufferType << TEXT(" ") << FShaderParametersMetadata::kRootUniformBufferBindingName << PlatformConfiguration.GetStableConstantBufferRegisterString(ShaderFrequency) << TEXT("\n{\n");
			ConstantBufferCode << ConstantBufferMembers;
			ConstantBufferCode << TEXT("}\n\n");

			FString NewShaderCode = (
				MakeInjectedShaderCodeBlock(TEXT("MoveShaderParametersToRootConstantBuffer"), *ConstantBufferCode) +
				PreprocessedShaderSource);

			PreprocessedShaderSource = MoveTemp(NewShaderCode);

			bMovedLoosedParametersToRootConstantBuffer = true;
			bModifiedShader = true;
		}
	} // if (CompilerInput.RootParametersStructure && bMoveGlobalsToRootCB)

	return bSuccess;
}

bool FShaderParameterParser::ParseAndModify(const FShaderCompilerInput& CompilerInput, TArray<FShaderCompilerError>& OutErrors, FString& PreprocessedShaderSource)
{
	bBindlessEnabled = CompilerInput.IsBindlessEnabled();

	const bool bUseStableConstantBuffer = EnumHasAnyFlags(PlatformConfiguration.Flags, EShaderParameterParserConfigurationFlags::UseStableConstantBuffer);
	const bool bSupportsBindless = EnumHasAnyFlags(PlatformConfiguration.Flags, EShaderParameterParserConfigurationFlags::SupportsBindless);
	const bool bAlwaysParseParams = EnumHasAnyFlags(PlatformConfiguration.Flags, EShaderParameterParserConfigurationFlags::AlwaysParseParams);
	const bool bHasRootParameters = (CompilerInput.RootParametersStructure != nullptr);
	const bool bRootParametersModification = bUseStableConstantBuffer && (CompilerInput.IsRayTracingShader() || CompilerInput.ShouldUseStableConstantBuffer());
	const bool bBindlessModifications = bSupportsBindless && bBindlessEnabled;

	const bool bShouldModify = bRootParametersModification || bBindlessModifications;

	// Always parse if we have root parameters since we need that data during reflection validation
	const bool bShouldParse = bHasRootParameters || bShouldModify || bAlwaysParseParams;

	// The shader doesn't have any parameter binding through shader structure, therefore don't do anything.
	if (!bShouldParse)
	{
		return true;
	}

	bMoveGlobalsToRootCB = bRootParametersModification;
	OriginalParsedShader = PreprocessedShaderSource;

	if (!ParseParameters(CompilerInput.RootParametersStructure, OutErrors))
	{
		return false;
	}

	bool bResult = true;

	if (bShouldModify)
	{
		RemoveMovingParametersFromSource(PreprocessedShaderSource);

		if (bSupportsBindless)
		{
			ApplyBindlessModifications(PreprocessedShaderSource);
		}

		if (bMoveGlobalsToRootCB)
		{
			bResult = MoveShaderParametersToRootConstantBuffer(CompilerInput.RootParametersStructure, CompilerInput.Target.GetFrequency(), PreprocessedShaderSource);
		}

#if DO_GUARD_SLOW
		if (bResult)
		{
			if (DidModifyShader())
			{
				checkSlow(PreprocessedShaderSource != OriginalParsedShader);
			}
			else
			{
				checkSlow(PreprocessedShaderSource == OriginalParsedShader);
			}
		}
#endif

	}

	return bResult;
}

void FShaderParameterParser::ValidateShaderParameterType(
	const FShaderCompilerInput& CompilerInput,
	FStringView ShaderBindingName,
	int32 ReflectionOffset,
	int32 ReflectionSize,
	bool bPlatformSupportsPrecisionModifier,
	FShaderCompilerOutput& CompilerOutput) const
{
	FString BindingName(ShaderBindingName);

	const bool bBindlessHack = RemoveBindlessParameterPrefix(BindingName);

	const FShaderParameterParser::FParsedShaderParameter& ParsedParameter = FindParameterInfos(BindingName);

	check(ParsedParameter.IsFound());
	check(CompilerInput.RootParametersStructure);

	if (ReflectionSize > 0 && bMovedLoosedParametersToRootConstantBuffer)
	{
		// Verify the offset of the parameter coming from shader reflections honor the packoffset()
		check(ReflectionOffset == ParsedParameter.ConstantBufferOffset);
	}

	// Validate the shader type.
	if (!bBindlessHack)
	{
		FString ExpectedShaderType;
		FShaderParametersMetadata::FMember::GenerateShaderParameterType(
			ExpectedShaderType, 
			bPlatformSupportsPrecisionModifier,
			ParsedParameter.BaseType,
			ParsedParameter.PrecisionModifier,
			ParsedParameter.NumRows,
			ParsedParameter.NumColumns);

		const bool bShouldBeInt = ParsedParameter.BaseType == UBMT_INT32;
		const bool bShouldBeUint = ParsedParameter.BaseType == UBMT_UINT32;

		// Match parsed type with expected shader type
		bool bIsTypeCorrect = ParsedParameter.ParsedType == ExpectedShaderType;

		if (!bIsTypeCorrect)
		{
			auto CheckTypeCorrect = [&ParsedParameter, &ExpectedShaderType](int32 ParsedOffset, int32 ExpectedOffset) -> bool
			{
				const FStringView Parsed = ParsedParameter.ParsedType.RightChop(ParsedOffset);
				const FStringView Expected = FStringView(ExpectedShaderType).RightChop(ExpectedOffset);

				return Parsed.Compare(Expected, ESearchCase::CaseSensitive) == 0;
			};

			// Accept half-precision floats when single-precision was requested
			if (ParsedParameter.ParsedType.StartsWith(TEXT("half")) && ParsedParameter.BaseType == UBMT_FLOAT32)
			{
				bIsTypeCorrect = CheckTypeCorrect(4, 5);
			}
			// Accept single-precision floats when half-precision was expected
			else if (ParsedParameter.ParsedType.StartsWith(TEXT("float")) && ExpectedShaderType.StartsWith(TEXT("half")))
			{
				bIsTypeCorrect = CheckTypeCorrect(5, 4);
			}
			// support for min16float
			else if (ParsedParameter.ParsedType.StartsWith(TEXT("min16float")) && ExpectedShaderType.StartsWith(TEXT("float")))
			{
				bIsTypeCorrect = CheckTypeCorrect(10, 5);
			}
			else if (ParsedParameter.ParsedType.StartsWith(TEXT("min16float")) && ExpectedShaderType.StartsWith(TEXT("half")))
			{
				bIsTypeCorrect = CheckTypeCorrect(10, 4);
			}
		}

		// Allow silent casting between signed and unsigned on shader bindings.
		if (!bIsTypeCorrect && (bShouldBeInt || bShouldBeUint))
		{
			FString NewExpectedShaderType;
			if (bShouldBeInt)
			{
				// tries up with an uint.
				NewExpectedShaderType = TEXT("u") + ExpectedShaderType;
			}
			else
			{
				// tries up with an int.
				NewExpectedShaderType = ExpectedShaderType;
				NewExpectedShaderType.RemoveAt(0);
			}

			bIsTypeCorrect = ParsedParameter.ParsedType == NewExpectedShaderType;
		}

		if (!bIsTypeCorrect)
		{
			FString CppCodeName = CompilerInput.RootParametersStructure->GetFullMemberCodeName(ParsedParameter.ConstantBufferOffset);

			FShaderCompilerError Error(FString::Printf(
				TEXT("Error: Type %.*s of shader parameter %.*s in shader mismatch the shader parameter structure: %s expects a %s"),
				ParsedParameter.ParsedType.Len(), ParsedParameter.ParsedType.GetData(),
				ShaderBindingName.Len(), ShaderBindingName.GetData(),
				*CppCodeName,
				*ExpectedShaderType));
			GetParameterFileAndLine(ParsedParameter, Error.ErrorVirtualFilePath, Error.ErrorLineString);

			CompilerOutput.Errors.Add(Error);
			CompilerOutput.bSucceeded = false;

			AddNoteToDisplayShaderParameterMemberOnCppSide(CompilerInput, ParsedParameter, CompilerOutput);
		}
	}

	// Validate parameter size, in case this is an array.
	if (!bBindlessHack && ReflectionSize > int32(ParsedParameter.MemberSize))
	{
		FString CppCodeName = CompilerInput.RootParametersStructure->GetFullMemberCodeName(ParsedParameter.ConstantBufferOffset);

		FShaderCompilerError Error(FString::Printf(
			TEXT("Error: The size required to bind shader parameter %.*s is %i bytes, smaller than %s that is %i bytes in the parameter structure."),
			ShaderBindingName.Len(), ShaderBindingName.GetData(),
			ReflectionSize,
			*CppCodeName,
			ParsedParameter.MemberSize));
		GetParameterFileAndLine(ParsedParameter, Error.ErrorVirtualFilePath, Error.ErrorLineString);

		CompilerOutput.Errors.Add(Error);
		CompilerOutput.bSucceeded = false;

		AddNoteToDisplayShaderParameterMemberOnCppSide(CompilerInput, ParsedParameter, CompilerOutput);
	}
}

void FShaderParameterParser::ValidateShaderParameterTypes(
	const FShaderCompilerInput& CompilerInput,
	bool bPlatformSupportsPrecisionModifier,
	FShaderCompilerOutput& CompilerOutput) const
{
	// The shader doesn't have any parameter binding through shader structure, therefore don't do anything.
	if (!CompilerInput.RootParametersStructure)
	{
		return;
	}

	if (!CompilerOutput.bSucceeded)
	{
		return;
	}

	const TMap<FString, FParameterAllocation>& ParametersFoundByCompiler = CompilerOutput.ParameterMap.GetParameterMap();

	IterateShaderParameterMembers(
		*CompilerInput.RootParametersStructure,
		[&](const FShaderParametersMetadata& ParametersMetadata,
			const FShaderParametersMetadata::FMember& Member,
			const TCHAR* ShaderBindingName,
			uint16 ByteOffset)
		{
			if (
				Member.GetBaseType() != UBMT_INT32 &&
				Member.GetBaseType() != UBMT_UINT32 &&
				Member.GetBaseType() != UBMT_FLOAT32)
			{
				return;
			}

			const FParsedShaderParameter& ParsedParameter = ParsedParameters[ShaderBindingName];

			// Did not find shader parameter in code.
			if (!ParsedParameter.IsFound())
			{
				// Verify the shader compiler also did not find this parameter to make sure there is no bug in the parser.
				checkf(
					!ParametersFoundByCompiler.Contains(ShaderBindingName),
					TEXT("Looks like there is a bug in FShaderParameterParser ParameterName=%s DumpDebugInfoPath=%s"),
					ShaderBindingName,
					*CompilerInput.DumpDebugInfoPath);
				return;
			}

			int32 BoundOffset = 0;
			int32 BoundSize = 0;
			if (const FParameterAllocation* ParameterAllocation = ParametersFoundByCompiler.Find(ShaderBindingName))
			{
				BoundOffset = ParameterAllocation->BaseIndex;
				BoundSize = ParameterAllocation->Size;
			}

			ValidateShaderParameterType(CompilerInput, ShaderBindingName, BoundOffset, BoundSize, bPlatformSupportsPrecisionModifier, CompilerOutput);
		});
}

void FShaderParameterParser::ExtractFileAndLine(int32 PragmaLineOffset, int32 LineOffset, FString& OutFile, FString& OutLine) const
{
	if (PragmaLineOffset == -1)
	{
		return;
	}

	check(FCString::Strncmp((*OriginalParsedShader) + PragmaLineOffset, TEXT("#line"), 5) == 0);

	const int32 ShaderSourceLen = OriginalParsedShader.Len();

	int32 StartFilePos = -1;
	int32 EndFilePos = -1;
	int32 StartLinePos = -1;
	int32 EndLinePos = -1;

	for (int32 Cursor = PragmaLineOffset + 5; Cursor < ShaderSourceLen; Cursor++)
	{
		const TCHAR Char = OriginalParsedShader[Cursor];

		if (Char == '\n')
		{
			break;
		}
		else if (StartLinePos == -1 && FChar::IsDigit(Char))
		{
			StartLinePos = Cursor;
		}
		else if (StartLinePos != -1 && EndLinePos == -1 && !FChar::IsDigit(Char))
		{
			EndLinePos = Cursor - 1;
		}
		else if (StartFilePos == -1 && Char == TEXT('"'))
		{
			StartFilePos = Cursor + 1;
		}
		else if (StartFilePos != -1 && EndFilePos == -1 && Char == TEXT('"'))
		{
			EndFilePos = Cursor - 1;
			break;
		}
	}

	check(StartFilePos != -1);
	check(EndFilePos != -1);
	check(EndLinePos != -1);

	OutFile = OriginalParsedShader.Mid(StartFilePos, EndFilePos - StartFilePos + 1);
	FString LineBasis = OriginalParsedShader.Mid(StartLinePos, EndLinePos - StartLinePos + 1);

	int32 FinalLine = FCString::Atoi(*LineBasis) + LineOffset;
	OutLine = FString::FromInt(FinalLine);
}
