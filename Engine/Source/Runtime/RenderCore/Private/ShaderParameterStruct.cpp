// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ShaderParameterStruct.cpp: Shader parameter struct implementations.
=============================================================================*/

#include "ShaderParameterStruct.h"

FRDGTextureAccess::FRDGTextureAccess(FRDGTexture* InTexture, ERHIAccess InAccess)
	: FRDGTextureAccess(InTexture, InTexture ? InTexture->GetSubresourceRange() : FRDGTextureSubresourceRange(), InAccess)
{}

FRDGTextureAccess::FRDGTextureAccess(FRDGTextureSRV* InTextureSRV, ERHIAccess InAccess)
	: FRDGTextureAccess(InTextureSRV ? InTextureSRV->GetParent() : nullptr, InTextureSRV ? InTextureSRV->GetSubresourceRange() : FRDGTextureSubresourceRange(), InAccess)
{}

FRDGTextureAccess::FRDGTextureAccess(FRDGTextureUAV* InTextureUAV, ERHIAccess InAccess)
	: FRDGTextureAccess(InTextureUAV ? InTextureUAV->GetParent() : nullptr, InTextureUAV ? InTextureUAV->GetSubresourceRange() : FRDGTextureSubresourceRange(), InAccess)
{}

/** Context of binding a map. */
struct FShaderParameterStructBindingContext
{
	// Shader having its parameter bound.
	const FShader* Shader;

	// Bindings to bind.
	FShaderParameterBindings* Bindings;

	// The shader parameter map from the compilation.
	const FShaderParameterMap* ParametersMap;

	// Map of global shader name that were bound to C++ members.
	TMap<FString, FString> ShaderGlobalScopeBindings;

	// C++ name of the render target binding slot.
	FString RenderTargetBindingSlotCppName;

	// Shader PermutationId
	int32 PermutationId;

	// Whether this is for legacy shader parameter settings, or root shader parameter structures/
	bool bUseRootShaderParameters;


	void Bind(
		const FShaderParametersMetadata& StructMetaData,
		const TCHAR* MemberPrefix,
		uint32 GeneralByteOffset)
	{
		const TArray<FShaderParametersMetadata::FMember>& StructMembers = StructMetaData.GetMembers();

		for (const FShaderParametersMetadata::FMember& Member : StructMembers)
		{
			EUniformBufferBaseType BaseType = Member.GetBaseType();

			FString CppName = FString::Printf(TEXT("%s::%s"), StructMetaData.GetStructTypeName(), Member.GetName());

			// Ignore rasterizer binding slots entirely since this actually have nothing to do with a shader.
			if (BaseType == UBMT_RENDER_TARGET_BINDING_SLOTS)
			{
				if (!RenderTargetBindingSlotCppName.IsEmpty())
				{
					UE_LOGF(LogShaders, Fatal, "Render target binding slots collision: %ls & %ls",
						*RenderTargetBindingSlotCppName, *CppName);
				}
				RenderTargetBindingSlotCppName = CppName;
				continue;
			}

			// Ignore RDG access types when binding to shaders.
			if (IsRDGResourceAccessType(BaseType))
			{
				continue;
			}

			// Compute the shader member name to look for according to nesting.
			FString ShaderBindingName = FString::Printf(TEXT("%s%s"), MemberPrefix, Member.GetName());

			uint16 ByteOffset = uint16(GeneralByteOffset + Member.GetOffset());
			check(uint32(ByteOffset) == GeneralByteOffset + Member.GetOffset());

			const uint32 ArraySize = Member.GetNumElements();
			const bool bIsArray = ArraySize > 0;
			const bool bIsRHIResource = (
				BaseType == UBMT_TEXTURE ||
				BaseType == UBMT_SRV ||
				BaseType == UBMT_UAV ||
				BaseType == UBMT_SAMPLER ||
				BaseType == UBMT_DESCRIPTOR_RANGE ||
				BaseType == UBMT_RESOURCE_COLLECTION);

			const bool bIsRDGResource =
				IsRDGResourceReferenceShaderParameterType(BaseType);

			const bool bIsVariableNativeType = Member.IsVariableNativeType();

			checkf(BaseType != UBMT_BOOL, TEXT("Should have failed in FShaderParametersMetadata::InitializeLayout()"));

			if (BaseType == UBMT_INCLUDED_STRUCT)
			{
				checkf(!bIsArray, TEXT("Array of included structure is impossible."));
				Bind(
					*Member.GetStructMetadata(),
					/* MemberPrefix = */ MemberPrefix,
					/* GeneralByteOffset = */ ByteOffset);
				continue;
			}
			else if (BaseType == UBMT_NESTED_STRUCT && bIsArray)
			{
				const FShaderParametersMetadata* ChildStruct = Member.GetStructMetadata();
				uint32 StructSize = ChildStruct->GetSize();
				for (uint32 ArrayElementId = 0; ArrayElementId < (bIsArray ? ArraySize : 1u); ArrayElementId++)
				{
					FString NewPrefix = FString::Printf(TEXT("%s%s_%d_"), MemberPrefix, Member.GetName(), ArrayElementId);
					Bind(
						*ChildStruct,
						/* MemberPrefix = */ *NewPrefix,
						/* GeneralByteOffset = */ ByteOffset + ArrayElementId * StructSize);
				}
				continue;
			}
			else if (BaseType == UBMT_NESTED_STRUCT && !bIsArray)
			{
				FString NewPrefix = FString::Printf(TEXT("%s%s_"), MemberPrefix, Member.GetName());
				Bind(
					*Member.GetStructMetadata(),
					/* MemberPrefix = */ *NewPrefix,
					/* GeneralByteOffset = */ ByteOffset);
				continue;
			}
			else if (BaseType == UBMT_REFERENCED_STRUCT || BaseType == UBMT_RDG_UNIFORM_BUFFER)
			{
				checkf(!bIsArray, TEXT("Array of referenced structure is not supported, because the structure is globally uniquely named."));
				// The member name of a globally referenced struct is the not name on the struct.
				ShaderBindingName = Member.GetStructMetadata()->GetShaderVariableName();
			}
			else if (bUseRootShaderParameters && bIsVariableNativeType)
			{
				// Constants are stored in the root shader parameter cbuffer when bUseRootShaderParameters == true.
				continue;
			}

			const bool bIsResourceArray = bIsArray && (bIsRHIResource || bIsRDGResource);

			for (uint32 ArrayElementId = 0; ArrayElementId < (bIsResourceArray ? ArraySize : 1u); ArrayElementId++)
			{
				FString ElementShaderBindingName;
				if (bIsResourceArray)
				{
					#if 0 // HLSLCC does not support array of resources.
						ElementShaderBindingName = FString::Printf(TEXT("%s[%d]"), *ShaderBindingName, ArrayElementId);
					#else
						ElementShaderBindingName = FString::Printf(TEXT("%s_%d"), *ShaderBindingName, ArrayElementId);
					#endif
				}
				else
				{
					ElementShaderBindingName = ShaderBindingName;
				}

				if (ShaderGlobalScopeBindings.Contains(ElementShaderBindingName))
				{
					UE_LOGF(LogShaders, Fatal, "%ls can't bind shader parameter %ls, because it has already be bound by %ls.", *CppName, *ElementShaderBindingName, **ShaderGlobalScopeBindings.Find(ShaderBindingName));
				}

				TOptional<FParameterAllocation> ParameterAllocation = ParametersMap->FindParameterAllocation(ElementShaderBindingName);
				if (!ParameterAllocation.IsSet())
				{
					continue;
				}

				ShaderGlobalScopeBindings.Add(ElementShaderBindingName, CppName);

				if (bIsVariableNativeType)
				{
					checkf(ArrayElementId == 0, TEXT("The entire array should be bound instead for RHI parameter submission performance."));
					uint32 ByteSize = Member.GetMemberSize();

					FShaderParameterBindings::FParameter Parameter;
					Parameter.BufferIndex = ParameterAllocation->BufferIndex;
					Parameter.BaseIndex = ParameterAllocation->BaseIndex;
					Parameter.ByteOffset = ByteOffset;
					Parameter.ByteSize = ParameterAllocation->Size;

					if (uint32(ParameterAllocation->Size) > ByteSize)
					{
						UE_LOGF(LogShaders, Fatal, "The size required to bind shader %ls's (Permutation Id %d) struct %ls parameter %ls is %i bytes, smaller than %ls's %i bytes.",
							Shader->GetTypeUnfrozen()->GetName(), PermutationId, StructMetaData.GetStructTypeName(),
							*ElementShaderBindingName, ParameterAllocation->Size, *CppName, ByteSize);
					}

					Bindings->Parameters.Add(Parameter);
				}
				else if (BaseType == UBMT_REFERENCED_STRUCT || BaseType == UBMT_RDG_UNIFORM_BUFFER)
				{
					check(!bIsArray);
					FShaderParameterBindings::FParameterStructReference Parameter;
					Parameter.BufferIndex = ParameterAllocation->BufferIndex;
					Parameter.ByteOffset = ByteOffset;

					if (BaseType == UBMT_REFERENCED_STRUCT)
					{
						Bindings->ParameterReferences.Add(Parameter);
					}
					else
					{
						Bindings->GraphUniformBuffers.Add(Parameter);
					}
				}
				else if (bIsRHIResource || bIsRDGResource)
				{
					if (IsParameterBindless(ParameterAllocation->Type) || ParameterAllocation->Type == EShaderParameterType::DescriptorRange)
					{
						FShaderParameterBindings::FBindlessResourceParameter Parameter;
						Parameter.GlobalConstantOffset = ParameterAllocation->BaseIndex;
						Parameter.ByteOffset = ByteOffset + ArrayElementId * SHADER_PARAMETER_POINTER_ALIGNMENT;
						Parameter.BaseType = BaseType;

						Bindings->BindlessResourceParameters.Add(Parameter);
					}
					else
					{
						checkf(ParameterAllocation->BaseIndex < 256, TEXT("BaseIndex does not fit into uint8. Change FResourceParameter::BaseIndex type to uint16"));

						FShaderParameterBindings::FResourceParameter Parameter;
						Parameter.BaseIndex = (uint8)ParameterAllocation->BaseIndex;
						Parameter.ByteOffset = ByteOffset + ArrayElementId * SHADER_PARAMETER_POINTER_ALIGNMENT;
						Parameter.BaseType = BaseType;

						if (ParameterAllocation->Size != 1)
						{
							UE_LOGF(LogShaders, Fatal,
								"Error with shader %ls's (Permutation Id %d) parameter %ls is %i bytes, cpp name = %ls." "The shader compiler should give precisely which elements of an array did not get compiled out, " "for optimal automatic render graph pass dependency with ClearUnusedGraphResources().",
								Shader->GetTypeUnfrozen()->GetName(), PermutationId,
								*ElementShaderBindingName, ParameterAllocation->Size, *CppName);
						}

						Bindings->ResourceParameters.Add(Parameter);
					}
				}
				else
				{
					checkf(0, TEXT("Unexpected base type for a shader parameter struct member."));
				}
			}
		}
	}
};

void FShaderParameterBindings::BindForLegacyShaderParameters(const FShader* Shader, int32 PermutationId, const FShaderParameterMap& ParametersMap, const FShaderParametersMetadata& StructMetaData, bool bShouldBindEverything)
{
	const FShaderType* Type = Shader->GetTypeUnfrozen();
	checkf(StructMetaData.GetSize() < (1 << (sizeof(uint16) * 8)), TEXT("Shader parameter structure can only have a size < 65536 bytes."));
	check(this == &Shader->Bindings);
	
	switch (Type->GetFrequency())
	{
#if PLATFORM_SUPPORTS_MESH_SHADERS
	case SF_Mesh:
	case SF_Amplification:
#endif
	case SF_Vertex:
	case SF_Pixel:
	case SF_Geometry:
	case SF_Compute:
	case SF_WorkGraphComputeNode:
		break;
	default:
		checkf(0, TEXT("Invalid shader frequency for this shader binding technique."));
		break;
	}

	FShaderParameterStructBindingContext BindingContext;
	BindingContext.Shader = Shader;
	BindingContext.PermutationId = PermutationId;
	BindingContext.Bindings = this;
	BindingContext.ParametersMap = &ParametersMap;

	// When using a stable root constant buffer, can submit all shader parameters in one RHISetShaderParameter()
	{
		const TCHAR* ShaderBindingName = FShaderParametersMetadata::kRootUniformBufferBindingName;
		BindingContext.bUseRootShaderParameters = ParametersMap.ContainsParameterAllocation(ShaderBindingName);

		if (BindingContext.bUseRootShaderParameters)
		{
			const FParameterAllocation& Allocation = ParametersMap.ParameterMap.FindChecked(ShaderBindingName);

			check(int32(Allocation.BufferIndex) == FShaderParametersMetadata::kRootCBufferBindingIndex);
			check(uint32(Allocation.BaseIndex + Allocation.Size) <= StructMetaData.GetSize());
			check(Allocation.Type == EShaderParameterType::LooseData);

			FShaderParameterBindings::FParameter Parameter;
			Parameter.BufferIndex = Allocation.BufferIndex;
			Parameter.BaseIndex = Allocation.BaseIndex;
			Parameter.ByteOffset = Allocation.BaseIndex;
			Parameter.ByteSize = Allocation.Size;

			Parameters.Add(Parameter);

			BindingContext.ShaderGlobalScopeBindings.Add(ShaderBindingName, StructMetaData.GetStructTypeName());
			Allocation.bBound = true;
		}
	}

	BindingContext.Bind(
		StructMetaData,
		/* MemberPrefix = */ TEXT(""),
		/* ByteOffset = */ 0);

	StructureLayoutHash = StructMetaData.GetLayoutHash();
	RootParameterBufferIndex = kInvalidBufferIndex;

	TArray<FString> AllParameterNames;
	ParametersMap.GetAllParameterNames(AllParameterNames);
	if (bShouldBindEverything && BindingContext.ShaderGlobalScopeBindings.Num() != AllParameterNames.Num())
	{
		FString ErrorString = FString::Printf(
			TEXT("Shader %s, permutation %d has unbound parameters not represented in the parameter struct:"), Type->GetName(), PermutationId);

		for (const FString& GlobalParameterName : AllParameterNames)
		{
			if (!BindingContext.ShaderGlobalScopeBindings.Contains(GlobalParameterName))
			{
				ErrorString += FString::Printf(TEXT("\n  %s"), *GlobalParameterName);
			}
		}

		ensureMsgf(false, TEXT("%s"), *ErrorString);
	}
}

void FShaderParameterBindings::BindForRootShaderParameters(const FShader* Shader, int32 PermutationId, const FShaderParameterMap& ParametersMap)
{
	const FShaderType* Type = Shader->GetTypeUnfrozen();
	check(this == &Shader->Bindings);
	check(Type->GetRootParametersMetadata());

	const FShaderParametersMetadata& StructMetaData = *Type->GetRootParametersMetadata();
	checkf(StructMetaData.GetSize() < (1 << (sizeof(uint16) * 8)), TEXT("Shader parameter structure can only have a size < 65536 bytes."));

	switch (Type->GetFrequency())
	{
	case SF_RayGen:
	case SF_RayMiss:
	case SF_RayHitGroup:
	case SF_RayCallable:
		break;
	default:
		checkf(0, TEXT("Invalid shader frequency for this shader binding technic."));
		break;
	}

	FShaderParameterStructBindingContext BindingContext;
	BindingContext.Shader = Shader;
	BindingContext.PermutationId = PermutationId;
	BindingContext.Bindings = this;
	BindingContext.ParametersMap = &ParametersMap;
	BindingContext.bUseRootShaderParameters = true;
	BindingContext.Bind(
		StructMetaData,
		/* MemberPrefix = */ TEXT(""),
		/* ByteOffset = */ 0);

	StructureLayoutHash = StructMetaData.GetLayoutHash();

	// Binds the uniform buffer that contains the root shader parameters.
	{
		const TCHAR* ShaderBindingName = FShaderParametersMetadata::kRootUniformBufferBindingName;
		if (TOptional<FParameterAllocation> Parameter = ParametersMap.FindParameterAllocation(ShaderBindingName))
		{
			BindingContext.ShaderGlobalScopeBindings.Add(ShaderBindingName, ShaderBindingName);
			RootParameterBufferIndex = Parameter->BufferIndex;
		}
		else
		{
			check(RootParameterBufferIndex == FShaderParameterBindings::kInvalidBufferIndex);
		}
	}

	TArray<FString> AllParameterNames;
	ParametersMap.GetAllParameterNames(AllParameterNames);
	if (BindingContext.ShaderGlobalScopeBindings.Num() != AllParameterNames.Num())
	{
		FString ErrorString = FString::Printf(
			TEXT("Shader %s, permutation %d has unbound parameters not represented in the parameter struct:"),
			Type->GetName(), PermutationId);

		for (const FString& GlobalParameterName : AllParameterNames)
		{
			if (!BindingContext.ShaderGlobalScopeBindings.Contains(GlobalParameterName))
			{
				ErrorString += FString::Printf(TEXT("\n  %s"), *GlobalParameterName);
			}
		}

		UE_LOGF(LogShaders, Fatal, "%ls", *ErrorString);
	}
}

bool FRenderTargetBinding::Validate() const
{
	if (!Texture)
	{
		checkf(LoadAction == ERenderTargetLoadAction::ENoAction,
			TEXT("Can't have a load action when no texture is bound."));

		checkf(!ResolveTexture, TEXT("Can't have a resolve texture when no render target texture is bound."));
	}
	
	return true;
}

bool FDepthStencilBinding::Validate() const
{
	if (Texture)
	{
		EPixelFormat PixelFormat = Texture->Desc.Format;
		const TCHAR* FormatString = GetPixelFormatString(PixelFormat);

		bool bIsDepthFormat = PixelFormat == PF_DepthStencil || PixelFormat == PF_ShadowDepth || PixelFormat == PF_D24 || PixelFormat == PF_R32_FLOAT;
		checkf(bIsDepthFormat,
			TEXT("Can't bind texture %s as a depth stencil because its pixel format is %s."),
			Texture->Name, FormatString);
		
		checkf(DepthStencilAccess != FExclusiveDepthStencil::DepthNop_StencilNop,
			TEXT("Texture %s is bound but both depth / stencil are set to no-op."),
			Texture->Name);

		bool bHasStencil = PixelFormat == PF_DepthStencil;
		if (!bHasStencil)
		{
			checkf(StencilLoadAction == ERenderTargetLoadAction::ENoAction,
				TEXT("Unable to load stencil of texture %s that have a pixel format %s that does not support stencil."),
				Texture->Name, FormatString);
		
			checkf(!DepthStencilAccess.IsUsingStencil(),
				TEXT("Unable to have stencil access on texture %s that have a pixel format %s that does not support stencil."),
				Texture->Name, FormatString);
		}

		bool bReadDepth = DepthStencilAccess.IsUsingDepth() && !DepthStencilAccess.IsDepthWrite();
		bool bReadStencil = DepthStencilAccess.IsUsingStencil() && !DepthStencilAccess.IsStencilWrite();

		checkf(!(bReadDepth && DepthLoadAction != ERenderTargetLoadAction::ELoad),
			TEXT("Depth read access without depth load action on texture %s."),
			Texture->Name);

		checkf(!(bReadStencil && StencilLoadAction != ERenderTargetLoadAction::ELoad),
			TEXT("Stencil read access without stencil load action on texture %s."),
			Texture->Name);
	}
	else
	{
		checkf(!ResolveTexture,
			TEXT("Can't have a depth stencil resolve texture when no depth stencil target texture is bound."));
		checkf(DepthLoadAction == ERenderTargetLoadAction::ENoAction,
			TEXT("Can't have a depth load action when no texture is bound."));
		checkf(StencilLoadAction == ERenderTargetLoadAction::ENoAction,
			TEXT("Can't have a stencil load action when no texture is bound."));
		checkf(DepthStencilAccess == FExclusiveDepthStencil::DepthNop_StencilNop,
			TEXT("Can't have a depth stencil access when no texture is bound."));
	}

	return true;
}

void EmitNullShaderParameterFatalError(const TShaderRef<FShader>& Shader, const FShaderParametersMetadata* ParametersMetadata, uint16 MemberOffset)
{
	FString MemberName = ParametersMetadata->GetFullMemberCodeName(MemberOffset);

	const TCHAR* ShaderClassName = Shader.GetType()->GetName();

	UE_LOGF(LogShaders, Fatal,
		"%ls's required shader parameter %ls::%ls was not set.",
		ShaderClassName,
		ParametersMetadata->GetStructTypeName(),
		*MemberName);
}

void EmitIncompatibleShaderParameterFatalError(const TShaderRef<FShader>& Shader, const FShaderParametersMetadata* ParametersMetadata, uint16 MemberOffset)
{
	const FString MemberName = ParametersMetadata->GetFullMemberCodeName(MemberOffset);

	const TCHAR* ShaderClassName = Shader.GetType()->GetName();

	UE_LOG(LogShaders, Fatal,
		TEXT("%s's required shader parameter %s::%s is not compatible with shader stage."),
		ShaderClassName,
		ParametersMetadata->GetStructTypeName(),
		*MemberName);
}

/// Utility class for reading shader parameters out of the data blob passed in.
struct FShaderParameterReader
{
	FShaderParameterReader() = delete;
	FShaderParameterReader(const void* InData, uint64 InDataSize) : Data(reinterpret_cast<const uint8*>(InData)), DataSize(InDataSize) { }
	FShaderParameterReader(TConstArrayView<uint8> InData) : Data(reinterpret_cast<const uint8*>(InData.GetData())), DataSize(InData.Num()) { }

	template<typename TParameterIn>
	const void* GetRawPointer(const TParameterIn& InParameter) const
	{
		checkSlow(InParameter.ByteOffset <= DataSize);
		return (Data + InParameter.ByteOffset);
	}

	template<typename TParameterOut, typename TParameterIn>
	const TParameterOut& Read(const TParameterIn& InParameter) const
	{
		checkSlow(InParameter.ByteOffset + sizeof(TParameterOut) <= DataSize);
		return *reinterpret_cast<const TParameterOut*>(Data + InParameter.ByteOffset);
	}

	const uint8* Data;
	const uint64 DataSize;
};

#if DO_CHECK

void ValidateShaderParameters(const TShaderRef<FShader>& Shader, const FShaderParametersMetadata* ParametersMetadata, const void* ParametersData, const FResourceExclusionArrayView ExclusionList)
{
	const FShaderParameterBindings& Bindings = Shader->Bindings;

	checkf(
		Bindings.StructureLayoutHash == ParametersMetadata->GetLayoutHash(),
		TEXT("Shader %s's parameter structure has changed without recompilation of the shader"),
		Shader.GetType()->GetName());

	const FShaderParameterReader Reader(ParametersData, ParametersMetadata->GetSize());

	const TCHAR* ShaderClassName = Shader.GetType()->GetName();
	const TCHAR* ShaderParameterStructName = ParametersMetadata->GetStructTypeName();

	FRHIShader* RHIShader = Shader.GetRHIShaderBase(Shader->GetFrequency());

	for (const FShaderParameterBindings::FResourceParameter& Parameter : Bindings.ResourceParameters)
	{
		const EUniformBufferBaseType BaseType = static_cast<EUniformBufferBaseType>(Parameter.BaseType);
		switch (BaseType)
		{
			case UBMT_TEXTURE:
			case UBMT_SRV:
			case UBMT_UAV:
			case UBMT_SAMPLER:
			case UBMT_RESOURCE_COLLECTION:
			{
				const FRHIResource* Resource = Reader.Read<const FRHIResource*>(Parameter);
				if (!Resource && !ExclusionList.Contains(Parameter.BaseIndex))
				{
					EmitNullShaderParameterFatalError(Shader, ParametersMetadata, Parameter.ByteOffset);
				}
				break;
			}
			case UBMT_RDG_TEXTURE:
			{
				const FRDGTexture* GraphTexture = Reader.Read<const FRDGTexture*>(Parameter);
				if (!GraphTexture && !ExclusionList.Contains(Parameter.BaseIndex))
				{
					EmitNullShaderParameterFatalError(Shader, ParametersMetadata, Parameter.ByteOffset);
				}
				break;
			}
			case UBMT_RDG_TEXTURE_SRV:
			{
				const FRDGResource* GraphResource = Reader.Read<const FRDGResource*>(Parameter);

				if (GRHIGlobals.SupportsDedicatedSRVGraphicsNonPixelAccess && IsValidPreRasterizationFrequency(Shader->GetFrequency()))
				{
					EmitIncompatibleShaderParameterFatalError(Shader, ParametersMetadata, Parameter.ByteOffset);
				}

				if (!GraphResource && !ExclusionList.Contains(Parameter.BaseIndex))
				{
					EmitNullShaderParameterFatalError(Shader, ParametersMetadata, Parameter.ByteOffset);
				}
				break;
			}
			case UBMT_RDG_TEXTURE_NON_PIXEL_SRV:
			case UBMT_RDG_BUFFER_SRV:
			case UBMT_RDG_TEXTURE_UAV:
			case UBMT_RDG_BUFFER_UAV:
			{
				const FRDGResource* GraphResource = Reader.Read<const FRDGResource*>(Parameter);

				if (!GraphResource && !ExclusionList.Contains(Parameter.BaseIndex))
				{
					EmitNullShaderParameterFatalError(Shader, ParametersMetadata, Parameter.ByteOffset);
				}
				break;
			}
			default:
				break;
		}
	}

	// Graph Uniform Buffers
	for (const FShaderParameterBindings::FParameterStructReference& Parameter : Bindings.GraphUniformBuffers)
	{
		const FRDGUniformBufferBinding& UniformBufferBinding = Reader.Read<FRDGUniformBufferBinding>(Parameter);
		if (!UniformBufferBinding)
		{
			EmitNullShaderParameterFatalError(Shader, ParametersMetadata, Parameter.ByteOffset);
		}
	}

	// Reference structures
	for (const FShaderParameterBindings::FParameterStructReference& Parameter : Bindings.ParameterReferences)
	{
		const FUniformBufferBinding& UniformBufferBinding = Reader.Read<FUniformBufferBinding>(Parameter);
		if (!UniformBufferBinding)
		{
			EmitNullShaderParameterFatalError(Shader, ParametersMetadata, Parameter.ByteOffset);
		}
	}
}

void ValidateShaderParameterResourcesRHI(const void* Contents, const FRHIUniformBufferLayout& Layout)
{
	for (int32 Index = 0, Count = Layout.Resources.Num(); Index < Count; ++Index)
	{
		const auto Parameter = Layout.Resources[Index];

		FRHIResource* Resource = GetShaderParameterResourceRHI(Contents, Parameter.MemberOffset, Parameter.MemberType);

		const bool bSRVorUAV =
			Parameter.MemberType == UBMT_SRV ||
			Parameter.MemberType == UBMT_UAV ||
			Parameter.MemberType == UBMT_RDG_TEXTURE_SRV ||
			Parameter.MemberType == UBMT_RDG_TEXTURE_UAV ||
			Parameter.MemberType == UBMT_RDG_TEXTURE_NON_PIXEL_SRV ||
			Parameter.MemberType == UBMT_RDG_BUFFER_SRV ||
			Parameter.MemberType == UBMT_RDG_BUFFER_UAV ||
			Parameter.MemberType == UBMT_RESOURCE_COLLECTION;

		// Allow null SRV's in uniform buffers for feature levels that don't support SRV's in shaders
		// Mobile does not set some UAVs to not break RDG pass auto-merging
		if (GMaxRHIFeatureLevel <= ERHIFeatureLevel::ES3_1 && bSRVorUAV)
		{
			continue;
		}

		checkf(Resource, TEXT("Null resource entry in uniform buffer parameters: %s.Resources[%u], ResourceType 0x%x."), *Layout.GetDebugName(), Index, Parameter.MemberType);
	}
}

#endif // DO_CHECK

FResourceExclusionArray GetSpecializationResourceExclusions(const TShaderRef<FShader>& Shader)
{
#if PLATFORM_SUPPORTS_SPECIALIZATION_CONSTANTS
	FRHIShader* RHIShader = Shader.GetRHIShaderBase(Shader->GetFrequency());
	if (RHIShader->HasSpecializationConstants())
	{
		FResourceExclusionArray ResourceExclusions;

		const uint32 SpecializationHash = GetTypeHash(Shader.GetSpecializationConstants());
		const FShaderParameterBindings& Bindings = Shader->Bindings;
		// Only one of the two following loops will fill ResourceExclusions (depending on if bindless is enabled)
		for (const FShaderParameterBindings::FResourceParameter& Parameter : Bindings.ResourceParameters)
		{
			// Check if this resource should be excluded from this combination of specialization constants
			if (RHIShader->IsExcludedFromSpecialization(Parameter.BaseIndex, SpecializationHash))
			{
				ResourceExclusions.Add(Parameter.BaseIndex);
			}
		}
		for (const FShaderParameterBindings::FBindlessResourceParameter& Parameter : Bindings.BindlessResourceParameters)
		{
			// Check if this resource should be excluded from this combination of specialization constants
			if (RHIShader->IsExcludedFromSpecialization(Parameter.GlobalConstantOffset, SpecializationHash))
			{
				ResourceExclusions.Add(Parameter.GlobalConstantOffset);
			}
		}

		return ResourceExclusions;
	}
#endif // PLATFORM_SUPPORTS_SPECIALIZATION_CONSTANTS
	return FResourceExclusionArray();
}

template<typename BindingParameterType>
FRHIShaderParameterResource ExtractShaderParameterResource(FShaderParameterReader Reader, const BindingParameterType& Parameter)
{
	const EUniformBufferBaseType BaseType = static_cast<EUniformBufferBaseType>(Parameter.BaseType);

	switch (BaseType)
	{
	case UBMT_TEXTURE:
	{
		FRHITexture* Texture = Reader.Read<FRHITexture*>(Parameter);
		checkSlow(Texture);
		return FRHIShaderParameterResource(Texture, GetParameterIndex(Parameter));
	}
	case UBMT_SRV:
	{
		FRHIShaderResourceView* ShaderResourceView = Reader.Read<FRHIShaderResourceView*>(Parameter);
		checkSlow(ShaderResourceView);
		return FRHIShaderParameterResource(ShaderResourceView, GetParameterIndex(Parameter));
	}
	case UBMT_UAV:
	{
		FRHIUnorderedAccessView* UnorderedAccessView = Reader.Read<FRHIUnorderedAccessView*>(Parameter);
		checkSlow(UnorderedAccessView);
		return FRHIShaderParameterResource(UnorderedAccessView, GetParameterIndex(Parameter));
	}
	case UBMT_SAMPLER:
	{
		FRHISamplerState* SamplerState = Reader.Read<FRHISamplerState*>(Parameter);
		checkSlow(SamplerState);
		return FRHIShaderParameterResource(SamplerState, GetParameterIndex(Parameter));
	}
	case UBMT_DESCRIPTOR_RANGE:
	{
		FRHIDescriptorRange* DescriptorRange = Reader.Read<FRHIDescriptorRange*>(Parameter);
		checkSlow(DescriptorRange);
		return FRHIShaderParameterResource(DescriptorRange, GetParameterIndex(Parameter));
	}
	case UBMT_RESOURCE_COLLECTION:
	{
		FRHIResourceCollection* Collection = Reader.Read<FRHIResourceCollection*>(Parameter);
		checkSlow(Collection);
		return FRHIShaderParameterResource(Collection, GetParameterIndex(Parameter));
	}
	case UBMT_RDG_TEXTURE:
	{
		FRDGTexture* RDGTexture = Reader.Read<FRDGTexture*>(Parameter);
		checkSlow(RDGTexture);
		RDGTexture->MarkResourceAsUsed();
		return FRHIShaderParameterResource(RDGTexture->GetRHI(), GetParameterIndex(Parameter));
	}
	case UBMT_RDG_TEXTURE_SRV:
	case UBMT_RDG_TEXTURE_NON_PIXEL_SRV:
	case UBMT_RDG_BUFFER_SRV:
	{
		FRDGShaderResourceView* RDGShaderResourceView = Reader.Read<FRDGShaderResourceView*>(Parameter);
		checkSlow(RDGShaderResourceView);
		RDGShaderResourceView->MarkResourceAsUsed();
		return FRHIShaderParameterResource(RDGShaderResourceView->GetRHI(), GetParameterIndex(Parameter));
	}
	case UBMT_RDG_TEXTURE_UAV:
	case UBMT_RDG_BUFFER_UAV:
	{
		FRDGUnorderedAccessView* RDGUnorderedAccessView = Reader.Read<FRDGUnorderedAccessView*>(Parameter);
		checkSlow(RDGUnorderedAccessView);
		RDGUnorderedAccessView->MarkResourceAsUsed();
		return FRHIShaderParameterResource(RDGUnorderedAccessView->GetRHI(), GetParameterIndex(Parameter));
	}
	default:
		checkf(false, TEXT("Unhandled resource type?"));
		return FRHIShaderParameterResource();
	}
}

template<typename TParameterFunction>
static void IterateShaderParameterMembersInternal(
	const FShaderParametersMetadata& ParametersMetadata,
	uint16 ByteOffset,
	TParameterFunction Lambda)
{
	for (const FShaderParametersMetadata::FMember& Member : ParametersMetadata.GetMembers())
	{
		EUniformBufferBaseType BaseType = Member.GetBaseType();
		const uint16 MemberOffset = ByteOffset + uint16(Member.GetOffset());
		const uint32 NumElements = Member.GetNumElements();

		if (BaseType == UBMT_INCLUDED_STRUCT)
		{
			check(NumElements == 0);
			IterateShaderParameterMembersInternal(*Member.GetStructMetadata(), MemberOffset, Lambda);
		}
		else if (BaseType == UBMT_NESTED_STRUCT && NumElements == 0)
		{
			IterateShaderParameterMembersInternal(*Member.GetStructMetadata(), MemberOffset, Lambda);
		}
		else if (BaseType == UBMT_NESTED_STRUCT && NumElements > 0)
		{
			const FShaderParametersMetadata& NewParametersMetadata = *Member.GetStructMetadata();
			for (uint32 ArrayElementId = 0; ArrayElementId < NumElements; ArrayElementId++)
			{
				uint16 NewStructOffset = MemberOffset + ArrayElementId * NewParametersMetadata.GetSize();
				IterateShaderParameterMembersInternal(NewParametersMetadata, NewStructOffset, Lambda);
			}
		}
		else
		{
			const bool bParametersAreExpanded = NumElements > 0 && (IsShaderParameterTypeRHIResource(BaseType) || IsRDGResourceReferenceShaderParameterType(BaseType));
			if (bParametersAreExpanded)
			{
				const uint16 ElementSize = SHADER_PARAMETER_POINTER_ALIGNMENT;

				for (uint32 Index = 0; Index < NumElements; Index++)
				{
					Lambda(BaseType, MemberOffset + Index * ElementSize);
				}
			}
			else
			{
				Lambda(BaseType, MemberOffset);
			}
		}
	}
}

static bool DoesBaseTypeSupportBindless(EUniformBufferBaseType Type)
{
	switch (Type)
	{
	case UBMT_TEXTURE:
	case UBMT_SRV:
	case UBMT_UAV:
	case UBMT_SAMPLER:
	case UBMT_RDG_TEXTURE:
	case UBMT_RDG_TEXTURE_SRV:
	case UBMT_RDG_TEXTURE_NON_PIXEL_SRV:
	case UBMT_RDG_TEXTURE_UAV:
	case UBMT_RDG_BUFFER_SRV:
	case UBMT_RDG_BUFFER_UAV:
	case UBMT_RESOURCE_COLLECTION:
	case UBMT_DESCRIPTOR_RANGE:
		return true;
	}
	return false;
}

void SetAllShaderParametersAsBindless(
	FRHIBatchedShaderParameters& BatchedParameters,
	const FShaderParametersMetadata* ParametersMetadata,
	const void* InParametersData)
{
	TConstArrayView<uint8> ParametersData((const uint8*)InParametersData, ParametersMetadata->GetSize());

	const FShaderParameterReader Reader(ParametersData);

	uint16 ByteOffset = 0;
	IterateShaderParameterMembersInternal(*ParametersMetadata, ByteOffset, [&BatchedParameters, Reader](EUniformBufferBaseType BaseType, uint16 ByteOffset)
		{
			if (DoesBaseTypeSupportBindless(BaseType))
			{
				FShaderParameterBindings::FBindlessResourceParameter Parameter{};
				Parameter.GlobalConstantOffset = ByteOffset;
				Parameter.ByteOffset = ByteOffset;
				Parameter.BaseType = BaseType;

				// Make sure the resource wasn't set to null
				if (Reader.Read<void*>(Parameter))
				{
					const FRHIShaderParameterResource ShaderParameterResource = ExtractShaderParameterResource(Reader, Parameter);
					BatchedParameters.AddBindlessParameter(ShaderParameterResource);
				}
			}
		});
}

static void ExtractShaderParameterResources(
	TArray<FRHIShaderParameterResource>& OutResourceParameters,
	TArray<FRHIShaderParameterResource>& OutBindlessParameters,
	const FShaderParameterBindings& Bindings,
	TConstArrayView<uint8> ParametersData,
	EUniformBufferBindingFlags UniformBufferBindingFlags)
{
	const FShaderParameterReader Reader(ParametersData);

#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
	if (int32 NumBindings = Bindings.BindlessResourceParameters.Num())
	{
		OutBindlessParameters.Reserve(NumBindings);

		for (const FShaderParameterBindings::FBindlessResourceParameter& Parameter : Bindings.BindlessResourceParameters)
		{
			const FRHIShaderParameterResource ShaderParameterResource = ExtractShaderParameterResource(Reader, Parameter);
			OutBindlessParameters.Emplace(ShaderParameterResource);
		}
	}
#endif

	const int32 NumBindings = Bindings.ResourceParameters.Num() + Bindings.GraphUniformBuffers.Num() + Bindings.ParameterReferences.Num();

	if (NumBindings)
	{
		OutResourceParameters.Reserve(NumBindings);

		for (const FShaderParameterBindings::FResourceParameter& Parameter : Bindings.ResourceParameters)
		{
			const FRHIShaderParameterResource ShaderParameterResource = ExtractShaderParameterResource(Reader, Parameter);
			OutResourceParameters.Emplace(ShaderParameterResource);
		}

		for (const FShaderParameterBindings::FParameterStructReference& Parameter : Bindings.GraphUniformBuffers)
		{
			const FRDGUniformBufferBinding& UniformBufferBinding = Reader.Read<FRDGUniformBufferBinding>(Parameter);
			if (EnumHasAnyFlags(UniformBufferBinding.GetBindingFlags(), UniformBufferBindingFlags))
			{
				UniformBufferBinding->MarkResourceAsUsed();

				OutResourceParameters.Emplace(UniformBufferBinding->GetRHI(), GetParameterIndex(Parameter));
			}
		}

		for (const FShaderParameterBindings::FParameterStructReference& Parameter : Bindings.ParameterReferences)
		{
			const FUniformBufferBinding& UniformBufferBinding = Reader.Read<FUniformBufferBinding>(Parameter);
			if (EnumHasAnyFlags(UniformBufferBinding.GetBindingFlags(), UniformBufferBindingFlags))
			{
				OutResourceParameters.Emplace(UniformBufferBinding.GetUniformBuffer(), GetParameterIndex(Parameter));
			}
		}
	}
}

static void ExtractShaderParameterResources(
	FRHIBatchedShaderParameters& BatchedParameters,
	const FShaderParameterBindings& Bindings,
	TConstArrayView<uint8> ParametersData,
	EUniformBufferBindingFlags UniformBufferBindingFlags,
	const FResourceExclusionArrayView ResourceExclusions)
{
	const FShaderParameterReader Reader(ParametersData);

#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
	if (Bindings.BindlessResourceParameters.Num())
	{
		for (const FShaderParameterBindings::FBindlessResourceParameter& Parameter : Bindings.BindlessResourceParameters)
		{
			if (ResourceExclusions.Contains(Parameter.GlobalConstantOffset))
			{
				continue;
			}

			const FRHIShaderParameterResource ShaderParameterResource = ExtractShaderParameterResource(Reader, Parameter);
			BatchedParameters.AddBindlessParameter(ShaderParameterResource);
		}
	}
#endif

	const int32 NumBindings = Bindings.ResourceParameters.Num() + Bindings.GraphUniformBuffers.Num() + Bindings.ParameterReferences.Num();

	if (NumBindings)
	{
		for (const FShaderParameterBindings::FResourceParameter& Parameter : Bindings.ResourceParameters)
		{
			if (ResourceExclusions.Contains(Parameter.BaseIndex))
			{
				continue;
			}

			const FRHIShaderParameterResource ShaderParameterResource = ExtractShaderParameterResource(Reader, Parameter);
			BatchedParameters.AddResourceParameter(ShaderParameterResource);
		}

		for (const FShaderParameterBindings::FParameterStructReference& Parameter : Bindings.GraphUniformBuffers)
		{
			const FRDGUniformBufferBinding& UniformBufferBinding = Reader.Read<FRDGUniformBufferBinding>(Parameter);
			if (EnumHasAnyFlags(UniformBufferBinding.GetBindingFlags(), UniformBufferBindingFlags))
			{
				UniformBufferBinding->MarkResourceAsUsed();

				BatchedParameters.AddResourceParameter(UniformBufferBinding->GetRHI(), GetParameterIndex(Parameter));
			}
		}

		for (const FShaderParameterBindings::FParameterStructReference& Parameter : Bindings.ParameterReferences)
		{
			const FUniformBufferBinding& UniformBufferBinding = Reader.Read<FUniformBufferBinding>(Parameter);
			if (EnumHasAnyFlags(UniformBufferBinding.GetBindingFlags(), UniformBufferBindingFlags))
			{
				BatchedParameters.AddResourceParameter(UniformBufferBinding.GetUniformBuffer(), GetParameterIndex(Parameter));
			}
		}
	}
}

/** Set batched parameters from a parameters struct. */
void SetShaderParameters(
	FRHIBatchedShaderParameters& BatchedParameters,
	const FShaderParameterBindings& Bindings,
	const FShaderParametersMetadata* ParametersMetadata,
	const void* InParametersData,
	const FResourceExclusionArrayView ResourceExclusions)
{
	TConstArrayView<uint8> FullParametersData((const uint8*)InParametersData, ParametersMetadata->GetSize());

	for (const FShaderParameterBindings::FParameter& Parameter : Bindings.Parameters)
	{
		BatchedParameters.SetShaderParameter(Parameter.BufferIndex, Parameter.BaseIndex, Parameter.ByteSize, FullParametersData.GetData() + Parameter.ByteOffset);
	}

	ExtractShaderParameterResources(BatchedParameters, Bindings, FullParametersData, EUniformBufferBindingFlags::Shader, ResourceExclusions);
}

/** Set shader's parameters from its parameters struct. */
template<typename TRHICmdList, typename TShaderRHI>
inline void SetShaderParametersInternal(
	TRHICmdList& RHICmdList,
	TShaderRHI* ShaderRHI,
	const FShaderParameterBindings& Bindings,
	const FShaderParametersMetadata* ParametersMetadata,
	const void* InParametersData,
	const FResourceExclusionArrayView ResourceExclusions)
{
	checkf(Bindings.RootParameterBufferIndex == FShaderParameterBindings::kInvalidBufferIndex, TEXT("Can't use SetShaderParameters() for root parameter buffer index."));
	checkf(!IsRayTracingShaderFrequency(ShaderRHI->GetFrequency()), TEXT("Can't use SetShaderParameters() with RHICmdList parameter for ray tracing shaders."));

	FRHIBatchedShaderParameters& ShaderParameters = RHICmdList.GetScratchShaderParameters();
	SetShaderParameters(ShaderParameters, Bindings, ParametersMetadata, InParametersData, ResourceExclusions);
	RHICmdList.SetBatchedShaderParameters(ShaderRHI, ShaderParameters);
}

void SetShaderParameters(
	FRHIComputeCommandList& RHICmdList,
	FRHIComputeShader* ShaderRHI,
	const FShaderParameterBindings& Bindings,
	const FShaderParametersMetadata* ParametersMetadata,
	const void* ParametersData,
	const FResourceExclusionArrayView ResourceExclusions)
{
	SetShaderParametersInternal(RHICmdList, ShaderRHI, Bindings, ParametersMetadata, ParametersData, ResourceExclusions);
}

void SetShaderParameters(
	FRHICommandList& RHICmdList,
	FRHIGraphicsShader* ShaderRHI,
	const FShaderParameterBindings& Bindings,
	const FShaderParametersMetadata* ParametersMetadata,
	const void* ParametersData,
	const FResourceExclusionArrayView ResourceExclusions)
{
	SetShaderParametersInternal(RHICmdList, ShaderRHI, Bindings, ParametersMetadata, ParametersData, ResourceExclusions);
}

void SetShaderParameters(
	FRHICommandList& RHICmdList,
	FRHIComputeShader* ShaderRHI,
	const FShaderParameterBindings& Bindings,
	const FShaderParametersMetadata* ParametersMetadata,
	const void* ParametersData,
	const FResourceExclusionArrayView ResourceExclusions)
{
	SetShaderParametersInternal(RHICmdList, ShaderRHI, Bindings, ParametersMetadata, ParametersData, ResourceExclusions);
}

#if RHI_RAYTRACING
PRAGMA_DISABLE_DEPRECATION_WARNINGS // Allow FRayTracingShaderBindingsWriter
void SetShaderParameters(
	FRayTracingShaderBindingsWriter& RTBindingsWriter,
	const FShaderParameterBindings& Bindings,
	const FShaderParametersMetadata* ParametersMetadata,
	const void* ParametersData)
{
	const FShaderParameterReader Reader(ParametersData, ParametersMetadata->GetSize());

#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
	if (int32 NumBindings = Bindings.BindlessResourceParameters.Num())
	{
		for (const FShaderParameterBindings::FBindlessResourceParameter& Parameter : Bindings.BindlessResourceParameters)
		{
			const FRHIShaderParameterResource ShaderParameterResource = ExtractShaderParameterResource(Reader, Parameter);
			RTBindingsWriter.AddBindlessParameter(ShaderParameterResource);
		}
	}
#endif // PLATFORM_SUPPORTS_BINDLESS_RENDERING

	for (const FShaderParameterBindings::FResourceParameter& Parameter : Bindings.ResourceParameters)
	{
		const EUniformBufferBaseType BaseType = static_cast<EUniformBufferBaseType>(Parameter.BaseType);
		switch (BaseType)
		{
		case UBMT_TEXTURE:
		{
			FRHITexture* Texture = Reader.Read<FRHITexture*>(Parameter);
			RTBindingsWriter.SetTexture(Parameter.BaseIndex, Texture);
		}
		break;
		case UBMT_SRV:
		{
			FRHIShaderResourceView* ShaderResourceView = Reader.Read<FRHIShaderResourceView*>(Parameter);
			RTBindingsWriter.SetSRV(Parameter.BaseIndex, ShaderResourceView);
		}
		break;
		case UBMT_UAV:
		{
			FRHIUnorderedAccessView* UnorderedAccessView = Reader.Read<FRHIUnorderedAccessView*>(Parameter);
			RTBindingsWriter.SetUAV(Parameter.BaseIndex, UnorderedAccessView);
		}
		break;
		case UBMT_SAMPLER:
		{
			FRHISamplerState* SamplerState = Reader.Read<FRHISamplerState*>(Parameter);
			RTBindingsWriter.SetSampler(Parameter.BaseIndex, SamplerState);
		}
		break;
		case UBMT_RDG_TEXTURE:
		{
			FRDGTexture* RDGTexture = Reader.Read<FRDGTexture*>(Parameter);
			checkSlow(RDGTexture);
			RDGTexture->MarkResourceAsUsed();
			RTBindingsWriter.SetTexture(Parameter.BaseIndex, RDGTexture->GetRHI());
		}
		break;
		case UBMT_RDG_TEXTURE_SRV:
		case UBMT_RDG_TEXTURE_NON_PIXEL_SRV:
		case UBMT_RDG_BUFFER_SRV:
		{
			FRDGShaderResourceView* RDGShaderResourceView = Reader.Read<FRDGShaderResourceView*>(Parameter);
			checkSlow(RDGShaderResourceView);
			RDGShaderResourceView->MarkResourceAsUsed();
			RTBindingsWriter.SetSRV(Parameter.BaseIndex, RDGShaderResourceView->GetRHI());
		}
		break;
		case UBMT_RDG_TEXTURE_UAV:
		case UBMT_RDG_BUFFER_UAV:
		{
			FRDGUnorderedAccessView* RDGUnorderedAccessView = Reader.Read<FRDGUnorderedAccessView*>(Parameter);
			checkSlow(RDGUnorderedAccessView);
			RDGUnorderedAccessView->MarkResourceAsUsed();
			RTBindingsWriter.SetUAV(Parameter.BaseIndex, RDGUnorderedAccessView->GetRHI());
		}
		break;
		default:
			checkf(false, TEXT("Unhandled resource type?"));
			break;
		}
	}

	// Graph Uniform Buffers
	for (const FShaderParameterBindings::FParameterStructReference& Parameter : Bindings.GraphUniformBuffers)
	{
		const FRDGUniformBufferBinding& UniformBufferBinding = Reader.Read<FRDGUniformBufferBinding>(Parameter);

		checkSlow(UniformBufferBinding);
		UniformBufferBinding->MarkResourceAsUsed();
		RTBindingsWriter.SetUniformBuffer(Parameter.BufferIndex, UniformBufferBinding->GetRHI());
	}

	// Referenced uniform buffers
	for (const FShaderParameterBindings::FParameterStructReference& Parameter : Bindings.ParameterReferences)
	{
		const FUniformBufferBinding& UniformBufferBinding = Reader.Read<FUniformBufferBinding>(Parameter);
		RTBindingsWriter.SetUniformBuffer(Parameter.BufferIndex, UniformBufferBinding.GetUniformBuffer());
	}

	// Root uniform buffer.
	if (Bindings.RootParameterBufferIndex != FShaderParameterBindings::kInvalidBufferIndex)
	{
		// Do not do any validation at some resources may have been removed from the structure because known to not be used by the shader.
		EUniformBufferValidation Validation = EUniformBufferValidation::None;

		RTBindingsWriter.RootUniformBuffer = RHICreateUniformBuffer(ParametersData, ParametersMetadata->GetLayoutPtr(), UniformBuffer_SingleDraw, Validation);
		RTBindingsWriter.SetUniformBuffer(Bindings.RootParameterBufferIndex, RTBindingsWriter.RootUniformBuffer);
	}
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

FRayTracingShaderBindings ConvertRayTracingShaderBindings(const FRHIBatchedShaderParameters& BatchedParameters)
{
	FRayTracingShaderBindings Result;

	// Use array views for bounds checking
	TArrayView<FRHITexture*> Textures = Result.Textures;
	TArrayView<FRHIShaderResourceView*> SRVs = Result.SRVs;
	TArrayView<FRHIUniformBuffer*> UniformBuffers = Result.UniformBuffers;
	TArrayView<FRHISamplerState*> Samplers = Result.Samplers;
	TArrayView<FRHIUnorderedAccessView*> UAVs = Result.UAVs;

	checkf(BatchedParameters.Parameters.IsEmpty(), TEXT("FRHIShaderParameter is not supported by FRayTracingShaderBindings"));

	for (const FRHIShaderParameterResource& It : BatchedParameters.ResourceParameters)
	{
		using EType = FRHIShaderParameterResource::EType;
		switch (It.Type)
		{
			case EType::Texture:
				Textures[It.Index] = static_cast<FRHITexture*>(It.Resource);
				break;
			case EType::ResourceView:
				SRVs[It.Index] = static_cast<FRHIShaderResourceView*>(It.Resource);
				break;
			case EType::UnorderedAccessView:
				UAVs[It.Index] = static_cast<FRHIUnorderedAccessView*>(It.Resource);
				break;
			case EType::Sampler:
				Samplers[It.Index] = static_cast<FRHISamplerState*>(It.Resource);
				break;
			case EType::UniformBuffer:
				UniformBuffers[It.Index] = static_cast<FRHIUniformBuffer*>(It.Resource);
				break;
			case EType::ResourceCollection:
				checkNoEntry(); // not supported
				break;
			case EType::DescriptorRange:
				checkNoEntry(); // not supported
				break;
			default:
				checkNoEntry();
		}
	}

	Result.BindlessParameters = BatchedParameters.BindlessParameters;

	return Result;
}

void SetRayTracingShaderParameters(
	FRHIBatchedShaderParameters& BatchedParameters,
	const FShaderParameterBindings& Bindings,
	const FShaderParametersMetadata* ParametersMetadata,
	const void* InParametersData)
{
	TConstArrayView<uint8> FullParametersData((const uint8*)InParametersData, ParametersMetadata->GetSize());

#if 0 // Loose parameter binding could be supported if the need arises, but we currently don't support it.

	for (const FShaderParameterBindings::FParameter& Parameter : Bindings.Parameters)
	{
		BatchedParameters.SetShaderParameter(Parameter.BufferIndex, Parameter.BaseIndex, Parameter.ByteSize, FullParametersData.GetData() + Parameter.ByteOffset);
	}

#else

	checkf(Bindings.Parameters.Num() == 0, TEXT("Ray tracing shader should use SHADER_USE_ROOT_PARAMETER_STRUCT() to passdown the cbuffer layout to the shader compiler."));

#endif

	const FResourceExclusionArrayView NoExclusions;  // Specialization constants not supported in ray tracing yet
	ExtractShaderParameterResources(BatchedParameters, Bindings, FullParametersData, EUniformBufferBindingFlags::StaticAndShader, NoExclusions);

	// Root uniform buffer.
	if (Bindings.RootParameterBufferIndex != FShaderParameterBindings::kInvalidBufferIndex)
	{
		// Do not do any validation at some resources may have been removed from the structure because known to not be used by the shader.
		EUniformBufferValidation Validation = EUniformBufferValidation::None;

		FUniformBufferRHIRef RootUniformBuffer = RHICreateUniformBuffer(InParametersData, ParametersMetadata->GetLayoutPtr(), UniformBuffer_SingleDraw, Validation);
		BatchedParameters.AddResourceParameter(RootUniformBuffer, Bindings.RootParameterBufferIndex);
	}

#if DO_CHECK
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS // Allow FRayTracingShaderBindingsWriter
		FRayTracingShaderBindingsWriter RTBindingsWriter;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS

		SetShaderParameters(RTBindingsWriter, Bindings, ParametersMetadata, InParametersData);

		FRayTracingShaderBindings ConvertedBindings = ConvertRayTracingShaderBindings(BatchedParameters);

		{
			for (int32 i = 0; i < UE_ARRAY_COUNT(ConvertedBindings.Textures); ++i)
			{
				check(RTBindingsWriter.Textures[i] == ConvertedBindings.Textures[i]);
			}

			for (int32 i = 0; i < UE_ARRAY_COUNT(ConvertedBindings.SRVs); ++i)
			{
				check(RTBindingsWriter.SRVs[i] == ConvertedBindings.SRVs[i]);
			}

			for (int32 i = 0; i < UE_ARRAY_COUNT(ConvertedBindings.UniformBuffers); ++i)
			{
				if (RTBindingsWriter.UniformBuffers[i])
				{
					check(ConvertedBindings.UniformBuffers[i]);
					check(RTBindingsWriter.UniformBuffers[i]->GetLayout() == ConvertedBindings.UniformBuffers[i]->GetLayout());
				}
				else
				{
					check(ConvertedBindings.UniformBuffers[i] == nullptr);
				}
			}

			for (int32 i = 0; i < UE_ARRAY_COUNT(ConvertedBindings.Samplers); ++i)
			{
				check(RTBindingsWriter.Samplers[i] == ConvertedBindings.Samplers[i]);
			}

			for (int32 i = 0; i < UE_ARRAY_COUNT(ConvertedBindings.UAVs); ++i)
			{
				check(RTBindingsWriter.UAVs[i] == ConvertedBindings.UAVs[i]);
			}

			check(RTBindingsWriter.BindlessParameters.Num() == ConvertedBindings.BindlessParameters.Num());
			for (int32 i = 0; i < ConvertedBindings.BindlessParameters.Num(); ++i)
			{
				check(RTBindingsWriter.BindlessParameters[i] == ConvertedBindings.BindlessParameters[i]);
			}
		}
	}
#endif // DO_CHECK
}


#endif // RHI_RAYTRACING
