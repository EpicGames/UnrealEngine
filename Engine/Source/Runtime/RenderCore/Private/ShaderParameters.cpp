// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ShaderParameters.cpp: Shader parameter implementation.
=============================================================================*/

#include "ShaderParameters.h"
#include "Containers/List.h"
#include "ShaderCore.h"
#include "Shader.h"
#include "ShaderParameterParser.h"
#include "VertexFactory.h"
#include "ShaderCodeLibrary.h"
#include "Misc/ScopeRWLock.h"
#include "Misc/ScopeLock.h"

IMPLEMENT_TYPE_LAYOUT(FShaderParameter);
IMPLEMENT_TYPE_LAYOUT(FShaderResourceParameter);
IMPLEMENT_TYPE_LAYOUT(FShaderUniformBufferParameter);
IMPLEMENT_TYPE_LAYOUT(FShaderUniformBufferMemberParameter);

static void FailureToBindNonOptionalParameter(const TCHAR* ParameterType, const TCHAR* ParameterName)
{
	if (!UE_LOG_ACTIVE(LogShaders, Log))
	{
		UE_LOGF(LogShaders, Fatal, "Failure to bind non-optional %ls %ls!  The parameter is either not present in the shader, or the shader compiler optimized it out.", ParameterType, ParameterName);
	}
	else
	{
		UE_LOGF(LogShaders, Log, "Failure to bind non-optional %ls %ls!  The parameter is either not present in the shader, or the shader compiler optimized it out.", ParameterType, ParameterName);

		// We use a non-Slate message box to avoid problem where we haven't compiled the shaders for Slate.
		FPlatformMisc::MessageBoxExt(EAppMsgType::Ok, *FText::Format(
			NSLOCTEXT("UnrealEd", "Error_FailedToBindShaderParameter", "Failure to bind non-optional shader parameter {0}! The parameter is either not present in the shader, or the shader compiler optimized it out. This will be an assert with LogShaders suppressed!"),
			FText::FromString(ParameterName)).ToString(), TEXT("Warning"));
	}
}

void FShaderParameter::Bind(const FShaderParameterMap& ParameterMap,const TCHAR* ParameterName,EShaderParameterFlags Flags)
{
	if (!ParameterMap.FindParameterAllocation(ParameterName,BufferIndex,BaseIndex,NumBytes) && Flags == SPF_Mandatory)
	{
		FailureToBindNonOptionalParameter(TEXT("shader parameter"), ParameterName);
	}
}

FArchive& operator<<(FArchive& Ar,FShaderParameter& P)
{
	uint16& PBufferIndex = P.BufferIndex;
	return Ar << P.BaseIndex << P.NumBytes << PBufferIndex;
}

void FShaderResourceParameter::Bind(const FShaderParameterMap& ParameterMap, const TCHAR* ParameterName, EShaderParameterFlags Flags)
{
	if (TOptional<FParameterAllocation> Allocation = ParameterMap.FindParameterAllocation(ParameterName))
	{
		BaseIndex = Allocation->BaseIndex;
		NumResources = Allocation->Size;
		Type = Allocation->Type;
	}
	else if (Flags == SPF_Mandatory)
	{
		FailureToBindNonOptionalParameter(TEXT("shader resource parameter"), ParameterName);
	}
}

FArchive& operator<<(FArchive& Ar,FShaderResourceParameter& P)
{
	return Ar << P.BaseIndex << P.NumResources;
}

void FShaderUniformBufferMemberParameter::Bind(const FShaderParameterMap& ParameterMap, const TCHAR* ParameterName)
{
	bIsBound = ParameterMap.ContainsParameterAllocation(ParameterName) ? 1 : 0;
}

FArchive& operator<<(FArchive& Ar, FShaderUniformBufferMemberParameter& P)
{
	return Ar << P.bIsBound;
}

#if WITH_EDITOR
void FShaderUniformBufferParameter::ModifyCompilationEnvironment(const TCHAR* ParameterName, const FShaderParametersMetadata& Struct, EShaderPlatform Platform, FShaderCompilerEnvironment& OutEnvironment)
{
	// Add the uniform buffer declaration to the compilation environment as an include: /Engine/Generated/UniformBuffers/<ParameterName>.usf
	const FString IncludeName = FString::Printf(TEXT("/Engine/Generated/UniformBuffers/%s.ush"), ParameterName);

	// if the name matches the struct's name, use the struct's cached version; otherwise, generate it now with the correct variable name.
	if (FCString::Strcmp(ParameterName, Struct.GetShaderVariableName()) != 0)
	{
		const FString Declaration = UE::ShaderParameters::CreateUniformBufferShaderDeclaration(ParameterName, Struct, nullptr);
		OutEnvironment.IncludeVirtualPathToContentsMap.Add(IncludeName, Declaration);
	}
	else
	{
		OutEnvironment.IncludeVirtualPathToContentsMap.Add(IncludeName, Struct.GetUniformBufferDeclaration());
	}

	FString& GeneratedUniformBuffersInclude = OutEnvironment.IncludeVirtualPathToContentsMap.FindOrAdd("/Engine/Generated/GeneratedUniformBuffers.ush");
	const FString Include = FString::Printf(TEXT("#include \"%s\"") HLSL_LINE_TERMINATOR, *IncludeName);

	GeneratedUniformBuffersInclude.Append(Include);
	Struct.AddResourceTableEntries(OutEnvironment.ResourceTableMap, OutEnvironment.UniformBufferMap);
}
#endif // WITH_EDITOR

void FShaderUniformBufferParameter::Bind(const FShaderParameterMap& ParameterMap,const TCHAR* ParameterName,EShaderParameterFlags Flags)
{
	uint16 UnusedBaseIndex = 0;
	uint16 UnusedNumBytes = 0;

	if (TOptional<FParameterAllocation> Parameter = ParameterMap.FindParameterAllocation(ParameterName))
	{
		// NOTE: the name difference is intentional (and confusing)
		BaseIndex = Parameter->BufferIndex;
		checkf(IsBound(), TEXT("UniformBuffer Parameter '%s' was not bound with a valid index. This can cause instability at runtime."), ParameterName);
	}
	else
	{
		BaseIndex = 0xffff;
		if (Flags == SPF_Mandatory)
		{
			FailureToBindNonOptionalParameter(TEXT("shader resource parameter"), ParameterName);
		}
	}
}

#if WITH_EDITOR

/** The individual bits of a uniform buffer declaration. */
struct FUniformBufferDecl
{
	/** Members to place in the constant buffer. */
	TStringBuilder<0> ConstantBufferMembers;

	/** Declarations of resource types. Used for all bindless scenarios. */
	TStringBuilder<0> ResourceDecls;

	/** Members to place in the resource table. */
	TStringBuilder<0> ResourceMembers;

	/** Remappings to apply in the UniformBuffer { } block */
	TStringBuilder<0> Remappings;

	/** Getter functions for bindless member resource access. */
	TStringBuilder<0> BindlessResourceGetters;

	/** Remappings to apply in the UniformBuffer { } block for when bindless UB access is enabled. */
	TStringBuilder<0> BindlessRemappings;
};

inline EShaderParameterType GetShaderParameterTypeFromMemberResourceType(EUniformBufferBaseType InMemberType)
{
	if (InMemberType == UBMT_SAMPLER)
	{
		return EShaderParameterType::Sampler;
	}

	if (InMemberType == UBMT_UAV || InMemberType == UBMT_RDG_TEXTURE_UAV || InMemberType == UBMT_RDG_BUFFER_UAV)
	{
		return EShaderParameterType::UAV;
	}

	return EShaderParameterType::SRV;
}

inline const TCHAR* GetShaderParameterTypeSuffix(EShaderParameterType ParameterType)
{
	if (ParameterType == EShaderParameterType::SRV)
	{
		return TEXT("SRV");
	}

	if (ParameterType == EShaderParameterType::UAV)
	{
		return TEXT("UAV");
	}

	if (ParameterType == EShaderParameterType::Sampler)
	{
		return TEXT("SAMPLER");
	}

	return TEXT("PARAMETER");
}

inline bool CanParameterTypeBeBindless(EShaderParameterType ParameterType)
{
	return ParameterType != EShaderParameterType::LooseData;
}

inline const TCHAR* GetBindlessPrefix(EUniformBufferBaseType InBaseType)
{
	if (InBaseType == UBMT_SAMPLER)
	{
		return FShaderParameterParser::kBindlessSamplerPrefix;
	}

	if (InBaseType == UBMT_UAV || InBaseType == UBMT_RDG_TEXTURE_UAV || InBaseType == UBMT_RDG_BUFFER_UAV)
	{
		return FShaderParameterParser::kBindlessUAVPrefix;
	}

	return FShaderParameterParser::kBindlessSRVPrefix;
};

using FTempParameterStringBuilder = TStringBuilder<256>;

/** Generates a HLSL struct declaration for a uniform buffer struct. */
static void CreateHLSLUniformBufferStructMembersDeclaration(
	const FShaderParametersMetadata& UniformBufferStruct,
	const FString& UniformBufferName,
	const FString& StructPrefix,
	const FString& GlobalPrefix,
	uint32 StructOffset,
	FUniformBufferDecl& Decl,
	uint32& HLSLBaseOffset)
{
	const TArray<FShaderParametersMetadata::FMember>& StructMembers = UniformBufferStruct.GetMembers();
	const bool bBindlessAccessible = EnumHasAnyFlags(UniformBufferStruct.GetUsageFlags(), FShaderParametersMetadata::EUsageFlags::BindlessAccessible);

	auto GenerateGlobalMemberName = [&GlobalPrefix](FTempParameterStringBuilder& Builder, const FShaderParametersMetadata::FMember& Member)
	{
		if (!GlobalPrefix.IsEmpty())
		{
			Builder << GlobalPrefix << TEXT("_");
		}
		Builder << Member.GetName();
	};

	auto GenerateStructMemberName = [&StructPrefix](FStringBuilderBase& Builder, const FShaderParametersMetadata::FMember& Member)
	{
		if (!StructPrefix.IsEmpty())
		{
			Builder << StructPrefix << TEXT(".");
		}
		Builder << Member.GetName();
	};

	auto AdjustConstantBufferPadding = [&HLSLBaseOffset, &Decl, StructOffset, &UniformBufferName](const FShaderParametersMetadata::FMember& Member, const TCHAR* PreviousBaseTypeName, uint32 HLSLMemberSize)
	{
		const uint32 AbsoluteMemberOffset = StructOffset + Member.GetOffset();

		// If the HLSL offset doesn't match the C++ offset, generate padding to fix it.
		if (HLSLBaseOffset != AbsoluteMemberOffset)
		{
			check(HLSLBaseOffset < AbsoluteMemberOffset);
			while (HLSLBaseOffset < AbsoluteMemberOffset)
			{
				Decl.ConstantBufferMembers.Appendf(TEXT("\t%s() UB_CB_MEMBER_NAME(%s, Padding%u);\n"), PreviousBaseTypeName, *UniformBufferName, HLSLBaseOffset);
				HLSLBaseOffset += 4;
			};
			check(HLSLBaseOffset == AbsoluteMemberOffset);
		}
		HLSLBaseOffset = AbsoluteMemberOffset + HLSLMemberSize;
	};

	auto AddStructResource = [&Decl, bBindlessAccessible, &UniformBufferName, &GenerateStructMemberName, &GenerateGlobalMemberName](const FShaderParametersMetadata::FMember& Member, const TCHAR* ShaderType, FStringView GlobalName, EShaderParameterType ParameterType)
	{
		if (bBindlessAccessible)
		{
			// Converting UniformBuffer.Member into a function call without string reallocations:
			//   The replacement code goes in *after* preprocessing, so we have to generate an actual function() call. At a minimum we have to cut 2 characters from "UniformBuffer.Member".
			//   Removing the '.' gives us one character, the second one can be trimmed off of either the uniform buffer name or member name. We have to do this without causing collisions with other uniform buffers or members. 
			//   If we want to keep things readable and not cut off _# array expansions, we should cut off the last letter of the uniform buffer name.

			const FString TrimmedUBName(FStringView(UniformBufferName).LeftChop(1));

			FTempParameterStringBuilder FullStructMemberName;
			GenerateStructMemberName(FullStructMemberName, Member);

			FTempParameterStringBuilder GlobalMemberName;
			GenerateGlobalMemberName(GlobalMemberName, Member);

			const TCHAR* LoadSuffix = GetShaderParameterTypeSuffix(ParameterType);

			// Global getter functions with the shortened name.
			Decl.BindlessResourceGetters.Appendf(
				TEXT("UB_RES_SAFE_TYPE(%s,%s) %s%s() { return UB_CBO_LOAD_%s(%s,%s,%s); }\n"),
				*UniformBufferName, *GlobalMemberName,
				*TrimmedUBName, *GlobalMemberName,
				LoadSuffix, *UniformBufferName, *UniformBufferName, *GlobalMemberName
			);

			// "UniformBuffer.Member" -> "UniformBuffeMember()"
			Decl.BindlessRemappings.Appendf(
				TEXT("%s.%s = %s%s();\n"),
				*UniformBufferName, *FullStructMemberName,
				*TrimmedUBName, *GlobalMemberName
			);
		}
	};

	auto AddRemapping = [&](const FShaderParametersMetadata::FMember& Member, const FTempParameterStringBuilder& GlobalName, EShaderParameterType ParameterType)
	{
		const bool bResource = (ParameterType != EShaderParameterType::LooseData);
		// Members for the UniformBuffer { ... } code that is used to remap struct accesses to global resources

		// UB_CB_REMAP_PARAMETER(UniformBufferName, StructPrefix_Name, GlobalPrefix_Name)
		Decl.Remappings << (bResource ? TEXT("UB_CB_REMAP_RESOURCE(") : TEXT("UB_CB_REMAP_PARAMETER(")) << UniformBufferName << TEXT(",");

		GenerateStructMemberName(Decl.Remappings, Member);

		Decl.Remappings << TEXT(",") << GlobalName << TEXT(");\n");
	};

	if (EnumHasAnyFlags(UniformBufferStruct.GetUsageFlags(), FShaderParametersMetadata::EUsageFlags::UniformView))
	{
		// UniformView struct is expected to have a single SRV member which serves as a uniform view
		check(StructMembers.Num() == 1);
		const FShaderParametersMetadata::FMember& Member = StructMembers[0];
		check(Member.GetBaseType() == UBMT_SRV || Member.GetBaseType() == UBMT_RDG_BUFFER_SRV);

		FTempParameterStringBuilder GlobalParameterName;
		GenerateGlobalMemberName(GlobalParameterName, Member);

		Decl.ConstantBufferMembers.Appendf(TEXT("UB_CB_UNIFORM_BLOCK(%s, %s);\n"), *UniformBufferName, *GlobalParameterName);

		AddRemapping(Member, GlobalParameterName, EShaderParameterType::LooseData);

		return;
	}

	const TCHAR* PreviousBaseTypeName = TEXT("UB_FLOAT");
	for (int32 MemberIndex = 0; MemberIndex < StructMembers.Num(); ++MemberIndex)
	{
		const FShaderParametersMetadata::FMember& Member = StructMembers[MemberIndex];

		TStringBuilder<8> ArrayDim;
		if (Member.GetNumElements() > 0)
		{
			ArrayDim.Appendf(TEXT("[%u]"), Member.GetNumElements());
		}

		if (Member.GetBaseType() == UBMT_NESTED_STRUCT)
		{
			checkf(Member.GetNumElements() == 0, TEXT("SHADER_PARAMETER_STRUCT_ARRAY() is not supported in uniform buffer yet."));

			FString NewStructPrefix = StructPrefix.IsEmpty() ? FString(Member.GetName()) : FString::Printf(TEXT("%s.%s"), *StructPrefix, Member.GetName());
			FString NewGlobalPrefix = GlobalPrefix.IsEmpty() ? FString(Member.GetName()) : FString::Printf(TEXT("%s_%s"), *GlobalPrefix, Member.GetName());

			CreateHLSLUniformBufferStructMembersDeclaration(*Member.GetStructMetadata(), UniformBufferName, NewStructPrefix, NewGlobalPrefix, StructOffset + Member.GetOffset(), Decl, HLSLBaseOffset);
		}
		else if (Member.GetBaseType() == UBMT_INCLUDED_STRUCT)
		{
			CreateHLSLUniformBufferStructMembersDeclaration(*Member.GetStructMetadata(), UniformBufferName, StructPrefix, GlobalPrefix, StructOffset + Member.GetOffset(), Decl, HLSLBaseOffset);
		}
		else if (IsShaderParameterTypeForUniformBufferLayout(Member.GetBaseType()))
		{
			// Add the constant buffer entry for bindless resource indices
			constexpr uint32 HLSLMemberSize = 4;

			AdjustConstantBufferPadding(Member, PreviousBaseTypeName, HLSLMemberSize);

			PreviousBaseTypeName = TEXT("UB_UINT");

			// Generate the member declaration.
			const TCHAR* MemberPrefix = GetBindlessPrefix(Member.GetBaseType());

			FTempParameterStringBuilder GlobalParameterName;
			GenerateGlobalMemberName(GlobalParameterName, Member);

			Decl.ConstantBufferMembers.Appendf(TEXT("\tUB_UINT() UB_CB_PREFIXED_MEMBER_NAME(%s, %s, %s);\n"), *UniformBufferName, MemberPrefix, *GlobalParameterName);
		}
		else
		{
			// Generate the base type name.
			const TCHAR* BaseTypeName = TEXT("");
			switch (Member.GetBaseType())
			{
			case UBMT_INT32:   BaseTypeName = TEXT("UB_INT"); break;
			case UBMT_UINT32:  BaseTypeName = TEXT("UB_UINT"); break;
			case UBMT_FLOAT32:
				if (Member.GetPrecision() == EShaderPrecisionModifier::Float)
				{
					BaseTypeName = TEXT("UB_FLOAT");
				}
				else if (Member.GetPrecision() == EShaderPrecisionModifier::Half)
				{
					BaseTypeName = TEXT("UB_HALF_FLOAT");
				}
				else if (Member.GetPrecision() == EShaderPrecisionModifier::Fixed)
				{
					BaseTypeName = TEXT("UB_FIXED_FLOAT");
				}
				break;
			default:           UE_LOGF(LogShaders, Fatal, "Unrecognized uniform buffer struct member base type.");
			};

			// Generate the type dimensions for vectors and matrices.
			TStringBuilder<16> TypeDim;
			uint32 HLSLMemberSize = 4;
			if (Member.GetNumRows() > 1)
			{
				TypeDim.Appendf(TEXT("%ux%u"), Member.GetNumRows(), Member.GetNumColumns());

				// Each row of a matrix is 16 byte aligned.
				HLSLMemberSize = (Member.GetNumRows() - 1) * 16 + Member.GetNumColumns() * 4;
			}
			else if (Member.GetNumColumns() > 1)
			{
				TypeDim.Appendf(TEXT("%u"), Member.GetNumColumns());
				HLSLMemberSize = Member.GetNumColumns() * 4;
			}

			// Array elements are 16 byte aligned.
			if (Member.GetNumElements() > 0)
			{
				HLSLMemberSize = (Member.GetNumElements() - 1) * Align(HLSLMemberSize, 16) + HLSLMemberSize;
			}

			AdjustConstantBufferPadding(Member, PreviousBaseTypeName, HLSLMemberSize);

			PreviousBaseTypeName = BaseTypeName;

			FTempParameterStringBuilder GlobalParameterName;
			GenerateGlobalMemberName(GlobalParameterName, Member);

			Decl.ConstantBufferMembers.Appendf(TEXT("\t%s(%s) UB_CB_MEMBER_NAME(%s, %s%s);\n"), BaseTypeName, *TypeDim, *UniformBufferName, *GlobalParameterName, *ArrayDim);

			AddRemapping(Member, GlobalParameterName, EShaderParameterType::LooseData);
		}
	}

	for (int32 MemberIndex = 0; MemberIndex < StructMembers.Num(); ++MemberIndex)
	{
		const FShaderParametersMetadata::FMember& Member = StructMembers[MemberIndex];

		if (IsShaderParameterTypeForUniformBufferLayout(Member.GetBaseType()))
		{
			// TODO: handle arrays?
			checkf(!IsRDGResourceAccessType(Member.GetBaseType()), TEXT("RDG access parameter types (e.g. RDG_TEXTURE_ACCESS) are not allowed in uniform buffers."));

			FTempParameterStringBuilder GlobalParameterName;
			GenerateGlobalMemberName(GlobalParameterName, Member);

			const EShaderParameterType ShaderParameterType = GetShaderParameterTypeFromMemberResourceType(Member.GetBaseType());
			const TCHAR* MemberSuffix = GetShaderParameterTypeSuffix(ShaderParameterType);

			Decl.ResourceDecls.Appendf(TEXT("UB_RES_DECL_%s(%s, %s, %s);\n"), MemberSuffix, Member.GetShaderType(), *UniformBufferName, *GlobalParameterName);
			Decl.ResourceMembers.Appendf(TEXT("UB_RESOURCE_MEMBER_%s(%s, %s, %s);\n"), MemberSuffix, Member.GetShaderType(), *UniformBufferName, *GlobalParameterName);

			AddRemapping(Member, GlobalParameterName, ShaderParameterType);
			AddStructResource(Member, Member.GetShaderType(), GlobalParameterName, ShaderParameterType);
		}
	}
}

/** Creates a HLSL declaration of a uniform buffer with the given structure. */
static FString CreateHLSLUniformBufferDeclaration(const TCHAR* UniformBufferName, const FShaderParametersMetadata& UniformBufferStruct, const FRHIUniformBufferShaderBindingLayout* UniformBufferSBLayout)
{
	// If the uniform buffer has no members, we don't want to write out anything.  Shader compilers throw errors when faced with empty cbuffers and structs.
	if (UniformBufferStruct.GetMembers().Num() > 0)
	{
		FUniformBufferDecl Decl;
		uint32 HLSLBaseOffset = 0;
		CreateHLSLUniformBufferStructMembersDeclaration(UniformBufferStruct, UniformBufferName, TEXT(""), TEXT(""), 0, Decl, HLSLBaseOffset);

		const bool bBindlessAccessible = EnumHasAnyFlags(UniformBufferStruct.GetUsageFlags(), FShaderParametersMetadata::EUsageFlags::BindlessAccessible);
		const bool bUseRegister = (UniformBufferSBLayout && UniformBufferSBLayout->RegisterSpace > 0);

		// UB_CB_DEFINITION_START and UB_CB_DEFINITION_END both have variants that take the register + space specifiers, so use those when appropriate
		const FString StartEndSuffix = bUseRegister
			? FString::Printf(TEXT("_REG(%s,%d,%d)"), UniformBufferName, UniformBufferSBLayout->CBVResourceIndex, UniformBufferSBLayout->RegisterSpace)
			: FString::Printf(TEXT("(%s)"), UniformBufferName);

		TStringBuilder<0> Builder;
		Builder.Append(TEXT("#pragma once\n"));

		// Type declarations are common between all UB generated code.
		Builder.Append(Decl.ResourceDecls);

		// Allow Texture2DMultiView to represent either a Texture2D or Texture2DArray depending on whether shaders were compiled to use mobile multi-view (VK_KHR_multiview)
		Builder.Append(TEXT("#if MOBILE_MULTI_VIEW\n"));
		Builder.Append(TEXT("#define Texture2DMultiView Texture2DArray\n"));
		Builder.Append(TEXT("#else\n"));
		Builder.Append(TEXT("#define Texture2DMultiView Texture2D\n"));
		Builder.Append(TEXT("#endif\n"));

		if (bBindlessAccessible)
		{
			Builder.Append(TEXT("#if !(UE_BINDLESS_ENABLED && SUPPORTS_BINDLESS_UNIFORM_BUFFER)\n"));
		}

		Builder.Appendf(
			TEXT("UB_CB_DEFINITION_START%s\n")
			TEXT("%s")
			TEXT("UB_CB_DEFINITION_END(%s)\n")
			TEXT("UB_CB_FINALIZE%s\n"),
			*StartEndSuffix,
			*Decl.ConstantBufferMembers,
			UniformBufferName,
			*StartEndSuffix
		);

		Builder.Append(Decl.ResourceMembers);
		Builder.Appendf(
			TEXT("UniformBuffer %s\n")
			TEXT("{\n")
			TEXT("%s")
			TEXT("};\n"),
			UniformBufferName,
			*Decl.Remappings
		);

		if (bBindlessAccessible)
		{
			Builder.Append(TEXT("#else\n"));

			// Force all ConstantBuffer members to CBO syntax.
			const FString CBOMembers = FString(*Decl.ConstantBufferMembers).Replace(TEXT("UB_CB_"), TEXT("UB_CBO_"), ESearchCase::CaseSensitive);


			Builder.Appendf(TEXT("#define UB_HAS_DECL_%s 1\n"), UniformBufferName);

			Builder.Appendf(
				TEXT("UB_CBO_DEFINITION_START(%s)\n")
				TEXT("%s")
				TEXT("UB_CBO_DEFINITION_END(%s)\n"),
				UniformBufferName,
				*CBOMembers,
				UniformBufferName
			);

			Builder.Appendf(
				TEXT("UB_BINDLESS_DECL(UB_CBO_TYPE(%s), %s);\n"),
				UniformBufferName,
				UniformBufferName
			);

			Builder.Appendf(TEXT("static UB_SAFE_TYPE(%s) %s;\n"),
				UniformBufferName, UniformBufferName
			);

			Builder.Append(Decl.BindlessResourceGetters);

			Builder.Appendf(
				TEXT("UniformBuffer %s\n")
				TEXT("{\n")
				TEXT("%s")
				TEXT("};\n"),
				UniformBufferName,
				*Decl.BindlessRemappings
			);
			Builder.Append(TEXT("#endif\n"));
		}

		Builder.Append(TEXT("\n"));

		FString Result(Builder);

		if (Decl.ResourceMembers.Len() == 0 && !bBindlessAccessible)
		{
			return Result.Replace(TEXT("UB_CB_"), TEXT("UB_CBO_"), ESearchCase::CaseSensitive);
		}

		return Result;
	}

	return FString("\n");
}

FString UE::ShaderParameters::CreateUniformBufferShaderDeclaration(const TCHAR* UniformBufferName, const FShaderParametersMetadata& UniformBufferStruct, const FRHIUniformBufferShaderBindingLayout* UniformBufferSBLayout)
{
	return CreateHLSLUniformBufferDeclaration(UniformBufferName, UniformBufferStruct, UniformBufferSBLayout);
}

void UE::ShaderParameters::AddUniformBufferIncludesToEnvironment(FShaderCompilerEnvironment& OutEnvironment, const TSet<const FShaderParametersMetadata*>& InUniformBuffers)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UE::ShaderParameters::AddUniformBufferIncludesToEnvironment);

	FString UniformBufferIncludes;

	auto AddUniformBufferData = [&OutEnvironment, &UniformBufferIncludes](const FShaderParametersMetadata* Metadata, FThreadSafeSharedAnsiStringPtr UniformBufferDeclaration)
	{
		FStringView UniformBufferNameView(Metadata->GetShaderVariableName());
		uint32 UniformBufferNameHash = GetTypeHash(UniformBufferNameView);
		if (!OutEnvironment.UniformBufferMap.FindByHash(UniformBufferNameHash, UniformBufferNameView))
		{
			check(UniformBufferDeclaration.Get() != NULL);
			check(!UniformBufferDeclaration.Get()->IsEmpty());

			UniformBufferIncludes += Metadata->GetUniformBufferInclude();

			OutEnvironment.IncludeVirtualPathToSharedContentsMap.AddByHash(Metadata->GetUniformBufferPathHash(), Metadata->GetUniformBufferPath(), UniformBufferDeclaration);

			Metadata->AddResourceTableEntries(OutEnvironment.ResourceTableMap, OutEnvironment.UniformBufferMap);
		}
	};

	// If there is a shader binding layout then always add all the entries (duplicated from InUniformBuffers will not be added twice because of Find function below)
	if (OutEnvironment.ShaderBindingLayout)
	{
		for (auto Iter = OutEnvironment.ShaderBindingLayout->GetUniformBufferDeclarations().CreateConstIterator(); Iter; ++Iter)
		{
			const FShaderParametersMetadata* UniformBufferStruct = FindUniformBufferStructByName(*Iter.Key());
			if (UniformBufferStruct)
			{
				AddUniformBufferData(UniformBufferStruct, Iter.Value());
			}
		}
	}

	for (const FShaderParametersMetadata* Metadata : InUniformBuffers)	
	{
		FStringView UniformBufferNameView(Metadata->GetShaderVariableName());
		uint32 UniformBufferNameHash = GetTypeHash(UniformBufferNameView);
		if (!OutEnvironment.UniformBufferMap.FindByHash(UniformBufferNameHash, UniformBufferNameView))
		{
			FThreadSafeSharedAnsiStringPtr UniformBufferDeclaration = Metadata->GetUniformBufferDeclarationAnsiPtr();
			AddUniformBufferData(Metadata, UniformBufferDeclaration);
		}
	}

	FString& GeneratedUniformBuffersInclude = OutEnvironment.IncludeVirtualPathToContentsMap.FindOrAdd("/Engine/Generated/GeneratedUniformBuffers.ush");
	GeneratedUniformBuffersInclude.Append(UniformBufferIncludes);
}

void FShaderType::AddUniformBufferIncludesToEnvironment(FShaderCompilerEnvironment& OutEnvironment, EShaderPlatform Platform) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FShaderType::AddReferencedUniformBufferIncludes);

	UE::ShaderParameters::AddUniformBufferIncludesToEnvironment(OutEnvironment, ReferencedUniformBuffers);
}

#endif // WITH_EDITOR

void FShaderType::DumpDebugInfo()
{
	UE_LOGF(LogConsoleResponse, Display, "----------------------------- GLobalShader %ls", GetName());
	UE_LOGF(LogConsoleResponse, Display, "               :Target %ls", GetShaderFrequencyString(GetFrequency()));
	UE_LOGF(LogConsoleResponse, Display, "               :TotalPermutationCount %d", TotalPermutationCount);
#if WITH_EDITOR
	UE_LOGF(LogConsoleResponse, Display, "               :SourceHash %ls", *GetSourceHash(GMaxRHIShaderPlatform).ToString());
#endif
	switch (ShaderTypeForDynamicCast)
	{
	case EShaderTypeForDynamicCast::Global:
		UE_LOGF(LogConsoleResponse, Display, "               :ShaderType Global");
		break;
	case EShaderTypeForDynamicCast::Material:
		UE_LOGF(LogConsoleResponse, Display, "               :ShaderType Material");
		break;
	case EShaderTypeForDynamicCast::MeshMaterial:
		UE_LOGF(LogConsoleResponse, Display, "               :ShaderType MeshMaterial");
		break;
	case EShaderTypeForDynamicCast::Niagara:
		UE_LOGF(LogConsoleResponse, Display, "               :ShaderType Niagara");
		break;
	}

#if 0
	UE_LOGF(LogConsoleResponse, Display, "  --- %d shaders", ShaderIdMap.Num());
	int32 Index = 0;
	for (auto& KeyValue : ShaderIdMap)
	{
		UE_LOGF(LogConsoleResponse, Display, "    --- shader %d", Index);
		FShader* Shader = KeyValue.Value;
		Shader->DumpDebugInfo();
		Index++;
	}
#endif
}

#if WITH_EDITOR
void FShaderType::GetShaderStableKeyParts(FStableShaderKeyAndValue& SaveKeyVal)
{
	static FName NAME_Material("Material");
	static FName NAME_MeshMaterial("MeshMaterial");
	static FName NAME_Niagara("Niagara");
	switch (ShaderTypeForDynamicCast)
	{
	case EShaderTypeForDynamicCast::Global:
		SaveKeyVal.ShaderClass = NAME_Global;
		break;
	case EShaderTypeForDynamicCast::Material:
		SaveKeyVal.ShaderClass = NAME_Material;
		break;
	case EShaderTypeForDynamicCast::MeshMaterial:
		SaveKeyVal.ShaderClass = NAME_MeshMaterial;
		break;
	case EShaderTypeForDynamicCast::Niagara:
		SaveKeyVal.ShaderClass = NAME_Niagara;
		break;
	}
	SaveKeyVal.ShaderType = FName(GetName() ? GetName() : TEXT("null"));
}

void FVertexFactoryType::AddUniformBufferIncludesToEnvironment(FShaderCompilerEnvironment& OutEnvironment, EShaderPlatform Platform) const
{
	UE::ShaderParameters::AddUniformBufferIncludesToEnvironment(OutEnvironment, ReferencedUniformBuffers);
}

#endif // WITH_EDITOR
