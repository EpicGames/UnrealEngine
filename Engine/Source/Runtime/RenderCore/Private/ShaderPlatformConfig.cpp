// Copyright Epic Games, Inc. All Rights Reserved.

#include "ShaderPlatformConfig.h"
#include "RHIShaderPlatformConfig.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "Misc/ConfigCacheIni.h"
#include "RHIStrings.h"
#include "RHI.h"

FShaderPlatformConfig::FPlatformConfig FShaderPlatformConfig::PlatformConfigs[SP_NumPlatforms];

FShaderPlatformConfig::FPlatformConfig::FPlatformConfig()
{
	SetDefaultValues();
}

void FShaderPlatformConfig::FPlatformConfig::SetDefaultValues()
{
	SubstrateMaxClosuresPerPixel = 8; // Similar to SUBSTRATE_MAX_CLOSURE_COUNT

	BindlessConfiguration = ERHIBindlessConfiguration::Disabled;
	bEnableNaniteBindlessShading = false;
	bEnableNaniteBindlessRasterization = false;
	bEnableNaniteBindlessPixelProgrammable = false;
}

static void SetupBindlessConfiguration(EShaderPlatform ShaderPlatform, FStringView InString, ERHIBindlessConfiguration& OutBindlessConfiguration)
{
	if (!FDataDrivenShaderPlatformInfo::GetSupportsBindless(ShaderPlatform))
	{
		OutBindlessConfiguration = ERHIBindlessConfiguration::Disabled;
	}
	else if (TOptional<ERHIBindlessConfiguration> ForcedBindlessConfiguration = RHIGetForcedBindlessConfiguration())
	{
		OutBindlessConfiguration = ForcedBindlessConfiguration.GetValue();
	}
	else
	{
		GetBindlessConfigurationFromString(InString, OutBindlessConfiguration);
	}
}

void FShaderPlatformConfig::FPlatformConfig::InitializeProperties(const FConfigSection& InConfigSection)
{
	check(bValidShaderPlatform);

#define GET_CONFIG_BOOL(SettingName) \
	if (const FConfigValue* ConfigValue = InConfigSection.Find(#SettingName)) \
	{ \
		this->SettingName = FCString::ToBool(*ConfigValue->GetValue()); \
	}

#define GET_CONFIG_UINT(SettingName) \
	if (const FConfigValue* ConfigValue = InConfigSection.Find(#SettingName)) \
	{ \
		const int32 IntValue = FCString::Atoi(*ConfigValue->GetValue()); \
		this->SettingName = static_cast<uint32>(FMath::Clamp<int32>(IntValue, 0, TNumericLimits<int32>::Max())); \
	}

#define GET_CONFIG_BINDLESS_CONFIGURATION(SettingName) \
	if (const FConfigValue* ConfigValue = InConfigSection.Find(#SettingName)) \
	{ \
		SetupBindlessConfiguration(this->ShaderPlatform, ConfigValue->GetValue(), this->SettingName); \
	}

	GET_CONFIG_UINT(SubstrateMaxClosuresPerPixel);
	GET_CONFIG_BINDLESS_CONFIGURATION(BindlessConfiguration);

	// These Nanite Bindless Shading/Rasterization settings should only be set when Bindless is enabled.
	if (IsBindlessEnabledForAnyGraphics(BindlessConfiguration))
	{
		GET_CONFIG_BOOL(bEnableNaniteBindlessShading);
		GET_CONFIG_BOOL(bEnableNaniteBindlessRasterization);
		GET_CONFIG_BOOL(bEnableNaniteBindlessPixelProgrammable);
	}

#undef GET_CONFIG_BINDLESS_CONFIGURATION
#undef GET_CONFIG_UINT
#undef GET_CONFIG_BOOL
}

#if WITH_EDITOR
void FShaderPlatformConfig::FPlatformConfig::InitializePropertiesForPreview(const FPlatformConfig& RuntimeConfig)
{
	check(bValidShaderPlatform);

#define PREVIEW_USE_RUNTIME_VALUE(SettingName) \
	this->SettingName = RuntimeConfig.SettingName

#define PREVIEW_DISABLE_IF_RUNTIME_UNSUPPORTED(SettingName) \
	this->SettingName &= RuntimeConfig.SettingName

#define PREVIEW_FORCE_SETTING(SettingName, Value) \
	this->SettingName = (Value)

#define PREVIEW_FORCE_DISABLE(SettingName) \
	PREVIEW_FORCE_SETTING(SettingName, false)

	// Metal platforms require bindless config to match
	if (IsMetalPlatform(GMaxRHIShaderPlatform))
	{
		PREVIEW_FORCE_SETTING(BindlessConfiguration, PlatformConfigs[GMaxRHIShaderPlatform].BindlessConfiguration);
	}

#undef PREVIEW_FORCE_DISABLE
#undef PREVIEW_FORCE_SETTING
#undef PREVIEW_DISABLE_IF_RUNTIME_UNSUPPORTED
#undef PREVIEW_USE_RUNTIME_VALUE
}
#endif // WITH_EDITOR

void FShaderPlatformConfig::FPlatformConfig::Initialize(EShaderPlatform InShaderPlatform)
{
	ShaderPlatform = InShaderPlatform;
	bValidShaderPlatform = true;

	SetDefaultValues();

	TStringBuilder<64> SectionName;
	if (const FConfigSection* Section = UE::RHIShaderPlatformConfig::GetConfigSectionForShaderPlatform(InShaderPlatform, SectionName))
	{
		ConfigSectionName = SectionName;
		bLoadedFromConfigFiles = true;

		InitializeProperties(*Section);
	}
}

#if WITH_EDITOR
void FShaderPlatformConfig::FPlatformConfig::InitializeForPreview(EShaderPlatform InShaderPlatform, const FPlatformConfig& RuntimeConfig)
{
	*this = RuntimeConfig;

	ShaderPlatform = InShaderPlatform;
	bValidShaderPlatform = true;

	InitializePropertiesForPreview(RuntimeConfig);
}
#endif

void FShaderPlatformConfig::Initialize()
{
#if USE_STATIC_SHADER_PLATFORM_ENUMS
	PlatformConfigs[GMaxRHIShaderPlatform].Initialize(GMaxRHIShaderPlatform);
#else
	for (uint32 ShaderPlatformIndex = 0; ShaderPlatformIndex < SP_NumPlatforms; ShaderPlatformIndex++)
	{
		const EShaderPlatform ShaderPlatform = static_cast<EShaderPlatform>(ShaderPlatformIndex);
		if (FDataDrivenShaderPlatformInfo::IsValid(ShaderPlatform))
		{
			if (FDataDrivenShaderPlatformInfo::GetIsPreviewPlatform(ShaderPlatform))
			{
#if WITH_EDITOR
				const ERHIFeatureLevel::Type PreviewFeatureLevel = FDataDrivenShaderPlatformInfo::GetMaxFeatureLevel(ShaderPlatform);
				const EShaderPlatform RuntimePlatform = FDataDrivenShaderPlatformInfo::GetPreviewRuntimePlatform(ShaderPlatform);

				if (RuntimePlatform < SP_NumPlatforms && PlatformConfigs[RuntimePlatform].IsValid())
				{
					PlatformConfigs[ShaderPlatform].InitializeForPreview(ShaderPlatform, PlatformConfigs[RuntimePlatform]);
				}
#endif
			}
			else
			{
				PlatformConfigs[ShaderPlatform].Initialize(ShaderPlatform);
			}
		}
	}
#endif
}
