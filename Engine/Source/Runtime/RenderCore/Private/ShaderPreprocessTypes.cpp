// Copyright Epic Games, Inc. All Rights Reserved.

#include "ShaderPreprocessTypes.h"

#include "ShaderCompilerCore.h"
#include "Containers/AnsiString.h"
#include "Templates/UnrealTemplate.h"

static TAutoConsoleVariable<bool> CVarEnableDebugHashAssociation(
	TEXT("r.ShaderCompiler.EnableDebugHashAssociation"),
	true,
	TEXT("If true, a // DebugHash_<hash> comment will be appended to all finalized shader source matching the DebugHash_<hash>.txt file output with "
		 "shader debug artifacts. This can be used to find debug artifacts from shader source visible in Pix/RenderDoc/etc when symbols are enabled. "
		 "Note that currently the only known reason to disable this is when comparing shader source or symbols across streams where versions may differ "
		 "but source would otherwise match."),
	ECVF_ReadOnly);

struct FPreprocessConstants
{
	FPreprocessConstants()
	{
		// Add a placeholder comment to the header to allocate enough space for the "debug hash" (which
		// will be used to associate shader source back to debug output). This will be replaced inline
		// by code calling into the preprocessing after the fact (necessary since the hash of the preprocessed
		// code itself is used in constructing this debug hash).
		if (CVarEnableDebugHashAssociation.GetValueOnAnyThread())
		{
			StrippedCodeHeaderBuilder << DebugHashCommentStart << DebugHashPrefix << FShaderCompilerInputHash() << DebugHashCommentEnd;
		}
		// Also add a sentinel line directive so we get the correct line offsets when remapping error/warning messages.
		StrippedCodeHeaderBuilder << LineDirectiveStart << FilenameSentinel << LineDirectiveEnd;

		check(StrippedCodeHeaderBuilder.GetAllocatedSize() == 0);

		StrippedCodeHeader = StrippedCodeHeaderBuilder.ToView();
	}

	static constexpr FShaderSource::FViewType DebugHashPrefix = ANSITEXTVIEW("DebugHash_");
	static constexpr FStringView DebugHashPrefixWide = TEXTVIEW("DebugHash_");
	static constexpr FShaderSource::FViewType FilenameSentinel = ANSITEXTVIEW("__UE_FILENAME_SENTINEL");

	FShaderSource::FViewType StrippedCodeHeader;

private:

	static constexpr FShaderSource::FViewType DebugHashCommentStart = ANSITEXTVIEW("// ");
	static constexpr FShaderSource::FViewType DebugHashCommentEnd = ANSITEXTVIEW("\n");
	static constexpr FShaderSource::FViewType LineDirectiveStart = ANSITEXTVIEW("#line 1 \"");
	static constexpr FShaderSource::FViewType LineDirectiveEnd = ANSITEXTVIEW("\"\n");

	static constexpr int32 HeaderLen =
		DebugHashCommentStart.Len() +
		DebugHashPrefix.Len() +
		DebugHashCommentEnd.Len() +
		2 * sizeof(FShaderCompilerInputHash::ByteArray) + // size of input hash converted to a hex string
		LineDirectiveStart.Len() +
		FilenameSentinel.Len() +
		LineDirectiveEnd.Len() +
		1; // +1 for null terminator, even though we don't need it builder always ensures there's space for one

	FShaderSource::TStringBuilder<HeaderLen> StrippedCodeHeaderBuilder;
};

static FPreprocessConstants& GetPreprocessConstants()
{
	static FPreprocessConstants PreprocessConstants;
	return PreprocessConstants;
}

FShaderSource::FViewType GetShaderSourceDebugHashPrefix()
{
	return GetPreprocessConstants().DebugHashPrefix;
}

FStringView GetShaderSourceDebugHashPrefixWide()
{
	return GetPreprocessConstants().DebugHashPrefixWide;
}

void FShaderDiagnosticRemapper::Remap(FShaderCompilerError& Diagnostic) const
{
	if (!Diagnostic.ErrorLineString.IsEmpty() && !Diagnostic.ErrorVirtualFilePath.IsEmpty() && Diagnostic.ErrorVirtualFilePath == GetPreprocessConstants().FilenameSentinel.GetData())
	{
		// only attempt remapping if the filename sentinel matches; this will bypass remapping in the case where compilation was retried on FXC
		// with a DXC precompile (this compilation will report error messages relative to an intermediate hlsl file, which is not a change in behaviour)

		// Some backends give us already-parsed FShaderCompilerError objects, so in this case we need to remap the parsed 
		// line number string & path (in addition to potentially needing to remap the message itself, in case the sentinel
		// file name occurred in the original diagnostic multiple times)
		const FString& ErrorLineStr = Diagnostic.ErrorLineString;
		check(FChar::IsDigit(ErrorLineStr[0]));
		int32 StrippedLineNum = 0, LineNumberEnd = 0;
		while (LineNumberEnd < ErrorLineStr.Len() && FChar::IsDigit(ErrorLineStr[LineNumberEnd]))
		{
			StrippedLineNum = StrippedLineNum * 10 + ErrorLineStr[LineNumberEnd++] - TEXT('0');
		}

		FRemapData RemapData = GetRemapData(StrippedLineNum);
		// if the given line number doesn't exist in the unstripped source/remap data, then just don't bother remapping
		// typically this only occurs if non-whitespace-preserving changes have been made in a shader format's compile implementation,
		// which in general should be caught before check-in, but in the case it's not reporting a non-remapped error is better than nothing
		if (RemapData.IsValid())
		{
			FString NewErrorLineString;
			NewErrorLineString.Reserve(ErrorLineStr.Len());
			NewErrorLineString.AppendInt(RemapData.LineNumber);
			NewErrorLineString.Append(ErrorLineStr.GetCharArray().GetData() + LineNumberEnd, ErrorLineStr.Len() - LineNumberEnd);

			Diagnostic.ErrorLineString = MoveTemp(NewErrorLineString);
			Diagnostic.ErrorVirtualFilePath = RemapData.Filename;
		}
	}

	int32 FilenameIndex = Diagnostic.StrippedErrorMessage.Find(GetPreprocessConstants().FilenameSentinel.GetData());
	// if we find our sentinel filename in the diagnostic message it needs remapping
	// need to loop in case the message contains the filename multiple times
	while (FilenameIndex >= 0)
	{
		const FString& OriginalMessage = Diagnostic.StrippedErrorMessage;
		// Parse the line number from the message and record start and end indices
		// Code assumes that the next integral number found in the message string after the filename is always line number
		int32 LineNumberStart = FilenameIndex + GetPreprocessConstants().FilenameSentinel.Len();
		while (LineNumberStart < OriginalMessage.Len() && !FChar::IsDigit(OriginalMessage[LineNumberStart]))
		{
			++LineNumberStart;
		}
		int32 LineNumberEnd = LineNumberStart;
		int32 StrippedLineNum = 0;
		while (LineNumberEnd < OriginalMessage.Len() && FChar::IsDigit(OriginalMessage[LineNumberEnd]))
		{
			StrippedLineNum = StrippedLineNum * 10 + (OriginalMessage[LineNumberEnd++] - TEXT('0'));
		}

		FRemapData RemapData = GetRemapData(StrippedLineNum);
		// if the given line number doesn't exist in the unstripped source/remap data, then just don't bother remapping
		// typically this only occurs if non-whitespace-preserving changes have been made in a shader format's compile implementation,
		// which in general should be caught before check-in, but in the case it's not reporting a non-remapped error is better than nothing
		if (!RemapData.IsValid())
		{
			break;
		}

		// Assume line number is the same number of digits for simplicity in reserve allocation size;
		// it's an upper bound but in most practical cases worst case we're allocating an extra byte
		int32 RemappedLen = OriginalMessage.Len() - GetPreprocessConstants().FilenameSentinel.Len() + RemapData.Filename.Len();
		FString RemappedMessage;
		RemappedMessage.Reserve(RemappedLen);
		RemappedMessage.AppendChars(OriginalMessage.GetCharArray().GetData(), FilenameIndex);
		RemappedMessage.Append(RemapData.Filename);
		int32 SeparatorStart = FilenameIndex + GetPreprocessConstants().FilenameSentinel.Len();
		RemappedMessage.Append(OriginalMessage.GetCharArray().GetData() + SeparatorStart, LineNumberStart - SeparatorStart);
		RemappedMessage.AppendInt(RemapData.LineNumber);
		RemappedMessage.Append(OriginalMessage.GetCharArray().GetData() + LineNumberEnd, OriginalMessage.Len() - LineNumberEnd);

		Diagnostic.StrippedErrorMessage = MoveTemp(RemappedMessage);
		FilenameIndex = Diagnostic.StrippedErrorMessage.Find(GetPreprocessConstants().FilenameSentinel.GetData());
	}
}

void FShaderDiagnosticRemapper::AddSourceBlock(int32 OriginalLineNum, int32 StrippedLineNum, FString && OriginalPath)
{
	if (Blocks.Num() && Blocks.Last().StrippedLineNum == StrippedLineNum)
	{
		// if no additional stripped source has been added since the last block, then the new block
		// can replace it (not possible to hit an error/warning in that block, it must be entirely
		// comments or whitespace)
		Blocks.Last().OriginalLineNum = OriginalLineNum;
		// If we weren't given a path, set path to the same as the last block (handles filename-less line directive)
		Blocks.Last().OriginalPath = OriginalPath.IsEmpty() ? Blocks.Last().OriginalPath : MoveTemp(OriginalPath);
	}
	else
	{
		// sanity check that ranges are always added in order of stripped line numbers
		check(Blocks.Num() == 0 || Blocks.Last().StrippedLineNum < StrippedLineNum);
		// if we're not given a path we must have a previous block from which to retrieve the path;
		// if we don't we don't bother adding the block. in practice this is only encountered in 
		// code automatically appended to the source by the shader system (since the preprocessor
		// always appends a line directive containing the file name when a file is loaded). we 
		// assume such code will not contain errors.
		if (!OriginalPath.IsEmpty() || Blocks.Num() > 0)
		{
			Blocks.Add(FSourceBlock{ StrippedLineNum, OriginalLineNum, OriginalPath.IsEmpty() ? Blocks.Last().OriginalPath : MoveTemp(OriginalPath) });
		}
	}
}

void FShaderDiagnosticRemapper::AddStrippedLine(int32 StrippedLineNum, int32 Offset)
{
	check(StrippedLineOffsets.Num() == (StrippedLineNum - 1));
	StrippedLineOffsets.Add(Offset);
}

void FShaderDiagnosticRemapper::Shrink()
{
	Blocks.Shrink();
	StrippedLineOffsets.Shrink();
}

FShaderDiagnosticRemapper::FRemapData FShaderDiagnosticRemapper::GetRemapData(int32 StrippedLineNum) const
{
	if (StrippedLineNum >= 1 && StrippedLineNum <= StrippedLineOffsets.Num())
	{
		// Find the index which should contain a block with starting line number greater than the one to which the message points; the preceding
		// block will be the one containing the stripped line in the unstripped source (in the case UpperBoundBy returns Blocks.Num() this means
		// the error must be in the last recorded block)
		int32 FoundIndex = Algo::UpperBoundBy(Blocks, StrippedLineNum, [](const FSourceBlock& Block) { return Block.StrippedLineNum; }) - 1;

		// Sanity check - we can't possibly have an error which occurs before the first recorded block unless the stripping didn't record blocks
		// properly (it only skips recording a block if it's empty in the stripped source, i.e. all comments or whitespace)
		check(FoundIndex >= 0);

		const FSourceBlock& FoundBlock = Blocks[FoundIndex];

		// Sanity check to validate the assumption made in AddSourceBlock - we should never be querying remap data for a line number
		// that doesn't have an associated file path in the unstripped source; this implies a warning or error has been emitted in
		// system-generated code and should be addressed by the developer prior to submission.
		checkf(!FoundBlock.OriginalPath.IsEmpty(), TEXT("Unexpected compile error/warning found in system-generated code"));

		// -1 when indexing stripped line offsets since line numbers are 1-based
		return FRemapData{ FoundBlock.OriginalPath, FoundBlock.OriginalLineNum + StrippedLineOffsets[StrippedLineNum - 1] };
	}
	else
	{
		// something went wrong and we're asking for remap data for a line number that didn't exist in the unstripped source.
		// return an invalid FRemapData which should be handled explicitly by calling code.
		static FString InvalidPath;
		return FRemapData{ InvalidPath };
	}
}

inline bool IsEndOfTheLine(FShaderSource::CharType C)
{
	return C == '\r' || C == '\n';
}

inline bool StripNeedsHandling(FShaderSource::CharType C)
{
	return IsEndOfTheLine(C) || C == '/' || C == '#' || C == 0;
}

inline void SkipNewLine(const FShaderSource::CharType*& Current, const FShaderSource::CharType* End)
{
	FShaderSource::CharType First = Current < End ? Current[0] : 0;
	FShaderSource::CharType Second = Current + 1 < End ? Current[1] : 0;
	Current += ((First + Second) == '\r' + '\n') ? 2 : 1;
}

void FShaderPreprocessOutput::StripCode(bool bCopyOriginalPreprocessdSource)
{
	// Reserve worst case slack (i.e. assuming there is nothing to strip) to avoid reallocation
	FShaderSource PreprocessedSourceStripped(GetPreprocessConstants().StrippedCodeHeader, PreprocessedSource.Len());
	FShaderSource::CharType* OutStrippedData = PreprocessedSourceStripped.GetData();
	FShaderSource::CharType* OutStripped = OutStrippedData + GetPreprocessConstants().StrippedCodeHeader.Len();

	const FShaderSource::CharType* Begin = PreprocessedSource.GetData(), * Current = Begin;
	const FShaderSource::CharType* End = Current + PreprocessedSource.Len();
	int32 CurrentBlockUnstrippedLineOffset = 0;
	int32 CurrentStrippedLineNum = 1;
	while (Current < End && *Current)
	{
		while (!StripNeedsHandling(*Current))
		{
			*OutStripped++ = *Current++;
		}

		if (IsEndOfTheLine(*Current))
		{
			// only emit \n if it wasn't preceded immediately by another linebreak 
			// (i.e. skip empty lines)
			if (!IsEndOfTheLine(*(OutStripped - 1)))
			{
				// Record the offset from the start of the block given by the last line directive for each line
				// output in the stripped code. 
				Remapper.AddStrippedLine(CurrentStrippedLineNum, CurrentBlockUnstrippedLineOffset);

				// normalize line endings
				*OutStripped++ = '\n';
				CurrentStrippedLineNum++;
			}
			CurrentBlockUnstrippedLineOffset++;
			SkipNewLine(Current, End);
		}
		else if (Current[0] == '/')
		{
			if (Current[1] == '/')
			{
				while (!IsEndOfTheLine(*Current) && Current < End)
				{
					++Current;
				}
			}
			else if (Current[1] == '*')
			{
				Current += 2;
				while (Current < End && !(Current[0] == '*' && Current[1] == '/'))
				{
					++Current;
				}
				Current += 2;
			}
			else
			{
				*OutStripped++ = *Current++;
			}
		}
		else if (Current[0] == '#')
		{
			if (Current + 4 < End
				&& Current[1] == 'l'
				&& Current[2] == 'i'
				&& Current[3] == 'n'
				&& Current[4] == 'e')
			{
				Current += 4;

				// scan past any whitespace until we find the first digit of the line number
				while (!FChar::IsDigit(*Current) && Current < End)
				{
					++Current;
				}
				int32 DirectiveLineNumber = 0;
				while (FChar::IsDigit(*Current) && Current < End)
				{
					DirectiveLineNumber = 10 * DirectiveLineNumber + (*Current - '0');
					++Current;
				}

				// scan past open quote
				while (*Current != '\"' && Current < End && !IsEndOfTheLine(*Current))
				{
					++Current;
				}
				
				if (*Current == '\"')
				{
					++Current;

					const FShaderSource::CharType* DirectiveFileNameStart = Current;
					int32 DirectiveFileNameLen = 0;

					// count chars in directive filename
					while (*Current != '\"' && Current < End)
					{
						++DirectiveFileNameLen;
						++Current;
					}

					FString DirectiveFileName = FString::ConstructFromPtrSize(DirectiveFileNameStart, DirectiveFileNameLen);

					// scan to end-of-line and skip past the newline; this would be handled by the newline case above as well,
					// but we don't want the newline at the end of the line directive to count in our calculated offsets for
					// emitted stripped code
					while (!IsEndOfTheLine(*Current) && Current < End)
					{
						++Current;
					}
					Remapper.AddSourceBlock(DirectiveLineNumber, CurrentStrippedLineNum, MoveTemp(DirectiveFileName));
				}
				else
				{
					// if we hit EOL before finding an open quote, assume this is a filename-less line directive
					Remapper.AddSourceBlock(DirectiveLineNumber, CurrentStrippedLineNum);
				}
				SkipNewLine(Current, End);

				CurrentBlockUnstrippedLineOffset = 0;
			}
			else
			{
				*OutStripped++ = *Current++;
			}
		}
	}
	check(OutStripped <= OutStrippedData + PreprocessedSourceStripped.Len());
	// ShrinkToLen null terminates for us by virtue of adding zero'd SIMD padding
	PreprocessedSourceStripped.ShrinkToLen((int32)(OutStripped - OutStrippedData));
	Remapper.Shrink();

	if (bCopyOriginalPreprocessdSource)
	{
		OriginalPreprocessedSource = MoveTemp(PreprocessedSource);
	}
	PreprocessedSource = MoveTemp(PreprocessedSourceStripped);
}

void FShaderPreprocessOutput::RemapErrors(FShaderCompilerOutput& Output) const
{
	for (FShaderCompilerError& Error : Output.Errors)
	{
		Remapper.Remap(Error);
	}
}

FArchive& operator<<(FArchive& Ar, FShaderPreprocessOutput& PreprocessOutput)
{
	Ar << PreprocessOutput.bSucceeded;
	Ar << PreprocessOutput.bIsSecondary;
	Ar << PreprocessOutput.Errors;
	Ar << PreprocessOutput.PragmaDirectives;
	Ar << PreprocessOutput.PreprocessedSource;
	Ar << PreprocessOutput.ShaderDiagnosticDatas;
	return Ar;
}
