// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ShaderResource.cpp: ShaderResource implementation.
=============================================================================*/

#include "Shader.h"
#include "Compression/OodleDataCompression.h"
#include "Containers/ArrayView.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "Misc/Compression.h"
#include "Misc/CoreMisc.h"
#include "Misc/StringBuilder.h"
#include "Interfaces/ITargetPlatform.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#include "Interfaces/IShaderFormat.h"
#include "IO/IoHash.h"
#include "Misc/MemStack.h"
#include "Misc/ScopeLock.h"
#include "RenderingThread.h"
#include "RHI.h"
#include "RHIResources.h"	// Access to FRHIRayTracingShader::RayTracingPayloadType requires this
#include "ShaderCompilerCore.h"
#include "ShaderCompilerJobTypes.h"
#include "ShaderCore.h"
#include "ShaderSerialization.h"
#include "UObject/RenderingObjectVersion.h"

#if WITH_EDITORONLY_DATA
#include "Interfaces/IShaderFormat.h"
#endif

#if (CSV_PROFILER_STATS && !UE_BUILD_SHIPPING) 
TCsvPersistentCustomStat<int>* CsvStatNumShaderMapsUsedForRendering = nullptr;
#endif

FName GetShaderCompressionFormat()
{
	// We always use oodle now. This was instituted because UnrealPak recompresses the shaders and doens't have
	// access to the INIs that drive the CVars and would always use default, resulting in mismatches for non
	// default encoder selection.
	return NAME_Oodle;
}

void GetShaderCompressionOodleSettings(FOodleDataCompression::ECompressor& OutCompressor, FOodleDataCompression::ECompressionLevel& OutLevel, const FName& ShaderFormat)
{
	// support an older developer-only CVar for compatibility and make it preempt
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	// Since we always use Oodle, we make SkipCompression tell Oodle to not compress.
	static const IConsoleVariable* CVarSkipCompression = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Shaders.SkipCompression"));
	static bool bSkipCompression = (CVarSkipCompression && CVarSkipCompression->GetInt() != 0);
	if (UNLIKELY(bSkipCompression))
	{
		OutCompressor = FOodleDataCompression::ECompressor::Selkie;
		OutLevel = FOodleDataCompression::ECompressionLevel::None;
		return;
	}
#endif

	// We just use mermaid/normal here since these settings get overwritten in unrealpak, so this is just for non pak'd builds.
	OutCompressor = FOodleDataCompression::ECompressor::Mermaid;
	OutLevel = FOodleDataCompression::ECompressionLevel::Normal;
}

bool FShaderMapResource::ArePlatformsCompatible(EShaderPlatform CurrentPlatform, EShaderPlatform TargetPlatform)
{
	bool bFeatureLevelCompatible = CurrentPlatform == TargetPlatform;

	if (!bFeatureLevelCompatible && IsPCPlatform(CurrentPlatform) && IsPCPlatform(TargetPlatform))
	{
		bFeatureLevelCompatible = GetMaxSupportedFeatureLevel(CurrentPlatform) >= GetMaxSupportedFeatureLevel(TargetPlatform);

		bool const bIsTargetD3D = IsD3DPlatform(TargetPlatform);

		bool const bIsCurrentPlatformD3D = IsD3DPlatform(CurrentPlatform);

		// For Metal in Editor we can switch feature-levels, but not in cooked projects when using Metal shader librariss.
		bool const bIsCurrentMetal = IsMetalPlatform(CurrentPlatform);
		bool const bIsTargetMetal = IsMetalPlatform(TargetPlatform);
		bool const bIsMetalCompatible = (bIsCurrentMetal == bIsTargetMetal)
#if !WITH_EDITOR	// Static analysis doesn't like (|| WITH_EDITOR)
			&& (!IsMetalPlatform(CurrentPlatform) || (CurrentPlatform == TargetPlatform))
#endif
			;

		bool const bIsCurrentOpenGL = IsOpenGLPlatform(CurrentPlatform);
		bool const bIsTargetOpenGL = IsOpenGLPlatform(TargetPlatform);

		bFeatureLevelCompatible = bFeatureLevelCompatible && (bIsCurrentPlatformD3D == bIsTargetD3D && bIsMetalCompatible && bIsCurrentOpenGL == bIsTargetOpenGL);
	}

	return bFeatureLevelCompatible;
}

#if RHI_RAYTRACING
class FRayTracingShaderLibrary
{
public:
	uint32 AddShader(EShaderPlatform ShaderPlatform, FRHIRayTracingShader* Shader)
	{
		const int32 PayloadIndex = FMath::CountTrailingZeros(Shader->RayTracingPayloadType);
		FScopeLock Lock(&CS);

		FShaderLibrary& ShaderLibrary = ShadersPerPlatform.FindOrAdd(ShaderPlatform);
		if (ShaderLibrary.UnusedIndicies[PayloadIndex].Num() != 0)
		{
			uint32 Index = ShaderLibrary.UnusedIndicies[PayloadIndex].Pop(EAllowShrinking::No);
			checkSlow(ShaderLibrary.Shaders[PayloadIndex][Index] == nullptr);
			ShaderLibrary.Shaders[PayloadIndex][Index] = Shader;
			return Index;
		}
		else
		{
			return ShaderLibrary.Shaders[PayloadIndex].Add(Shader);
		}
	}

	void RemoveShader(EShaderPlatform ShaderPlatform, uint32 Index, FRHIRayTracingShader* Shader)
	{
		if (Index != ~0u)
		{
			const int32 PayloadIndex = FMath::CountTrailingZeros(Shader->RayTracingPayloadType);

			FScopeLock Lock(&CS);
			FShaderLibrary* ShaderLibrary = ShadersPerPlatform.Find(ShaderPlatform);
			check(ShaderLibrary);
			checkSlow(ShaderLibrary->Shaders[PayloadIndex][Index] == Shader);
			ShaderLibrary->UnusedIndicies[PayloadIndex].Push(Index);
			ShaderLibrary->Shaders[PayloadIndex][Index] = nullptr;
		}
	}

	void GetShaders(EShaderPlatform ShaderPlatform, TArray<FRHIRayTracingShader*>& OutShaders, FRHIRayTracingShader* DefaultShader)
	{
		const int32 PayloadIndex = FMath::CountTrailingZeros(DefaultShader->RayTracingPayloadType);
		const int32 BaseOutIndex = OutShaders.Num();

		FScopeLock Lock(&CS);

		FShaderLibrary* ShaderLibrary = ShadersPerPlatform.Find(ShaderPlatform);
		if (ShaderLibrary)
		{
			OutShaders.Append(ShaderLibrary->Shaders[PayloadIndex]);

			for (uint32 Index : ShaderLibrary->UnusedIndicies[PayloadIndex])
			{
				OutShaders[BaseOutIndex + Index] = DefaultShader;
			}
		}
	}

private:
	struct FShaderLibrary
	{
		TArray<uint32> UnusedIndicies[32];
		TArray<FRHIRayTracingShader*> Shaders[32];
	};
	TMap<EShaderPlatform, FShaderLibrary> ShadersPerPlatform;
	FCriticalSection CS;
};

static FRayTracingShaderLibrary GlobalRayTracingHitGroupLibrary;
static FRayTracingShaderLibrary GlobalRayTracingCallableShaderLibrary;
static FRayTracingShaderLibrary GlobalRayTracingMissShaderLibrary;

void FShaderMapResource::GetRayTracingHitGroupLibrary(EShaderPlatform ShaderPlatform, TArray<FRHIRayTracingShader*>& RayTracingShaders, FRHIRayTracingShader* DefaultShader)
{
	GlobalRayTracingHitGroupLibrary.GetShaders(ShaderPlatform, RayTracingShaders, DefaultShader);
}

void FShaderMapResource::GetRayTracingCallableShaderLibrary(EShaderPlatform ShaderPlatform, TArray<FRHIRayTracingShader*>& RayTracingCallableShaders, FRHIRayTracingShader* DefaultShader)
{
	GlobalRayTracingCallableShaderLibrary.GetShaders(ShaderPlatform, RayTracingCallableShaders, DefaultShader);
}

void FShaderMapResource::GetRayTracingMissShaderLibrary(EShaderPlatform ShaderPlatform, TArray<FRHIRayTracingShader*>& RayTracingMissShaders, FRHIRayTracingShader* DefaultShader)
{
	GlobalRayTracingMissShaderLibrary.GetShaders(ShaderPlatform, RayTracingMissShaders, DefaultShader);
}
#endif // RHI_RAYTRACING

static void ApplyResourceStats(FShaderMapResourceCode& Resource)
{
#if STATS
	INC_DWORD_STAT_BY(STAT_Shaders_ShaderResourceMemory, Resource.GetSizeBytes());
	for (const FShaderCodeResource& Shader : Resource.ShaderCodeResources)
	{
		INC_DWORD_STAT_BY_FName(GetMemoryStatType(Shader.GetFrequency()).GetName(), Shader.GetCodeBuffer().GetSize());
	}
#endif // STATS
}

static void RemoveResourceStats(FShaderMapResourceCode& Resource)
{
#if STATS
	DEC_DWORD_STAT_BY(STAT_Shaders_ShaderResourceMemory, Resource.GetSizeBytes());
	for (const FShaderCodeResource& Shader : Resource.ShaderCodeResources)
	{
		DEC_DWORD_STAT_BY_FName(GetMemoryStatType(Shader.GetFrequency()).GetName(), Shader.GetCodeBuffer().GetSize());
	}
#endif // STATS
}

FShaderMapResourceCode::FShaderMapResourceCode(const FShaderMapResourceCode& Other)
{
	ResourceHash = Other.ResourceHash;
	ShaderHashes = Other.ShaderHashes;
	ShaderCodeResources = Other.ShaderCodeResources;

#if WITH_EDITORONLY_DATA
	ShaderEditorOnlyDataEntries = Other.ShaderEditorOnlyDataEntries;
#endif // WITH_EDITORONLY_DATA
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
FShaderMapResourceCode::~FShaderMapResourceCode()
{
	RemoveResourceStats(*this);
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void FShaderMapResourceCode::Finalize()
{
	uint64 Combined = 0;
	for (const FShaderHash& Hash : ShaderHashes)
	{
		Combined += Hash.Hash;
	}
	ResourceHash.Hash = Combined;
	ApplyResourceStats(*this);
}

uint32 FShaderMapResourceCode::GetSizeBytes() const
{
	uint64 Size = sizeof(*this) + ShaderHashes.GetAllocatedSize() + ShaderCodeResources.GetAllocatedSize();
	for (const FShaderCodeResource& Entry : ShaderCodeResources)
	{
		Size += Entry.GetCacheBuffer().GetSize();
	}
	check(Size <= TNumericLimits<uint32>::Max());
	return static_cast<uint32>(Size);
}

int32 FShaderMapResourceCode::FindShaderIndex(const FShaderHash& InHash) const
{
	return Algo::BinarySearch(ShaderHashes, InHash);
}

#if WITH_EDITORONLY_DATA
void FShaderMapResourceCode::FShaderEditorOnlyDataEntry::ConditionalSetSymbolBuffer(FCompressedBuffer InSymbols)
{
	// Intentional truncation of hash; 20 bytes is overkill for this purpose so we just take the first 8
	const uint64 InSymbolHash = reinterpret_cast<uint64*>(InSymbols.GetRawHash().GetBytes())[0];
	if (SymbolBuffer.IsNull() || InSymbolHash < SymbolHash)
	{
		SymbolBuffer = InSymbols;
		SymbolHash = InSymbolHash;
	}
}
#endif

void FShaderMapResourceCode::AddShaderCompilerOutput(const FShaderCompilerOutput& Output, const FShaderCompileJobKey& Key, FString DebugInfo)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FShaderMapResourceCode::AddShaderCode);

	const FShaderHash& InHash = Output.OutputHash;
	FString DebugName = Key.ToString();
	uint64 ShaderTypeHash = Key.ShaderType->GetHashedName().GetHash();
	const int32 Index = Algo::LowerBound(ShaderHashes, InHash);
	const FShaderCodeResource& CodeResource = Output.GetFinalizedCodeResource();

#if WITH_EDITORONLY_DATA
	TArray<FGenericShaderStat> ShaderStatsSorted(Output.ShaderStatistics);
	ShaderStatsSorted.StableSort();
#endif

	if (Index >= ShaderHashes.Num() || ShaderHashes[Index] != InHash)
	{
		ShaderHashes.Insert(InHash, Index);

#if WITH_EDITORONLY_DATA
		// Output.Errors contains warnings in the case any exist (no errors since if there were the job would have failed)
		AddEditorOnlyData(Index, DebugName, ShaderTypeHash, CodeResource.GetSymbolsBuffer(), MoveTemp(ShaderStatsSorted), DebugInfo);
#endif

		ShaderCodeResources.Insert(CodeResource, Index);
	}
#if WITH_EDITORONLY_DATA
	else
	{
		// Output.Errors contains warnings in the case any exist (no errors since if there were the job would have failed)
		// We append the warnings and deduplicate other data like DebugInfo for any additional jobs which resulted in the
		// same bytecode for the sake of determinism in the results saved to DDC.
		UpdateEditorOnlyData(Index, DebugName, ShaderTypeHash, CodeResource.GetSymbolsBuffer(), DebugInfo);
		ValidateShaderStatisticsEditorOnlyData(Index, ShaderStatsSorted);
	}
#endif
}

#if WITH_EDITORONLY_DATA
void FShaderMapResourceCode::AddEditorOnlyData(int32 Index, const FString& DebugName, uint64 ShaderTypeHash, FCompressedBuffer InSymbols, TArray<FGenericShaderStat>&& ShaderStatistics, const FString& DebugInfo)
{
	FShaderEditorOnlyDataEntry& Entry = ShaderEditorOnlyDataEntries.InsertDefaulted_GetRef(Index);

	// This should be a newly created shader entry.
	check(Entry.ShaderStatistics.Num() == 0);
	Entry.ShaderStatistics = MoveTemp(ShaderStatistics);

	UpdateEditorOnlyData(Index, DebugName, ShaderTypeHash, InSymbols, DebugInfo);
}

void FShaderMapResourceCode::UpdateEditorOnlyData(int32 Index, const FString& DebugName, uint64 ShaderTypeHash, FCompressedBuffer InSymbols, const FString& DebugInfo)
{
	FShaderEditorOnlyDataEntry& Entry = ShaderEditorOnlyDataEntries[Index];

	// Keep a single DebugInfo as it doesn't matter which one we use, but make sure it is the same one for determinism
	if (!DebugInfo.IsEmpty() && (Entry.DebugInfo.IsEmpty() || (DebugInfo < Entry.DebugInfo)))
	{
		Entry.DebugInfo = DebugInfo;
	}

	Entry.ConditionalSetSymbolBuffer(InSymbols);

	const int32 HashIndex = Algo::LowerBound(Entry.ShaderTypeHashes, ShaderTypeHash);
	if (HashIndex >= Entry.ShaderTypeHashes.Num() || Entry.ShaderTypeHashes[HashIndex] != ShaderTypeHash)
	{
		Entry.ShaderTypeHashes.Insert(ShaderTypeHash, HashIndex);
	}
}

void FShaderMapResourceCode::ValidateShaderStatisticsEditorOnlyData(int32 Index, const TArray<FGenericShaderStat>& ShaderStatistics)
{
	check(ShaderEditorOnlyDataEntries.IsValidIndex(Index));
	FShaderEditorOnlyDataEntry& Entry = ShaderEditorOnlyDataEntries[Index];

	auto PrintMismatchWarning = [](FShaderEditorOnlyDataEntry& Entry, const TArray<FGenericShaderStat>& ShaderStatistics)
		{
			TStringBuilder<512> WarningMessage;
			WarningMessage << TEXT("Non-determinism detected in shader statistics; existing entry contains:\n");
			WarningMessage.Join(Entry.ShaderStatistics, TEXT(", "));
			WarningMessage << TEXT("\nNew entry contains:\n");
			WarningMessage.Join(ShaderStatistics, TEXT(", "));
			WarningMessage << TEXT("\nShader statistics are assumed to be a property of the bytecode and so must match when bytecode matches.");
			UE_LOGF(LogShaders, Warning, "%ls", WarningMessage.ToString());
		};

	if (Entry.ShaderStatistics.Num() != ShaderStatistics.Num())
	{
		PrintMismatchWarning(Entry, ShaderStatistics);
		return;
	}

	for (int i = 0; i < ShaderStatistics.Num(); ++i)
	{
		const FGenericShaderStat& StatA = Entry.ShaderStatistics[i];
		const FGenericShaderStat& StatB = ShaderStatistics[i];
		if (!(StatA == StatB))
		{
			PrintMismatchWarning(Entry, ShaderStatistics);
			return;
		}
	}
}

#endif // WITH_EDITORONLY_DATA

void FShaderMapResourceCode::ToString(FStringBuilderBase& OutString) const
{
	OutString.Appendf(TEXT("Shaders: Num=%d\n"), ShaderHashes.Num());
	for (int32 i = 0; i < ShaderHashes.Num(); ++i)
	{
		const FShaderCodeResource& Res = ShaderCodeResources[i];
		OutString.Appendf(TEXT("    [%d]: { Hash: %s, Freq: %s, Size: %llu, UncompressedSize: %d }\n"),
			i, *ShaderHashes[i].ToString(), GetShaderFrequencyString(Res.GetFrequency()), Res.GetCodeBuffer().GetSize(), Res.GetUncompressedSize());
	}
}

void FShaderMapResourceCode::Serialize(FShaderSerializeContext& Ctx)
{
	FArchive& Ar = Ctx.GetMainArchive();
	Ar << ResourceHash;
	Ar << ShaderHashes;
	if (!Ctx.EnableCustomCodeSerialize())
	{
		Ar << ShaderCodeResources;
	}
	else
	{
		if (Ar.IsLoading())
		{
			ShaderCodeResources.SetNum(ShaderHashes.Num());
		}
	
		Ctx.ReserveCode(ShaderCodeResources.Num());

		for (int32 CodeIndex = 0; CodeIndex < ShaderCodeResources.Num(); ++CodeIndex)
		{
			Ctx.SerializeCode(ShaderCodeResources[CodeIndex], CodeIndex);
		}
	}
	check(ShaderCodeResources.Num() == ShaderHashes.Num());
#if WITH_EDITORONLY_DATA
	const bool bSerializeEditorOnlyData = !Ctx.bLoadingCooked && (!Ar.IsCooking() || Ar.CookingTarget()->HasEditorOnlyData());
	if (bSerializeEditorOnlyData)
	{
		Ar << ShaderEditorOnlyDataEntries;
		if (Ar.IsLoading())
		{
			// only need to set SymbolBuffer when loading, saving is handled automatically by the serialization of ShaderCodeResources
			// (this just sets the FSharedBuffer reference in a second place to avoid needing to fix up a bunch of code which retrieves
			// symbols from the editoronly data)
			for (int32 CodeIndex = 0; CodeIndex < ShaderCodeResources.Num(); ++CodeIndex)
			{
				ShaderEditorOnlyDataEntries[CodeIndex].ConditionalSetSymbolBuffer(ShaderCodeResources[CodeIndex].GetSymbolsBuffer());
			}
		}
	}
#endif // WITH_EDITORONLY_DATA
	ApplyResourceStats(*this);
}

#if WITH_EDITORONLY_DATA
void FShaderMapResourceCode::NotifyShadersCompiled(FName FormatName)
{
#if WITH_ENGINE
	// Notify the platform shader format that this particular shader is being used in the cook.
	// We discard this data in cooked builds unless Ar.CookingTarget()->HasEditorOnlyData() is true.
	if (ShaderEditorOnlyDataEntries.Num())
	{
		if (const IShaderFormat* ShaderFormat = GetTargetPlatformManagerRef().FindShaderFormat(FormatName))
		{
			for (const FShaderEditorOnlyDataEntry& Entry : ShaderEditorOnlyDataEntries)
			{
				ShaderFormat->NotifyShaderCompiled(Entry.SymbolBuffer, FormatName, Entry.DebugInfo);
			}
		}
	}
#endif // WITH_ENGINE
}
#endif // WITH_EDITORONLY_DATA

FShaderMapResource::FShaderMapResource(EShaderPlatform InPlatform, int32 NumShaders)
	: NumRHIShaders(static_cast<uint32>(NumShaders))
	, bAtLeastOneRHIShaderCreated(0)
	, Platform(InPlatform)
	, NumRefs(0)
{
	RHIShaders = MakeUnique<std::atomic<FRHIShader*>[]>(NumRHIShaders); // this MakeUnique() zero-initializes the array
#if RHI_RAYTRACING
	if (GRHISupportsRayTracing && GRHISupportsRayTracingShaders)
	{
		RayTracingLibraryIndices.AddUninitialized(NumShaders);
		FMemory::Memset(RayTracingLibraryIndices.GetData(), 0xff, NumShaders * RayTracingLibraryIndices.GetTypeSize());
	}
#endif // RHI_RAYTRACING
}

FShaderMapResource::~FShaderMapResource()
{
	ReleaseShaders();
	check(NumRefs.load(std::memory_order_relaxed) == 0);
}

void FShaderMapResource::AddRef()
{
	NumRefs.fetch_add(1, std::memory_order_relaxed);
}

uint32 FShaderMapResource::GetShaderSizeBytes(int32 ShaderIndex) const
{
	// default FShaderMapResource does not contain inline shader bytecode
	return 0;
}

void FShaderMapResource::Release()
{
	check(NumRefs.load(std::memory_order_relaxed) > 0);
	if (NumRefs.fetch_sub(1, std::memory_order_release) - 1 == 0 && TryRelease())
	{
		//check https://www.boost.org/doc/libs/1_55_0/doc/html/atomic/usage_examples.html for explanation
		std::atomic_thread_fence(std::memory_order_acquire);
		// Send a release message to the rendering thread when the shader loses its last reference.
		BeginReleaseResource(this);
		BeginCleanup(this);

		DEC_DWORD_STAT_BY(STAT_Shaders_ShaderResourceMemory, GetSizeBytes());
	}
}

void FShaderMapResource::ReleaseShaders()
{
	// Note that this function must not reset the RHIShaders pointer nor NumRHIShaders. This will be destroyed automatically in the destructor.
	// This function could also be used by a subclass to releases its RHI shaders to allow them to be re-created after they have been moved to a new shader library when the old one was unmounted.
	FScopeLock ScopeLock(&RHIShadersCreationGuard);

	if (bAtLeastOneRHIShaderCreated)
	{
		int NumReleaseShaders = 0;

		for (uint32 Idx = 0; Idx < NumRHIShaders; ++Idx)
		{
			if (FRHIShader* Shader = ExchangeShader(Idx, nullptr))
			{
				Shader->Release();
				NumReleaseShaders++;
				DEC_DWORD_STAT(STAT_Shaders_NumShadersCreated);
			}
		}

#if (CSV_PROFILER_STATS && !UE_BUILD_SHIPPING) 
		TCsvPersistentCustomStat<int>* CsvStatNumShadersCreated = FCsvProfiler::Get()->GetOrCreatePersistentCustomStatInt(TEXT("NumShadersCreated"), CSV_CATEGORY_INDEX(Shaders));
		CsvStatNumShadersCreated->Sub(NumReleaseShaders);
#endif

		{
			DEC_DWORD_STAT(STAT_Shaders_NumShaderMapsUsedForRendering);

#if (CSV_PROFILER_STATS && !UE_BUILD_SHIPPING) 
			if (CsvStatNumShaderMapsUsedForRendering == nullptr)
			{
				CsvStatNumShaderMapsUsedForRendering = FCsvProfiler::Get()->GetOrCreatePersistentCustomStatInt(TEXT("NumShaderMapsUsedForRendering"), CSV_CATEGORY_INDEX(Shaders));
			}
			CsvStatNumShaderMapsUsedForRendering->Sub(1);
#endif
		}
		bAtLeastOneRHIShaderCreated = false;
	}
}


void FShaderMapResource::ReleaseRHI()
{
#if RHI_RAYTRACING
	if (GRHISupportsRayTracing && GRHISupportsRayTracingShaders)
	{
		check(NumRHIShaders == static_cast<uint32>(RayTracingLibraryIndices.Num()));

		for (uint32 Idx = 0; Idx < NumRHIShaders; ++Idx)
		{
			if (FRHIShader* Shader = RHIShaders[Idx].load(std::memory_order_acquire))
			{
				int32 IndexInLibrary = RayTracingLibraryIndices[Idx];
				switch (Shader->GetFrequency())
				{
				case SF_RayHitGroup:
					GlobalRayTracingHitGroupLibrary.RemoveShader(Platform, IndexInLibrary, static_cast<FRHIRayTracingShader*>(Shader));
					break;
				case SF_RayCallable:
					GlobalRayTracingCallableShaderLibrary.RemoveShader(Platform, IndexInLibrary, static_cast<FRHIRayTracingShader*>(Shader));
					break;
				case SF_RayMiss:
					GlobalRayTracingMissShaderLibrary.RemoveShader(Platform, IndexInLibrary, static_cast<FRHIRayTracingShader*>(Shader));
					break;
				default:
					break;
				}
			}
		}
	}
	RayTracingLibraryIndices.Empty();
#endif // RHI_RAYTRACING

	ReleaseShaders();
}

void FShaderMapResource::BeginCreateAllShaders()
{
	FShaderMapResource* Resource = this;
	ENQUEUE_RENDER_COMMAND(InitCommand)(
		[Resource](FRHICommandListImmediate& RHICmdList)
	{
		for (int32 ShaderIndex = 0; ShaderIndex < Resource->GetNumShaders(); ++ShaderIndex)
		{
			Resource->GetShader(ShaderIndex, true /*bRequired*/);
		}
	});
}

FRHIShader* FShaderMapResource::CreateShaderOrCrash(int32 ShaderIndex, bool bRequired)
{
	FRHIShader* Shader = nullptr;
	// create before taking the lock. This may cause multiple creations, but it's better
	// than a potential oversubscription deadlock, since CreateShader can spawn async tasks
	FRHIShader* CreatedShader = CreateRHIShaderOrCrash(ShaderIndex, bRequired);	// guaranteed to return non-null if required is set
	if (CreatedShader == nullptr)
	{
		check(!bRequired);
		return nullptr;
	}

	{
		// Most shadermaps have <100 shaders, and less than a half of them can be created. 
		// However, if this path is often contended, you can slice this lock (but remember to take care of STAT_Shaders_NumShaderMapsUsedForRendering!)
		FScopeLock ScopeLock(&RHIShadersCreationGuard);

		Shader = RHIShaders[ShaderIndex].load(std::memory_order_relaxed);
		if (UNLIKELY(Shader == nullptr))
		{
			Shader = CreatedShader;
			CreatedShader = nullptr;

			if (!bAtLeastOneRHIShaderCreated)
			{
				INC_DWORD_STAT(STAT_Shaders_NumShaderMapsUsedForRendering);

#if (CSV_PROFILER_STATS && !UE_BUILD_SHIPPING) 
				if (CsvStatNumShaderMapsUsedForRendering == nullptr)
				{
					CsvStatNumShaderMapsUsedForRendering = FCsvProfiler::Get()->GetOrCreatePersistentCustomStatInt(TEXT("NumShaderMapsUsedForRendering"), CSV_CATEGORY_INDEX(Shaders));
				}
				CsvStatNumShaderMapsUsedForRendering->Add(1);
#endif
				bAtLeastOneRHIShaderCreated = 1;
			}

#if RHI_RAYTRACING
			// Registers RT shaders in global "libraries" that track all shaders potentially usable in a scene for adding to RTPSO
			EShaderFrequency Frequency = Shader->GetFrequency();
			if (LIKELY(GRHISupportsRayTracing && GRHISupportsRayTracingShaders))
			{
				switch (Frequency)
				{
					case SF_RayHitGroup:
						RayTracingLibraryIndices[ShaderIndex] = GlobalRayTracingHitGroupLibrary.AddShader(Platform, static_cast<FRHIRayTracingShader*>(Shader));
						break;
					case SF_RayCallable:
						RayTracingLibraryIndices[ShaderIndex] = GlobalRayTracingCallableShaderLibrary.AddShader(Platform, static_cast<FRHIRayTracingShader*>(Shader));
						break;
					case SF_RayMiss:
						RayTracingLibraryIndices[ShaderIndex] = GlobalRayTracingMissShaderLibrary.AddShader(Platform, static_cast<FRHIRayTracingShader*>(Shader));
						break;
					case SF_RayGen:
						// NOTE: we do not maintain a library for raygen shaders since the list of rayshaders we care about is usually small and consistent
						break;
					default:
						break;
				}
			}
#endif // RHI_RAYTRACING

			RHIShaders[ShaderIndex].store(Shader, std::memory_order_release);

			// When using shader library, shader code is usually preloaded during the material load. Release it
			// since we won't need it anymore for this shader.
			ReleasePreloadedShaderCode(ShaderIndex);
		}
	}

	if (LIKELY(CreatedShader))
	{
		// free redundantly created shader
		checkSlow(Shader != nullptr);
		CreatedShader->Release();
	}

	return Shader;
}

FShaderHash FShaderMapResource_InlineCode::GetShaderHash(int32 ShaderIndex)
{
	return Code->ShaderHashes[ShaderIndex];
}

FRHIShader* FShaderMapResource_InlineCode::CreateRHIShaderOrCrash(int32 ShaderIndex, bool bRequired)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FShaderMapResource_InlineCode::CreateRHIShaderOrCrash);

	// we can't have this called on the wrong platform's shaders
	if (!ArePlatformsCompatible(GMaxRHIShaderPlatform, GetPlatform()))
	{
		UE_LOGF(LogShaders, Fatal, "FShaderMapResource_InlineCode::InitRHI got platform %ls but it is not compatible with %ls",
			*LegacyShaderPlatformToShaderFormat(GetPlatform()).ToString(), *LegacyShaderPlatformToShaderFormat(GMaxRHIShaderPlatform).ToString());
		// unreachable
		return nullptr;
	}

	FMemStackBase& MemStack = FMemStack::Get();
	const FShaderCodeResource& ShaderCodeResource = Code->ShaderCodeResources[ShaderIndex];
	FSharedBuffer ShaderCode = ShaderCodeResource.GetCodeBuffer();
	TConstArrayView<uint8> ShaderCodeView = ShaderCodeResource.GetCodeView();

	FMemMark Mark(MemStack);
	int32 UncompressedSize = ShaderCodeResource.GetUncompressedSize();
	if (ShaderCode.GetSize() != UncompressedSize)
	{
		void* UncompressedCode = MemStack.Alloc(UncompressedSize, 16);
		bool bSucceed = FCompression::UncompressMemory(GetShaderCompressionFormat(), UncompressedCode, UncompressedSize, ShaderCode.GetData(), ShaderCode.GetSize());
		check(bSucceed);
		ShaderCodeView = MakeArrayView(reinterpret_cast<const uint8*>(UncompressedCode), UncompressedSize);
	}

	FRHICreateShaderDesc CreateShaderDesc = FRHICreateShaderDesc(ShaderCodeView);
	const EShaderFrequency Frequency = ShaderCodeResource.GetFrequency();

	TRefCountPtr<FRHIShader> RHIShader;
	switch (Frequency)
	{
	case SF_Vertex: RHIShader = RHICreateVertexShader(CreateShaderDesc); break;
	case SF_Mesh: RHIShader = RHICreateMeshShader(CreateShaderDesc); break;
	case SF_Amplification: RHIShader = RHICreateAmplificationShader(CreateShaderDesc); break;
	case SF_Pixel: RHIShader = RHICreatePixelShader(CreateShaderDesc); break;
	case SF_Geometry: RHIShader = RHICreateGeometryShader(CreateShaderDesc); break;
	case SF_Compute: RHIShader = RHICreateComputeShader(CreateShaderDesc); break;
	case SF_WorkGraphRoot: RHIShader = RHICreateWorkGraphShader(CreateShaderDesc, SF_WorkGraphRoot); break;
	case SF_WorkGraphComputeNode: RHIShader = RHICreateWorkGraphShader(CreateShaderDesc, SF_WorkGraphComputeNode); break;
	case SF_RayGen: case SF_RayMiss: case SF_RayHitGroup: case SF_RayCallable:
#if RHI_RAYTRACING
		if (GRHISupportsRayTracing && GRHISupportsRayTracingShaders)
		{
			RHIShader = RHICreateRayTracingShader(CreateShaderDesc, Frequency);
		}
#endif // RHI_RAYTRACING
		break;
	default:
		checkNoEntry();
		break;
	}
	const FShaderHash& ShaderHash = Code->ShaderHashes[ShaderIndex];
	if (UNLIKELY(RHIShader == nullptr))
	{
		if (bRequired)
		{
			UE_LOGF(LogShaders, Fatal, "FShaderMapResource_InlineCode::InitRHI is unable to create a shader: frequency=%d, hash=%ls.", static_cast<int32>(Frequency), *ShaderHash.ToString());
		}
		return nullptr;
	}

	INC_DWORD_STAT(STAT_Shaders_NumShadersCreated);

#if (CSV_PROFILER_STATS && !UE_BUILD_SHIPPING) 
	TCsvPersistentCustomStat<int>* CsvStatNumShadersCreated = FCsvProfiler::Get()->GetOrCreatePersistentCustomStatInt(TEXT("NumShadersCreated"), CSV_CATEGORY_INDEX(Shaders));
	CsvStatNumShadersCreated->Add(1);
#endif

	RHIShader->SetHash(ShaderHash);

	// contract of this function is to return a shader with an already held reference
	RHIShader->AddRef();
	return RHIShader;
}

uint32 FShaderMapResource_InlineCode::GetSizeBytes() const
{
	uint32 TotalSize = 0;

	if (Code)
	{
		TotalSize += Code->GetSizeBytes();
	}

	TotalSize += sizeof(FShaderMapResource_InlineCode);
	TotalSize += GetAllocatedSize();

	return TotalSize;
}

uint32 FShaderMapResource_InlineCode::GetShaderSizeBytes(int32 ShaderIndex) const
{
	if (Code)
	{
		return Code->ShaderCodeResources[ShaderIndex].GetUncompressedSize();
	}
	return 0u;
}


