// Copyright Epic Games, Inc. All Rights Reserved.

#include "ShaderSerialization.h"

#include "DerivedDataCacheKey.h"
#include "DerivedDataCacheRecord.h"
#include "DerivedDataValue.h"
#include "Serialization/CompactBinaryWriter.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/MemoryWriter.h"
#include "ShaderCore.h"

FShaderCacheSaveContext::FShaderCacheSaveContext()
{
	Reset();
}

#if WITH_EDITOR
const UE::DerivedData::FValueId ShaderObjectDataValue = UE::DerivedData::FValueId::FromName(TEXT("ShaderObjectData"));
const UE::DerivedData::FValueId ShaderCodeDataValue = UE::DerivedData::FValueId::FromName(TEXT("ShaderCodeData"));
const UE::DerivedData::FValueId ShaderSymbolsDataValue = UE::DerivedData::FValueId::FromName(TEXT("ShaderSymbolsData"));
const FAnsiStringView CodeCountMetaField = ANSITEXTVIEW("CodeCount");
const FAnsiStringView HasSymbolsMetaField = ANSITEXTVIEW("bHasSymbols");
#endif

void FShaderCacheSaveContext::SerializeCode(FShaderCodeResource& Resource, int32 Index)
{
	OwnedShaderCode.Add(Resource.GetCacheBuffer());
	// reset the array view any time an entry is added; we do this instead of calling Resize in the reserve delegate
	// and setting it there since not all code paths (i.e. single job cache records) call reserve
	ShaderCode = OwnedShaderCode;

	OwnedShaderSymbols.Add(Resource.GetSymbolsBuffer());
	ShaderSymbols = OwnedShaderSymbols;

	checkf(ShaderCode.Num() == ShaderSymbols.Num(), TEXT("It is required to serialize a (possibly empty, but non-null) symbols buffer for every code buffer."));
}

void FShaderCacheSaveContext::ReserveCode(int32 Count)
{
	OwnedShaderCode.Reserve(Count);
	OwnedShaderSymbols.Reserve(Count);
}

void FShaderCacheSaveContext::Reset()
{
	ShaderObjectData.Reset();
	OwnedShaderCode.Reset();
	OwnedShaderSymbols.Reset();
	Writer = MakeUnique<FMemoryWriter64>(ShaderObjectRawData);
	Ar = Writer.Get();
}

void FShaderCacheSaveContext::Finalize()
{
	if (!ShaderObjectData)
	{
		ShaderObjectData = MakeSharedBufferFromArray(MoveTemp(ShaderObjectRawData));
	}
}

#if WITH_EDITOR
UE::DerivedData::FCacheRecord FShaderCacheSaveContext::BuildCacheRecord(const UE::DerivedData::FCacheKey& Key)
{
	Finalize();

	UE::DerivedData::FCacheRecordBuilder RecordBuilder(Key);
	RecordBuilder.AddValue(ShaderObjectDataValue, ShaderObjectData);
	int32 CodeIndex = 0;
	// Code buffers are already compressed, don't waste cycles attempting (and failing) to recompress them
	const ECompressedBufferCompressor CodeComp = ECompressedBufferCompressor::NotSet;
	const ECompressedBufferCompressionLevel CodeCompLevel = ECompressedBufferCompressionLevel::None;

	// Use a meta field to indicate whether we are adding symbol values to this record; we do so to prevent per-value overhead of symbol buffers if they are empty
	// (each value in a cache record has a 64 byte overhead, which can be significant when we're pushing millions of individual shader cache records)
	bool bHasSymbols = false;
	for (FCompressedBuffer& SymbolsBuf : ShaderSymbols)
	{
		bHasSymbols |= (SymbolsBuf.GetRawSize() > 0);
	}

	for (FCompositeBuffer& CodeBuf : ShaderCode)
	{
		RecordBuilder.AddValue(ShaderCodeDataValue.MakeIndexed(CodeIndex), UE::DerivedData::FValue(FCompressedBuffer::Compress(CodeBuf, CodeComp, CodeCompLevel)));
		if (bHasSymbols)
		{
			RecordBuilder.AddValue(ShaderSymbolsDataValue.MakeIndexed(CodeIndex), UE::DerivedData::FValue(ShaderSymbols[CodeIndex]));
		}
		CodeIndex++;
	}

	TCbWriter<16> MetaWriter;
	MetaWriter.BeginObject();
	MetaWriter.AddInteger(CodeCountMetaField, ShaderCode.Num());
	MetaWriter.AddBool(HasSymbolsMetaField, bHasSymbols);
	MetaWriter.EndObject();

	RecordBuilder.SetMeta(MetaWriter.Save().AsObject());

	return RecordBuilder.Build();
}
#endif


FShaderCacheLoadContext::FShaderCacheLoadContext(FSharedBuffer InShaderObjectData, TArrayView<FCompositeBuffer> InCodeBuffers, TArrayView<FCompressedBuffer> InSymbolBuffers)
{
	Reset(InShaderObjectData, InCodeBuffers, InSymbolBuffers);
}

void FShaderCacheLoadContext::Reset(FSharedBuffer InShaderObjectData, TArrayView<FCompositeBuffer> InCodeBuffers, TArrayView<FCompressedBuffer> InSymbolBuffers)
{
	ShaderObjectData = InShaderObjectData;
	ShaderCode = InCodeBuffers;
	ShaderSymbols = InSymbolBuffers;

	checkf(ShaderCode.Num() == ShaderSymbols.Num(), TEXT("It is required to serialize a (possibly empty, but non-null) symbols buffer for every code buffer."));

	Reader = MakeUnique<FMemoryReaderView>(ShaderObjectData);
	Ar = Reader.Get();
}

void FShaderCacheLoadContext::SerializeCode(FShaderCodeResource& Resource, int32 Index)
{
	Resource.PopulateFromComposite(ShaderCode[Index], ShaderSymbols[Index]);
}

void FShaderCacheLoadContext::Reuse()
{
	Reader->Seek(0u);
}

#if WITH_EDITOR
void FShaderCacheLoadContext::ReadFromRecord(const UE::DerivedData::FCacheRecord& Record, bool bIsPersistent)
{
	ShaderObjectData = Record.GetValue(ShaderObjectDataValue).GetData().Decompress();
	
	// Must initialize a memory reader (and the base class archive pointer) after reading the base shadermap data buffer
	// from the DDC record
	Reader = MakeUnique<FMemoryReaderView>(ShaderObjectData, bIsPersistent);
	Ar = Reader.Get();
	
	int32 CodeCount = Record.GetMeta()[CodeCountMetaField].AsInt32();
	OwnedShaderCode.Reserve(CodeCount);
	OwnedShaderSymbols.Reserve(CodeCount);

	const bool bHasSymbols = Record.GetMeta()[HasSymbolsMetaField].AsBool();

	for (int32 CodeIndex = 0; CodeIndex < CodeCount; ++CodeIndex)
	{
		FSharedBuffer CombinedBuffer = Record.GetValue(ShaderCodeDataValue.MakeIndexed(CodeIndex)).GetData().Decompress();
		OwnedShaderCode.Add(FShaderCodeResource::Unpack(CombinedBuffer));
		if (bHasSymbols)
		{
			UE::DerivedData::FValueWithId Value = Record.GetValue(ShaderSymbolsDataValue.MakeIndexed(CodeIndex));
			FCompressedBuffer SymbolsBuffer = Value.GetData();
			OwnedShaderSymbols.Add(SymbolsBuffer);
		}
		else
		{
			// We need to set an empty symbol buffer if symbols were not saved to the cache, instead of a null one
			// This is due to FCompressedBuffer serialization not supporting null buffers
			static FCompressedBuffer Empty = FCompressedBuffer::Compress(FUniqueBuffer::Alloc(0).MoveToShared());
			OwnedShaderSymbols.Add(Empty);
		}
	}
	ShaderCode = OwnedShaderCode;
	ShaderSymbols = OwnedShaderSymbols;
}
#endif

