// Copyright Epic Games, Inc. All Rights Reserved.

#include "ShaderSource.h"
#include "Containers/StringConv.h"
#include "Compression/OodleDataCompression.h"
#include "HAL/IConsoleManager.h"
#include "HAL/UnrealMemory.h"

static int32 GShaderSourceCompressionMethod = 2;
static FAutoConsoleVariableRef CVarShaderSourceCompressionMethod(
	TEXT("r.ShaderSource.CompressionMethod"),
	GShaderSourceCompressionMethod,
	TEXT("Compression method for shader source stored in memory. See FOodleDataCompression::ECompressor enum for supported values; defaults to Mermaid."),
	ECVF_Default);

static int32 GShaderSourceCompressionLevel = 1;
FAutoConsoleVariableRef CVarShaderSourceCompressionLevel(
	TEXT("r.ShaderSource.CompressionLevel"),
	GShaderSourceCompressionLevel,
	TEXT("Compression level for shader source stored in memory. See FOodleDataCompression::ECompressionLevel enum for supported values; default is SuperFast."),
	ECVF_Default);

FShaderSource::FShaderSource(FShaderSource::FViewType InSrc, int32 AdditionalSlack)
{
	Set(InSrc, AdditionalSlack);
}

void FShaderSource::Set(FShaderSource::FViewType InSrc, int32 AdditionalSlack)
{
	SetLen(InSrc.Len() + AdditionalSlack);
	FShaderSource::CharType* SourceData = Source.GetData();
	FMemory::Memcpy(SourceData, InSrc.GetData(), sizeof(FShaderSource::CharType) * InSrc.Len());
}

FShaderSource& FShaderSource::operator=(FShaderSource::FStringType&& InSrc)
{
	SourceCompressed.Empty();
	DecompressedCharCount = 0;

	int32 InInitialLen = InSrc.Len();
	// input char array already has a null terminator appended, so can add one less padding char
	TArray<CharType>& InSrcArray = InSrc.GetCharArray();
	InSrcArray.AddZeroed(ShaderSourceSimdPadding - 1);
	Source = MoveTemp(InSrcArray);
	return *this;
}

void FShaderSource::Compress()
{
	FOodleDataCompression::ECompressor Compressor = static_cast<FOodleDataCompression::ECompressor>(GShaderSourceCompressionMethod);
	if (Compressor == FOodleDataCompression::ECompressor::NotSet)
	{
		return;
	}

	checkf(!IsCompressed(), TEXT("FShaderSource is already compressed"));

	FOodleDataCompression::ECompressionLevel CompressionLevel = static_cast<FOodleDataCompression::ECompressionLevel>(GShaderSourceCompressionLevel);
	
	DecompressedCharCount = Source.Num();
	int32 CompressionBufferSize = FOodleDataCompression::CompressedBufferSizeNeeded(GetDecompressedSize());
	SourceCompressed.SetNumUninitialized(CompressionBufferSize);

	int32 CompressedSize = FOodleDataCompression::Compress(
		SourceCompressed.GetData(), 
		CompressionBufferSize, 
		Source.GetData(), 
		GetDecompressedSize(),
		Compressor,
		CompressionLevel);

	SourceCompressed.SetNum(CompressedSize, EAllowShrinking::Yes);
	Source.Empty();
}

void FShaderSource::Decompress()
{
	if (IsCompressed())
	{
		Source.SetNumUninitialized(DecompressedCharCount);
		FOodleDataCompression::Decompress(Source.GetData(), GetDecompressedSize(), SourceCompressed.GetData(), SourceCompressed.Num());
		SourceCompressed.Empty();
		DecompressedCharCount = 0;
	}
}

FArchive& operator<<(FArchive& Ar, FShaderSource& Src)
{
	Ar << Src.DecompressedCharCount;
	if (Src.IsCompressed())
	{
		Ar << Src.SourceCompressed;
	}
	else
	{
		Ar << Src.Source;
	}

	return Ar;
}

