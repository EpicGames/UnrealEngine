// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * Code for reading .shk files and using that information to map shader hashes back to human-readable identifies.
 */

#include "ShaderStableKeyDebugInfoReader.h"
#include "PipelineCacheUtilities.h"
#include "RHIResources.h"

// Disable in Shipping just in case (also shaderpipelineutils aren't available there)
#define UE_WITH_SHK_READER			(!UE_BUILD_SHIPPING)

bool UE::ShaderUtils::FShaderStableKeyDebugInfoReader::Initialize(const FString& ShaderStableKeyFile)
{
	bool bResult = false;

#if UE_WITH_SHK_READER
	TArray<FStableShaderKeyAndValue> StableArray;
	if (UE::PipelineCacheUtilities::LoadStableKeysFile(ShaderStableKeyFile, StableArray))
	{
		ShaderHashesToSource.Empty();
		for (const FStableShaderKeyAndValue& StableKey : StableArray)
		{
			TSet<FStableShaderKeyAndValue>& DedupShaders = ShaderHashesToSource.FindOrAdd(StableKey.OutputHash);
			DedupShaders.Add(StableKey);
		}

		bResult = true;
	}
#endif // UE_WITH_SHK_READER

	return bResult;
}

FString UE::ShaderUtils::FShaderStableKeyDebugInfoReader::GetShaderStableNameOptions(const FShaderHash& ShaderHash, int32 MaxOptionsToInclude)
{
	FString Output;	// FIXME: change to builder

#if UE_WITH_SHK_READER
	if (TSet<FStableShaderKeyAndValue>* SourceSet = ShaderHashesToSource.Find(ShaderHash))
	{
		Output += FString::Printf(TEXT("hash %s maps to %d shaders"), *ShaderHash.ToString(), SourceSet->Num());

		if (SourceSet->Num() > MaxOptionsToInclude)
		{
			Output += FString::Printf(TEXT(", limiting output to %d"), MaxOptionsToInclude);
		}
		Output += TEXT(":\n");

		int32 OptionsToInclude = FMath::Min(MaxOptionsToInclude, SourceSet->Num());
		for (const FStableShaderKeyAndValue& StableKey : *SourceSet)
		{
			if (--OptionsToInclude < 0)
			{
				break;
			}
			Output += FString::Printf(TEXT("\t%s\n"), *StableKey.ToString());
		}
	}
	else
	{
		Output += FString::Printf(TEXT("hash %s not in the key file(s)! (were key file(s) produced in the same cook as this build?)\n"), *ShaderHash.ToString());
	}
#endif // UE_WITH_SHK_READER

	return Output;
}

FString UE::ShaderUtils::FShaderStableKeyDebugInfoReader::GetPSOStableNameOptions(const FGraphicsPipelineStateInitializer& Initializer, int32 MaxOptionsToInclude)
{
	FString Output;	// FIXME: change to builder

#if UE_WITH_SHK_READER

#define UE_FPTSM_DUMPSHADER(ShaderType) \
	if (FRHIShader* Shader = Initializer.BoundShaderState.Get##ShaderType()) \
	{ \
		const FShaderHash& Hash = Shader->GetHash(); \
		Output += FString::Printf(TEXT("%s: %s"), TEXT(#ShaderType), *GetShaderStableNameOptions(Hash, MaxOptionsToInclude)); \
	}

	Output += FString::Printf(TEXT("---- begin FGraphicsPipelineStateInitializer -----\n"));
	UE_FPTSM_DUMPSHADER(VertexShader);
	UE_FPTSM_DUMPSHADER(MeshShader);
	UE_FPTSM_DUMPSHADER(AmplificationShader);
	UE_FPTSM_DUMPSHADER(PixelShader);
	UE_FPTSM_DUMPSHADER(GeometryShader);

#undef UE_FPTSM_DUMPSHADER

	Output += FString::Printf(TEXT("----- end -----\n"));

#endif // UE_WITH_SHK_READER

	return Output;
}

void UE::ShaderUtils::FShaderStableKeyDebugInfoReader::DumpPSOToLogIfConfigured(const FGraphicsPipelineStateInitializer& Initializer)
{
#if UE_WITH_SHK_READER
	UE_LOGF(LogInit, Log, "%ls", *GetPSOStableNameOptions(Initializer));
#endif // UE_WITH_SHK_READER
}
