// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	StaticBoundShaderState.cpp: Static bound shader state implementation.
=============================================================================*/

#include "StaticBoundShaderState.h"
#include "RenderingThread.h"

PRAGMA_DISABLE_DEPRECATION_WARNINGS

TLinkedList<FGlobalBoundShaderStateResource*>*& FGlobalBoundShaderStateResource::GetGlobalBoundShaderStateList()
{
	static TLinkedList<FGlobalBoundShaderStateResource*>* List = NULL;
	return List;
}

FGlobalBoundShaderStateResource::FGlobalBoundShaderStateResource()
	: GlobalListLink(this)
#if DO_CHECK
	, BoundVertexDeclaration(nullptr)
	, BoundVertexShader(nullptr)
	, BoundPixelShader(nullptr)
	, BoundGeometryShader(nullptr)
#endif 
{
	// Add this resource to the global list in the rendering thread.
	if(IsInRenderingThread())
	{
		GlobalListLink.LinkHead(GetGlobalBoundShaderStateList());
	}
	else
	{
		FGlobalBoundShaderStateResource* Resource = this;
		ENQUEUE_RENDER_COMMAND(LinkGlobalBoundShaderStateResource)(
			[Resource](FRHICommandList& RHICmdList)
			{
				Resource->GlobalListLink.LinkHead(GetGlobalBoundShaderStateList());
			});
	}
}

FGlobalBoundShaderStateResource::~FGlobalBoundShaderStateResource()
{
	// Remove this resource from the global list.
	GlobalListLink.Unlink();
}

void FGlobalBoundShaderStateResource::ReleaseRHI()
{
	// Release the cached bound shader state.
	BoundShaderState.SafeRelease();
}

PRAGMA_ENABLE_DEPRECATION_WARNINGS
