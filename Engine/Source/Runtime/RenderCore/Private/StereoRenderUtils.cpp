// Copyright Epic Games, Inc. All Rights Reserved.

#include "StereoRenderUtils.h"
#include "Misc/App.h"
#include "ShaderPlatformCachedIniValue.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "RHI.h"
#include "RenderUtils.h"

// enable this to printf-debug stereo rendering aspects on a device
#define UE_DEBUG_STEREO_ASPECTS			(!UE_BUILD_SHIPPING && !UE_BUILD_TEST && !UE_EDITOR)

#if UE_DEBUG_STEREO_ASPECTS
	#define UE_DEBUG_SSA_LOG_INIT(Platform) \
	bool bSSALogEnable = false; \
	{ \
		static EShaderPlatform LastLogged[2] = { SP_NumPlatforms, SP_NumPlatforms };	\
		if (UNLIKELY(LastLogged[0] != Platform && LastLogged[1] != Platform)) \
		{ \
			LastLogged[1] = LastLogged[0]; \
			LastLogged[0] = Platform; \
			bSSALogEnable = FApp::CanEverRender(); \
		} \
	}

	#define UE_DEBUG_SSA_LOG(Verbosity, Format, ...) \
	{ \
		if (UNLIKELY(bSSALogEnable)) \
		{ \
			UE_LOGF(LogInit, Verbosity, "FStereoShaderAspects: %ls", *FString::Printf(Format, ##__VA_ARGS__)); \
		} \
	}
#else
	#define UE_DEBUG_SSA_LOG_INIT(Platform)
	#define UE_DEBUG_SSA_LOG(Verbosity, Format, ...)
#endif

#define UE_DEBUG_SSA_LOG_BOOL(Boolean) \
{ \
	UE_DEBUG_SSA_LOG(Log, TEXT(#Boolean) TEXT(" = %d"), Boolean); \
}

namespace UE::StereoRenderUtils
{
static bool bInstancedStereoIsMultiViewport = true;	
static FAutoConsoleVariableRef CVarInstancedStereoIsMultiViewport(
	TEXT("r.InstancedStereoIsMultiViewport"),
	bInstancedStereoIsMultiViewport,
	TEXT("If true, considers instanced stereo to mean multiviewport since there are now no other instanced stereo paths"),
	ECVF_Scalability | ECVF_RenderThreadSafe);
	
PRAGMA_DISABLE_DEPRECATION_WARNINGS
RENDERCORE_API FStereoShaderAspects::FStereoShaderAspects(EShaderPlatform Platform) :
	bInstancedStereoEnabled(false)
	, bMobileMultiViewEnabled(false)
	, bInstancedMultiViewportEnabled(false)
	, bInstancedStereoNative(false)
	, bMobileMultiViewNative(false)
{
	check(Platform < EShaderPlatform::SP_NumPlatforms);
	UE_DEBUG_SSA_LOG_INIT(Platform);
	
	// Would be nice to use URendererSettings, but not accessible in RenderCore
	static FShaderPlatformCachedIniValue<bool> CVarInstancedStereo(TEXT("vr.InstancedStereo"));
	static FShaderPlatformCachedIniValue<bool> CVarMobileMultiView(TEXT("vr.MobileMultiView"));
	
	const bool bInstancedStereo = CVarInstancedStereo.Get(Platform);

	const bool bMobilePlatform = IsMobilePlatform(Platform);
	const bool bMobilePostprocessing = IsMobileHDR();
	const bool bMobileMultiView = CVarMobileMultiView.Get(Platform);
	// If we're in a non-rendering run (cooker, DDC commandlet, anything with -nullrhi), don't check GRHI* setting, as it reflects runtime RHI capabilities.
	const bool bMultiViewportCapable = (GRHISupportsArrayIndexFromAnyShader || !FApp::CanEverRender()) && RHISupportsMultiViewport(Platform);

	bInstancedStereoNative = !bMobilePlatform && bInstancedStereo && RHISupportsInstancedStereo(Platform);

	UE_DEBUG_SSA_LOG(Log, TEXT("--- StereoAspects begin ---"));
	UE_DEBUG_SSA_LOG(Log, TEXT("Platform=%s (%d)"), *LexToString(Platform), static_cast<int32>(Platform));
	UE_DEBUG_SSA_LOG_BOOL(bInstancedStereo);
	UE_DEBUG_SSA_LOG_BOOL(bMobilePlatform);
	UE_DEBUG_SSA_LOG_BOOL(bMobilePostprocessing);
	UE_DEBUG_SSA_LOG_BOOL(bMobileMultiView);
	UE_DEBUG_SSA_LOG_BOOL(bMultiViewportCapable);
	UE_DEBUG_SSA_LOG_BOOL(bInstancedStereoNative);
	UE_DEBUG_SSA_LOG(Log, TEXT("---"));

	const bool bMobileMultiViewCoreSupport = bMobilePlatform && bMobileMultiView; 
	if (bMobileMultiViewCoreSupport)
	{
		UE_DEBUG_SSA_LOG(Log, TEXT("RHISupportsMobileMultiView(%s) = %d."), *LexToString(Platform), RHISupportsMobileMultiView(Platform));
		UE_DEBUG_SSA_LOG(Log, TEXT("RHISupportsInstancedStereo(%s) = %d."), *LexToString(Platform), RHISupportsInstancedStereo(Platform));

		if (RHISupportsMobileMultiView(Platform))
		{
			bMobileMultiViewNative = true;
		}
	}

	UE_DEBUG_SSA_LOG_BOOL(bMobileMultiViewCoreSupport);
	UE_DEBUG_SSA_LOG_BOOL(bMobileMultiViewNative);
	UE_DEBUG_SSA_LOG(Log, TEXT("---"));

	// DEPRECATED: Since instanced stereo now relies on multi-viewport capability, it cannot be separately enabled from it.
	bInstancedMultiViewportEnabled = bInstancedStereoNative && bMultiViewportCapable;

	bInstancedStereoEnabled = bInstancedStereoIsMultiViewport ? bInstancedStereoNative && bMultiViewportCapable : bInstancedStereoNative;
	bMobileMultiViewEnabled = bMobileMultiViewNative;

	UE_DEBUG_SSA_LOG_BOOL(bInstancedStereoEnabled);
	UE_DEBUG_SSA_LOG_BOOL(bMobileMultiViewEnabled);
	UE_DEBUG_SSA_LOG(Log, TEXT("--- StereoAspects end ---"));

	// check the following invariants
	checkf(!bMobileMultiViewNative || !bInstancedStereoEnabled, TEXT("When a platform supports MMV natively, ISR should not be enabled."));
	
	if (!bInstancedStereoIsMultiViewport)
	{
		checkf(!bInstancedStereoEnabled || bInstancedMultiViewportEnabled, TEXT("If ISR is enabled, we need either multi-viewport (since we no longer support clip-distance method)."));
	}
}

RENDERCORE_API FStereoShaderAspects::FStereoShaderAspects() :
	bInstancedStereoEnabled(false)
	, bMobileMultiViewEnabled(false)
	, bInstancedMultiViewportEnabled(false)
	, bInstancedStereoNative(false)
	, bMobileMultiViewNative(false)
{}

FStereoShaderAspects::FStereoShaderAspects(const FStereoShaderAspects& Copy) :
	bInstancedStereoEnabled(Copy.bInstancedStereoEnabled)
	, bMobileMultiViewEnabled(Copy.IsMobileMultiViewEnabled())
	, bInstancedMultiViewportEnabled(Copy.bInstancedMultiViewportEnabled)
	, bInstancedStereoNative(Copy.bInstancedStereoNative)
	, bMobileMultiViewNative(Copy.bMobileMultiViewNative)
{}

bool FStereoShaderAspects::IsInstancedMultiViewportEnabled() const
{
	return bInstancedMultiViewportEnabled;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

RENDERCORE_API void LogISRInit(const UE::StereoRenderUtils::FStereoShaderAspects& Aspects)
{
	UE_LOGF(LogInit, Log, "XR: Instanced Stereo Rendering is %ls", (Aspects.IsInstancedStereoEnabled() ? TEXT("Enabled") : TEXT("Disabled")));
	UE_LOGF(LogInit, Log, "XR: Mobile Multiview is %ls", (Aspects.IsMobileMultiViewEnabled() ? TEXT("Enabled") : TEXT("Disabled")));
}

} // namespace UE::StereoRenderUtils
