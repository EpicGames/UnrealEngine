// Copyright Epic Games, Inc. All Rights Reserved.

#include "TickableObjectRenderThread.h"
#include "RHICommandList.h"

/** Static array of tickable objects that are ticked from rendering thread*/
FTickableObjectRenderThread::FRenderingThreadTickableObjectsArray FTickableObjectRenderThread::RenderingThreadTickableObjects;
FTickableObjectRenderThread::FRenderingThreadTickableObjectsArray FTickableObjectRenderThread::RenderingThreadHighFrequencyTickableObjects;

void TickHighFrequencyTickables(FRHICommandListImmediate& RHICmdList, double CurTime)
{
	static double LastHighFreqTime = FPlatformTime::Seconds();
	float DeltaSecondsHighFreq = float(CurTime - LastHighFreqTime);

	// tick any high frequency rendering thread tickables.
	for (int32 ObjectIndex = 0; ObjectIndex < FTickableObjectRenderThread::RenderingThreadHighFrequencyTickableObjects.Num(); ObjectIndex++)
	{
		FTickableObjectRenderThread* TickableObject = FTickableObjectRenderThread::RenderingThreadHighFrequencyTickableObjects[ObjectIndex];
		// make sure it wants to be ticked and the rendering thread isn't suspended
		if (TickableObject->IsTickable())
		{
			STAT(FScopeCycleCounter(TickableObject->GetStatId());)
			TickableObject->Tick(RHICmdList, DeltaSecondsHighFreq);
		}
	}

	LastHighFreqTime = CurTime;
}

extern float GRenderingThreadMaxIdleTickFrequency;

void TickRenderingTickables(FRHICommandListImmediate& RHICmdList)
{
	static double LastTickTime = FPlatformTime::Seconds();

	// calc how long has passed since last tick
	double CurTime = FPlatformTime::Seconds();
	float DeltaSeconds = float(CurTime - LastTickTime);

	TickHighFrequencyTickables(RHICmdList, CurTime);

	if (DeltaSeconds < (1.f/GRenderingThreadMaxIdleTickFrequency))
	{
		return;
	}

	// tick any rendering thread tickables
	for (int32 ObjectIndex = 0; ObjectIndex < FTickableObjectRenderThread::RenderingThreadTickableObjects.Num(); ObjectIndex++)
	{
		FTickableObjectRenderThread* TickableObject = FTickableObjectRenderThread::RenderingThreadTickableObjects[ObjectIndex];
		// make sure it wants to be ticked and the rendering thread isn't suspended
		if (TickableObject->IsTickable())
		{
			STAT(FScopeCycleCounter(TickableObject->GetStatId());)
			TickableObject->Tick(RHICmdList, DeltaSeconds);
		}
	}

	// update the last time we ticked
	LastTickTime = CurTime;
}

// DEPRECATED
void TickRenderingTickables()
{
	TickRenderingTickables(FRHICommandListImmediate::Get());
}

