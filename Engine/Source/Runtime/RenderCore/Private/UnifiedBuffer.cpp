// Copyright Epic Games, Inc. All Rights Reserved.

#include "UnifiedBuffer.h"
#include "Containers/ResourceArray.h"
#include "RHI.h"
#include "ShaderParameters.h"
#include "ShaderParameterStruct.h"
#include "Shader.h"
#include "GlobalShader.h"
#include "RenderGraphUtils.h"
#include "RHIResourceUtils.h"
#include "DataDrivenShaderPlatformInfo.h"

// Uploads use a storage buffers which are at least 128m elements
static uint64 GetMaxUploadBufferElements()
{
	return (1 << 27);
}

enum class EByteBufferResourceType
{
	Float4_Buffer,
	StructuredBuffer,
	Uint_Buffer,
	Uint4Aligned_Buffer,
	Count
};

enum class EByteBufferStructuredSize
{
	Uint1,
	Uint2,
	Uint4,
	Uint8,
	Count
};

// placeholder struct, not really used (on the host side)
struct FUint8
{
	uint32 Values[8];
};

class FByteBufferShader : public FGlobalShader
{
	DECLARE_INLINE_TYPE_LAYOUT(FByteBufferShader, NonVirtual);

	FByteBufferShader() {}
	FByteBufferShader(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FGlobalShader(Initializer)
	{}

	class ResourceTypeDim : SHADER_PERMUTATION_INT("RESOURCE_TYPE", (int)EByteBufferResourceType::Count);
	class StructuredElementSizeDim : SHADER_PERMUTATION_INT("STRUCTURED_ELEMENT_SIZE", (int)EByteBufferStructuredSize::Count);

	using FPermutationDomain = TShaderPermutationDomain<ResourceTypeDim, StructuredElementSizeDim>;

	static bool ShouldCompilePermutation( const FGlobalShaderPermutationParameters& Parameters )
	{
		FPermutationDomain PermutationVector( Parameters.PermutationId );

		EByteBufferResourceType ResourceType = (EByteBufferResourceType)PermutationVector.Get<ResourceTypeDim>();

		if (ResourceType == EByteBufferResourceType::Uint_Buffer || ResourceType == EByteBufferResourceType::Uint4Aligned_Buffer)
		{
			return true;
		}
		// Don't compile structured buffer size variations unless we need them
		else if (ResourceType != EByteBufferResourceType::StructuredBuffer && static_cast<EByteBufferStructuredSize>(PermutationVector.Get<StructuredElementSizeDim>()) != EByteBufferStructuredSize::Uint4)
		{
			return false;
		}
		else
		{
			return true;
		}

	}

	BEGIN_SHADER_PARAMETER_STRUCT( FParameters, )
		SHADER_PARAMETER(uint32, Value)
		SHADER_PARAMETER(uint32, Size)
		SHADER_PARAMETER(uint32, SrcOffset)
		SHADER_PARAMETER(uint32, DstOffset)
		SHADER_PARAMETER_UAV(RWBuffer<float4>, DstBuffer)
		SHADER_PARAMETER_UAV(RWStructuredBuffer<float4>, DstStructuredBuffer4x)
		SHADER_PARAMETER_UAV(RWByteAddressBuffer, DstByteAddressBuffer)
	END_SHADER_PARAMETER_STRUCT()
};

class FMemsetBufferCS : public FByteBufferShader
{
	DECLARE_GLOBAL_SHADER( FMemsetBufferCS );
	SHADER_USE_PARAMETER_STRUCT( FMemsetBufferCS, FByteBufferShader );

	static bool ShouldCompilePermutation( const FGlobalShaderPermutationParameters& Parameters )
	{
		FPermutationDomain PermutationVector( Parameters.PermutationId );
		// Don't compile structured buffer size variations
		if (static_cast<EByteBufferStructuredSize>(PermutationVector.Get<StructuredElementSizeDim>()) != EByteBufferStructuredSize::Uint4)
		{
			return false;
		}
		return FByteBufferShader::ShouldCompilePermutation(Parameters);
	}
};
IMPLEMENT_GLOBAL_SHADER( FMemsetBufferCS, "/Engine/Private/ByteBuffer.usf", "MemsetBufferCS", SF_Compute );

class FMemcpyCS : public FByteBufferShader
{
	DECLARE_GLOBAL_SHADER( FMemcpyCS );
	SHADER_USE_PARAMETER_STRUCT( FMemcpyCS, FByteBufferShader );

	static bool ShouldCompilePermutation( const FGlobalShaderPermutationParameters& Parameters )
	{
		FPermutationDomain PermutationVector( Parameters.PermutationId );
		// Don't compile structured buffer size variations
		if (static_cast<EByteBufferStructuredSize>(PermutationVector.Get<StructuredElementSizeDim>()) != EByteBufferStructuredSize::Uint4)
		{
			return false;
		}
		return FByteBufferShader::ShouldCompilePermutation(Parameters);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FByteBufferShader::FParameters, Common)
		SHADER_PARAMETER_SRV(Buffer<float4>, SrcBuffer)
		SHADER_PARAMETER_SRV(StructuredBuffer<float4>, SrcStructuredBuffer4x)
		SHADER_PARAMETER_SRV(ByteAddressBuffer, SrcByteAddressBuffer)
	END_SHADER_PARAMETER_STRUCT()
};
IMPLEMENT_GLOBAL_SHADER( FMemcpyCS, "/Engine/Private/ByteBuffer.usf", "MemcpyCS", SF_Compute );

class FScatterCopyCS : public FByteBufferShader
{
	DECLARE_GLOBAL_SHADER( FScatterCopyCS );
	SHADER_USE_PARAMETER_STRUCT( FScatterCopyCS, FByteBufferShader );

	static bool ShouldCompilePermutation( const FGlobalShaderPermutationParameters& Parameters )
	{
		FPermutationDomain PermutationVector( Parameters.PermutationId );
		// Don't compile structured buffer size variations
		if (static_cast<EByteBufferStructuredSize>(PermutationVector.Get<StructuredElementSizeDim>()) != EByteBufferStructuredSize::Uint4)
		{
			return false;
		}
		return FByteBufferShader::ShouldCompilePermutation(Parameters);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FByteBufferShader::FParameters, Common)
		SHADER_PARAMETER(uint32, NumScatters)
		SHADER_PARAMETER_SRV(ByteAddressBuffer, UploadByteAddressBuffer)
		SHADER_PARAMETER_SRV(StructuredBuffer<float4>, UploadStructuredBuffer4x)
		SHADER_PARAMETER_SRV(ByteAddressBuffer, ScatterByteAddressBuffer)
		SHADER_PARAMETER_SRV(StructuredBuffer<uint>, ScatterStructuredBuffer)
	END_SHADER_PARAMETER_STRUCT()
};
IMPLEMENT_GLOBAL_SHADER( FScatterCopyCS, "/Engine/Private/ByteBuffer.usf", "ScatterCopyCS", SF_Compute );

enum class EResourceType
{
	BUFFER,
	STRUCTURED_BUFFER,
	BYTEBUFFER
};

template<typename ResourceType>
struct ResourceTypeTraits;

template<>
struct ResourceTypeTraits<FRWBuffer>
{
	static const EResourceType Type = EResourceType::BUFFER;
};

template<>
struct ResourceTypeTraits<FRWBufferStructured>
{
	static const EResourceType Type = EResourceType::STRUCTURED_BUFFER;
};

template<>
struct ResourceTypeTraits<FRWByteAddressBuffer>
{
	static const EResourceType Type = EResourceType::BYTEBUFFER;
};

///////////////////////////////////////////////////////////////////////////////////////////////////

class FRDGByteBufferShader : public FGlobalShader
{
	DECLARE_INLINE_TYPE_LAYOUT(FRDGByteBufferShader, NonVirtual);

	FRDGByteBufferShader() {}
	FRDGByteBufferShader(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FGlobalShader(Initializer)
	{}

	using ResourceTypeDim = FByteBufferShader::ResourceTypeDim;
	using StructuredElementSizeDim = FByteBufferShader::StructuredElementSizeDim;
	using FPermutationDomain = FByteBufferShader::FPermutationDomain;

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return FByteBufferShader::ShouldCompilePermutation(Parameters);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(uint32, Value)
		SHADER_PARAMETER(uint32, Size)
		SHADER_PARAMETER(uint32, SrcOffset)
		SHADER_PARAMETER(uint32, DstOffset)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<float4>, DstBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<FUint8>, DstStructuredBuffer8x)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<uint4>, DstStructuredBuffer4x)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<uint2>, DstStructuredBuffer2x)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<uint>, DstStructuredBuffer1x)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWByteAddressBuffer, DstByteAddressBuffer)
	END_SHADER_PARAMETER_STRUCT()
};

class FRDGMemsetBufferCS : public FRDGByteBufferShader
{
	DECLARE_GLOBAL_SHADER(FRDGMemsetBufferCS);
	SHADER_USE_PARAMETER_STRUCT(FRDGMemsetBufferCS, FRDGByteBufferShader);
};
IMPLEMENT_GLOBAL_SHADER(FRDGMemsetBufferCS, "/Engine/Private/ByteBuffer.usf", "MemsetBufferCS", SF_Compute);

class FRDGMemcpyCS : public FRDGByteBufferShader
{
	DECLARE_GLOBAL_SHADER(FRDGMemcpyCS);
	SHADER_USE_PARAMETER_STRUCT(FRDGMemcpyCS, FRDGByteBufferShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FRDGByteBufferShader::FParameters, Common)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float4>, SrcBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<FUint8>, SrcStructuredBuffer8x)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint4>, SrcStructuredBuffer4x)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint2>, SrcStructuredBuffer2x)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint>, SrcStructuredBuffer1x)
		SHADER_PARAMETER_RDG_BUFFER_SRV(ByteAddressBuffer, SrcByteAddressBuffer)
	END_SHADER_PARAMETER_STRUCT()
};
IMPLEMENT_GLOBAL_SHADER(FRDGMemcpyCS, "/Engine/Private/ByteBuffer.usf", "MemcpyCS", SF_Compute);

class FRDGScatterCopyCS : public FRDGByteBufferShader
{
	DECLARE_GLOBAL_SHADER(FRDGScatterCopyCS);
	SHADER_USE_PARAMETER_STRUCT(FRDGScatterCopyCS, FRDGByteBufferShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FRDGByteBufferShader::FParameters, Common)
		SHADER_PARAMETER(uint32, NumScatters)
		SHADER_PARAMETER_RDG_BUFFER_SRV(ByteAddressBuffer, UploadByteAddressBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<FUint8>, UploadStructuredBuffer8x)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint4>, UploadStructuredBuffer4x)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint2>, UploadStructuredBuffer2x)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint>, UploadStructuredBuffer1x)
		SHADER_PARAMETER_RDG_BUFFER_SRV(ByteAddressBuffer, ScatterByteAddressBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint>, ScatterStructuredBuffer)
	END_SHADER_PARAMETER_STRUCT()
};
IMPLEMENT_GLOBAL_SHADER(FRDGScatterCopyCS, "/Engine/Private/ByteBuffer.usf", "ScatterCopyCS", SF_Compute);

EResourceType GetBufferType(FRDGBuffer* Buffer)
{
	const FRDGBufferDesc& Desc = Buffer->Desc;

	if (EnumHasAnyFlags(Desc.Usage, EBufferUsageFlags::ByteAddressBuffer))
	{
		return EResourceType::BYTEBUFFER;
	}
	else if (EnumHasAnyFlags(Desc.Usage, EBufferUsageFlags::StructuredBuffer))
	{
		return EResourceType::STRUCTURED_BUFFER;
	}
	else
	{
		return EResourceType::BUFFER;
	}
}

EResourceType GetResourceType(FRDGViewableResource* Resource)
{
	check(Resource);
	switch (Resource->Type)
	{
	case ERDGViewableResourceType::Buffer:
		return GetBufferType(GetAsBuffer(Resource));
	}
	checkNoEntry();
	return EResourceType::BUFFER;
}

EPixelFormat GetBufferPixelFormat(FRDGBuffer* Buffer)
{
	return EnumHasAnyFlags(Buffer->Desc.Usage, EBufferUsageFlags::ByteAddressBuffer | EBufferUsageFlags::StructuredBuffer) ? PF_Unknown : PF_A32B32G32R32F;
}

void MemsetResource(FRDGBuilder& GraphBuilder, FRDGBuffer* DstResource, const FMemsetResourceParams& Params)
{
	MemsetResource(GraphBuilder, GraphBuilder.CreateUAV(DstResource, GetBufferPixelFormat(DstResource), ERDGUnorderedAccessViewFlags::SkipBarrier), Params);
}

void MemcpyResource(FRDGBuilder& GraphBuilder, FRDGBuffer* DstResource, FRDGBuffer* SrcResource, const FMemcpyResourceParams& Params)
{
	MemcpyResource(GraphBuilder, GraphBuilder.CreateUAV(DstResource, GetBufferPixelFormat(DstResource), ERDGUnorderedAccessViewFlags::SkipBarrier), GraphBuilder.CreateSRV(SrcResource, GetBufferPixelFormat(SrcResource)), Params);
}

static EByteBufferStructuredSize GetStructuredBufferElementSize(FRDGBufferUAV * RDGBufferUAV)
{
	int32 BytesPerElement = RDGBufferUAV->Desc.Buffer->Desc.BytesPerElement;
	uint32 Log2NumElements = FMath::FloorLog2(BytesPerElement / 4);
	checkf((BytesPerElement % 4) == 0 && FMath::IsPowerOfTwo(BytesPerElement / 4) && Log2NumElements < uint32(EByteBufferStructuredSize::Count), TEXT("Unsupported structured buffer BytesPerElement size (%d) for buffer '%s' (supported sizes are 4,8,16,32)."), BytesPerElement, RDGBufferUAV->Name);

	return static_cast<EByteBufferStructuredSize>(Log2NumElements);

}

void MemsetResource(FRDGBuilder& GraphBuilder, FRDGUnorderedAccessView* UAV, const FMemsetResourceParams& Params)
{
	check(UAV);
	FRDGViewableResource* Resource = UAV->GetParent();

	EByteBufferResourceType ResourceTypeEnum = EByteBufferResourceType::Count;
	// This is only used for structured buffers, since that is where we must match the format specified when it is created / used or whatever the platform happens to care about.
	EByteBufferStructuredSize ByteBufferStructuredSize = EByteBufferStructuredSize::Uint4;

	auto* PassParameters = GraphBuilder.AllocParameters<FRDGMemsetBufferCS::FParameters>();
	PassParameters->Value = Params.Value;
	PassParameters->Size = Params.Count;
	PassParameters->DstOffset = Params.DstOffset;

	// each thread will set 4 floats / uints
	uint32 Divisor = 1;

	switch (GetResourceType(Resource))
	{
	case EResourceType::BYTEBUFFER:
		ResourceTypeEnum = EByteBufferResourceType::Uint_Buffer;
		PassParameters->DstByteAddressBuffer = GetAs<FRDGBufferUAV>(UAV);
		Divisor = 4;
		break;
	case EResourceType::BUFFER:
		ResourceTypeEnum = EByteBufferResourceType::Float4_Buffer;
		PassParameters->DstBuffer = GetAs<FRDGBufferUAV>(UAV);
		break;
	case EResourceType::STRUCTURED_BUFFER:
	{
		ResourceTypeEnum = EByteBufferResourceType::StructuredBuffer;
		FRDGBufferUAV * RDGBufferUAV = GetAs<FRDGBufferUAV>(UAV);
		ByteBufferStructuredSize = GetStructuredBufferElementSize(RDGBufferUAV);
		switch(ByteBufferStructuredSize)
		{
		case EByteBufferStructuredSize::Uint1:
			PassParameters->DstStructuredBuffer1x = RDGBufferUAV;
			break;
		case EByteBufferStructuredSize::Uint2:
			PassParameters->DstStructuredBuffer2x = RDGBufferUAV;
			break;
		case EByteBufferStructuredSize::Uint4:
			PassParameters->DstStructuredBuffer4x = RDGBufferUAV;
			break;
		case EByteBufferStructuredSize::Uint8:
			PassParameters->DstStructuredBuffer8x = RDGBufferUAV;
			break;
		default:
			break;
		};
		break;
	}
	default:
		checkNoEntry();
	}

	FRDGMemsetBufferCS::FPermutationDomain PermutationVector;
	PermutationVector.Set<FRDGMemsetBufferCS::ResourceTypeDim >((int)ResourceTypeEnum);
	PermutationVector.Set<FRDGMemsetBufferCS::StructuredElementSizeDim >((int)ByteBufferStructuredSize);
	TShaderMapRef<FRDGMemsetBufferCS> ComputeShader(GetGlobalShaderMap(GMaxRHIFeatureLevel), PermutationVector);

	FComputeShaderUtils::AddPass(
		GraphBuilder,
		RDG_EVENT_NAME("MemsetResource (%s)", Resource->Name),
		ComputeShader,
		PassParameters,
		FComputeShaderUtils::GetGroupCountWrapped(FMath::DivideAndRoundUp(Params.Count, Divisor), 64));
}

void MemsetResource(FRDGBuilder& GraphBuilder, FRDGBufferUAV* UAV, const FMemsetResourceParams& Params)
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	MemsetResource(GraphBuilder, static_cast<FRDGUnorderedAccessView*>(UAV), Params);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void MemcpyResource(FRDGBuilder& GraphBuilder, FRDGUnorderedAccessView* UAV, FRDGShaderResourceView* SRV, const FMemcpyResourceParams& Params)
{
	check(UAV && SRV);
	FRDGViewableResource* DstResource = UAV->GetParent();
	FRDGViewableResource* SrcResource = SRV->GetParent();

	const EResourceType ResourceType = GetResourceType(DstResource);
	checkf(ResourceType == GetResourceType(SrcResource), TEXT("Unable to MemcpyResource because the source and destination view types don't match."));

	// each thread will copy 4 floats / uints
	const uint32 Divisor = ResourceType == EResourceType::BYTEBUFFER ? 4 : 1;

	{
		EByteBufferResourceType ResourceTypeEnum = EByteBufferResourceType::Count;
		EByteBufferStructuredSize ByteBufferStructuredSize = EByteBufferStructuredSize::Uint4;

		auto* PassParameters = GraphBuilder.AllocParameters<FRDGMemcpyCS::FParameters>();
		PassParameters->Common.Size = Params.Count;
		PassParameters->Common.SrcOffset = Params.SrcOffset;
		PassParameters->Common.DstOffset = Params.DstOffset;

		switch (ResourceType)
		{
		case EResourceType::BYTEBUFFER:
			ResourceTypeEnum = EByteBufferResourceType::Uint_Buffer;
			PassParameters->SrcByteAddressBuffer           = GetAs<FRDGBufferSRV>(SRV);
			PassParameters->Common.DstByteAddressBuffer    = GetAs<FRDGBufferUAV>(UAV);
			break;
		case EResourceType::STRUCTURED_BUFFER:
		{
			ResourceTypeEnum = EByteBufferResourceType::StructuredBuffer;

			FRDGBufferUAV * RDGBufferUAV = GetAs<FRDGBufferUAV>(UAV);
			ByteBufferStructuredSize = GetStructuredBufferElementSize(RDGBufferUAV);
			switch(ByteBufferStructuredSize)
			{
			case EByteBufferStructuredSize::Uint1:
				PassParameters->SrcStructuredBuffer1x = GetAs<FRDGBufferSRV>(SRV);
				PassParameters->Common.DstStructuredBuffer1x = RDGBufferUAV;
				break;
			case EByteBufferStructuredSize::Uint2:
				PassParameters->SrcStructuredBuffer2x = GetAs<FRDGBufferSRV>(SRV);
				PassParameters->Common.DstStructuredBuffer2x = RDGBufferUAV;
				break;
			case EByteBufferStructuredSize::Uint4:
				PassParameters->SrcStructuredBuffer4x = GetAs<FRDGBufferSRV>(SRV);
				PassParameters->Common.DstStructuredBuffer4x = RDGBufferUAV;
				break;
			case EByteBufferStructuredSize::Uint8:
				PassParameters->SrcStructuredBuffer8x = GetAs<FRDGBufferSRV>(SRV);
				PassParameters->Common.DstStructuredBuffer8x = RDGBufferUAV;
				break;
			default:
				break;
			};
			break;
		}
		case EResourceType::BUFFER:
			ResourceTypeEnum = EByteBufferResourceType::Float4_Buffer;
			PassParameters->SrcBuffer                      = GetAs<FRDGBufferSRV>(SRV);
			PassParameters->Common.DstBuffer               = GetAs<FRDGBufferUAV>(UAV);
			break;
		default:
			checkNoEntry();
		}

		FRDGMemcpyCS::FPermutationDomain PermutationVector;
		PermutationVector.Set<FRDGMemcpyCS::ResourceTypeDim >((int)ResourceTypeEnum);
		PermutationVector.Set<FRDGMemcpyCS::StructuredElementSizeDim >((int)ByteBufferStructuredSize);
		TShaderMapRef<FRDGMemcpyCS> ComputeShader(GetGlobalShaderMap(GMaxRHIFeatureLevel), PermutationVector);

		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("Memcpy %s -> %s", DstResource->Name, SrcResource->Name),
			ComputeShader,
			PassParameters,
			FComputeShaderUtils::GetGroupCountWrapped(FMath::DivideAndRoundUp(Params.Count, Divisor), 64));
	}
}

void MemcpyResource(FRDGBuilder& GraphBuilder, FRDGBufferUAV* UAV, FRDGBufferSRV* SRV, const FMemcpyResourceParams& Params)
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	MemcpyResource(GraphBuilder, static_cast<FRDGUnorderedAccessView*>(UAV), static_cast<FRDGShaderResourceView*>(SRV), Params);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

FRDGBuffer* ResizeBufferIfNeeded(FRDGBuilder& GraphBuilder, TRefCountPtr<FRDGPooledBuffer>& ExternalBuffer, const FRDGBufferDesc& BufferDesc, const TCHAR* Name)
{
	FRDGBuffer* InternalBufferNew = nullptr;

	if (!ExternalBuffer)
	{
		InternalBufferNew = GraphBuilder.CreateBuffer(BufferDesc, Name);
		ExternalBuffer = GraphBuilder.ConvertToExternalBuffer(InternalBufferNew);
		return InternalBufferNew;
	}

	const uint32 BufferSizeNew = BufferDesc.GetSize();
	const uint32 BufferSizeOld = ExternalBuffer->GetCommittedSize();

	FRDGBuffer* InternalBufferOld = GraphBuilder.RegisterExternalBuffer(ExternalBuffer);

	if (BufferSizeNew == BufferSizeOld)
	{
		return InternalBufferOld;
	}

	if (EnumHasAllFlags(ExternalBuffer->Desc.Usage, EBufferUsageFlags::ReservedResource)
		&& ensureMsgf(ExternalBuffer->GetSize() >= BufferSizeNew, TEXT("Reserved buffers can't grow beyond the size specified at creation")))
	{
		GraphBuilder.QueueCommitReservedBuffer(InternalBufferOld, BufferSizeNew);

		return InternalBufferOld;
	}
	else
	{
		RDG_EVENT_SCOPE(GraphBuilder, "ResizeIfNeeded::MemcpyResource %d->%d", BufferSizeOld, BufferSizeNew);

		InternalBufferNew = GraphBuilder.CreateBuffer(BufferDesc, Name);
		ExternalBuffer = GraphBuilder.ConvertToExternalBuffer(InternalBufferNew);

		// Copy data to new buffer
		FMemcpyResourceParams Params;
		Params.Count = FMath::Min(BufferSizeNew, BufferSizeOld) / BufferDesc.BytesPerElement;
		Params.SrcOffset = 0;
		Params.DstOffset = 0;
		MemcpyResource(GraphBuilder, InternalBufferNew, InternalBufferOld, Params);

		return InternalBufferNew;
	}
}

FRDGBuffer* ResizeBufferIfNeeded(FRDGBuilder& GraphBuilder, TRefCountPtr<FRDGPooledBuffer>& ExternalBuffer, EPixelFormat Format, uint32 NumElements, const TCHAR* Name)
{
	const uint32 BytesPerElement = GPixelFormats[Format].BlockBytes;

	return ResizeBufferIfNeeded(GraphBuilder, ExternalBuffer, FRDGBufferDesc::CreateBufferDesc(BytesPerElement, NumElements), Name);
}

FRDGBuffer* ResizeStructuredBufferIfNeeded(FRDGBuilder& GraphBuilder, TRefCountPtr<FRDGPooledBuffer>& ExternalBuffer, uint32 NumBytes, const TCHAR* Name)
{
	const uint32 BytesPerElement = 16;

	check((NumBytes & (BytesPerElement - 1)) == 0);

	const uint32 NumElements = NumBytes / BytesPerElement;

	return ResizeBufferIfNeeded(GraphBuilder, ExternalBuffer, FRDGBufferDesc::CreateStructuredDesc(BytesPerElement, NumElements), Name);
}

FRDGBuffer* ResizeStructuredBufferSOAIfNeeded(FRDGBuilder& GraphBuilder, TRefCountPtr<FRDGPooledBuffer>& ExternalBuffer, const FResizeResourceSOAParams& Params, const TCHAR* Name)
{
	const uint32 BytesPerElement = 16;
	const uint32 ExternalBufferSize = TryGetSize(ExternalBuffer);

	checkf(Params.NumBytes % BytesPerElement == 0, TEXT("NumBytes (%u) must be a multiple of BytesPerElement (%u)"), Params.NumBytes, BytesPerElement);
	checkf(ExternalBufferSize % BytesPerElement == 0, TEXT("NumBytes (%u) must be a multiple of BytesPerElement (%u)"), ExternalBufferSize, BytesPerElement);

	uint32 NumElements = Params.NumBytes / BytesPerElement;
	uint32 NumElementsOld = ExternalBufferSize / BytesPerElement;

	checkf(NumElements % Params.NumArrays == 0, TEXT("NumElements (%u) must be a multiple of NumArrays (%u)"), NumElements, Params.NumArrays);
	checkf(NumElementsOld % Params.NumArrays == 0, TEXT("NumElements (%u) must be a multiple of NumArrays (%u)"), NumElementsOld, Params.NumArrays);

	const FRDGBufferDesc BufferDesc = FRDGBufferDesc::CreateStructuredDesc(BytesPerElement, NumElements);

	FRDGBuffer* InternalBufferNew = nullptr;

	if (!ExternalBuffer)
	{
		InternalBufferNew = GraphBuilder.CreateBuffer(BufferDesc, Name);
		ExternalBuffer = GraphBuilder.ConvertToExternalBuffer(InternalBufferNew);
		return InternalBufferNew;
	}

	FRDGBuffer* InternalBufferOld = GraphBuilder.RegisterExternalBuffer(ExternalBuffer);

	const uint32 BufferSize = BufferDesc.GetSize();
	const uint32 BufferSizeOld = InternalBufferOld->GetSize();

	if (BufferSize != BufferSizeOld)
	{
		InternalBufferNew = GraphBuilder.CreateBuffer(BufferDesc, Name);
		ExternalBuffer = GraphBuilder.ConvertToExternalBuffer(InternalBufferNew);

		FRDGBufferUAV* NewBufferUAV = GraphBuilder.CreateUAV(InternalBufferNew, ERDGUnorderedAccessViewFlags::SkipBarrier);
		FRDGBufferSRV* OldBufferSRV = GraphBuilder.CreateSRV(InternalBufferOld);

		// Copy data to new buffer
		uint32 OldArraySize = NumElementsOld / Params.NumArrays;
		uint32 NewArraySize = NumElements / Params.NumArrays;

		FMemcpyResourceParams MemcpyParams;
		MemcpyParams.Count = FMath::Min(NewArraySize, OldArraySize);

		for (uint32 Index = 0; Index < Params.NumArrays; Index++)
		{
			MemcpyParams.SrcOffset = Index * OldArraySize;
			MemcpyParams.DstOffset = Index * NewArraySize;
			MemcpyResource(GraphBuilder, NewBufferUAV, OldBufferSRV, MemcpyParams);
		}

		return InternalBufferNew;
	}

	return InternalBufferOld;
}

FRDGBuffer* ResizeByteAddressBufferIfNeeded(FRDGBuilder& GraphBuilder, TRefCountPtr<FRDGPooledBuffer>& ExternalBuffer, uint32 NumBytes, const TCHAR* Name)
{
	// Needs to be aligned to 16 bytes to MemcpyResource to work correctly (otherwise it skips last unaligned elements of the buffer during resize)
	check((NumBytes & 15) == 0);

	return ResizeBufferIfNeeded(GraphBuilder, ExternalBuffer, FRDGBufferDesc::CreateByteAddressDesc(NumBytes), Name);
}

#if ENABLE_LOW_LEVEL_MEM_TRACKER || UE_MEMORY_TRACE_ENABLED
FRDGBuffer* ResizeByteAddressBufferIfNeededWithCurrentLLMTag(FRDGBuilder& GraphBuilder, TRefCountPtr<FRDGPooledBuffer>& ExternalBuffer, uint32 NumBytes, const TCHAR* Name)
{
	const bool bCurrentBufferValid = ExternalBuffer && (ExternalBuffer->GetCommittedSize() == NumBytes);
	
	FRDGBuffer* RDGBuffer = ResizeByteAddressBufferIfNeeded(GraphBuilder, ExternalBuffer, NumBytes, Name);
	if (!bCurrentBufferValid && ExternalBuffer && ExternalBuffer->GetRHI())
	{
		GraphBuilder.RHICmdList.UpdateAllocationTags(ExternalBuffer->GetRHI());
	}
	return RDGBuffer;
}
#endif

void FRDGScatterUploadBuffer::Release()
{
	check(!ScatterData);
	ScatterBuffer = nullptr;
	UploadBuffer = nullptr;
}

uint32 FRDGScatterUploadBuffer::GetNumBytes() const
{
	return TryGetSize(ScatterBuffer) + TryGetSize(UploadBuffer);
}

void FRDGScatterUploadBuffer::Init(FRDGBuilder& GraphBuilder, TArrayView<const uint32> ElementScatterOffsets, uint32 InNumBytesPerElement, bool bInFloat4Buffer, const TCHAR* DebugName)
{
	Init(GraphBuilder, ElementScatterOffsets.Num(), InNumBytesPerElement, bInFloat4Buffer, DebugName);
	FMemory::ParallelMemcpy(ScatterData, ElementScatterOffsets.GetData(), ElementScatterOffsets.Num() * ElementScatterOffsets.GetTypeSize(), EMemcpyCachePolicy::StoreUncached);
	NumScatters = ElementScatterOffsets.Num();
}

void FRDGScatterUploadBuffer::InitPreSized(FRDGBuilder& GraphBuilder, uint32 NumElements, uint32 InNumBytesPerElement, bool bInFloat4Buffer, const TCHAR* DebugName)
{
	Init(GraphBuilder, NumElements, InNumBytesPerElement, bInFloat4Buffer, DebugName);
	NumScatters = NumElements;
}

struct FScatterUploadConfig
{
	uint32 NumBytesPerThread;
	uint32 NumThreadsPerScatter;
};

static FScatterUploadConfig GetScatterUploadConfig(uint32 NumBytesPerElement, int32 NumElementsPerScatter = -1)
{
	FScatterUploadConfig Config;
	if (NumElementsPerScatter != INDEX_NONE)
	{
		Config.NumBytesPerThread = NumBytesPerElement;
		Config.NumThreadsPerScatter = NumElementsPerScatter; 
	}
	else
	{
		Config.NumBytesPerThread = (NumBytesPerElement & 15) == 0 ? 16 : 4;
		Config.NumThreadsPerScatter = NumBytesPerElement / Config.NumBytesPerThread;
	}
	return Config;
}

struct FScatterUploadDispatchConfig
{
	uint32 ThreadGroupSize;
	uint32 NumThreads;
	FIntVector GroupCountWrapped;
};

static FScatterUploadDispatchConfig GetScatterUploadDispatchConfig(const FScatterUploadConfig& Config, uint32 NumScatters)
{
	constexpr uint32 ThreadGroupSize = 64u;

	FScatterUploadDispatchConfig DispatchConfig;
	DispatchConfig.ThreadGroupSize = ThreadGroupSize;
	DispatchConfig.NumThreads = NumScatters * Config.NumThreadsPerScatter;
	DispatchConfig.GroupCountWrapped = FComputeShaderUtils::GetGroupCountWrapped(DispatchConfig.NumThreads, DispatchConfig.ThreadGroupSize);
	return DispatchConfig;
}

static void GetScatterCopyParamsAndPermutation(
	FRDGBuilder& GraphBuilder,
	FRDGViewableResource* DstResource,
	FRDGBufferSRV* ScatterBufferSRV,
	FRDGBufferSRV* UploadBufferSRV,
	const FScatterUploadConfig& Config,
	uint32 NumScatters,
	FRDGScatterCopyCS::FParameters& OutParameters,
	FRDGScatterCopyCS::FPermutationDomain& OutPermutationVector)
{
	const EResourceType DstResourceType = GetResourceType(DstResource);

	EByteBufferResourceType ResourceTypeEnum = EByteBufferResourceType::Count;
	EByteBufferStructuredSize ByteBufferStructuredSize = EByteBufferStructuredSize::Uint4;

	OutParameters.Common.Size = Config.NumThreadsPerScatter;
	OutParameters.Common.SrcOffset = 0;
	OutParameters.Common.DstOffset = 0;
	OutParameters.NumScatters = NumScatters;

	if (DstResourceType == EResourceType::BYTEBUFFER)
	{
		if (Config.NumBytesPerThread == 16)
		{
			ResourceTypeEnum = EByteBufferResourceType::Uint4Aligned_Buffer;
		}
		else
		{
			ResourceTypeEnum = EByteBufferResourceType::Uint_Buffer;
		}
		OutParameters.UploadByteAddressBuffer = UploadBufferSRV;
		OutParameters.ScatterByteAddressBuffer = ScatterBufferSRV;
		OutParameters.Common.DstByteAddressBuffer = GraphBuilder.CreateUAV(GetAsBuffer(DstResource), ERDGUnorderedAccessViewFlags::SkipBarrier);
	}
	else if (DstResourceType == EResourceType::STRUCTURED_BUFFER)
	{
		ResourceTypeEnum = EByteBufferResourceType::StructuredBuffer;

		FRDGBufferUAV* RDGBufferUAV = GraphBuilder.CreateUAV(GetAsBuffer(DstResource), ERDGUnorderedAccessViewFlags::SkipBarrier);
		ByteBufferStructuredSize = GetStructuredBufferElementSize(RDGBufferUAV);
		switch (ByteBufferStructuredSize)
		{
		case EByteBufferStructuredSize::Uint1:
			OutParameters.UploadStructuredBuffer1x = UploadBufferSRV;
			OutParameters.Common.DstStructuredBuffer1x = RDGBufferUAV;
			break;
		case EByteBufferStructuredSize::Uint2:
			OutParameters.UploadStructuredBuffer2x = UploadBufferSRV;
			OutParameters.Common.DstStructuredBuffer2x = RDGBufferUAV;
			break;
		case EByteBufferStructuredSize::Uint4:
			OutParameters.UploadStructuredBuffer4x = UploadBufferSRV;
			OutParameters.Common.DstStructuredBuffer4x = RDGBufferUAV;
			break;
		case EByteBufferStructuredSize::Uint8:
			OutParameters.UploadStructuredBuffer8x = UploadBufferSRV;
			OutParameters.Common.DstStructuredBuffer8x = RDGBufferUAV;
			break;
		default:
			break;
		};
		OutParameters.ScatterStructuredBuffer = ScatterBufferSRV;
	}
	else if (DstResourceType == EResourceType::BUFFER)
	{
		ResourceTypeEnum = EByteBufferResourceType::Float4_Buffer;

		OutParameters.UploadStructuredBuffer4x = UploadBufferSRV;
		OutParameters.ScatterStructuredBuffer = ScatterBufferSRV;
		OutParameters.Common.DstBuffer = GraphBuilder.CreateUAV(GetAsBuffer(DstResource), PF_A32B32G32R32F, ERDGUnorderedAccessViewFlags::SkipBarrier);
	}

	OutPermutationVector.Set<FRDGScatterCopyCS::ResourceTypeDim>((int)ResourceTypeEnum);
	OutPermutationVector.Set<FRDGScatterCopyCS::StructuredElementSizeDim >((int)ByteBufferStructuredSize);
}

void ScatterCopyResource(FRDGBuilder& GraphBuilder, FRDGViewableResource* DstResource, FRDGBufferSRV* ScatterBufferSRV, FRDGBufferSRV* UploadBufferSRV, const FScatterCopyParams &Params)
{
	const FScatterUploadConfig Config = GetScatterUploadConfig(Params.NumBytesPerElement, Params.NumElementsPerScatter);
	const FScatterUploadDispatchConfig DispatchConfig = GetScatterUploadDispatchConfig(Config, Params.NumScatters);

	FRDGScatterCopyCS::FParameters Parameters;
	FRDGScatterCopyCS::FPermutationDomain PermutationVector;
	GetScatterCopyParamsAndPermutation(GraphBuilder, DstResource, ScatterBufferSRV, UploadBufferSRV, Config, Params.NumScatters, Parameters, PermutationVector);

	TShaderMapRef<FRDGScatterCopyCS> ComputeShader(GetGlobalShaderMap(GMaxRHIFeatureLevel), PermutationVector);

	FComputeShaderUtils::AddPass(
		GraphBuilder,
		RDG_EVENT_NAME("ScatterUpload (Resource: %s)", DstResource->Name),
		ComputeShader,
		GraphBuilder.AllocParameters(&Parameters),
		DispatchConfig.GroupCountWrapped);
}

void ScatterCopyResource(FRDGBuilder& GraphBuilder, FRDGBuffer* DstResource, FRDGBufferSRV* ScatterBufferSRV, FRDGBufferSRV* UploadBufferSRV, const FAsyncScatterCopyParams& Params)
{
	const FScatterUploadConfig Config = GetScatterUploadConfig(Params.NumBytesPerElement, Params.NumElementsPerScatter);

	FRDGScatterCopyCS::FParameters Parameters;
	FRDGScatterCopyCS::FPermutationDomain PermutationVector;
	GetScatterCopyParamsAndPermutation(GraphBuilder, DstResource, ScatterBufferSRV, UploadBufferSRV, Config, 0, Parameters, PermutationVector);

	TShaderMapRef<FRDGScatterCopyCS> ComputeShader(GetGlobalShaderMap(GMaxRHIFeatureLevel), PermutationVector);

	FRDGScatterCopyCS::FParameters* PassParameters = GraphBuilder.AllocParameters(&Parameters);

	GraphBuilder.AddPass(
		RDG_EVENT_NAME("ScatterUpload"),
		PassParameters,
		ERDGPassFlags::Compute,
		[PassParameters, ComputeShader, Config, GetNumScatters = Params.GetNumScatters](FRDGAsyncTask, FRHIComputeCommandList& RHICmdList)
		{
			PassParameters->NumScatters = GetNumScatters();
			if (PassParameters->NumScatters)
			{
				const FScatterUploadDispatchConfig DispatchConfig = GetScatterUploadDispatchConfig(Config, PassParameters->NumScatters);

				FComputeShaderUtils::Dispatch(
					RHICmdList,
					ComputeShader,
					*PassParameters,
					DispatchConfig.GroupCountWrapped);
			}
		}
	);
}

void ScatterCopyResource(FRDGBuilder& GraphBuilder, FRDGBuffer* DstResource, FRDGBufferSRV* ScatterBufferSRV, FRDGBufferSRV* UploadBufferSRV, const FScatterCopyParams &Params)
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	ScatterCopyResource(GraphBuilder, static_cast<FRDGViewableResource*>(DstResource), ScatterBufferSRV, UploadBufferSRV, Params);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void FRDGScatterUploadBuffer::Init(FRDGBuilder& GraphBuilder, uint32 NumElements, uint32 InNumBytesPerElement, bool bInFloat4Buffer, const TCHAR* Name)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FRDGScatterUploadBuffer::Init);

	NumScatters = 0;
	MaxScatters = NumElements;
	NumBytesPerElement = InNumBytesPerElement;
	bFloat4Buffer = bInFloat4Buffer;

	FRHICommandListBase& RHICmdList = GraphBuilder.RHICmdList;

	const EBufferUsageFlags Usage = bInFloat4Buffer ? BUF_None : BUF_ByteAddressBuffer;
	const uint32 TypeSize = bInFloat4Buffer ? 16 : 4;

	const uint32 ScatterNumBytesPerElement = sizeof(uint32);
	const uint32 ScatterBytes = NumElements * ScatterNumBytesPerElement;
	const uint32 ScatterBufferSize = (uint32)FMath::Min(FMath::RoundUpToPowerOfTwo64((uint64)ScatterBytes), GetMaxUploadBufferElements() * sizeof(uint32));
	check(ScatterBufferSize >= ScatterBytes);

	const uint32 UploadNumBytesPerElement = TypeSize;
	const uint32 UploadBytes = NumElements * NumBytesPerElement;
	const uint32 UploadBufferSize = (uint32)FMath::Min(FMath::RoundUpToPowerOfTwo64((uint64)UploadBytes), GetMaxUploadBufferElements() * TypeSize);
	check(UploadBufferSize >= UploadBytes);

	// Recreate buffers is they are already queued into RDG from a previous call.
	if (IsRegistered(GraphBuilder, ScatterBuffer))
	{
		ScatterBuffer = nullptr;
		UploadBuffer = nullptr;
	}

	if (!ScatterBuffer || ScatterBytes > ScatterBuffer->GetSize() || ScatterBufferSize < ScatterBuffer->GetSize() / 2)
	{
		FRDGBufferDesc Desc = FRDGBufferDesc::CreateStructuredUploadDesc(ScatterNumBytesPerElement, ScatterBufferSize / ScatterNumBytesPerElement);
		Desc.Usage |= Usage;

		AllocatePooledBuffer(Desc, ScatterBuffer, Name, ERDGPooledBufferAlignment::None);
	}

	if (!UploadBuffer || UploadBytes > UploadBuffer->GetSize() || UploadBufferSize < UploadBuffer->GetSize() / 2)
	{
		FRDGBufferDesc Desc = FRDGBufferDesc::CreateStructuredUploadDesc(TypeSize, UploadBufferSize / UploadNumBytesPerElement);
		Desc.Usage |= Usage;

		AllocatePooledBuffer(Desc, UploadBuffer, Name, ERDGPooledBufferAlignment::None);
	}

	ScatterData = (uint32*)RHICmdList.LockBuffer(ScatterBuffer->GetRHI(), 0, ScatterBytes, RLM_WriteOnly);
	UploadData = (uint8*)RHICmdList.LockBuffer(UploadBuffer->GetRHI(), 0, UploadBytes, RLM_WriteOnly);
}

void FRDGScatterUploadBuffer::ResourceUploadToInternal(FRDGBuilder& GraphBuilder, FRDGViewableResource* DstResource)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FRDGScatterUploadBuffer::ResourceUploadTo);
	FRHICommandListBase& RHICmdList = GraphBuilder.RHICmdList;

	RHICmdList.UnlockBuffer(ScatterBuffer->GetRHI());
	RHICmdList.UnlockBuffer(UploadBuffer->GetRHI());

	ScatterData = nullptr;
	UploadData = nullptr;

	if (NumScatters == 0)
	{
		return;
	}

	const EResourceType DstResourceType = GetResourceType(DstResource);
	check(bFloat4Buffer != (DstResourceType == EResourceType::BYTEBUFFER));

	FRDGBufferSRV* ScatterBufferSRV = GraphBuilder.CreateSRV(GraphBuilder.RegisterExternalBuffer(ScatterBuffer));
	FRDGBufferSRV* UploadBufferSRV  = GraphBuilder.CreateSRV(GraphBuilder.RegisterExternalBuffer(UploadBuffer));
	
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	ScatterCopyResource(GraphBuilder, DstResource, ScatterBufferSRV, UploadBufferSRV, FScatterCopyParams { NumScatters, NumBytesPerElement, INDEX_NONE});
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	Reset();
}

void FRDGScatterUploadBuffer::Reset()
{
	NumScatters = 0;
	MaxScatters = 0;
	NumBytesPerElement = 0;
}

void FRDGScatterUploader::Lock(FRHICommandListBase& RHICmdList, uint32 ActualMaxScatters)
{
	check(State == EState::Empty);
	check(ActualMaxScatters <= MaxScatters);
	State = EState::Locked;

	ScatterBytes = ActualMaxScatters * sizeof(uint32);
	UploadBytes = ActualMaxScatters * NumBytesPerElement;
	MaxScatters = ActualMaxScatters;

	ScatterData = (uint32*)RHICmdList.LockBuffer(ScatterBuffer, 0, ScatterBytes, RLM_WriteOnly);
	UploadData = (uint8*)RHICmdList.LockBuffer(UploadBuffer, 0, UploadBytes, RLM_WriteOnly);
}

void FRDGScatterUploader::Unlock(FRHICommandListBase& RHICmdList)
{
	check(State == EState::Locked);
	State = EState::Unlocked;
	RHICmdList.UnlockBuffer(ScatterBuffer);
	RHICmdList.UnlockBuffer(UploadBuffer);
}

FRDGScatterUploader* FRDGAsyncScatterUploadBuffer::Begin(FRDGBuilder& GraphBuilder, FRDGViewableResource* DstResource, uint32 NumElements, uint32 NumBytesPerElement, const TCHAR* Name)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FRDGAsyncScatterUploadBuffer::Upload);

	const EResourceType DstResourceType = GetResourceType(DstResource);
	const EBufferUsageFlags Usage = DstResourceType == EResourceType::BYTEBUFFER ? BUF_ByteAddressBuffer : BUF_None;
	const uint32 TypeSize = DstResourceType == EResourceType::BYTEBUFFER ? 4 : 16;

	const uint32 ScatterNumBytesPerElement = sizeof(uint32);
	const uint32 ScatterBytes = NumElements * ScatterNumBytesPerElement;
	const uint32 ScatterBufferSize = (uint32)FMath::Min((uint64)FMath::RoundUpToPowerOfTwo(ScatterBytes), GetMaxUploadBufferElements() * sizeof(uint32));
	check(ScatterBufferSize >= ScatterBytes);

	const uint32 UploadNumBytesPerElement = TypeSize;
	const uint32 UploadBytes = NumElements * NumBytesPerElement;
	const uint32 UploadBufferSize = (uint32)FMath::Min((uint64)FMath::RoundUpToPowerOfTwo(UploadBytes), GetMaxUploadBufferElements() * TypeSize);
	check(UploadBufferSize >= UploadBytes);

	// Recreate buffers is they are already queued into RDG from a previous call.
	if (IsRegistered(GraphBuilder, ScatterBuffer))
	{
		ScatterBuffer = nullptr;
		UploadBuffer = nullptr;
	}

	if (!ScatterBuffer || ScatterBytes > ScatterBuffer->GetSize() || ScatterBufferSize < ScatterBuffer->GetSize() / 2)
	{
		FRDGBufferDesc Desc = FRDGBufferDesc::CreateStructuredUploadDesc(ScatterNumBytesPerElement, ScatterBufferSize / ScatterNumBytesPerElement);
		Desc.Usage |= Usage;

		AllocatePooledBuffer(Desc, ScatterBuffer, Name, ERDGPooledBufferAlignment::None);
	}

	if (!UploadBuffer || UploadBytes > UploadBuffer->GetSize() || UploadBufferSize < UploadBuffer->GetSize() / 2)
	{
		FRDGBufferDesc Desc = FRDGBufferDesc::CreateStructuredUploadDesc(TypeSize, UploadBufferSize / UploadNumBytesPerElement);
		Desc.Usage |= Usage;

		AllocatePooledBuffer(Desc, UploadBuffer, Name, ERDGPooledBufferAlignment::None);
	}

	FRDGScatterUploader* Uploader = GraphBuilder.AllocObject<FRDGScatterUploader>();
	Uploader->MaxScatters = NumElements;
	Uploader->NumBytesPerElement = NumBytesPerElement;
	Uploader->DstResource = DstResource;
	Uploader->ScatterBuffer = ScatterBuffer->GetRHI();
	Uploader->UploadBuffer = UploadBuffer->GetRHI();
	Uploader->ScatterBytes = ScatterBytes;
	Uploader->UploadBytes = UploadBytes;
	return Uploader;
}

FRDGScatterUploader* FRDGAsyncScatterUploadBuffer::BeginPreSized(FRDGBuilder& GraphBuilder, FRDGViewableResource* DstResource, uint32 NumElements, uint32 NumBytesPerElement, const TCHAR* Name)
{
	FRDGScatterUploader* Uploader = Begin(GraphBuilder, DstResource, NumElements, NumBytesPerElement, Name);
	Uploader->NumScatters = NumElements;
	Uploader->bNumScattersPreSized = true;
	return Uploader;
}

void FRDGAsyncScatterUploadBuffer::End(FRDGBuilder& GraphBuilder, FRDGScatterUploader* Uploader)
{
	check(Uploader);
	checkf(!FRDGBuilder::IsImmediateMode() || Uploader->State == FRDGScatterUploader::EState::Unlocked, TEXT("In immediate mode, you must fill the uploader prior to calling End."));

	const uint32 NumScatters = Uploader->bNumScattersPreSized ? Uploader->NumScatters : Uploader->MaxScatters;

	const FScatterUploadConfig Config = GetScatterUploadConfig(Uploader->NumBytesPerElement);
	const FScatterUploadDispatchConfig DispatchConfig = GetScatterUploadDispatchConfig(Config, NumScatters);

	FRDGViewableResource* DstResource = Uploader->DstResource;

	FRDGBufferSRV* ScatterBufferSRV = GraphBuilder.CreateSRV(GraphBuilder.RegisterExternalBuffer(ScatterBuffer));
	FRDGBufferSRV* UploadBufferSRV = GraphBuilder.CreateSRV(GraphBuilder.RegisterExternalBuffer(UploadBuffer));

	FRDGScatterCopyCS::FParameters Parameters;
	FRDGScatterCopyCS::FPermutationDomain PermutationVector;
	GetScatterCopyParamsAndPermutation(GraphBuilder, DstResource, ScatterBufferSRV, UploadBufferSRV, Config, NumScatters, Parameters, PermutationVector);

	TShaderMapRef<FRDGScatterCopyCS> ComputeShader(GetGlobalShaderMap(GMaxRHIFeatureLevel), PermutationVector);

	{
		FRDGScatterCopyCS::FParameters* PassParameters = GraphBuilder.AllocParameters(&Parameters);

		if (Uploader->bNumScattersPreSized)
		{
			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("ScatterUpload (Resource: %s)", DstResource->Name),
				ComputeShader,
				PassParameters,
				DispatchConfig.GroupCountWrapped);
		}
		else
		{
			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("ScatterUpload (Resource: %s)", DstResource->Name),
				ComputeShader,
				PassParameters,
				[PassParameters, Config, Uploader]
			{
				const uint32 NumScatters = Uploader->GetFinalNumScatters();
				const FScatterUploadDispatchConfig DispatchConfig = GetScatterUploadDispatchConfig(Config, NumScatters);

				PassParameters->NumScatters = NumScatters;

				return DispatchConfig.GroupCountWrapped;
			});
		}
	}
}

void FRDGAsyncScatterUploadBuffer::Release()
{
	ScatterBuffer = nullptr;
	UploadBuffer = nullptr;
}

uint32 FRDGAsyncScatterUploadBuffer::GetNumBytes() const
{
	return TryGetSize(ScatterBuffer) + TryGetSize(UploadBuffer);
}

FRDGScatterUploadBuilder* FRDGScatterUploadBuilder::Create(FRDGBuilder& GraphBuilder)
{
	return GraphBuilder.AllocObject<FRDGScatterUploadBuilder>();
}

UE::Tasks::FTask FRDGScatterUploadBuilder::Execute(FRDGBuilder& GraphBuilder)
{
	UE::Tasks::FTask Task;

	if (!Passes.IsEmpty() || !CommandListFunctions.IsEmpty())
	{
		const bool bUseAsyncTask = MaxBytes > 32 * 1024 || !CommandListFunctions.IsEmpty();

		Task = GraphBuilder.AddCommandListSetupTask([Passes = Passes, CommandListFunctions = MoveTemp(CommandListFunctions)] (FRHICommandList& RHICmdList)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(FRDGScatterUploadBuilder);
			for (auto& Pass : Passes)
			{
				if (Pass.GetMaxScattersFunction)
				{
					Pass.Uploader->Lock(RHICmdList, Pass.GetMaxScattersFunction());
				}
				else
				{
					Pass.Uploader->Lock(RHICmdList);
				}
				Pass.Function(*Pass.Uploader);
				Pass.Uploader->Unlock(RHICmdList);
			}

			for (auto& CommandListFunction : CommandListFunctions)
			{
				CommandListFunction(RHICmdList);
			}

		}, PrerequisiteTasks, UE::Tasks::ETaskPriority::Normal, bUseAsyncTask);
	}

	for (auto& Pass : Passes)
	{
		Pass.UploadBuffer.End(GraphBuilder, Pass.Uploader);
	}

	Passes.Empty();
	CommandListFunctions.Empty();
	PrerequisiteTasks.Empty();
	return Task;
}

template<typename ResourceType>
void MemsetResource(FRHICommandList& RHICmdList, const ResourceType& DstBuffer, const FMemsetResourceParams& Params)
{
	EByteBufferResourceType ResourceTypeEnum;

	FMemsetBufferCS::FParameters Parameters;
	Parameters.Value = Params.Value;
	Parameters.Size = Params.Count;
	Parameters.DstOffset = Params.DstOffset;

	if (ResourceTypeTraits<ResourceType>::Type == EResourceType::BYTEBUFFER)
	{
		ResourceTypeEnum = EByteBufferResourceType::Uint_Buffer;

		Parameters.DstByteAddressBuffer = DstBuffer.UAV;
	}
	else if (ResourceTypeTraits<ResourceType>::Type == EResourceType::BUFFER)
	{
		ResourceTypeEnum = EByteBufferResourceType::Float4_Buffer;

		Parameters.DstBuffer = DstBuffer.UAV;
	}
	else if (ResourceTypeTraits<ResourceType>::Type == EResourceType::STRUCTURED_BUFFER)
	{
		ResourceTypeEnum = EByteBufferResourceType::StructuredBuffer;

		Parameters.DstStructuredBuffer4x = DstBuffer.UAV;
	}

	FMemcpyCS::FPermutationDomain PermutationVector;
	PermutationVector.Set<FMemcpyCS::ResourceTypeDim >((int)ResourceTypeEnum);
	PermutationVector.Set<FMemcpyCS::StructuredElementSizeDim>((int32)EByteBufferStructuredSize::Uint4);

	auto ShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);

	auto ComputeShader = GetGlobalShaderMap(GMaxRHIFeatureLevel)->GetShader<FMemsetBufferCS>(PermutationVector);

	// each thread will set 4 floats / uints
	const uint32 Divisor = ResourceTypeTraits<ResourceType>::Type == EResourceType::BYTEBUFFER ? 4 : 1;

	FComputeShaderUtils::Dispatch(RHICmdList, ComputeShader, Parameters, FIntVector(FMath::DivideAndRoundUp(Params.Count / Divisor, 64u), 1, 1));
}

template<typename ResourceType>
void MemcpyResource(FRHICommandList& RHICmdList, const ResourceType& DstBuffer, const ResourceType& SrcBuffer, const FMemcpyResourceParams& Params, bool bAlreadyInUAVOverlap)
{
	// each thread will copy 4 floats / uints
	const uint32 Divisor = ResourceTypeTraits<ResourceType>::Type == EResourceType::BYTEBUFFER ? 4 : 1;

	if (!bAlreadyInUAVOverlap)	// TODO: Get rid of this check once BeginUAVOverlap/EndUAVOverlap supports nesting.
		RHICmdList.BeginUAVOverlap(DstBuffer.UAV);

	{
		EByteBufferResourceType ResourceTypeEnum;

		FMemcpyCS::FParameters Parameters;
		Parameters.Common.Size = Params.Count;
		Parameters.Common.SrcOffset = Params.SrcOffset;
		Parameters.Common.DstOffset = Params.DstOffset;

		if (ResourceTypeTraits<ResourceType>::Type == EResourceType::BYTEBUFFER)
		{
			ResourceTypeEnum = EByteBufferResourceType::Uint_Buffer;

			Parameters.SrcByteAddressBuffer = SrcBuffer.SRV;
			Parameters.Common.DstByteAddressBuffer = DstBuffer.UAV;
		}
		else if (ResourceTypeTraits<ResourceType>::Type == EResourceType::STRUCTURED_BUFFER)
		{
			ResourceTypeEnum = EByteBufferResourceType::StructuredBuffer;

			Parameters.SrcStructuredBuffer4x = SrcBuffer.SRV;
			Parameters.Common.DstStructuredBuffer4x = DstBuffer.UAV;
		}
		else if (ResourceTypeTraits<ResourceType>::Type == EResourceType::BUFFER)
		{
			ResourceTypeEnum = EByteBufferResourceType::Float4_Buffer;

			Parameters.SrcBuffer = SrcBuffer.SRV;
			Parameters.Common.DstBuffer = DstBuffer.UAV;
		}
		else
		{
			check(false);
		}

		FMemcpyCS::FPermutationDomain PermutationVector;
		PermutationVector.Set<FMemcpyCS::ResourceTypeDim >((int)ResourceTypeEnum);
		PermutationVector.Set<FMemcpyCS::StructuredElementSizeDim>((int32)EByteBufferStructuredSize::Uint4);

		auto ComputeShader = GetGlobalShaderMap(GMaxRHIFeatureLevel)->GetShader<FMemcpyCS >(PermutationVector);

		FComputeShaderUtils::Dispatch(
			RHICmdList,
			ComputeShader,
			Parameters,
			FComputeShaderUtils::GetGroupCountWrapped(Params.Count / Divisor, 64));
	}

	if(!bAlreadyInUAVOverlap)
		RHICmdList.EndUAVOverlap(DstBuffer.UAV);
}

template<>
RENDERCORE_API bool ResizeResourceIfNeeded<FRWBufferStructured>(FRHICommandList& RHICmdList, FRWBufferStructured& Buffer, uint32 NumBytes, const TCHAR* DebugName)
{
	const uint32 BytesPerElement = 16;

	check((NumBytes & (BytesPerElement - 1)) == 0);

	uint32 NumElements = NumBytes / BytesPerElement;

	if (Buffer.NumBytes == 0)
	{
		Buffer.Initialize(RHICmdList, DebugName, BytesPerElement, NumElements);
		return true;
	}
	else if (NumBytes != Buffer.NumBytes)
	{
		FRWBufferStructured NewBuffer;
		NewBuffer.Initialize(RHICmdList, DebugName, BytesPerElement, NumElements);

		RHICmdList.Transition(FRHITransitionInfo(Buffer.UAV, ERHIAccess::Unknown, ERHIAccess::SRVCompute));
		RHICmdList.Transition(FRHITransitionInfo(NewBuffer.UAV, ERHIAccess::Unknown, ERHIAccess::UAVCompute));

		// Copy data to new buffer
		FMemcpyResourceParams Params;
		Params.Count = FMath::Min(NumBytes, Buffer.NumBytes) / BytesPerElement;
		Params.SrcOffset = 0;
		Params.DstOffset = 0;
		MemcpyResource(RHICmdList, NewBuffer, Buffer, Params);

		Buffer = NewBuffer;
		return true;
	}

	return false;
}

template<>
RENDERCORE_API bool ResizeResourceIfNeeded<FRWByteAddressBuffer>(FRHICommandList& RHICmdList, FRWByteAddressBuffer& Buffer, uint32 NumBytes, const TCHAR* DebugName)
{
	const uint32 BytesPerElement = 4;

	// Needs to be aligned to 16 bytes to MemcpyResource to work correctly (otherwise it skips last unaligned elements of the buffer during resize)
	check((NumBytes & 15) == 0);

	if (Buffer.NumBytes == 0)
	{
		Buffer.Initialize(RHICmdList, DebugName, NumBytes);
		return true;
	}
	else if (NumBytes != Buffer.NumBytes)
	{
		FRWByteAddressBuffer NewBuffer;
		NewBuffer.Initialize(RHICmdList, DebugName, NumBytes);

		RHICmdList.Transition({
			FRHITransitionInfo(Buffer.UAV, ERHIAccess::Unknown, ERHIAccess::SRVCompute),
			FRHITransitionInfo(NewBuffer.UAV, ERHIAccess::Unknown, ERHIAccess::UAVCompute)
		});

		// Copy data to new buffer
		FMemcpyResourceParams Params;
		Params.Count = FMath::Min(NumBytes, Buffer.NumBytes) / BytesPerElement;
		Params.SrcOffset = 0;
		Params.DstOffset = 0;
		MemcpyResource(RHICmdList, NewBuffer, Buffer, Params);

		Buffer = NewBuffer;
		return true;
	}

	return false;
}

RENDERCORE_API bool ResizeResourceIfNeeded(FRHICommandList& RHICmdList, FRWBuffer& Buffer, EPixelFormat Format, uint32 NumElements, const TCHAR* DebugName)
{
	const uint32 BytesPerElement = GPixelFormats[Format].BlockBytes;
	const uint32 NumBytes = BytesPerElement * NumElements;

	if (Buffer.NumBytes == 0)
	{
		Buffer.Initialize(RHICmdList, DebugName, BytesPerElement, NumElements, Format);
		return true;
	}
	else if (NumBytes != Buffer.NumBytes)
	{
		FRWBuffer NewBuffer;
		NewBuffer.Initialize(RHICmdList, DebugName, BytesPerElement, NumElements, Format);

		RHICmdList.Transition(FRHITransitionInfo(Buffer.UAV, ERHIAccess::Unknown, ERHIAccess::SRVCompute));
		RHICmdList.Transition(FRHITransitionInfo(NewBuffer.UAV, ERHIAccess::Unknown, ERHIAccess::UAVCompute));

		// Copy data to new buffer
		FMemcpyResourceParams MemcpyParams;
		MemcpyParams.Count = FMath::Min(NumBytes, Buffer.NumBytes) / BytesPerElement;
		MemcpyParams.SrcOffset = 0;
		MemcpyParams.DstOffset = 0;
		MemcpyResource(RHICmdList, NewBuffer, Buffer, MemcpyParams);

		Buffer = NewBuffer;
		return true;
	}

	return false;
}

RENDERCORE_API bool ResizeResourceSOAIfNeeded(FRHICommandList& RHICmdList, FRWBufferStructured& Buffer, const FResizeResourceSOAParams& Params, const TCHAR* DebugName)
{
	const uint32 BytesPerElement = 16;

	checkf(Params.NumBytes % BytesPerElement == 0, TEXT("NumBytes (%u) must be a multiple of BytesPerElement (%u)"), Params.NumBytes, BytesPerElement);
	checkf(Buffer.NumBytes % BytesPerElement == 0, TEXT("NumBytes (%u) must be a multiple of BytesPerElement (%u)"), Buffer.NumBytes, BytesPerElement);

	uint32 NumElements = Params.NumBytes / BytesPerElement;
	uint32 NumElementsOld = Buffer.NumBytes / BytesPerElement;

	checkf(NumElements % Params.NumArrays == 0, TEXT("NumElements (%u) must be a multiple of NumArrays (%u)"), NumElements, Params.NumArrays);
	checkf(NumElementsOld % Params.NumArrays == 0, TEXT("NumElements (%u) must be a multiple of NumArrays (%u)"), NumElementsOld, Params.NumArrays);

	if (Buffer.NumBytes == 0)
	{
		Buffer.Initialize(RHICmdList, DebugName, BytesPerElement, NumElements);
		return true;
	}
	else if (Params.NumBytes != Buffer.NumBytes)
	{
		FRWBufferStructured NewBuffer;
		NewBuffer.Initialize(RHICmdList, DebugName, BytesPerElement, NumElements);

		RHICmdList.Transition({
			FRHITransitionInfo(Buffer.UAV, ERHIAccess::Unknown, ERHIAccess::SRVCompute),
			FRHITransitionInfo(NewBuffer.UAV, ERHIAccess::Unknown, ERHIAccess::UAVCompute)
		});

		// Copy data to new buffer
		uint32 OldArraySize = NumElementsOld / Params.NumArrays;
		uint32 NewArraySize = NumElements / Params.NumArrays;

		RHICmdList.BeginUAVOverlap(NewBuffer.UAV);

		FMemcpyResourceParams MemcpyParams;
		MemcpyParams.Count = FMath::Min(NewArraySize, OldArraySize);

		for( uint32 i = 0; i < Params.NumArrays; i++ )
		{
			MemcpyParams.SrcOffset = i * OldArraySize;
			MemcpyParams.DstOffset = i * NewArraySize;
			MemcpyResource( RHICmdList, NewBuffer, Buffer, MemcpyParams, true );
		}

		RHICmdList.EndUAVOverlap(NewBuffer.UAV);

		Buffer = NewBuffer;
		return true;
	}

	return false;
}

RENDERCORE_API bool ResizeResourceSOAIfNeeded(FRDGBuilder& GraphBuilder, FRWBufferStructured& Buffer, const FResizeResourceSOAParams& Params, const TCHAR* DebugName)
{
	const uint32 BytesPerElement = 16;

	checkf(Params.NumBytes % BytesPerElement == 0, TEXT("NumBytes (%u) must be a multiple of BytesPerElement (%u)"), Params.NumBytes, BytesPerElement);
	checkf(Buffer.NumBytes % BytesPerElement == 0, TEXT("NumBytes (%u) must be a multiple of BytesPerElement (%u)"), Buffer.NumBytes, BytesPerElement);

	uint32 NumElements = Params.NumBytes / BytesPerElement;
	uint32 NumElementsOld = Buffer.NumBytes / BytesPerElement;

	checkf(NumElements % Params.NumArrays == 0, TEXT("NumElements (%u) must be a multiple of NumArrays (%u)"), NumElements, Params.NumArrays);
	checkf(NumElementsOld % Params.NumArrays == 0, TEXT("NumElements (%u) must be a multiple of NumArrays (%u)"), NumElementsOld, Params.NumArrays);

	if (Buffer.NumBytes == 0)
	{
		Buffer.Initialize(GraphBuilder.RHICmdList, DebugName, BytesPerElement, NumElements);
		return true;
	}
	else if (Params.NumBytes != Buffer.NumBytes)
	{
		FRWBufferStructured NewBuffer;
		FRWBufferStructured OldBuffer = Buffer;
		NewBuffer.Initialize(GraphBuilder.RHICmdList, DebugName, BytesPerElement, NumElements);

		AddPass(GraphBuilder, RDG_EVENT_NAME("ResizeResourceSOAIfNeeded"), 
			[OldBuffer, NewBuffer, NumElements, NumElementsOld, Params](FRDGAsyncTask, FRHICommandList& RHICmdList)
		{
			RHICmdList.Transition({
				FRHITransitionInfo(OldBuffer.UAV, ERHIAccess::Unknown, ERHIAccess::SRVCompute),
				FRHITransitionInfo(NewBuffer.UAV, ERHIAccess::Unknown, ERHIAccess::UAVCompute)
			});

			// Copy data to new buffer
			uint32 OldArraySize = NumElementsOld / Params.NumArrays;
			uint32 NewArraySize = NumElements / Params.NumArrays;

			RHICmdList.BeginUAVOverlap(NewBuffer.UAV);

			FMemcpyResourceParams MemcpyParams;
			MemcpyParams.Count = FMath::Min(NewArraySize, OldArraySize);

			for (uint32 i = 0; i < Params.NumArrays; i++)
			{
				MemcpyParams.SrcOffset = i * OldArraySize;
				MemcpyParams.DstOffset = i * NewArraySize;
				MemcpyResource(RHICmdList, NewBuffer, OldBuffer, MemcpyParams, true);
			}
			RHICmdList.EndUAVOverlap(NewBuffer.UAV);
		});

		Buffer = NewBuffer;
		return true;
	}

	return false;
}

template <typename FBufferType>
void AddCopyBufferPass(FRDGBuilder& GraphBuilder, const FBufferType &NewBuffer, const FBufferType &OldBuffer, uint32 ElementSize)
{
	AddPass(GraphBuilder, RDG_EVENT_NAME("ResizeResourceIfNeeded-Copy"), 
		[OldBuffer, NewBuffer, ElementSize](FRDGAsyncTask, FRHICommandList& RHICmdList)
	{
		RHICmdList.Transition({
			FRHITransitionInfo(OldBuffer.UAV, ERHIAccess::Unknown, ERHIAccess::SRVCompute),
			FRHITransitionInfo(NewBuffer.UAV, ERHIAccess::Unknown, ERHIAccess::UAVCompute)
		});

		// Copy data to new buffer
		FMemcpyResourceParams MemcpyParams;
		MemcpyParams.Count = FMath::Min(NewBuffer.NumBytes, OldBuffer.NumBytes) / ElementSize;
		MemcpyParams.SrcOffset = 0;
		MemcpyParams.DstOffset = 0;

		MemcpyResource(RHICmdList, NewBuffer, OldBuffer, MemcpyParams);
	});
}

RENDERCORE_API bool ResizeResourceIfNeeded(FRDGBuilder& GraphBuilder, FRWBufferStructured& Buffer, uint32 NumBytes, const TCHAR* DebugName)
{
	const uint32 BytesPerElement = 16;

	checkf((NumBytes % BytesPerElement) == 0, TEXT("NumBytes (%u) must be a multiple of BytesPerElement (%u)"), NumBytes, BytesPerElement);

	uint32 NumElements = NumBytes / BytesPerElement;

	if (Buffer.NumBytes == 0)
	{
		Buffer.Initialize(GraphBuilder.RHICmdList, DebugName, BytesPerElement, NumElements);
		return true;
	}
	else if (NumBytes != Buffer.NumBytes)
	{
		FRWBufferStructured NewBuffer;
		NewBuffer.Initialize(GraphBuilder.RHICmdList, DebugName, BytesPerElement, NumElements);

		AddCopyBufferPass(GraphBuilder, NewBuffer, Buffer, BytesPerElement);

		Buffer = NewBuffer;
		return true;
	}

	return false;
}

RENDERCORE_API bool ResizeResourceIfNeeded(FRDGBuilder& GraphBuilder, FRWByteAddressBuffer& Buffer, uint32 NumBytes, const TCHAR* DebugName)
{
	// Needs to be aligned to 16 bytes to MemcpyResource to work correctly (otherwise it skips last unaligned elements of the buffer during resize)
	check((NumBytes & 15) == 0);

	if (Buffer.NumBytes == 0)
	{
		Buffer.Initialize(GraphBuilder.RHICmdList, DebugName, NumBytes);
		return true;
	}
	else if (NumBytes != Buffer.NumBytes)
	{
		FRWByteAddressBuffer NewBuffer;
		NewBuffer.Initialize(GraphBuilder.RHICmdList, DebugName, NumBytes);

		AddCopyBufferPass(GraphBuilder, NewBuffer, Buffer, 4);

		Buffer = NewBuffer;
		return true;
	}

	return false;
}

RENDERCORE_API bool ResizeResourceIfNeeded(FRDGBuilder& GraphBuilder, FRWBuffer& Buffer, EPixelFormat Format, uint32 NumElements, const TCHAR* DebugName)
{
	const uint32 BytesPerElement = GPixelFormats[Format].BlockBytes;
	const uint32 NumBytes = BytesPerElement * NumElements;

	if (Buffer.NumBytes == 0)
	{
		Buffer.Initialize(GraphBuilder.RHICmdList, DebugName, BytesPerElement, NumElements, Format);
		return true;
	}
	else if (NumBytes != Buffer.NumBytes)
	{
		FRWBuffer NewBuffer;
		NewBuffer.Initialize(GraphBuilder.RHICmdList, DebugName, BytesPerElement, NumElements, Format);

		AddCopyBufferPass(GraphBuilder, NewBuffer, Buffer, BytesPerElement);

		Buffer = NewBuffer;
		return true;
	}

	return false;
}

void FScatterUploadBuffer::Init( uint32 NumElements, uint32 InNumBytesPerElement, bool bInFloat4Buffer, const TCHAR* DebugName )
{
	FRHICommandListBase& RHICmdList = FRHICommandListImmediate::Get();
	NumScatters = 0;
	MaxScatters = NumElements;
	NumBytesPerElement = InNumBytesPerElement;
	bFloat4Buffer = bInFloat4Buffer;

	const EBufferUsageFlags Usage = BUF_ShaderResource | BUF_Volatile | (bInFloat4Buffer ? BUF_None : BUF_ByteAddressBuffer);
	const uint32 TypeSize = bInFloat4Buffer ? 16 : 4;

	uint32 ScatterBytes = NumElements * sizeof( uint32 );
	uint32 ScatterBufferSize = FMath::RoundUpToPowerOfTwo( ScatterBytes );
	
	uint32 UploadBytes = NumElements * NumBytesPerElement;
	uint32 UploadBufferSize = FMath::RoundUpToPowerOfTwo( UploadBytes );

	if (bUploadViaCreate)
	{
		if (ScatterBytes > ScatterDataSize || ScatterBufferSize < ScatterDataSize / 2)
		{
			FMemory::Free(ScatterData);
			ScatterData = (uint32*)FMemory::Malloc(ScatterBufferSize);
			ScatterDataSize = ScatterBufferSize;
		}

		if (UploadBytes > UploadDataSize || UploadBufferSize < UploadDataSize / 2)
		{
			FMemory::Free(UploadData);
			UploadData = (uint8*)FMemory::Malloc(UploadBufferSize);
			UploadDataSize = UploadBufferSize;
		}
	}
	else
	{
		check(ScatterData == nullptr);
		check(UploadData == nullptr);

		if (ScatterBytes > ScatterBuffer.NumBytes || ScatterBufferSize < ScatterBuffer.NumBytes / 2)
		{
			// Resize Scatter Buffer
			ScatterBuffer.Release();
			ScatterBuffer.NumBytes = ScatterBufferSize;

			const FRHIBufferCreateDesc ScatterBufferDesc =
				FRHIBufferCreateDesc::CreateStructured(DebugName, ScatterBuffer.NumBytes, sizeof(uint32))
				.AddUsage(Usage);

			ScatterBuffer.Buffer = RHICmdList.CreateBuffer(ScatterBufferDesc);
			ScatterBuffer.SRV = RHICmdList.CreateShaderResourceView(ScatterBuffer.Buffer, FRHIViewDesc::CreateBufferSRV().SetTypeFromBuffer(ScatterBuffer.Buffer));
		}

		if (UploadBytes > UploadBuffer.NumBytes || UploadBufferSize < UploadBuffer.NumBytes / 2)
		{
			// Resize Upload Buffer
			UploadBuffer.Release();
			UploadBuffer.NumBytes = UploadBufferSize;

			const FRHIBufferCreateDesc UploadBufferDesc =
				FRHIBufferCreateDesc::CreateStructured(DebugName, UploadBuffer.NumBytes, TypeSize)
				.AddUsage(Usage);

			UploadBuffer.Buffer = RHICmdList.CreateBuffer(UploadBufferDesc);
			UploadBuffer.SRV = RHICmdList.CreateShaderResourceView(UploadBuffer.Buffer, FRHIViewDesc::CreateBufferSRV().SetTypeFromBuffer(UploadBuffer.Buffer));
		}

		ScatterData = (uint32*)RHICmdList.LockBuffer(ScatterBuffer.Buffer, 0, ScatterBytes, RLM_WriteOnly);
		UploadData = (uint8*)RHICmdList.LockBuffer(UploadBuffer.Buffer, 0, UploadBytes, RLM_WriteOnly);
	}
}

void FScatterUploadBuffer::Init(TArrayView<const uint32> ElementScatterOffsets, uint32 InNumBytesPerElement, bool bInFloat4Buffer, const TCHAR* DebugName)
{
	Init(ElementScatterOffsets.Num(), InNumBytesPerElement, bInFloat4Buffer, DebugName);
	FMemory::ParallelMemcpy(ScatterData, ElementScatterOffsets.GetData(), ElementScatterOffsets.Num() * ElementScatterOffsets.GetTypeSize(), EMemcpyCachePolicy::StoreUncached);
	NumScatters = ElementScatterOffsets.Num();
}

void FScatterUploadBuffer::InitPreSized(uint32 NumElements, uint32 InNumBytesPerElement, bool bInFloat4Buffer, const TCHAR* DebugName)
{
	Init(NumElements, InNumBytesPerElement, bInFloat4Buffer, DebugName);
	NumScatters = NumElements;
}

template<typename ResourceType>
void FScatterUploadBuffer::ResourceUploadTo(FRHICommandList& RHICmdList, const ResourceType& DstBuffer)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FScatterUploadBuffer::ResourceUploadTo);

	if (bUploadViaCreate)
	{
		ScatterBuffer.Release();
		UploadBuffer.Release();

		ScatterBuffer.NumBytes = ScatterDataSize;
		UploadBuffer.NumBytes = UploadDataSize;

		const uint32 TypeSize = bFloat4Buffer ? 16 : 4;
		const EBufferUsageFlags Usage = EBufferUsageFlags::StructuredBuffer | EBufferUsageFlags::ShaderResource | EBufferUsageFlags::Volatile | (bFloat4Buffer ? EBufferUsageFlags::None : EBufferUsageFlags::ByteAddressBuffer);

		{
			ScatterBuffer.Buffer = UE::RHIResourceUtils::CreateBufferFromArray(RHICmdList, TEXT("ScatterResourceArray"), Usage, sizeof(uint32), ScatterData, ScatterDataSize);
			ScatterBuffer.SRV = RHICmdList.CreateShaderResourceView(ScatterBuffer.Buffer, FRHIViewDesc::CreateBufferSRV().SetTypeFromBuffer(ScatterBuffer.Buffer));
		}
		{
			UploadBuffer.Buffer = UE::RHIResourceUtils::CreateBufferFromArray(RHICmdList, TEXT("ScatterUploadBuffer"), Usage, TypeSize, UploadData, UploadDataSize);
			UploadBuffer.SRV = RHICmdList.CreateShaderResourceView(UploadBuffer.Buffer, FRHIViewDesc::CreateBufferSRV().SetTypeFromBuffer(UploadBuffer.Buffer));
		}
	}
	else
	{
		RHICmdList.UnlockBuffer(ScatterBuffer.Buffer);
		RHICmdList.UnlockBuffer(UploadBuffer.Buffer);

		ScatterData = nullptr;
		UploadData = nullptr;
	}

	if (NumScatters == 0)
	{
		return;
	}

	const FScatterUploadConfig Config = GetScatterUploadConfig(NumBytesPerElement);
	const FScatterUploadDispatchConfig DispatchConfig = GetScatterUploadDispatchConfig(Config, NumScatters);

	EByteBufferResourceType ResourceTypeEnum;

	FScatterCopyCS::FParameters Parameters;
	Parameters.Common.Size = Config.NumThreadsPerScatter;
	Parameters.Common.SrcOffset = 0;
	Parameters.Common.DstOffset = 0;
	Parameters.NumScatters = NumScatters;

	check(bFloat4Buffer || ResourceTypeTraits<ResourceType>::Type == EResourceType::BYTEBUFFER);

	if (ResourceTypeTraits<ResourceType>::Type == EResourceType::BYTEBUFFER)
	{
		if (Config.NumBytesPerThread == 16)
		{
			ResourceTypeEnum = EByteBufferResourceType::Uint4Aligned_Buffer;
		}
		else
		{
			ResourceTypeEnum = EByteBufferResourceType::Uint_Buffer;
		}
		Parameters.UploadByteAddressBuffer = UploadBuffer.SRV;
		Parameters.ScatterByteAddressBuffer = ScatterBuffer.SRV;
		Parameters.Common.DstByteAddressBuffer = DstBuffer.UAV;
	}
	else if (ResourceTypeTraits<ResourceType>::Type == EResourceType::STRUCTURED_BUFFER)
	{
		ResourceTypeEnum = EByteBufferResourceType::StructuredBuffer;

		Parameters.UploadStructuredBuffer4x = UploadBuffer.SRV;
		Parameters.ScatterStructuredBuffer = ScatterBuffer.SRV;
		Parameters.Common.DstStructuredBuffer4x = DstBuffer.UAV;
	}
	else if (ResourceTypeTraits<ResourceType>::Type == EResourceType::BUFFER)
	{
		ResourceTypeEnum = EByteBufferResourceType::Float4_Buffer;

		Parameters.UploadStructuredBuffer4x = UploadBuffer.SRV;
		Parameters.ScatterStructuredBuffer = ScatterBuffer.SRV;
		Parameters.Common.DstBuffer = DstBuffer.UAV;
	}

	FByteBufferShader::FPermutationDomain PermutationVector;
	PermutationVector.Set<FByteBufferShader::ResourceTypeDim>((int)ResourceTypeEnum);
	PermutationVector.Set<FMemcpyCS::StructuredElementSizeDim>((int32)EByteBufferStructuredSize::Uint4);

	auto ComputeShader = GetGlobalShaderMap(GMaxRHIFeatureLevel)->GetShader<FScatterCopyCS>(PermutationVector);

	RHICmdList.BeginUAVOverlap(DstBuffer.UAV);
	
	FComputeShaderUtils::Dispatch(RHICmdList, ComputeShader, Parameters, DispatchConfig.GroupCountWrapped);

	RHICmdList.EndUAVOverlap(DstBuffer.UAV);

	// We need to unbind shader SRVs in this case, because scatter upload buffers are sometimes used more than once in a
	// frame, and this can cause rendering bugs on D3D12, where the driver fails to update the bound SRV with new data.
	UnsetShaderSRVs(RHICmdList, ComputeShader, ComputeShader.GetComputeShader());
}

template RENDERCORE_API void MemsetResource<FRWBufferStructured>(FRHICommandList& RHICmdList, const FRWBufferStructured& DstBuffer, const FMemsetResourceParams& Params);
template RENDERCORE_API void MemsetResource<FRWByteAddressBuffer>(FRHICommandList& RHICmdList, const FRWByteAddressBuffer& DstBuffer, const FMemsetResourceParams& Params);

template RENDERCORE_API void MemcpyResource<FRWBuffer>(FRHICommandList& RHICmdList, const FRWBuffer& DstBuffer, const FRWBuffer& SrcBuffer, const FMemcpyResourceParams& Params, bool bAlreadyInUAVOverlap);
template RENDERCORE_API void MemcpyResource<FRWBufferStructured>(FRHICommandList& RHICmdList, const FRWBufferStructured& DstBuffer, const FRWBufferStructured& SrcBuffer, const FMemcpyResourceParams& Params, bool bAlreadyInUAVOverlap);
template RENDERCORE_API void MemcpyResource<FRWByteAddressBuffer>(FRHICommandList& RHICmdList, const FRWByteAddressBuffer& DstBuffer, const FRWByteAddressBuffer& SrcBuffer, const FMemcpyResourceParams& Params, bool bAlreadyInUAVOverlap);

template RENDERCORE_API void FScatterUploadBuffer::ResourceUploadTo<FRWBuffer>(FRHICommandList& RHICmdList, const FRWBuffer& DstBuffer, bool bFlush);
template RENDERCORE_API void FScatterUploadBuffer::ResourceUploadTo<FRWBufferStructured>(FRHICommandList& RHICmdList, const FRWBufferStructured& DstBuffer, bool bFlush);
template RENDERCORE_API void FScatterUploadBuffer::ResourceUploadTo<FRWByteAddressBuffer>(FRHICommandList& RHICmdList, const FRWByteAddressBuffer& DstBuffer, bool bFlush);