// Copyright Epic Games, Inc. All Rights Reserved.


#include "UpdateDescriptorHandle.h"
#include "ShaderParameterUtils.h"

IMPLEMENT_GLOBAL_SHADER(FUpdateDescriptorHandleCS, "/Engine/Private/UpdateDescriptorHandle.usf", "MainCS", SF_Compute);
