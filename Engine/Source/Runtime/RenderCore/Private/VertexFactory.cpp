// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	VertexFactory.cpp: Vertex factory implementation
=============================================================================*/

#include "VertexFactory.h"
#include "PipelineStateCache.h"
#include "RHICommandList.h"
#include "ShaderCompilerCore.h"
#include "RenderUtils.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "RHIResourceUtils.h"
#if WITH_EDITOR
#include "Serialization/CompactBinary.h"
#include "Serialization/CompactBinaryWriter.h"
#endif

IMPLEMENT_TYPE_LAYOUT(FVertexFactoryShaderParameters);

uint32 FVertexFactoryType::NumVertexFactories = 0;
bool FVertexFactoryType::bVertexFactoriesInitialized = false;

static TLinkedList<FVertexFactoryType*>* GVFTypeList = nullptr;

static TArray<FVertexFactoryType*>& GetSortedMaterialVertexFactoryTypes()
{
	static TArray<FVertexFactoryType*> Types;
	return Types;
}

static TMap<FHashedName, FVertexFactoryType*>& GetVFTypeMap()
{
	static TMap<FHashedName, FVertexFactoryType*> VTTypeMap;
	return VTTypeMap;
}

void FVertexInputStream::SetOnRHICommandList(FRHICommandList& RHICmdList) const
{
	if (bStreamSourceSlot)
	{
		RHICmdList.SetStreamSourceSlot(StreamIndex, StreamSourceSlot, Offset);
	}
	else
	{
		RHICmdList.SetStreamSource(StreamIndex, VertexBuffer, Offset);
	}
}

/**
 * @return The global shader factory list.
 */
TLinkedList<FVertexFactoryType*>*& FVertexFactoryType::GetTypeList()
{
	return GVFTypeList;
}

const TArray<FVertexFactoryType*>& FVertexFactoryType::GetSortedMaterialTypes()
{
	return GetSortedMaterialVertexFactoryTypes();
}

/**
 * Finds a FVertexFactoryType by name.
 */
FVertexFactoryType* FVertexFactoryType::GetVFByName(const FHashedName& VFName)
{
	FVertexFactoryType** Type = GetVFTypeMap().Find(VFName);
	return Type ? *Type : nullptr;
}

#if WITH_EDITOR
void FVertexFactoryType::UpdateReferencedUniformBufferNames(const TMap<FString, TArray<const TCHAR*>>& ShaderFileToUniformBufferVariables)
{
	ReferencedUniformBuffers.Empty();
	GenerateReferencedUniformBuffers(ShaderFilename, Name, ShaderFileToUniformBufferVariables, ReferencedUniformBuffers);
	if (IncludesFwdShaderFile())
	{
		GenerateReferencedUniformBuffers(GetShaderFwdFilename(), Name, ShaderFileToUniformBufferVariables, ReferencedUniformBuffers);
	}
}
#endif

void FVertexFactoryType::Initialize(const TMap<FString, TArray<const TCHAR*> >& ShaderFileToUniformBufferVariables)
{
#if WITH_EDITOR
	if (!FPlatformProperties::RequiresCookedData())
	{
		// Cache serialization history for each VF type
		// This history is used to detect when shader serialization changes without a corresponding .usf change
		for(TLinkedList<FVertexFactoryType*>::TIterator It(FVertexFactoryType::GetTypeList()); It; It.Next())
		{
			FVertexFactoryType* Type = *It;
			Type->UpdateReferencedUniformBufferNames(ShaderFileToUniformBufferVariables);
		}
	}
#endif // WITH_EDITOR

	bVertexFactoriesInitialized = true;
}

FVertexFactoryType::FVertexFactoryType(
	const TCHAR* InName,
	const TCHAR* InShaderFilename,
	EVertexFactoryFlags InFlags,
	ConstructParametersType InConstructParameters,
	GetParameterTypeLayoutType InGetParameterTypeLayout,
	GetParameterTypeElementShaderBindingsType InGetParameterTypeElementShaderBindings,
	GetPSOPrecacheVertexFetchElementsType InGetPSOPrecacheVertexFetchElements,
	ShouldCacheType InShouldCache
#if WITH_EDITOR
	, ModifyCompilationEnvironmentType InModifyCompilationEnvironment
	, ValidateCompiledResultType InValidateCompiledResult
#endif // WITH_EDITOR
	):
	Name(InName),
	ShaderFilename(InShaderFilename),
	TypeName(InName),
	HashedName(TypeName),
	Flags(InFlags),
	ConstructParameters(InConstructParameters),
	GetParameterTypeLayout(InGetParameterTypeLayout),
	GetParameterTypeElementShaderBindings(InGetParameterTypeElementShaderBindings),
	GetPSOPrecacheVertexFetchElements(InGetPSOPrecacheVertexFetchElements),
	ShouldCacheRef(InShouldCache),
#if WITH_EDITOR
	ModifyCompilationEnvironmentRef(InModifyCompilationEnvironment),
	ValidateCompiledResultRef(InValidateCompiledResult),
#endif // WITH_EDITOR
	GlobalListLink(this)
{
	// Make sure the format of the source file path is right.
	check(CheckVirtualShaderFilePath(InShaderFilename));

	checkf(FPaths::GetExtension(InShaderFilename) == TEXT("ush"),
		TEXT("Incorrect virtual shader path extension for vertex factory shader header '%s': Only .ush files should be included."),
		InShaderFilename);

	if (IncludesFwdShaderFile())
	{
		ShaderFwdFilename = FPaths::GetBaseFilename(InShaderFilename, false) + TEXT("Fwd.ush");
		check(CheckVirtualShaderFilePath(ShaderFwdFilename));
	}

	// This will trigger if an IMPLEMENT_VERTEX_FACTORY_TYPE was in a module not loaded before InitializeShaderTypes
	// Vertex factory types need to be implemented in modules that are loaded before that
	checkf(!bVertexFactoriesInitialized, TEXT("VF type was loaded after engine init, use ELoadingPhase::PostConfigInit on your module to cause it to load earlier."));

	// Add this vertex factory type to the global list.
	GlobalListLink.LinkHead(GetTypeList()); 
	GetVFTypeMap().Add(HashedName, this);

	if (IsUsedWithMaterials())
	{
		TArray<FVertexFactoryType*>& SortedTypes = GetSortedMaterialVertexFactoryTypes();
		const int32 SortedIndex = Algo::LowerBoundBy(SortedTypes, HashedName, [](const FVertexFactoryType* InType) { return InType->GetHashedName(); });
		SortedTypes.Insert(this, SortedIndex);
	}

	++NumVertexFactories;
}

FVertexFactoryType::~FVertexFactoryType()
{
	GlobalListLink.Unlink();
	verify(GetVFTypeMap().Remove(HashedName) == 1);

	if (IsUsedWithMaterials())
	{
		TArray<FVertexFactoryType*>& SortedTypes = GetSortedMaterialVertexFactoryTypes();
		const int32 SortedIndex = Algo::BinarySearchBy(SortedTypes, HashedName, [](const FVertexFactoryType* InType) { return InType->GetHashedName(); });
		check(SortedIndex != INDEX_NONE);
		SortedTypes.RemoveAt(SortedIndex);
	}

	check(NumVertexFactories > 0u);
	--NumVertexFactories;
}

bool FVertexFactoryType::CheckManualVertexFetchSupport(ERHIFeatureLevel::Type InFeatureLevel)
{
	check(InFeatureLevel != ERHIFeatureLevel::Num);
	return (InFeatureLevel > ERHIFeatureLevel::ES3_1) && RHISupportsManualVertexFetch(GShaderPlatformForFeatureLevel[InFeatureLevel]);
}

/** Calculates a Hash based on this vertex factory type's source code and includes */
const FSHAHash& FVertexFactoryType::GetSourceHash(EShaderPlatform ShaderPlatform) const
{
	return GetShaderFileHash(GetShaderFilename(), ShaderPlatform);
}

FArchive& operator<<(FArchive& Ar,FVertexFactoryType*& TypeRef)
{
	if(Ar.IsSaving())
	{
		FName TypeName = TypeRef ? FName(TypeRef->GetName()) : NAME_None;
		Ar << TypeName;
	}
	else if(Ar.IsLoading())
	{
		FName TypeName = NAME_None;
		Ar << TypeName;
		TypeRef = FindVertexFactoryType(TypeName);
	}
	return Ar;
}

FVertexFactoryType* FindVertexFactoryType(const FHashedName& TypeName)
{
	return FVertexFactoryType::GetVFByName(TypeName);
}

void FVertexFactoryTypeDependency::RefreshCachedSourceHash(EShaderPlatform ShaderPlatform)
{
	const FVertexFactoryType* VertexFactory = FindVertexFactoryType(VertexFactoryTypeName);
	if (!VertexFactory)
	{
		VFSourceHash = FSHAHash();
		return;
	}
	VFSourceHash = VertexFactory->GetSourceHash(ShaderPlatform);
}

#if WITH_EDITOR
void FVertexFactoryTypeDependency::Save(FCbWriter& Writer) const
{
	Writer.BeginArray();
	Writer << VertexFactoryTypeName;
	Writer << VFSourceHash;
	Writer.EndArray();
}

bool FVertexFactoryTypeDependency::TryLoad(FCbFieldView Field)
{
	*this = FVertexFactoryTypeDependency();
	FCbFieldViewIterator ElementField(Field.CreateViewIterator());
	if (!LoadFromCompactBinary(ElementField++, VertexFactoryTypeName))
	{
		return false;
	}
	if (!LoadFromCompactBinary(ElementField++, VFSourceHash))
	{
		return false;
	}
	return true;
}

bool LoadFromCompactBinary(FCbFieldView Field, FVertexFactoryTypeDependency& OutValue)
{
	return OutValue.TryLoad(Field);
}
#endif // WITH_EDITOR

FVertexFactory::~FVertexFactory() = default;

void FVertexFactory::GetStreams(ERHIFeatureLevel::Type InFeatureLevel, EVertexInputStreamType VertexStreamType, FVertexInputStreamArray& OutVertexStreams) const
{
	check(IsInitialized());
	if (VertexStreamType == EVertexInputStreamType::Default)
	{
		const bool bSupportsVertexFetch = SupportsManualVertexFetch(InFeatureLevel);

		for (int32 StreamIndex = 0; StreamIndex < Streams.Num(); StreamIndex++)
		{
			const FVertexStream& Stream = Streams[StreamIndex];

			// Skip streams that are bound using manual vertex fetch if we support that.
			if (EnumHasAnyFlags(Stream.VertexStreamUsage, EVertexStreamUsage::ManualFetch) && bSupportsVertexFetch)
			{
				continue;
			}

			// Skip streams that are overridden as they will be provided manually by the vertex factory shader bindings.
			if (EnumHasAnyFlags(Stream.VertexStreamUsage, EVertexStreamUsage::Overridden))
			{
				continue;
			}

			// Issue a null binding since we don't appear to have one available.
			if (!Stream.VertexBuffer || !Stream.VertexBuffer->IsInitialized())
			{
				OutVertexStreams.Add(FVertexInputStream(StreamIndex, 0, nullptr));
			}
			else
			{
				OutVertexStreams.Add(FVertexInputStream(StreamIndex, Stream.Offset, Stream.VertexBuffer->VertexBufferRHI));
			}
		}
	}
	else if (VertexStreamType == EVertexInputStreamType::PositionOnly)
	{
		// Set the predefined vertex streams.
		for (int32 StreamIndex = 0; StreamIndex < PositionStream.Num(); StreamIndex++)
		{
			const FVertexStream& Stream = PositionStream[StreamIndex];
			check(Stream.VertexBuffer->IsInitialized());
			OutVertexStreams.Add(FVertexInputStream(StreamIndex, Stream.Offset, Stream.VertexBuffer->VertexBufferRHI));
		}
	}
	else if (VertexStreamType == EVertexInputStreamType::PositionAndNormalOnly)
	{
		// Set the predefined vertex streams.
		for (int32 StreamIndex = 0; StreamIndex < PositionAndNormalStream.Num(); StreamIndex++)
		{
			const FVertexStream& Stream = PositionAndNormalStream[StreamIndex];
			check(Stream.VertexBuffer->IsInitialized());
			OutVertexStreams.Add(FVertexInputStream(StreamIndex, Stream.Offset, Stream.VertexBuffer->VertexBufferRHI));
		}
	}
	else
	{
		// NOT_IMPLEMENTED
	}
}

void FVertexFactory::OffsetInstanceStreams(uint32 InstanceOffset, EVertexInputStreamType VertexStreamType, FVertexInputStreamArray& VertexStreams) const
{
	const TArrayView<const FVertexStream>& StreamArray = 
		  VertexStreamType == EVertexInputStreamType::PositionOnly ?			MakeArrayView(PositionStream) : 
		( VertexStreamType == EVertexInputStreamType::PositionAndNormalOnly ?	MakeArrayView(PositionAndNormalStream) : 
		/*VertexStreamType == EVertexInputStreamType::Default*/					MakeArrayView(Streams));

	for (int32 StreamIndex = 0; StreamIndex < StreamArray.Num(); StreamIndex++)
	{
		const FVertexStream& Stream = StreamArray[StreamIndex];

		if (EnumHasAnyFlags(EVertexStreamUsage::Instancing, Stream.VertexStreamUsage))
		{
			for (int32 BindingIndex = 0; BindingIndex < VertexStreams.Num(); BindingIndex++)
			{
				if (VertexStreams[BindingIndex].StreamIndex == StreamIndex)
				{
					VertexStreams[BindingIndex].Offset = Stream.Offset + Stream.Stride * InstanceOffset;
				}
			}
		}
	}
}

void FVertexFactory::ReleaseRHI()
{
	Declaration.SafeRelease();
	PositionDeclaration.SafeRelease();
	PositionAndNormalDeclaration.SafeRelease();
	Streams.Empty();
	PositionStream.Empty();
	PositionAndNormalStream.Empty();
}

bool FVertexFactory::AddPrimitiveIdStreamElement(EVertexInputStreamType InputStreamType, FVertexDeclarationElementList& Elements, uint8 AttributeIndex, uint8 AttributeIndex_Mobile)
{
	if (GetType()->SupportsPrimitiveIdStream() && UseGPUScene(GMaxRHIShaderPlatform))
	{
		// mobile PrimitiveId stream should either disabled or same as desktop
		check(AttributeIndex_Mobile == 0xff || AttributeIndex == AttributeIndex_Mobile);

		if (GMaxRHIFeatureLevel > ERHIFeatureLevel::ES3_1 || AttributeIndex_Mobile != 0xff)
		{
			// UniformView path does not use PrimitiveId stream, we still need to set it to a non-negative index
			int32 AddedStreamIndex = 0;
			if (!PlatformGPUSceneUsesUniformBufferView(GMaxRHIShaderPlatform))
			{
				// When the VF is used for rendering in normal mesh passes, this vertex buffer and offset will be overridden
				Elements.Add(AccessStreamComponent(FVertexStreamComponent(&GPrimitiveIdDummy, 0, 0, 0u, VET_UInt, EVertexStreamUsage::Instancing), AttributeIndex, InputStreamType));
				AddedStreamIndex = Elements.Last().StreamIndex;
			}
			
			SetPrimitiveIdStreamIndex(GMaxRHIFeatureLevel, InputStreamType, AddedStreamIndex);
			
			if (GIsEditor && (AttributeIndex_Mobile != 0xff && GMaxRHIFeatureLevel != ERHIFeatureLevel::ES3_1))
			{
				SetPrimitiveIdStreamIndex(ERHIFeatureLevel::ES3_1, InputStreamType, AddedStreamIndex);
			}
		}

		return true;
	}

	return false;
}

FVertexElement FVertexFactory::AccessStreamComponent(const FVertexStreamComponent& Component, uint8 AttributeIndex, EVertexInputStreamType InputStreamType)
{
	if (InputStreamType == EVertexInputStreamType::PositionOnly)
	{
		return AccessStreamComponent(Component, AttributeIndex, PositionStream);
	}
	else if (InputStreamType == EVertexInputStreamType::PositionAndNormalOnly)
	{
		return AccessStreamComponent(Component, AttributeIndex, PositionAndNormalStream);
	}
	else /* (InputStreamType == EVertexInputStreamType::Default) */
	{
		return AccessStreamComponent(Component, AttributeIndex, Streams);
	}
}

void FVertexFactory::InitDeclaration(const FVertexDeclarationElementList& Elements, EVertexInputStreamType StreamType)
{
	
	if (StreamType == EVertexInputStreamType::PositionOnly)
	{
		PositionDeclaration = PipelineStateCache::GetOrCreateVertexDeclaration(Elements);
	}
	else if (StreamType == EVertexInputStreamType::PositionAndNormalOnly)
	{
		PositionAndNormalDeclaration = PipelineStateCache::GetOrCreateVertexDeclaration(Elements);
	}
	else // (StreamType == EVertexInputStreamType::Default)
	{
		// Create the vertex declaration for rendering the factory normally.
		Declaration = PipelineStateCache::GetOrCreateVertexDeclaration(Elements);
	}
}

void FPrimitiveIdDummyBuffer::InitRHI(FRHICommandListBase& RHICmdList) 
{
	// create a static vertex buffer
	const FRHIBufferCreateDesc CreateDesc =
		FRHIBufferCreateDesc::CreateVertex(TEXT("FPrimitiveIdDummyBuffer"), sizeof(uint32))
		.AddUsage(EBufferUsageFlags::Static | EBufferUsageFlags::ShaderResource)
		.SetInitialState(ERHIAccess::VertexOrIndexBuffer | ERHIAccess::SRVMask);

	VertexBufferRHI = UE::RHIResourceUtils::CreateBufferZeroed(RHICmdList, CreateDesc);
	VertexBufferSRV = RHICmdList.CreateShaderResourceView(
		VertexBufferRHI,
		FRHIViewDesc::CreateBufferSRV()
			.SetType(FRHIViewDesc::EBufferType::Typed)
			.SetFormat(PF_R32_UINT));
}

TGlobalResource<FPrimitiveIdDummyBuffer> GPrimitiveIdDummy;
