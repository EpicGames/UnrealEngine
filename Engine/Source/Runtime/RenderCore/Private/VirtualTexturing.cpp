// Copyright Epic Games, Inc. All Rights Reserved.

#include "VirtualTexturing.h"
#include "VirtualTextureEnum.h"

DEFINE_LOG_CATEGORY(LogVirtualTexturing);


// ----------------------------------------------------------------------------------

FVTProducerDescription::FVTProducerDescription()
	: Priority(EVTProducerPriority::Normal) // Cannot inline-initialize because EVTProducerPriority is forward-declared
{}


// ----------------------------------------------------------------------------------

void FVirtualTextureLocalTileRequest::ValidatePriorities() const
{
	static_assert(static_cast<uint64>(EVTProducerPriority::Count) <= (1 << 3), "EVTProducerPriority should be packable on 3 bits");
	static_assert(static_cast<uint64>(EVTInvalidatePriority::Count) <= (1 << 1), "EVTInvalidatePriority should be packable on 1 bit");
	checkSlow(ProducerPriority < (1 << 3));
	checkSlow(InvalidatePriority < (1 << 1));
}
