// Copyright Epic Games, Inc. All Rights Reserved.

#include "VisualizeTexture.h"
#include "Misc/App.h"
#include "ShaderParameters.h"
#include "Misc/WildcardString.h"
#include "RHIStaticStates.h"
#include "RenderingThread.h"
#include "Shader.h"
#include "RenderTargetPool.h"
#include "GlobalShader.h"
#include "PipelineStateCache.h"
#include "PixelShaderUtils.h"
#include "Misc/FileHelper.h"
#include "RenderCore.h"

void FVisualizeTexture::ParseCommands(const TCHAR* Cmd, FOutputDevice &Ar)
{
#if SUPPORTS_VISUALIZE_TEXTURE
	// Find out what command to do based on first parameter.
	ECommand Command = ECommand::Unknown;
	int32 ViewId = 0;
	FString ViewName;
	FString RDGResourceName;
	TOptional<uint32> ResourceVersion;
	TOptional<FWildcardString> ResourceListWildCard;
	{
		FString FirstParameter = FParse::Token(Cmd, 0);


		if (FirstParameter.IsEmpty())
		{
			// NOP
		}
		else if (FChar::IsDigit(**FirstParameter))
		{
			Command = ECommand::DisableVisualization;
		}
		else if (FirstParameter == TEXT("help"))
		{
			Command = ECommand::DisplayHelp;
		}
		else if (FirstParameter == TEXT("pool"))
		{
			Command = ECommand::DisplayPoolResourceList;
		}
		else if (FirstParameter.StartsWith(TEXT("view=")))
		{
			if (FirstParameter.Len() == 5)
			{
				// Empty payload, reset view ID to zero
				Command = ECommand::SetViewId;
				ViewId = 0;
			}
			else if (FirstParameter[5] == TEXT('?'))
			{
				Command = ECommand::DisplayViewList;
			}
			else
			{
				Command = ECommand::SetViewId;

				// Supports view ID or string name of view
				TCHAR* EndOfNumber;
				ViewId = FCString::Strtoi(&FirstParameter[5], &EndOfNumber, 0);
				if (EndOfNumber - &FirstParameter[0] != FirstParameter.Len())
				{
					// Didn't parse as a number, treat it as a string
					ViewId = 0;
					ViewName = FirstParameter.Right(FirstParameter.Len() - 5);
				}
			}
		}
		else
		{
			const TCHAR* AfterAt = *FirstParameter;

			while (*AfterAt != 0 && *AfterAt != TCHAR('@'))
			{
				++AfterAt;
			}

			if (*AfterAt == TCHAR('@'))
			{
				RDGResourceName = FirstParameter.Left(AfterAt - *FirstParameter);
				ResourceVersion = FCString::Atoi(AfterAt + 1);
			}
			else
			{
				RDGResourceName = FirstParameter;
			}

			if (RDGResourceName.Contains(TEXT("*")))
			{
				ResourceListWildCard = FWildcardString(RDGResourceName);
				Command = ECommand::DisplayResourceList;
			}
			else
			{
				Command = ECommand::VisualizeResource;
				Visualize(RDGResourceName, ResourceVersion);
			}
		}
	}

	if (Command == ECommand::Unknown)
	{
		FVisualizeTexture::DisplayHelp(Ar);
		DisplayResourceListToLog(TOptional<FWildcardString>());
	}
	else if (Command == ECommand::DisplayHelp)
	{
		FVisualizeTexture::DisplayHelp(Ar);
	}
	else if (Command == ECommand::DisableVisualization)
	{
		Visualize({});
	}
	else if (Command == ECommand::VisualizeResource)
	{
		Config = {};
		Visualize(RDGResourceName, ResourceVersion);
		for (;;)
		{
			FString Parameter = FParse::Token(Cmd, 0);

			if (Parameter.IsEmpty())
			{
				break;
			}
			else if (Parameter == TEXT("uv0"))
			{
				Config.InputUVMapping = EInputUVMapping::LeftTop;
			}
			else if (Parameter == TEXT("uv1"))
			{
				Config.InputUVMapping = EInputUVMapping::Whole;
			}
			else if (Parameter == TEXT("uv2"))
			{
				Config.InputUVMapping = EInputUVMapping::PixelPerfectCenter;
			}
			else if (Parameter == TEXT("pip"))
			{
				Config.InputUVMapping = EInputUVMapping::PictureInPicture;
			}
			else if (Parameter == TEXT("bmp"))
			{
				Config.Flags |= EFlags::SaveBitmap;
			}
			else if (Parameter == TEXT("stencil"))
			{
				Config.Flags |= EFlags::SaveBitmapAsStencil;
			}
			else if (Parameter == TEXT("frac"))
			{
				Config.ShaderOp = EShaderOp::Frac;
			}
			else if (Parameter == TEXT("sat"))
			{
				Config.ShaderOp = EShaderOp::Saturate;
			}
			else if (Parameter.Left(3) == TEXT("mip"))
			{
				Parameter.RightInline(Parameter.Len() - 3, EAllowShrinking::No);
				Config.MipIndex = FCString::Atoi(*Parameter);
			}
			else if (Parameter.Left(5) == TEXT("index"))
			{
				Parameter.RightInline(Parameter.Len() - 5, EAllowShrinking::No);
				Config.ArrayIndex = FCString::Atoi(*Parameter);
			}
			// e.g. RGB*6, A, *22, /2.7, A*7
			else if (Parameter.Left(3) == TEXT("rgb")
				|| Parameter.Left(1) == TEXT("a")
				|| Parameter.Left(1) == TEXT("r")
				|| Parameter.Left(1) == TEXT("g")
				|| Parameter.Left(1) == TEXT("b")
				|| Parameter.Left(1) == TEXT("*")
				|| Parameter.Left(1) == TEXT("/"))
			{
				Config.SingleChannel = -1;

				if (Parameter.Left(3) == TEXT("rgb"))
				{
					Parameter.RightInline(Parameter.Len() - 3, EAllowShrinking::No);
				}
				else if (Parameter.Left(1) == TEXT("r")) Config.SingleChannel = 0;
				else if (Parameter.Left(1) == TEXT("g")) Config.SingleChannel = 1;
				else if (Parameter.Left(1) == TEXT("b")) Config.SingleChannel = 2;
				else if (Parameter.Left(1) == TEXT("a")) Config.SingleChannel = 3;
				if (Config.SingleChannel >= 0)
				{
					Parameter.RightInline(Parameter.Len() - 1, EAllowShrinking::No);
					Config.SingleChannelMul = 1.0f;
					Config.RGBMul = 0.0f;
				}

				float Mul = 1.0f;

				// * or /
				if (Parameter.Left(1) == TEXT("*"))
				{
					Parameter.RightInline(Parameter.Len() - 1, EAllowShrinking::No);
					Mul = FCString::Atof(*Parameter);
				}
				else if (Parameter.Left(1) == TEXT("/"))
				{
					Parameter.RightInline(Parameter.Len() - 1, EAllowShrinking::No);
					Mul = 1.0f / FCString::Atof(*Parameter);
				}
				Config.RGBMul *= Mul;
				Config.SingleChannelMul *= Mul;
				Config.AMul *= Mul;
			}
			else
			{
				Ar.Logf(TEXT("Error: parameter \"%s\" not recognized"), *Parameter);
			}
		}

	}
	else if (Command == ECommand::DisplayPoolResourceList)
	{
		ESortBy SortBy = ESortBy::Index;

		for (;;)
		{
			FString Parameter = FParse::Token(Cmd, 0);

			if (Parameter.IsEmpty())
			{
				break;
			}
			else if (Parameter == TEXT("byname"))
			{
				SortBy = ESortBy::Name;
			}
			else if (Parameter == TEXT("bysize"))
			{
				SortBy = ESortBy::Size;
			}
			else
			{
				Ar.Logf(TEXT("Error: parameter \"%s\" not recognized"), *Parameter);
			}
		}

		DisplayPoolResourceListToLog(SortBy);
	}
	else if (Command == ECommand::DisplayResourceList)
	{
		bool bListAllocated = false;
		ESortBy SortBy = ESortBy::Index;

		for (;;)
		{
			FString Parameter = FParse::Token(Cmd, 0);

			if (Parameter.IsEmpty())
			{
				break;
			}
			else
			{
				Ar.Logf(TEXT("Error: parameter \"%s\" not recognized"), *Parameter);
			}
		}

		DisplayResourceListToLog(ResourceListWildCard);
	}
	else if (Command == ECommand::DisplayViewList)
	{
		DisplayViewListToLog();
	}
	else if (Command == ECommand::SetViewId) //-V547
	{
		Requested.ViewUniqueId = ViewId;
		Requested.ViewName = ViewName;
	}
	else
	{
		unimplemented();
	}

	// Enable tracking when the system is first interacted with
	if (State == EState::Inactive)
	{
		State = EState::TrackResources;
	}
#endif
}

void FVisualizeTexture::DebugLogOnCrash()
{
#if SUPPORTS_VISUALIZE_TEXTURE
	DisplayPoolResourceListToLog(ESortBy::Size);
	DisplayResourceListToLog(TOptional<FWildcardString>());
#endif
}

void FVisualizeTexture::GetTextureInfos_GameThread(TArray<FString>& Infos) const
{
	check(IsInGameThread());
	FlushRenderingCommands();

	for (uint32 Index = 0, Num = GRenderTargetPool.GetElementCount(); Index < Num; ++Index)
	{
		FPooledRenderTarget* RenderTarget = GRenderTargetPool.GetElementById(Index);

		if (!RenderTarget)
		{
			continue;
		}

		FPooledRenderTargetDesc Desc = RenderTarget->GetDesc();
		uint32 SizeInKB = (RenderTarget->ComputeMemorySize() + 1023) / 1024;
		FString Entry = FString::Printf(TEXT("%s %d %s %d"),
			*Desc.GenerateInfoString(),
			Index + 1,
			Desc.DebugName ? Desc.DebugName : TEXT("<Unnamed>"),
			SizeInKB);
		Infos.Add(Entry);
	}
}

TGlobalResource<FVisualizeTexture> GVisualizeTexture;

#if SUPPORTS_VISUALIZE_TEXTURE

void FVisualizeTexture::DisplayHelp(FOutputDevice &Ar)
{
	Ar.Logf(TEXT("VisualizeTexture/Vis <RDGResourceNameWildcard>:"));
	Ar.Logf(TEXT("  Lists all RDG resource names with wildcard filtering."));
	Ar.Logf(TEXT(""));
	Ar.Logf(TEXT("VisualizeTexture/Vis <RDGResourceName>[@<Version>] [<Mode>] [PIP/UV0/UV1/UV2] [BMP] [FRAC/SAT] [FULL]:"));
	Ar.Logf(TEXT("  RDGResourceName = Name of the resource set when creating it with RDG."));
	Ar.Logf(TEXT("  Version = Integer to specify a specific intermediate version."));
	Ar.Logf(TEXT("  Mode (examples):"));
	Ar.Logf(TEXT("    RGB      = RGB in range 0..1 (default)"));
	Ar.Logf(TEXT("    *8       = RGB * 8"));
	Ar.Logf(TEXT("    A        = alpha channel in range 0..1"));
	Ar.Logf(TEXT("    R        = red channel in range 0..1"));
	Ar.Logf(TEXT("    G        = green channel in range 0..1"));
	Ar.Logf(TEXT("    B        = blue channel in range 0..1"));
	Ar.Logf(TEXT("    A*16     = Alpha * 16"));
	Ar.Logf(TEXT("    RGB/2    = RGB / 2"));
	Ar.Logf(TEXT("  SubResource:"));
	Ar.Logf(TEXT("    MIP5     = Mip level 5 (0 is default)"));
	Ar.Logf(TEXT("    INDEX5   = Array Element 5 (0 is default)"));
	Ar.Logf(TEXT("  InputMapping:"));
	Ar.Logf(TEXT("    PIP      = like UV1 but as picture in picture with normal rendering  (default)"));
	Ar.Logf(TEXT("    UV0      = UV in left top"));
	Ar.Logf(TEXT("    UV1      = full texture"));
	Ar.Logf(TEXT("    UV2      = pixel perfect centered"));
	Ar.Logf(TEXT("  Flags:"));
	Ar.Logf(TEXT("    BMP      = save out bitmap to the screenshots folder (not on console, normalized)"));
	Ar.Logf(TEXT("    STENCIL  = Stencil normally displayed in alpha channel of depth.  This option is used for BMP to get a stencil only BMP."));
	Ar.Logf(TEXT("    FRAC     = use frac() in shader (default)"));
	Ar.Logf(TEXT("    SAT      = use saturate() in shader"));
	Ar.Logf(TEXT(""));
	Ar.Logf(TEXT("VisualizeTexture/Vis 0"));
	Ar.Logf(TEXT("  Stops visualizing a resource."));
	Ar.Logf(TEXT(""));
	Ar.Logf(TEXT("VisualizeTexture/Vis pool [BYNAME/BYSIZE]:"));
	Ar.Logf(TEXT("  Shows list of all resources in the pool."));
	Ar.Logf(TEXT("  BYNAME   = sort pool list by name"));
	Ar.Logf(TEXT("  BYSIZE   = show pool list by size"));
	Ar.Logf(TEXT(""));
	Ar.Logf(TEXT("VisualizeTexture/Vis view=[ID/NAME]"));
	Ar.Logf(TEXT("  Unique ID or name of view to visualize textures from, \"view=?\" to dump list of available views"));
	Ar.Logf(TEXT(""));
}

void FVisualizeTexture::DisplayPoolResourceListToLog(FVisualizeTexture::ESortBy SortBy)
{
	struct FSortedLines
	{
		FString Line;
		int32   SortIndex = 0;
		uint32  PoolIndex = 0;

		bool operator < (const FSortedLines& B) const
		{
			// first large ones
			if (SortIndex < B.SortIndex)
			{
				return true;
			}
			if (SortIndex > B.SortIndex)
			{
				return false;
			}

			return Line < B.Line;
		}
	};

	TArray<FSortedLines> SortedLines;

	for (uint32 Index = 0, Num = GRenderTargetPool.GetElementCount(); Index < Num; ++Index)
	{
		FPooledRenderTarget* RenderTarget = GRenderTargetPool.GetElementById(Index);

		if (!RenderTarget)
		{
			continue;
		}

		const FPooledRenderTargetDesc Desc = RenderTarget->GetDesc();

		const uint32 SizeInKB = (RenderTarget->ComputeMemorySize() + 1023) / 1024;

		FString UnusedStr;

		if (RenderTarget->GetUnusedForNFrames() > 0)
		{
			UnusedStr = FString::Printf(TEXT(" unused(%d)"), RenderTarget->GetUnusedForNFrames());
		}

		FSortedLines Element;
		Element.PoolIndex = Index;
		Element.SortIndex = Index;

		FString InfoString = Desc.GenerateInfoString();

		switch (SortBy)
		{
		case ESortBy::Index:
		{
			// Constant works well with the average name length
			const uint32 TotalSpacerSize = 36;
			const uint32 SpaceCount = FMath::Max<int32>(0, TotalSpacerSize - InfoString.Len());

			for (uint32 Space = 0; Space < SpaceCount; ++Space)
			{
				InfoString.AppendChar((TCHAR)' ');
			}

			// Sort by index
			Element.Line = FString::Printf(TEXT("%s %s %d KB%s"), *InfoString, Desc.DebugName, SizeInKB, *UnusedStr);
		}
		break;

		case ESortBy::Name:
		{
			Element.Line = FString::Printf(TEXT("%s %s %d KB%s"), Desc.DebugName, *InfoString, SizeInKB, *UnusedStr);
			Element.SortIndex = 0;
		}
		break;

		case ESortBy::Size:
		{
			Element.Line = FString::Printf(TEXT("%d KB %s %s%s"), SizeInKB, *InfoString, Desc.DebugName, *UnusedStr);
			Element.SortIndex = -(int32)SizeInKB;
		}
		break;

		default:
			checkNoEntry();
		}

		SortedLines.Add(Element);
	}

	SortedLines.Sort();

	for (int32 Index = 0; Index < SortedLines.Num(); Index++)
	{
		const FSortedLines& Entry = SortedLines[Index];

		UE_LOGF(LogConsoleResponse, Log, "   %3d = %ls", Entry.PoolIndex + 1, *Entry.Line);
	}

	UE_LOGF(LogConsoleResponse, Log, "");

	uint32 WholeCount;
	uint32 WholePoolInKB;
	uint32 UsedInKB;
	GRenderTargetPool.GetStats(WholeCount, WholePoolInKB, UsedInKB);

	UE_LOGF(LogConsoleResponse, Log, "Pool: %d/%d MB (referenced/allocated)", (UsedInKB + 1023) / 1024, (WholePoolInKB + 1023) / 1024);

	UE_LOGF(LogConsoleResponse, Log, "");
}

void FVisualizeTexture::DisplayResourceListToLog(const TOptional<FWildcardString>& Wildcard)
{
	if (!IsActive())
	{
		State = EState::DisplayResources;
		DisplayResourcesParam = Wildcard;
		return;
	}

	UE_LOGF(LogConsoleResponse, Log, "RDGResourceName (what was rendered this frame, use <RDGResourceName>@<Version> to get intermediate versions):");

	TArray<FString> Entries;
	Entries.Reserve(VersionCountMap.Num());
	for (auto KV : VersionCountMap)
	{
		if (Wildcard.IsSet())
		{
			if (Wildcard->IsMatch(KV.Key))
			{
				Entries.Add(KV.Key);
			}
		}
		else
		{
			Entries.Add(KV.Key);
		}
	}
	Entries.Sort();

	// Magic number works well with the name length we have
	const int32 MaxColumnCount = 5;
	const int32 SpaceBetweenColumns = 1;
	const int32 TargetColumnHeight = 8;

	int32 ColumnCount = FMath::Clamp(FMath::DivideAndRoundUp(Entries.Num(), TargetColumnHeight), 1, MaxColumnCount);
	int32 ColumnHeight = FMath::DivideAndRoundUp(Entries.Num(), ColumnCount);

	// Width of the column in characters, init with 0
	TStaticArray<int32, MaxColumnCount> ColumnWidths;
	for (int32 ColumnId = 0; ColumnId < MaxColumnCount; ColumnId++)
	{
		ColumnWidths[ColumnId] = 0;
	}

	for (int32 Index = 0, Count = Entries.Num(); Index < Count; ++Index)
	{
		const FString& Entry = *Entries[Index];
		const int32 Column = Index / ColumnHeight;
		ColumnWidths[Column] = FMath::Max(ColumnWidths[Column], Entry.Len());
	}

	// Print them sorted, if possible multiple in a line
	for (int32 RowId = 0; RowId < ColumnHeight; RowId++)
	{
		FString Line;
		int32 ColumnAlignment = 0;

		for (int32 ColumnId = 0; ColumnId < ColumnCount; ColumnId++)
		{
			int32 EntryId = ColumnId * ColumnHeight + RowId;

			if (EntryId >= Entries.Num())
			{
				break;
			}

			const FString& Entry = *Entries[EntryId];

			const int32 SpaceCount = ColumnAlignment - Line.Len();
			check(SpaceCount >= 0);
			for (int32 Space = 0; Space < SpaceCount; ++Space)
			{
				Line.AppendChar((TCHAR)' ');
			}

			Line += Entry;

			ColumnAlignment += SpaceBetweenColumns + ColumnWidths[ColumnId];
		}

		UE_LOGF(LogConsoleResponse, Log, "   %ls", *Line);
	}

	UE_LOGF(LogConsoleResponse, Log, "");
}

void FVisualizeTexture::DisplayViewListToLog()
{
	if (!IsActive())
	{
		State = EState::DisplayViews;
		return;
	}

	// Display view list sorted by unique ID
	TArray<FSetElementId> Entries;
	Entries.Reserve(ViewDescriptionMap.Num());
	for (auto ViewIt = ViewDescriptionMap.CreateConstIterator(); ViewIt; ++ViewIt)
	{
		Entries.Add(ViewIt.GetId());
	}

	Entries.Sort([ViewDescriptionMap = ViewDescriptionMap](const FSetElementId& A, const FSetElementId& B)
	{
		return ViewDescriptionMap.Get(A).Key < ViewDescriptionMap.Get(B).Key;
	});

	UE_LOGF(LogConsoleResponse, Log, "Visualize Texture available views:");

	for (FSetElementId ElementId : Entries)
	{
		UE_LOGF(LogConsoleResponse, Log, "   %d  %ls", ViewDescriptionMap.Get(ElementId).Key, *ViewDescriptionMap.Get(ElementId).Value);
	}
}

static TAutoConsoleVariable<int32> CVarAllowBlinking(
	TEXT("r.VisualizeTexture.AllowBlinking"), 1,
	TEXT("Whether to allow blinking when visualizing NaN or inf that can become irritating over time.\n"),
	ECVF_RenderThreadSafe);

enum class EVisualisePSType
{
	Cube = 0,
	Texture1D = 1, //not supported
	Texture2DNoMSAA = 2,
	Texture3D = 3,
	CubeArray = 4,
	Texture2DMSAA = 5,
	Texture2DDepthStencilNoMSAA = 6,
	Texture2DUINT8 = 7,
	Texture2DUINT32 = 8,
	MAX
};

/** A pixel shader which filters a texture. */
// @param TextureType 0:Cube, 1:1D(not yet supported), 2:2D no MSAA, 3:3D, 4:Cube[], 5:2D MSAA, 6:2D DepthStencil no MSAA (needed to avoid D3DDebug error)
class FVisualizeTexturePS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FVisualizeTexturePS);
	SHADER_USE_PARAMETER_STRUCT(FVisualizeTexturePS, FGlobalShader);

	class FVisualisePSTypeDim : SHADER_PERMUTATION_ENUM_CLASS("TEXTURE_TYPE", EVisualisePSType);

	using FPermutationDomain = TShaderPermutationDomain<FVisualisePSTypeDim>;

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		FPermutationDomain PermutationVector(Parameters.PermutationId);
		return PermutationVector.Get<FVisualisePSTypeDim>() != EVisualisePSType::Texture1D;
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(FVector3f, TextureExtent)
		SHADER_PARAMETER_ARRAY(FVector4f, VisualizeParam, [3])

		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, VisualizeTexture2D)
		SHADER_PARAMETER_SAMPLER(SamplerState, VisualizeTexture2DSampler)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture3D, VisualizeTexture3D)
		SHADER_PARAMETER_SAMPLER(SamplerState, VisualizeTexture3DSampler)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(TextureCube, VisualizeTextureCube)
		SHADER_PARAMETER_SAMPLER(SamplerState, VisualizeTextureCubeSampler)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(TextureCubeArray, VisualizeTextureCubeArray)
		SHADER_PARAMETER_SAMPLER(SamplerState, VisualizeTextureCubeArraySampler)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D<uint4>, VisualizeDepthStencil)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2DMS<float4>, VisualizeTexture2DMS)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D<uint>, VisualizeUINT8Texture2D)

		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()
};

IMPLEMENT_GLOBAL_SHADER(FVisualizeTexturePS, "/Engine/Private/Tools/VisualizeTexture.usf", "VisualizeTexturePS", SF_Pixel);

static EVisualisePSType GetVisualizePSType(const FRDGTextureDesc& Desc)
{
	if(Desc.IsTexture2D())
	{
		// 2D		
		if(Desc.NumSamples > 1)
		{
			// MSAA
			return EVisualisePSType::Texture2DMSAA;
		}
		else
		{
			if(Desc.Format == PF_DepthStencil)
			{
				// DepthStencil non MSAA (needed to avoid D3DDebug error)
				return EVisualisePSType::Texture2DDepthStencilNoMSAA;
			}
			else if (Desc.Format == PF_R8_UINT)
			{
				return EVisualisePSType::Texture2DUINT8;
			}
			else if (Desc.Format == PF_R32_UINT)
			{
				return EVisualisePSType::Texture2DUINT32;
			}
			else
			{
				// non MSAA
				return EVisualisePSType::Texture2DNoMSAA;
			}
		}
	}
	else if(Desc.IsTextureCube())
	{
		if(Desc.IsTextureArray())
		{
			// Cube[]
			return EVisualisePSType::CubeArray;
		}
		else
		{
			// Cube
			return EVisualisePSType::Cube;
		}
	}

	check(Desc.IsTexture3D());
	return EVisualisePSType::Texture3D;
}

void FVisualizeTexture::ReleaseRHI()
{
	Config = {};
	Requested = {};
	Captured = {};
}

// static
FRDGTextureRef FVisualizeTexture::AddVisualizeTexturePass(
	FRDGBuilder& GraphBuilder,
	FGlobalShaderMap* ShaderMap,
	const FRDGTextureRef InputTexture,
	const FConfig& VisualizeConfig,
	EInputValueMapping InputValueMapping,
	uint32 CaptureId)
{
	check(InputTexture);
	check(!EnumHasAnyFlags(InputTexture->Desc.Flags, TexCreate_CPUReadback));

	const FRDGTextureDesc& InputDesc = InputTexture->Desc;
	FIntPoint InputExtent = InputDesc.Extent;
	FIntPoint OutputExtent = InputExtent;

	// Scene textures are padded and shared across scene renderers, with a given scene renderer using a viewport in the shared buffer.
	// We only want to visualize the portion actually used by the given scene renderer, as the rest will be blank or garbage.  The info
	// text will display the actual texture size in addition to the viewport being visualized.
	FIntPoint VisualizeTextureExtent = InputTexture->GetVisualizeExtent();
	if ((VisualizeTextureExtent.X > 0) && (VisualizeTextureExtent.Y > 0))
	{
		// Clamp extent at actual dimensions of texture
		OutputExtent.X = FMath::Min(VisualizeTextureExtent.X, OutputExtent.X);
		OutputExtent.Y = FMath::Min(VisualizeTextureExtent.Y, OutputExtent.Y);
	}

	if (InputDesc.IsTextureCube())
	{
		// For pixel perfect display of cube map, we'll use a 4x3 flat unwrapping of the cube map, rather than a projection.  The visualization
		// shader detects the 4x3 aspect ratio, and generates a seamless panorama in the middle, with the adjacent floor and sky tiles above
		// and below.  There will be seams between floor and sky tiles, but the pixels shown will otherwise be exact.
		if (GVisualizeTexture.Config.InputUVMapping == EInputUVMapping::PixelPerfectCenter)
		{
			InputExtent.X *= 4;
			InputExtent.Y *= 3;
			OutputExtent.X *= 4;
			OutputExtent.Y *= 3;
		}
		else
		{
			// Longitudinal rendered cube maps look better with 2 to 1 aspect ratio (same as how the texture resource viewer displays cube maps)
			InputExtent.X *= 2;
			OutputExtent.X *= 2;
		}
	}

	GVisualizeTexture.Captured.OutputExtent = OutputExtent;

	// Clamp to reasonable value to prevent crash
	OutputExtent.X = FMath::Max(OutputExtent.X, 1);
	OutputExtent.Y = FMath::Max(OutputExtent.Y, 1);

	FRDGTextureRef OutputTexture = GraphBuilder.CreateTexture(FRDGTextureDesc::Create2D(OutputExtent, PF_B8G8R8A8, FClearValueBinding(FLinearColor(1, 1, 0, 1)), TexCreate_RenderTargetable | TexCreate_ShaderResource), InputTexture->Name);

	{
		const EVisualisePSType VisualizeType = GetVisualizePSType(InputDesc);

		FVisualizeTexturePS::FParameters* PassParameters = GraphBuilder.AllocParameters<FVisualizeTexturePS::FParameters>();

		{
			PassParameters->TextureExtent = FVector3f(InputExtent.X, InputExtent.Y, InputDesc.Depth);

			{
				// Alternates between 0 and 1 with a short pause
				const float FracTimeScale = 1.0f / 4.0f;
				float FracTime = FApp::GetCurrentTime() * FracTimeScale - floor(FApp::GetCurrentTime() * FracTimeScale);
				float BlinkState = (FracTime < 1.0f / 16.0f) ? 1.0f : 0.0f;

				FVector4f VisualizeParamValue[3];

				float Add = 0.0f;
				float FracScale = 1.0f;

				// w * almost_1 to avoid frac(1) => 0
				PassParameters->VisualizeParam[0] = FVector4f(VisualizeConfig.RGBMul, VisualizeConfig.SingleChannelMul, Add, FracScale * 0.9999f);
				PassParameters->VisualizeParam[1] = FVector4f(CVarAllowBlinking.GetValueOnRenderThread() ? BlinkState : 0.0f, (VisualizeConfig.ShaderOp == EShaderOp::Saturate) ? 1.0f : 0.0f, VisualizeConfig.ArrayIndex, VisualizeConfig.MipIndex);
				PassParameters->VisualizeParam[2] = FVector4f((float)InputValueMapping, 0.0f, VisualizeConfig.SingleChannel);
			}

			FRDGTextureSRV* InputSRV = nullptr;
			FRHISamplerState* PointSampler = TStaticSamplerState<SF_Point, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
			if (InputTexture->Desc.Dimension == ETextureDimension::Texture2DArray)
			{
				InputSRV = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::CreateForSlice(InputTexture, FMath::Clamp(int32(VisualizeConfig.ArrayIndex), 0, int32(InputDesc.ArraySize) - 1)));
			}
			else
			{
				InputSRV = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::Create(InputTexture));
			}

			PassParameters->VisualizeTexture2D = InputSRV;
			PassParameters->VisualizeTexture2DSampler = PointSampler;
			PassParameters->VisualizeTexture3D = InputSRV;
			PassParameters->VisualizeTexture3DSampler = PointSampler;
			PassParameters->VisualizeTextureCube = InputSRV;
			PassParameters->VisualizeTextureCubeSampler = PointSampler;
			PassParameters->VisualizeTextureCubeArray = InputSRV;
			PassParameters->VisualizeTextureCubeArraySampler = PointSampler;

			if (VisualizeType == EVisualisePSType::Texture2DDepthStencilNoMSAA)
			{
				FRDGTextureSRVDesc SRVDesc = FRDGTextureSRVDesc::CreateWithPixelFormat(InputTexture, PF_X24_G8);
				PassParameters->VisualizeDepthStencil = GraphBuilder.CreateSRV(SRVDesc);
			}

			PassParameters->VisualizeTexture2DMS = InputTexture;
			PassParameters->VisualizeUINT8Texture2D = InputTexture;

			PassParameters->RenderTargets[0] = FRenderTargetBinding(OutputTexture, ERenderTargetLoadAction::EClear);
		}

		FVisualizeTexturePS::FPermutationDomain PermutationVector;
		PermutationVector.Set<FVisualizeTexturePS::FVisualisePSTypeDim>(VisualizeType);

		TShaderMapRef<FVisualizeTexturePS> PixelShader(ShaderMap, PermutationVector);

		FString ExtendedDrawEvent;
		if (GraphBuilder.ShouldEmitEvents())
		{
			if (InputDesc.IsTexture3D())
			{
				ExtendedDrawEvent += FString::Printf(TEXT("x%d CapturedSlice=%d"), InputDesc.Depth, VisualizeConfig.ArrayIndex);
			}

			if (InputDesc.IsTextureArray())
			{
				ExtendedDrawEvent += FString::Printf(TEXT(" ArraySize=%d CapturedSlice=%d"), InputDesc.ArraySize, VisualizeConfig.ArrayIndex);
			}

			// Precise the mip level being captured in the mip level when there is a mip chain.
			if (InputDesc.IsMipChain())
			{
				ExtendedDrawEvent += FString::Printf(TEXT(" Mips=%d CapturedMip=%d"), InputDesc.NumMips, VisualizeConfig.MipIndex);
			}
		}

		FPixelShaderUtils::AddFullscreenPass(
			GraphBuilder,
			ShaderMap,
			RDG_EVENT_NAME("VisualizeTextureCapture(%s@%d %s %dx%d%s)",
				InputTexture->Name, CaptureId,
				GPixelFormats[InputDesc.Format].Name,
				InputExtent.X, InputExtent.Y,
				*ExtendedDrawEvent),
			PixelShader,
			PassParameters,
			FIntRect(0, 0, OutputExtent.X, OutputExtent.Y));
	}

	return OutputTexture;
}

void FVisualizeTexture::CreateContentCapturePass(FRDGBuilder& GraphBuilder, const FRDGTextureRef InputTexture, uint32 CaptureId)
{
	if (!InputTexture)
	{
		return;
	}

	const FRDGTextureDesc& InputDesc = InputTexture->Desc;
	const FIntPoint InputExtent = InputDesc.Extent;

	if (EnumHasAnyFlags(InputDesc.Flags, TexCreate_CPUReadback))
	{
		return;
	}

	EInputValueMapping InputValueMapping = EInputValueMapping::Color;
	{
		if (InputDesc.Format == PF_ShadowDepth)
		{
			InputValueMapping = EInputValueMapping::Shadow;
		}
		else if (EnumHasAnyFlags(InputDesc.Flags, TexCreate_DepthStencilTargetable))
		{
			InputValueMapping = EInputValueMapping::Depth;
		}
	}

	FGlobalShaderMap* ShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);
	FRDGTextureRef OutputTexture = AddVisualizeTexturePass(GraphBuilder, ShaderMap, InputTexture, Config, InputValueMapping, CaptureId);

	FIntPoint OutputExtent = InputExtent;
	OutputExtent.X = FMath::Max(OutputExtent.X, 1);
	OutputExtent.Y = FMath::Max(OutputExtent.Y, 1);

	{
		Captured.Desc = Translate(InputDesc);
		Captured.Desc.DebugName = InputTexture->Name;
		Captured.PooledRenderTarget = nullptr;
		Captured.Texture = OutputTexture;
		Captured.InputValueMapping = InputValueMapping;
		Captured.ViewRects = FamilyViewRects;

		GraphBuilder.QueueTextureExtraction(OutputTexture, &Captured.PooledRenderTarget);
	}

	if (EnumHasAnyFlags(Config.Flags, EFlags::SaveBitmap | EFlags::SaveBitmapAsStencil))
	{
		uint32 MipAdjustedExtentX = FMath::Clamp(OutputExtent.X >> Config.MipIndex, 0, OutputExtent.X);
		uint32 MipAdjustedExtentY = FMath::Clamp(OutputExtent.Y >> Config.MipIndex, 0, OutputExtent.Y);
		FIntPoint Extent(MipAdjustedExtentX, MipAdjustedExtentY);

		FReadSurfaceDataFlags ReadDataFlags;
		ReadDataFlags.SetLinearToGamma(false);
		ReadDataFlags.SetOutputStencil(EnumHasAnyFlags(Config.Flags, EFlags::SaveBitmapAsStencil));
		ReadDataFlags.SetMip(Config.MipIndex);

		const TCHAR* DebugName = Captured.Desc.DebugName;

		AddReadbackTexturePass(GraphBuilder, RDG_EVENT_NAME("SaveBitmap"), OutputTexture,
			[OutputTexture, Extent, ReadDataFlags, DebugName](FRHICommandListImmediate& RHICmdList)
		{
			TArray<FColor> Bitmap;
			RHICmdList.ReadSurfaceData(OutputTexture->GetRHI(), FIntRect(0, 0, Extent.X, Extent.Y), Bitmap, ReadDataFlags);

			// if the format and texture type is supported
			if (Bitmap.Num())
			{
				// Create screenshot folder if not already present.
				IFileManager::Get().MakeDirectory(*FPaths::ScreenShotDir(), true);

				const FString Filename(FPaths::ScreenShotDir() / TEXT("VisualizeTexture"));

				uint32 ExtendXWithMSAA = Bitmap.Num() / Extent.Y;

				// Save the contents of the array to a bitmap file. (24bit only so alpha channel is dropped)
				FFileHelper::CreateBitmap(*Filename, ExtendXWithMSAA, Extent.Y, Bitmap.GetData());

				UE_LOGF(LogRendererCore, Display, "Content was saved to \"%ls\"", *FPaths::ScreenShotDir());
			}
			else
			{
				UE_LOGF(LogRendererCore, Error, "Failed to save BMP for VisualizeTexture, format or texture type is not supported");
			}
		});
	}
}

void FVisualizeTexture::BeginFrameRenderThread()
{
	bAnyViewRendered = false;
	bIsRequestedView = false;
	bFoundRequestedView = false;
}

static bool VisualizeTextureViewNameMatches(const FString& ViewName, const TCHAR* Description)
{
	// Description will be of the form "EditorName (FName)" or "Name", with EditorName being user facing.  Match name followed by space or null terminator.
	int32 ViewNameLen = ViewName.Len();
	return !FCString::Strnicmp(*ViewName, Description, ViewNameLen) && (Description[ViewNameLen] == ' ' || Description[ViewNameLen] == 0);
}

void FVisualizeTexture::BeginViewRenderThread(ERHIFeatureLevel::Type InFeatureLevel, int32 UniqueId, const TCHAR* Description, bool bIsSceneCapture, bool bIsReflectionCapture)
{
	// Only support visualization for views with a unique ID, or reflection scene captures
	if (State == EState::Inactive || (!UniqueId && !(bIsSceneCapture && bIsReflectionCapture)))
	{
		return;
	}

	FeatureLevel = InFeatureLevel;

	if (!bAnyViewRendered)
	{
		// Clear list of views out when we encounter the first view on the current frame
		ViewDescriptionMap.Empty();
		bAnyViewRendered = true;
	}

	ViewDescriptionMap.FindOrAdd(UniqueId) = Description;

	if (!Requested.ViewName.IsEmpty() ? VisualizeTextureViewNameMatches(Requested.ViewName, Description) : (UniqueId && Requested.ViewUniqueId == UniqueId))
	{
		// Found the specific view we requested
		bIsRequestedView = true;
		bFoundRequestedView = true;
	}
	else if (!bFoundRequestedView)
	{
		// If specific requested view hasn't been found, visualize any view that's not a scene capture, so we still get some sort of result
		bIsRequestedView = !bIsSceneCapture;
	}

	// Clear outputs when we are processing a requested view
	if (bIsRequestedView)
	{
		VersionCountMap.Empty();
		Captured = {};
		Captured.ViewUniqueId = UniqueId;
	}
}

void FVisualizeTexture::SetSceneTextures(const TArray<FRDGTextureRef>& InSceneTextures, FIntPoint InFamilySize, const TArray<FIntRect>& InFamilyViewRects)
{
	for (FRDGTextureRef Texture : InSceneTextures)
	{
		Texture->EncloseVisualizeExtent(InFamilySize);
	}
	FamilyViewRects = InFamilyViewRects;
}

TOptional<uint32> FVisualizeTexture::ShouldCapture(const TCHAR* InName, uint32 InMipIndex)
{
	TOptional<uint32> CaptureId;
	uint32& VersionCount = VersionCountMap.FindOrAdd(InName);
	if (!Requested.Name.IsEmpty() && Requested.Name == InName)
	{
		if (!Requested.Version || VersionCount == Requested.Version.GetValue())
		{
			CaptureId = VersionCount;
		}
	}
	++VersionCount;
	return CaptureId;
}

void FVisualizeTexture::EndViewRenderThread()
{
	if (bIsRequestedView)
	{
		bIsRequestedView = false;
		FamilyViewRects.Empty();
	}
}

void FVisualizeTexture::EndFrameRenderThread()
{
	if (bAnyViewRendered)
	{
		if (State == EState::DisplayResources)
		{
			DisplayResourceListToLog(DisplayResourcesParam);
			DisplayResourcesParam.Reset();
			State = EState::TrackResources;
		}
		else if (State == EState::DisplayViews)
		{
			DisplayViewListToLog();
			State = EState::TrackResources;
		}
	}
}

uint32 FVisualizeTexture::GetVersionCount(const TCHAR* InName) const
{
	if (const uint32* VersionCount = VersionCountMap.Find(InName))
	{
		return *VersionCount;
	}
	return 0;
}

void FVisualizeTexture::SetCheckPoint(FRDGBuilder& GraphBuilder, IPooledRenderTarget* PooledRenderTarget)
{
	check(IsInRenderingThread());

	if (!PooledRenderTarget)
	{
		return;
	}

	const FPooledRenderTargetDesc& Desc = PooledRenderTarget->GetDesc();

	if (!EnumHasAnyFlags(Desc.Flags, TexCreate_ShaderResource))
	{
		return;
	}

	TOptional<uint32> CaptureId = ShouldCapture(Desc.DebugName, Config.MipIndex);
	if (!CaptureId)
	{
		return;
	}

	FRDGTextureRef TextureToCapture = GraphBuilder.RegisterExternalTexture(PooledRenderTarget);
	CreateContentCapturePass(GraphBuilder, TextureToCapture, CaptureId.GetValue());
}

void FVisualizeTexture::SetCheckPoint(FRHICommandListImmediate& RHICmdList, IPooledRenderTarget* PooledRenderTarget)
{
	FRDGBuilder GraphBuilder(RHICmdList);
	SetCheckPoint(GraphBuilder, PooledRenderTarget);
	GraphBuilder.Execute();
}

void FVisualizeTexture::Visualize(const FString& InName, TOptional<uint32> InVersion)
{
	Requested.Name = InName;
	Requested.Version = InVersion;
}

#endif // SUPPORTS_VISUALIZE_TEXTURE

bool FVisualizeTexture::IsNamedView(const TCHAR* Name) const
{
#if SUPPORTS_VISUALIZE_TEXTURE
	return VisualizeTextureViewNameMatches(Requested.ViewName, Name);
#else
	return false;
#endif
}

// static
FRDGTextureRef FVisualizeTexture::AddVisualizeTexturePass(
	FRDGBuilder& GraphBuilder,
	class FGlobalShaderMap* ShaderMap,
	const FRDGTextureRef InputTexture,
	bool bUseConfig)
#if SUPPORTS_VISUALIZE_TEXTURE
{
	check(InputTexture);

	if (bUseConfig)
	{
		// This function is called to visualize an external resource that isn't written by RDG, so we only want to add a visualization pass the first time
		// it's encountered, rather than each time, as we don't expect it to change.  So if it's already in VersionCountMap, return.  Also skip if this
		// isn't the active view.
		if (!GVisualizeTexture.IsRequestedView() || GVisualizeTexture.VersionCountMap.Find(InputTexture->Name))
		{
			return InputTexture;
		}

		// Add it to the map, so it shows up in the list of textures that can be visualized (even if it's not the requested texture).
		GVisualizeTexture.VersionCountMap.Add(InputTexture->Name) = 1;

		// If not the requested texture, just return.
		if (InputTexture->Name != GVisualizeTexture.Requested.Name)
		{
			return InputTexture;
		}
	}

	EInputValueMapping InputValueMapping = EInputValueMapping::Color;
	{
		if (InputTexture->Desc.Format == PF_ShadowDepth)
		{
			InputValueMapping = EInputValueMapping::Shadow;
		}
		else if (EnumHasAnyFlags(InputTexture->Desc.Flags, TexCreate_DepthStencilTargetable))
		{
			InputValueMapping = EInputValueMapping::Depth;
		}
	}

	FConfig VisualizeConfig;

	return AddVisualizeTexturePass(GraphBuilder, ShaderMap, InputTexture, bUseConfig ? GVisualizeTexture.Config : VisualizeConfig, InputValueMapping, /* CaptureId = */ 0);
}
#else
{
	return InputTexture;
}
#endif

// static
FRDGTextureRef FVisualizeTexture::AddVisualizeTextureAlphaPass(
	FRDGBuilder& GraphBuilder,
	class FGlobalShaderMap* ShaderMap,
	const FRDGTextureRef InputTexture)
#if SUPPORTS_VISUALIZE_TEXTURE
{
	check(InputTexture);
	FConfig VisualizeConfig;
	VisualizeConfig.SingleChannel = 3;
	VisualizeConfig.SingleChannelMul = 1.0f;
	VisualizeConfig.RGBMul = 0.0f;

	return AddVisualizeTexturePass(GraphBuilder, ShaderMap, InputTexture, VisualizeConfig, EInputValueMapping::Color, /* CaptureId = */ 0);
}
#else
{
	return InputTexture;
}
#endif
