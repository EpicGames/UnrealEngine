// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "CoreMinimal.h"
#include "Features/IModularFeature.h"
#include "UObject/SoftObjectPtr.h"

enum EShaderPlatform : uint16;

/**
 * Modular feature interface for mesh deformer providers. 
 * Modules that inherit from this need to be loaded before shader compilation starts (PostConfigInit)
 * so that the correct vertex factories can be created.
 */
class IMeshDeformerProvider : public IModularFeature
{
public:
	virtual ~IMeshDeformerProvider() = default;

	static RENDERCORE_API const FName ModularFeatureName; // "MeshDeformer"
	static RENDERCORE_API bool IsAvailable();
	static RENDERCORE_API IMeshDeformerProvider* Get();

	/** Returns true if the platform is supported. */
	virtual bool IsSupported(EShaderPlatform Platform) const = 0;

	/** Structure for passing to GetDefaultMeshDeformer(). */
	struct FDefaultMeshDeformerSetup
	{
		bool bIsRequestingDeformer = false;
		bool bIsRequestingRecomputeTangent = false;
	};

	/** 
	 * Returns a default mesh deformer. 
	 * This can allow a mesh deformer plugin to automatically replace the UE fixed function animation path.
	 */
	virtual TObjectPtr<class UMeshDeformer> GetDefaultMeshDeformer(FDefaultMeshDeformerSetup const& Setup) = 0;
};
