// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GlobalShader.h"
#include "HAL/PreprocessorHelpers.h"
#include "Math/Vector4.h"
#include "PipelineStateCache.h"
#include "RHI.h"
#include "RenderResource.h"
#include "Serialization/MemoryLayout.h"
#include "Shader.h"
#include "ShaderParameterMacros.h"
#include "ShaderParameterMetadata.h"
#include "ShaderParameterStruct.h"
#include "ShaderParameterUtils.h"
#include "ShaderPermutation.h"

class FPointerTableBase;
class FRDGTexture;
struct FShaderCompilerEnvironment;

BEGIN_SHADER_PARAMETER_STRUCT(FBinkParameters, )
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, tex0)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, tex1)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, tex2)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, tex3)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, tex4)
	SHADER_PARAMETER_SAMPLER(SamplerState, samp0)
	SHADER_PARAMETER_SAMPLER(SamplerState, samp1)
	SHADER_PARAMETER_SAMPLER(SamplerState, samp2)
	SHADER_PARAMETER_SAMPLER(SamplerState, samp3)
	SHADER_PARAMETER_SAMPLER(SamplerState, samp4)
	SHADER_PARAMETER(FVector4f, consta)
	SHADER_PARAMETER(FVector4f, crc)
	SHADER_PARAMETER(FVector4f, cbc)
	SHADER_PARAMETER(FVector4f, adj)
	SHADER_PARAMETER(FVector4f, yscale)
	SHADER_PARAMETER(FVector4f, xy_xform0)
	SHADER_PARAMETER(FVector4f, xy_xform1)
	SHADER_PARAMETER(FVector4f, uv_xform0)
	SHADER_PARAMETER(FVector4f, uv_xform1)
	SHADER_PARAMETER(FVector4f, uv_xform2)
	RENDER_TARGET_BINDING_SLOTS()
END_SHADER_PARAMETER_STRUCT()

struct FBinkDrawVS : FGlobalShader
{
	DECLARE_EXPORTED_SHADER_TYPE(FBinkDrawVS, Global, RENDERCORE_API);

	SHADER_USE_PARAMETER_STRUCT(FBinkDrawVS, FGlobalShader);

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) 
	{
		return true;
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FBinkParameters, BinkParameters)
	END_SHADER_PARAMETER_STRUCT()
};

struct FBinkDrawYCbCrPS : FGlobalShader
{
	DECLARE_EXPORTED_SHADER_TYPE(FBinkDrawYCbCrPS, Global, RENDERCORE_API);
	SHADER_USE_PARAMETER_STRUCT(FBinkDrawYCbCrPS, FGlobalShader);

	class FALPHA    : SHADER_PERMUTATION_BOOL("ALPHA");
	class FSRGB     : SHADER_PERMUTATION_BOOL("SRGB");

	using FPermutationDomain = TShaderPermutationDomain<FALPHA, FSRGB>;

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) 
	{
		return true;
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FBinkParameters, BinkParameters)
	END_SHADER_PARAMETER_STRUCT()
};

struct FBinkDrawICtCpPS : FGlobalShader
{
	DECLARE_EXPORTED_SHADER_TYPE(FBinkDrawICtCpPS, Global, RENDERCORE_API);
	SHADER_USE_PARAMETER_STRUCT(FBinkDrawICtCpPS, FGlobalShader);

	class FALPHA    : SHADER_PERMUTATION_BOOL("ALPHA");
	class FTONEMAP  : SHADER_PERMUTATION_BOOL("TONEMAP");
	class FST2084   : SHADER_PERMUTATION_BOOL("ST2084");

	using FPermutationDomain = TShaderPermutationDomain<FALPHA, FTONEMAP, FST2084>;

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) 
	{
		return true;
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FBinkParameters, BinkParameters)
	END_SHADER_PARAMETER_STRUCT()
};

