// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataDrivenShaderPlatformInfo.h"
#include "GlobalShader.h"
#include "RHIDefinitions.h"
#include "ShaderParameterMacros.h"
#include "ShaderParameterStruct.h"
#include "RenderUtils.h"

#if RHI_RAYTRACING

class FBuiltInRayTracingShader : public FGlobalShader
{
protected:
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{}

public:
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return ShouldCompileRayTracingShadersForProject(Parameters.Platform);
	}

	FBuiltInRayTracingShader() = default;
	FBuiltInRayTracingShader(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FGlobalShader(Initializer)
	{}
};


class FDefaultPayloadMS : public FBuiltInRayTracingShader
{
	DECLARE_EXPORTED_SHADER_TYPE(FDefaultPayloadMS, Global, RENDERCORE_API);
public:
	static ERayTracingPayloadType GetRayTracingPayloadType(const int32 PermutationId);

	FDefaultPayloadMS() = default;
	FDefaultPayloadMS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FBuiltInRayTracingShader(Initializer)
	{}
};

class FPackedMaterialClosestHitPayloadMS : public FBuiltInRayTracingShader
{
	DECLARE_EXPORTED_SHADER_TYPE(FPackedMaterialClosestHitPayloadMS, Global, RENDERCORE_API);
public:
	static ERayTracingPayloadType GetRayTracingPayloadType(const int32 PermutationId);

	FPackedMaterialClosestHitPayloadMS() = default;
	FPackedMaterialClosestHitPayloadMS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FBuiltInRayTracingShader(Initializer)
	{}
};

class FRayTracingDispatchDescCS : public FBuiltInRayTracingShader
{
	DECLARE_GLOBAL_SHADER(FRayTracingDispatchDescCS);

public:
	FRayTracingDispatchDescCS() = default;
	FRayTracingDispatchDescCS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FBuiltInRayTracingShader(Initializer)
	{
		DispatchDescInputParam.Bind(Initializer.ParameterMap, TEXT("DispatchDescInput"), SPF_Mandatory);
		DispatchDescSizeDwordsParam.Bind(Initializer.ParameterMap, TEXT("DispatchDescSizeDwords"), SPF_Mandatory);
		DispatchDescDimensionsOffsetDwordsParam.Bind(Initializer.ParameterMap, TEXT("DispatchDescDimensionsOffsetDwords"), SPF_Mandatory);
		DimensionsBufferOffsetDwordsParam.Bind(Initializer.ParameterMap, TEXT("DimensionsBufferOffsetDwords"), SPF_Mandatory);
		DispatchDimensionsParam.Bind(Initializer.ParameterMap, TEXT("DispatchDimensions"), SPF_Mandatory);
		DispatchDescOutputParam.Bind(Initializer.ParameterMap, TEXT("DispatchDescOutput"), SPF_Mandatory);
	}

	static inline void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		OutEnvironment.SetDefine(TEXT("DISPATCH_DESC_MAX_SIZE_DWORDS"), DispatchDescMaxSizeDwords);
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
	}

	static RENDERCORE_API void Dispatch(FRHICommandList& RHICmdList,
		const void* DispatchDescInput, uint32 DispatchDescSize, uint32 DispatchDescDimensionsOffset,
		FRHIShaderResourceView* DispatchDimensionsSRV, uint32 DimensionsBufferOffset,
		FRHIUnorderedAccessView* DispatchDescOutputUAV);

	static constexpr uint32 DispatchDescMaxSizeDwords = 32;

	LAYOUT_FIELD(FShaderParameter, DispatchDescInputParam);
	LAYOUT_FIELD(FShaderParameter, DispatchDescSizeDwordsParam);
	LAYOUT_FIELD(FShaderParameter, DispatchDescDimensionsOffsetDwordsParam);
	LAYOUT_FIELD(FShaderParameter, DimensionsBufferOffsetDwordsParam);
	LAYOUT_FIELD(FShaderResourceParameter, DispatchDimensionsParam);
	LAYOUT_FIELD(FShaderResourceParameter, DispatchDescOutputParam);
};

#endif // RHI_RAYTRACING

