// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/Color.h"
#include "Math/IntPoint.h"
#include "Math/IntRect.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector4.h"
#include "RenderResource.h"
#include "Templates/Function.h"

class FGraphicsPipelineStateInitializer;
class FRHICommandList;
class FRHIUnorderedAccessView;
struct FRWBuffer;
struct FRWBufferStructured;

class FClearVertexBuffer : public FVertexBuffer
{
public:
	/**
	* Initialize the RHI for this rendering resource
	*/
	virtual void InitRHI(FRHICommandListBase& RHICmdList) override;
};
extern RENDERCORE_API TGlobalResource<FClearVertexBuffer> GClearVertexBuffer;

struct FClearQuadCallbacks
{
	TFunction<void(FGraphicsPipelineStateInitializer&)> PSOModifier = nullptr;
	TFunction<void(FRHICommandList&)> PreClear = nullptr;
	TFunction<void(FRHICommandList&)> PostClear = nullptr;
};

extern RENDERCORE_API void ClearUAV(FRHICommandList& RHICmdList, FRHIUnorderedAccessView* Buffer, uint32 NumBytes, uint32 Value, bool bBarriers = true);
extern RENDERCORE_API void DrawClearQuadAlpha(FRHICommandList& RHICmdList, float Alpha);
extern RENDERCORE_API void DrawClearQuadMRT(FRHICommandList& RHICmdList, bool bClearColor, int32 NumClearColors, const FLinearColor* ClearColorArray, bool bClearDepth, float Depth, bool bClearStencil, uint32 Stencil);
extern RENDERCORE_API void DrawClearQuadMRT(FRHICommandList& RHICmdList, bool bClearColor, int32 NumClearColors, const FLinearColor* ClearColorArray, bool bClearDepth, float Depth, bool bClearStencil, uint32 Stencil, FClearQuadCallbacks ClearQuadCallbacks);
extern RENDERCORE_API void DrawClearQuadMRT(FRHICommandList& RHICmdList, bool bClearColor, int32 NumClearColors, const FLinearColor* ClearColorArray, bool bClearDepth, float Depth, bool bClearStencil, uint32 Stencil, FIntPoint ViewSize, FIntRect ExcludeRect);
extern RENDERCORE_API void DrawClearQuadMRTWithUints(FRHICommandList& RHICmdList, bool bClearColor, int32 NumClearColors, const FLinearColor* ClearColorArray, bool bClearDepth, float Depth, bool bClearStencil, uint32 Stencil, uint8 NumUintOutput);

inline void DrawClearQuad(FRHICommandList& RHICmdList, bool bClearColor, const FLinearColor& Color, bool bClearDepth, float Depth, bool bClearStencil, uint32 Stencil)
{
	DrawClearQuadMRT(RHICmdList, bClearColor, 1, &Color, bClearDepth, Depth, bClearStencil, Stencil);
}

inline void DrawClearQuad(FRHICommandList& RHICmdList, bool bClearColor, const FLinearColor& Color, bool bClearDepth, float Depth, bool bClearStencil, uint32 Stencil, FIntPoint ViewSize, FIntRect ExcludeRect)
{
	DrawClearQuadMRT(RHICmdList, bClearColor, 1, &Color, bClearDepth, Depth, bClearStencil, Stencil, ViewSize, ExcludeRect);
}

inline void DrawClearQuad(FRHICommandList& RHICmdList, const FLinearColor& Color)
{
	DrawClearQuadMRT(RHICmdList, true, 1, &Color, false, 0, false, 0);
}

inline void DrawClearQuad(FRHICommandList& RHICmdList, const FLinearColor& Color, FClearQuadCallbacks ClearQuadCallbacks)
{
	DrawClearQuadMRT(RHICmdList, true, 1, &Color, false, 0, false, 0, ClearQuadCallbacks);
}
