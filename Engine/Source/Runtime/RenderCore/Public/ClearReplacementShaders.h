// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "GlobalShader.h"
#include "Math/IntVector.h"
#include "Math/UnrealMathSSE.h"
#include "PipelineStateCache.h"
#include "PixelFormat.h"
#include "RenderGraphUtils.h"
#include "RHI.h"
#include "RHICommandList.h"
#include "RHIDefinitions.h"
#include "Serialization/MemoryLayout.h"
#include "Shader.h"
#include "ShaderCompilerCore.h"
#include "ShaderCore.h"
#include "ShaderParameterUtils.h"
#include "ShaderParameters.h"
#include "Templates/EnableIf.h"
#include "Templates/Function.h"

class FPointerTableBase;

enum class EClearReplacementResourceType
{
	Buffer = 0,
	Texture2D = 1,
	Texture2DArray = 2,
	Texture3D = 3,
	StructuredBuffer = 4,
	LargeBuffer = 5
};

enum class EClearReplacementValueType
{
	Float,
	Int32,
	Uint32
};

template <EClearReplacementValueType Type> struct TClearReplacementTypeSelector {};
template <> struct TClearReplacementTypeSelector<EClearReplacementValueType::Float> { typedef float Type; };
template <> struct TClearReplacementTypeSelector<EClearReplacementValueType::Int32> { typedef int32 Type; };
template <> struct TClearReplacementTypeSelector<EClearReplacementValueType::Uint32> { typedef uint32 Type; };

template <EClearReplacementValueType ValueType, uint32 NumChannels, bool bZeroOutput = false, bool bEnableBounds = false>
struct TClearReplacementBase : public FGlobalShader
{
	static_assert(NumChannels >= 1 && NumChannels <= 4, "Only 1 to 4 channels are supported.");
	DECLARE_INLINE_TYPE_LAYOUT(TClearReplacementBase, NonVirtual);

	// Wrapping group counts will require bounds checking to account for additional groups from the wrapping stride.
	static constexpr bool bSupportsWrappedGroupCount = bEnableBounds;

protected:
	TClearReplacementBase() {}
	TClearReplacementBase(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FGlobalShader(Initializer)
	{
		if (!bZeroOutput)
		{
			ClearValueParam.Bind(Initializer.ParameterMap, TEXT("ClearValue"), SPF_Mandatory);
		}
		if (bEnableBounds)
		{
			MinBoundsParam.Bind(Initializer.ParameterMap, TEXT("MinBounds"), SPF_Mandatory);
			MaxBoundsParam.Bind(Initializer.ParameterMap, TEXT("MaxBounds"), SPF_Mandatory);
		}
	}

public:
	static const TCHAR* GetSourceFilename() { return TEXT("/Engine/Private/ClearReplacementShaders.usf"); }

	static inline void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);

		OutEnvironment.SetDefine(TEXT("ENABLE_CLEAR_VALUE"), !bZeroOutput);
		OutEnvironment.SetDefine(TEXT("ENABLE_BOUNDS"), bEnableBounds);

		switch (ValueType)
		{
		case EClearReplacementValueType::Float:
			switch (NumChannels)
			{
			case 1: OutEnvironment.SetDefine(TEXT("VALUE_TYPE"), TEXT("float")); break;
			case 2: OutEnvironment.SetDefine(TEXT("VALUE_TYPE"), TEXT("float2")); break;
			case 3: OutEnvironment.SetDefine(TEXT("VALUE_TYPE"), TEXT("float3")); break;
			case 4: OutEnvironment.SetDefine(TEXT("VALUE_TYPE"), TEXT("float4")); break;
			}
			break;

		case EClearReplacementValueType::Int32:
			switch (NumChannels)
			{
			case 1: OutEnvironment.SetDefine(TEXT("VALUE_TYPE"), TEXT("int")); break;
			case 2: OutEnvironment.SetDefine(TEXT("VALUE_TYPE"), TEXT("int2")); break;
			case 3: OutEnvironment.SetDefine(TEXT("VALUE_TYPE"), TEXT("int3")); break;
			case 4: OutEnvironment.SetDefine(TEXT("VALUE_TYPE"), TEXT("int4")); break;
			}
			break;

		case EClearReplacementValueType::Uint32:
			switch (NumChannels)
			{
			case 1: OutEnvironment.SetDefine(TEXT("VALUE_TYPE"), TEXT("uint")); break;
			case 2: OutEnvironment.SetDefine(TEXT("VALUE_TYPE"), TEXT("uint2")); break;
			case 3: OutEnvironment.SetDefine(TEXT("VALUE_TYPE"), TEXT("uint3")); break;
			case 4: OutEnvironment.SetDefine(TEXT("VALUE_TYPE"), TEXT("uint4")); break;
			}
			break;
		}
	}
	
	template <typename T = const FShaderParameter&>	inline typename TEnableIf<!bZeroOutput, T>::Type GetClearValueParam() const { return ClearValueParam; }
	template <typename T = const FShaderParameter&> inline typename TEnableIf<bEnableBounds, T>::Type GetMinBoundsParam() const { return MinBoundsParam;  }
	template <typename T = const FShaderParameter&> inline typename TEnableIf<bEnableBounds, T>::Type GetMaxBoundsParam() const { return MaxBoundsParam;  }
	
private:
	LAYOUT_FIELD(FShaderParameter, ClearValueParam);
	LAYOUT_FIELD(FShaderParameter, MinBoundsParam);
	LAYOUT_FIELD(FShaderParameter, MaxBoundsParam);

};
	
namespace ClearReplacementCS
{
	template <EClearReplacementResourceType> struct TThreadGroupSize {};
	template <> struct TThreadGroupSize<EClearReplacementResourceType::Buffer>				{ static constexpr int32 X =  64, Y = 1, Z = 1; };
	template <> struct TThreadGroupSize<EClearReplacementResourceType::Texture2D>			{ static constexpr int32 X =   8, Y = 8, Z = 1; };
	template <> struct TThreadGroupSize<EClearReplacementResourceType::Texture2DArray>		{ static constexpr int32 X =   8, Y = 8, Z = 1; };
	template <> struct TThreadGroupSize<EClearReplacementResourceType::Texture3D>			{ static constexpr int32 X =   4, Y = 4, Z = 4; };
	template <> struct TThreadGroupSize<EClearReplacementResourceType::StructuredBuffer>	{ static constexpr int32 X =  64, Y = 1, Z = 1; };
	template <> struct TThreadGroupSize<EClearReplacementResourceType::LargeBuffer>			{ static constexpr int32 X = 512, Y = 1, Z = 1; };
}

template <EClearReplacementResourceType ResourceType, typename BaseType>
class TClearReplacementCS : public BaseType
{
	DECLARE_EXPORTED_SHADER_TYPE(TClearReplacementCS, Global, RENDERCORE_API);

public:
	static constexpr uint32 ThreadGroupSizeX = ClearReplacementCS::TThreadGroupSize<ResourceType>::X;
	static constexpr uint32 ThreadGroupSizeY = ClearReplacementCS::TThreadGroupSize<ResourceType>::Y;
	static constexpr uint32 ThreadGroupSizeZ = ClearReplacementCS::TThreadGroupSize<ResourceType>::Z;

	// Use wrapping for 1D layouts if supported.
	static constexpr bool bUseWrappedGroupCount = (ThreadGroupSizeY == 1) && (ThreadGroupSizeZ == 1) && BaseType::bSupportsWrappedGroupCount;
	
	TClearReplacementCS() {}
	TClearReplacementCS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: BaseType(Initializer)
	{
		ClearResourceParam.Bind(Initializer.ParameterMap, TEXT("ClearResource"), SPF_Mandatory);
	}
	
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return true;
	}

	static inline void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		BaseType::ModifyCompilationEnvironment(Parameters, OutEnvironment);

		if (FDataDrivenShaderPlatformInfo::GetRequiresBindfulUtilityShaders(Parameters.Platform))
		{
			OutEnvironment.CompilerFlags.Add(CFLAG_ForceBindful);
		}
	
		OutEnvironment.SetDefine(TEXT("THREADGROUPSIZE_X"), ThreadGroupSizeX);
		OutEnvironment.SetDefine(TEXT("THREADGROUPSIZE_Y"), ThreadGroupSizeY);
		OutEnvironment.SetDefine(TEXT("THREADGROUPSIZE_Z"), ThreadGroupSizeZ);
		OutEnvironment.SetDefine(TEXT("USE_WRAPPED_GROUP_COUNT"), bUseWrappedGroupCount ? 1 : 0);
		OutEnvironment.SetDefine(TEXT("RESOURCE_TYPE"), uint32(ResourceType));
	}
	
	static const TCHAR* GetFunctionName() { return TEXT("ClearCS"); }

	inline const FShaderResourceParameter& GetClearResourceParam() const { return ClearResourceParam; }
	inline uint32 GetResourceParamIndex() const { return ClearResourceParam.GetBaseIndex(); }

	static inline FIntVector GetGroupCount(uint32 SizeX, uint32 SizeY, uint32 SizeZ)
	{
		if (bUseWrappedGroupCount)
		{
			return FComputeShaderUtils::GetGroupCountWrapped(SizeX, ThreadGroupSizeX);
		}

		return FIntVector(
			FMath::DivideAndRoundUp(SizeX, ThreadGroupSizeX), 
			FMath::DivideAndRoundUp(SizeY, ThreadGroupSizeY), 
			FMath::DivideAndRoundUp(SizeZ, ThreadGroupSizeZ));
	}

private:
	LAYOUT_FIELD(FShaderResourceParameter, ClearResourceParam);
};

template <bool bEnableDepth, typename BaseType>
class TClearReplacementVS : public BaseType
{
	DECLARE_EXPORTED_SHADER_TYPE(TClearReplacementVS, Global, RENDERCORE_API);

public:
	TClearReplacementVS() {}
	TClearReplacementVS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: BaseType(Initializer)
	{
		if (bEnableDepth)
		{
			DepthParam.Bind(Initializer.ParameterMap, TEXT("Depth"), SPF_Mandatory);
		}
	}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return true;
	}

	static inline void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		BaseType::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("ENABLE_DEPTH"), bEnableDepth);
	}

	static const TCHAR* GetFunctionName() { return TEXT("ClearVS"); }

	template <typename T = const FShaderParameter&>
	inline typename TEnableIf<bEnableDepth, T>::Type GetDepthParam() const
	{
		return DepthParam;
	}

private:
	LAYOUT_FIELD(FShaderParameter, DepthParam);
};

template <bool b128BitOutput, typename BaseType>
class TClearReplacementPS : public BaseType
{
	DECLARE_EXPORTED_SHADER_TYPE(TClearReplacementPS, Global, RENDERCORE_API);

public:
	TClearReplacementPS() {}
	TClearReplacementPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: BaseType(Initializer)
	{
	}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return true;
	}

	static inline void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		BaseType::ModifyCompilationEnvironment(Parameters, OutEnvironment);

		if (b128BitOutput)
		{
			OutEnvironment.SetRenderTargetOutputFormat(0, PF_A32B32G32R32F);
		}
		OutEnvironment.SetDefine(TEXT("ENABLE_DEPTH"), false);
	}

	static const TCHAR* GetFunctionName() { return TEXT("ClearPS"); }
};

// Not all combinations are defined here. Add more if required.
//                             Type  NC  Zero   Bounds
typedef TClearReplacementBase<EClearReplacementValueType::Uint32, 1, false, false> FClearReplacementBase_Uint;
typedef TClearReplacementBase<EClearReplacementValueType::Uint32, 4, false, false> FClearReplacementBase_Uint4;
typedef TClearReplacementBase<EClearReplacementValueType::Float , 4, false, false> FClearReplacementBase_Float4;
typedef TClearReplacementBase<EClearReplacementValueType::Uint32, 1, true , false> FClearReplacementBase_Uint_Zero;
typedef TClearReplacementBase<EClearReplacementValueType::Float , 4, true , false> FClearReplacementBase_Float4_Zero;
typedef TClearReplacementBase<EClearReplacementValueType::Float , 4, true , true > FClearReplacementBase_Float4_Zero_Bounds;
typedef TClearReplacementBase<EClearReplacementValueType::Uint32, 1, false, true > FClearReplacementBase_Uint_Bounds;
typedef TClearReplacementBase<EClearReplacementValueType::Uint32, 4, false, true > FClearReplacementBase_Uint4_Bounds;
typedef TClearReplacementBase<EClearReplacementValueType::Int32,  4, false, true > FClearReplacementBase_Sint4_Bounds;
typedef TClearReplacementBase<EClearReplacementValueType::Float , 1, false, true > FClearReplacementBase_Float_Bounds;
typedef TClearReplacementBase<EClearReplacementValueType::Float , 4, false, true > FClearReplacementBase_Float4_Bounds;

// Simple vertex shaders for generating screen quads. Optionally with a min/max bounds in NDC space, and depth value.
typedef TClearReplacementVS<false, FClearReplacementBase_Float4_Zero       > FClearReplacementVS;
typedef TClearReplacementVS<false, FClearReplacementBase_Float4_Zero_Bounds> FClearReplacementVS_Bounds;
typedef TClearReplacementVS<true,  FClearReplacementBase_Float4_Zero       > FClearReplacementVS_Depth;

// Simple pixel shader which outputs a specified solid color to MRT0.
typedef TClearReplacementPS<false, FClearReplacementBase_Float4>             FClearReplacementPS;
typedef TClearReplacementPS<true,  FClearReplacementBase_Float4>             FClearReplacementPS_128;
// Simple pixel shader which outputs zero to MRT0
typedef TClearReplacementPS<false, FClearReplacementBase_Float4_Zero>        FClearReplacementPS_Zero;
	
// Compute shaders for clearing each resource type, with a min/max bounds enabled.
typedef TClearReplacementCS<EClearReplacementResourceType::Buffer,           FClearReplacementBase_Uint_Bounds>   FClearReplacementCS_Buffer_Uint_Bounds;
typedef TClearReplacementCS<EClearReplacementResourceType::Buffer,           FClearReplacementBase_Float_Bounds>  FClearReplacementCS_Buffer_Float_Bounds;
typedef TClearReplacementCS<EClearReplacementResourceType::Buffer,           FClearReplacementBase_Float4_Bounds> FClearReplacementCS_Buffer_Float4_Bounds;
typedef TClearReplacementCS<EClearReplacementResourceType::LargeBuffer,      FClearReplacementBase_Uint_Bounds>   FClearReplacementCS_LargeBuffer_Uint_Bounds;
typedef TClearReplacementCS<EClearReplacementResourceType::LargeBuffer,      FClearReplacementBase_Float_Bounds>  FClearReplacementCS_LargeBuffer_Float_Bounds;
typedef TClearReplacementCS<EClearReplacementResourceType::LargeBuffer,      FClearReplacementBase_Float4_Bounds> FClearReplacementCS_LargeBuffer_Float4_Bounds;
typedef TClearReplacementCS<EClearReplacementResourceType::StructuredBuffer, FClearReplacementBase_Uint_Bounds>   FClearReplacementCS_StructuredBuffer_Uint_Bounds;
typedef TClearReplacementCS<EClearReplacementResourceType::StructuredBuffer, FClearReplacementBase_Float_Bounds>  FClearReplacementCS_StructuredBuffer_Float_Bounds;
typedef TClearReplacementCS<EClearReplacementResourceType::StructuredBuffer, FClearReplacementBase_Float4_Bounds> FClearReplacementCS_StructuredBuffer_Float4_Bounds;
typedef TClearReplacementCS<EClearReplacementResourceType::Texture3D,        FClearReplacementBase_Float4_Bounds> FClearReplacementCS_Texture3D_Float4_Bounds;
typedef TClearReplacementCS<EClearReplacementResourceType::Texture2D,		 FClearReplacementBase_Float4_Bounds> FClearReplacementCS_Texture2D_Float4_Bounds;
typedef TClearReplacementCS<EClearReplacementResourceType::Texture2DArray,	 FClearReplacementBase_Float4_Bounds> FClearReplacementCS_Texture2DArray_Float4_Bounds;

// Compute shaders for clearing each resource type. No bounds checks enabled.
typedef TClearReplacementCS<EClearReplacementResourceType::Buffer,           FClearReplacementBase_Uint_Zero>     FClearReplacementCS_Buffer_Uint_Zero;
typedef TClearReplacementCS<EClearReplacementResourceType::StructuredBuffer, FClearReplacementBase_Uint_Zero>     FClearReplacementCS_StructuredBuffer_Uint_Zero;
typedef TClearReplacementCS<EClearReplacementResourceType::Texture2DArray,   FClearReplacementBase_Uint_Zero>     FClearReplacementCS_Texture2DArray_Uint_Zero;
typedef TClearReplacementCS<EClearReplacementResourceType::Buffer,           FClearReplacementBase_Uint>          FClearReplacementCS_Buffer_Uint;
typedef TClearReplacementCS<EClearReplacementResourceType::LargeBuffer,      FClearReplacementBase_Uint>          FClearReplacementCS_LargeBuffer_Uint;
typedef TClearReplacementCS<EClearReplacementResourceType::StructuredBuffer, FClearReplacementBase_Uint>          FClearReplacementCS_StructuredBuffer_Uint;
typedef TClearReplacementCS<EClearReplacementResourceType::Texture2DArray,   FClearReplacementBase_Uint>          FClearReplacementCS_Texture2DArray_Uint;
	
typedef TClearReplacementCS<EClearReplacementResourceType::Texture3D,        FClearReplacementBase_Float4>        FClearReplacementCS_Texture3D_Float4;
typedef TClearReplacementCS<EClearReplacementResourceType::Texture2D,        FClearReplacementBase_Float4>        FClearReplacementCS_Texture2D_Float4;
typedef TClearReplacementCS<EClearReplacementResourceType::Texture2DArray,   FClearReplacementBase_Float4>        FClearReplacementCS_Texture2DArray_Float4;
	
// Used by ClearUAV_T in ClearQuad.cpp
typedef TClearReplacementCS<EClearReplacementResourceType::Texture3D,        FClearReplacementBase_Uint4>         FClearReplacementCS_Texture3D_Uint4;
typedef TClearReplacementCS<EClearReplacementResourceType::Texture2D,        FClearReplacementBase_Uint4>         FClearReplacementCS_Texture2D_Uint4;
typedef TClearReplacementCS<EClearReplacementResourceType::Texture2DArray,   FClearReplacementBase_Uint4>         FClearReplacementCS_Texture2DArray_Uint4;
typedef TClearReplacementCS<EClearReplacementResourceType::Buffer,			 FClearReplacementBase_Uint4>         FClearReplacementCS_Buffer_Uint4;
typedef TClearReplacementCS<EClearReplacementResourceType::LargeBuffer,		 FClearReplacementBase_Uint4>         FClearReplacementCS_LargeBuffer_Uint4;
typedef TClearReplacementCS<EClearReplacementResourceType::StructuredBuffer, FClearReplacementBase_Uint4>         FClearReplacementCS_StructuredBuffer_Uint4;

typedef TClearReplacementCS<EClearReplacementResourceType::Buffer,		     FClearReplacementBase_Uint4_Bounds>  FClearReplacementCS_Buffer_Uint4_Bounds;
typedef TClearReplacementCS<EClearReplacementResourceType::LargeBuffer,		 FClearReplacementBase_Uint4_Bounds>  FClearReplacementCS_LargeBuffer_Uint4_Bounds;
typedef TClearReplacementCS<EClearReplacementResourceType::StructuredBuffer, FClearReplacementBase_Uint4_Bounds>  FClearReplacementCS_StructuredBuffer_Uint4_Bounds;
typedef TClearReplacementCS<EClearReplacementResourceType::Texture3D,        FClearReplacementBase_Uint4_Bounds>  FClearReplacementCS_Texture3D_Uint4_Bounds;
typedef TClearReplacementCS<EClearReplacementResourceType::Texture2D,        FClearReplacementBase_Uint4_Bounds>  FClearReplacementCS_Texture2D_Uint4_Bounds;
typedef TClearReplacementCS<EClearReplacementResourceType::Texture2DArray,   FClearReplacementBase_Uint4_Bounds>  FClearReplacementCS_Texture2DArray_Uint4_Bounds;

typedef TClearReplacementCS<EClearReplacementResourceType::Buffer,           FClearReplacementBase_Sint4_Bounds>  FClearReplacementCS_Buffer_Sint4_Bounds;
typedef TClearReplacementCS<EClearReplacementResourceType::LargeBuffer,      FClearReplacementBase_Sint4_Bounds>  FClearReplacementCS_LargeBuffer_Sint4_Bounds;
typedef TClearReplacementCS<EClearReplacementResourceType::StructuredBuffer, FClearReplacementBase_Sint4_Bounds>  FClearReplacementCS_StructuredBuffer_Sint4_Bounds;
typedef TClearReplacementCS<EClearReplacementResourceType::Texture3D,        FClearReplacementBase_Sint4_Bounds>  FClearReplacementCS_Texture3D_Sint4_Bounds;
typedef TClearReplacementCS<EClearReplacementResourceType::Texture2D,        FClearReplacementBase_Sint4_Bounds>  FClearReplacementCS_Texture2D_Sint4_Bounds;
typedef TClearReplacementCS<EClearReplacementResourceType::Texture2DArray,   FClearReplacementBase_Sint4_Bounds>  FClearReplacementCS_Texture2DArray_Sint4_Bounds;

/**
 * Helper functions for running the clear replacement shader for specific resource types, values types and number of channels.
 * Can be used from inside RHIs via FRHICommandList_RecursiveHazardous. ResourceBindCallback is provided to allow the RHI to override
 * how the UAV resource is bound to the underlying platform context..
 */
template <EClearReplacementResourceType ResourceType, EClearReplacementValueType ValueType, uint32 NumChannels, bool bBarriers>
inline void ClearUAVShader_T(FRHIComputeCommandList& RHICmdList, FRHIUnorderedAccessView* UAV, uint32 SizeX, uint32 SizeY, uint32 SizeZ, const typename TClearReplacementTypeSelector<ValueType>::Type(&ClearValues)[NumChannels], TFunctionRef<void(FRHIComputeShader*, const FShaderResourceParameter&, bool)> ResourceBindCallback)
{
	typedef TClearReplacementCS<ResourceType, TClearReplacementBase<ValueType, NumChannels, false, true>> FClearShader;

	TShaderMapRef<FClearShader> ComputeShader(GetGlobalShaderMap(GMaxRHIFeatureLevel));
	FRHIComputeShader* ShaderRHI = ComputeShader.GetComputeShader();
	SetComputePipelineState(RHICmdList, ShaderRHI);

	FRHIBatchedShaderParameters& BatchedParameters = RHICmdList.GetScratchShaderParameters();

	SetShaderValue(BatchedParameters, ComputeShader->GetClearValueParam(), ClearValues);
	SetShaderValue(BatchedParameters, ComputeShader->GetMinBoundsParam(), FUintVector4(0, 0, 0, 0));
	SetShaderValue(BatchedParameters, ComputeShader->GetMaxBoundsParam(), FUintVector4(SizeX, SizeY, SizeZ, 0));

	RHICmdList.SetBatchedShaderParameters(ShaderRHI, BatchedParameters);

	if (bBarriers)
	{
		RHICmdList.Transition(FRHITransitionInfo(UAV, ERHIAccess::Unknown, ERHIAccess::UAVCompute));
	}

	ResourceBindCallback(ShaderRHI, ComputeShader->GetClearResourceParam(), true);

	FIntVector GroupCount = FClearShader::GetGroupCount(SizeX, SizeY, SizeZ);

	RHICmdList.DispatchComputeShader(GroupCount.X, GroupCount.Y, GroupCount.Z);

	ResourceBindCallback(ShaderRHI, ComputeShader->GetClearResourceParam(), false);

	if (bBarriers)
	{
		RHICmdList.Transition(FRHITransitionInfo(UAV, ERHIAccess::UAVCompute, ERHIAccess::SRVCompute));
	}
}

// Default implementation of ClearUAVShader_T which simply binds the UAV to the compute shader via RHICmdList.SetUAVParameter
template <EClearReplacementResourceType ResourceType, EClearReplacementValueType ValueType, uint32 NumChannels, bool bBarriers>
inline void ClearUAVShader_T(FRHIComputeCommandList& RHICmdList, FRHIUnorderedAccessView* UAV, uint32 SizeX, uint32 SizeY, uint32 SizeZ, const typename TClearReplacementTypeSelector<ValueType>::Type(&ClearValues)[NumChannels])
{
	typedef TClearReplacementCS<ResourceType, TClearReplacementBase<ValueType, NumChannels, false, true>> FClearShader;

	TShaderMapRef<FClearShader> ComputeShader(GetGlobalShaderMap(GMaxRHIFeatureLevel));
	FRHIComputeShader* ShaderRHI = ComputeShader.GetComputeShader();
	SetComputePipelineState(RHICmdList, ShaderRHI);

	FRHIBatchedShaderParameters& BatchedParameters = RHICmdList.GetScratchShaderParameters();

	SetShaderValue(BatchedParameters, ComputeShader->GetClearValueParam(), ClearValues);
	SetShaderValue(BatchedParameters, ComputeShader->GetMinBoundsParam(), FUintVector4(0, 0, 0, 0));
	SetShaderValue(BatchedParameters, ComputeShader->GetMaxBoundsParam(), FUintVector4(SizeX, SizeY, SizeZ, 0));

	if (bBarriers)
	{
		RHICmdList.Transition(FRHITransitionInfo(UAV, ERHIAccess::Unknown, ERHIAccess::UAVCompute));
	}

	SetUAVParameter(BatchedParameters, ComputeShader->GetClearResourceParam(), UAV);

	RHICmdList.SetBatchedShaderParameters(ShaderRHI, BatchedParameters);

	const FIntVector GroupCount = FClearShader::GetGroupCount(SizeX, SizeY, SizeZ);

	RHICmdList.DispatchComputeShader(GroupCount.X, GroupCount.Y, GroupCount.Z);

	if (RHICmdList.NeedsShaderUnbinds())
	{
		FRHIBatchedShaderUnbinds& BatchedUnbinds = RHICmdList.GetScratchShaderUnbinds();
		UnsetUAVParameter(BatchedUnbinds, ComputeShader->GetClearResourceParam());
		RHICmdList.SetBatchedShaderUnbinds(ShaderRHI, BatchedUnbinds);
	}

	if (bBarriers)
	{
		RHICmdList.Transition(FRHITransitionInfo(UAV, ERHIAccess::UAVCompute, ERHIAccess::SRVCompute));
	}
}

// Helper version of ClearUAVShader_T for determining float vs uint32 at runtime. Uses the above default implementation.
template <EClearReplacementResourceType ResourceType, uint32 NumChannels, bool bBarriers>
inline void ClearUAVShader_T(FRHIComputeCommandList& RHICmdList, FRHIUnorderedAccessView* UAV, uint32 SizeX, uint32 SizeY, uint32 SizeZ, const void* ClearValues, EClearReplacementValueType ValueType)
{
	switch (ValueType)
	{
	case EClearReplacementValueType::Float:
		ClearUAVShader_T<ResourceType, EClearReplacementValueType::Float, NumChannels, bBarriers>(RHICmdList, UAV, SizeX, SizeY, SizeZ, *reinterpret_cast<const float(*)[NumChannels]>(ClearValues));
		break;

	case EClearReplacementValueType::Uint32:
		ClearUAVShader_T<ResourceType, EClearReplacementValueType::Uint32, NumChannels, bBarriers>(RHICmdList, UAV, SizeX, SizeY, SizeZ, *reinterpret_cast<const uint32(*)[NumChannels]>(ClearValues));
		break;

	case EClearReplacementValueType::Int32:
		ClearUAVShader_T<ResourceType, EClearReplacementValueType::Int32, NumChannels, bBarriers>(RHICmdList, UAV, SizeX, SizeY, SizeZ, *reinterpret_cast<const int32(*)[NumChannels]>(ClearValues));
		break;
	}
}
