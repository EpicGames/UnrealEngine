// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	DummyRenderResource.h: Frequently used rendering resources
=============================================================================*/

#pragma once

#include "GlobalShader.h"
#include "HAL/Platform.h"
#include "Math/Vector2D.h"
#include "Math/Vector4.h"
#include "PipelineStateCache.h"
#include "RHI.h"
#include "RHIDefinitions.h"
#include "RenderResource.h"
#include "Serialization/MemoryLayout.h"
#include "Shader.h"
#include "ShaderParameterMacros.h"
#include "ShaderParameterStruct.h"
#include "Templates/UnrealTemplate.h"
#include "DataDrivenShaderPlatformInfo.h"

class FPointerTableBase;
class FRDGTexture;


/** The vertex data used to filter a texture. */
struct FFilterVertex
{
public:
	FVector4f Position;
	FVector2f UV;
};

/** The filter vertex declaration resource type. */
class FFilterVertexDeclaration : public FRenderResource
{
public:
	FVertexDeclarationRHIRef VertexDeclarationRHI;

	/** Destructor. */
	virtual ~FFilterVertexDeclaration() = default;

	virtual void InitRHI(FRHICommandListBase& RHICmdList)
	{
		FVertexDeclarationElementList Elements;
		uint16 Stride = sizeof(FFilterVertex);
		Elements.Add(FVertexElement(0, STRUCT_OFFSET(FFilterVertex, Position), VET_Float4, 0, Stride));
		Elements.Add(FVertexElement(0, STRUCT_OFFSET(FFilterVertex, UV), VET_Float2, 1, Stride));
		VertexDeclarationRHI = PipelineStateCache::GetOrCreateVertexDeclaration(Elements);
	}

	virtual void ReleaseRHI()
	{
		VertexDeclarationRHI.SafeRelease();
	}
};

extern RENDERCORE_API TGlobalResource<FFilterVertexDeclaration, FRenderResource::EInitPhase::Pre> GFilterVertexDeclaration;

/** The empty vertex declaration resource type. */
class FEmptyVertexDeclaration : public FRenderResource
{
public:
	FVertexDeclarationRHIRef VertexDeclarationRHI;

	/** Destructor. */
	virtual ~FEmptyVertexDeclaration() = default;

	virtual void InitRHI(FRHICommandListBase& RHICmdList)
	{
		FVertexDeclarationElementList Elements;
		VertexDeclarationRHI = PipelineStateCache::GetOrCreateVertexDeclaration(Elements);
	}

	virtual void ReleaseRHI()
	{
		VertexDeclarationRHI.SafeRelease();
	}
};

extern RENDERCORE_API TGlobalResource<FEmptyVertexDeclaration, FRenderResource::EInitPhase::Pre> GEmptyVertexDeclaration;

/**
* Static vertex and index buffer used for 2D screen rectangles.
*/
class FScreenRectangleVertexBuffer : public FVertexBuffer
{
public:
	/** Initialize the RHI for this rendering resource */
	void InitRHI(FRHICommandListBase& RHICmdList) override;
};

extern RENDERCORE_API TGlobalResource<FScreenRectangleVertexBuffer, FRenderResource::EInitPhase::Pre> GScreenRectangleVertexBuffer;


class FScreenRectangleIndexBuffer : public FIndexBuffer
{
public:
	/** Initialize the RHI for this rendering resource */
	void InitRHI(FRHICommandListBase& RHICmdList) override;
};

extern RENDERCORE_API TGlobalResource<FScreenRectangleIndexBuffer, FRenderResource::EInitPhase::Pre> GScreenRectangleIndexBuffer;


/** Vertex shader to draw a screen quad that works on all platforms. Does not have any shader parameters.
 * The pixel shader should just use SV_Position. */
class FScreenVertexShaderVS : public FGlobalShader
{
	DECLARE_EXPORTED_GLOBAL_SHADER(FScreenVertexShaderVS, RENDERCORE_API);
	SHADER_USE_PARAMETER_STRUCT(FScreenVertexShaderVS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
	END_SHADER_PARAMETER_STRUCT()
};

/** Vertex shader to draw an instanced quad covering all the viewports (SV_ViewportArrayIndex is output for each SV_InstanceID). Does not have any shader parameters.
 * The pixel shader should just use SV_Position. */
class FInstancedScreenVertexShaderVS : public FScreenVertexShaderVS
{
	DECLARE_GLOBAL_SHADER(FInstancedScreenVertexShaderVS);
	SHADER_USE_PARAMETER_STRUCT(FInstancedScreenVertexShaderVS, FScreenVertexShaderVS);

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters);

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FScreenVertexShaderVS::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SCREEN_VS_FOR_INSTANCED_VIEWS"), 1);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
	END_SHADER_PARAMETER_STRUCT()
};

/** Vertex shader to draw a quad covering all the viewports with mobile multi view (SV_RenderTargetArrayIndex is output for each SV_InstanceID if using the multi view fallback path). Does not have any shader parameters.
 * The pixel shader should expect UV in TEXCOORD0 (and EyeIndex in TEXCOORD1 if using the mobile multi view fallback path). */
class FMobileMultiViewVertexShaderVS : public FScreenVertexShaderVS
{
	DECLARE_EXPORTED_GLOBAL_SHADER(FMobileMultiViewVertexShaderVS, RENDERCORE_API);
	SHADER_USE_PARAMETER_STRUCT(FMobileMultiViewVertexShaderVS, FScreenVertexShaderVS);

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters);

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FScreenVertexShaderVS::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SCREEN_VS_FOR_MOBILE_MULTI_VIEW"), 1);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
	END_SHADER_PARAMETER_STRUCT()
};

/** Pixel shader to copy pixels from src to dst performing a format change that works on all platforms. */
class FCopyRectPS : public FGlobalShader
{
	DECLARE_EXPORTED_GLOBAL_SHADER(FCopyRectPS, RENDERCORE_API);
	SHADER_USE_PARAMETER_STRUCT(FCopyRectPS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, InputTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, InputSampler)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()
};

/** Pixel shader to copy pixels from src to dst performing a format change that works on all platforms -- variation that accepts SRV */
class FCopyRectSrvPS : public FGlobalShader
{
	DECLARE_EXPORTED_GLOBAL_SHADER(FCopyRectSrvPS, RENDERCORE_API);
	SHADER_USE_PARAMETER_STRUCT(FCopyRectSrvPS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, InputTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, InputSampler)
		RENDER_TARGET_BINDING_SLOTS()
		END_SHADER_PARAMETER_STRUCT()
};

/** Vertex shader to perform a screen rotation for Vulkan pre-rotation on mobile. */
class FImagePreTransformVS : public FGlobalShader
{
	DECLARE_EXPORTED_GLOBAL_SHADER(FImagePreTransformVS, RENDERCORE_API);
	SHADER_USE_PARAMETER_STRUCT(FImagePreTransformVS, FGlobalShader);

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsVulkanMobilePlatform(Parameters.Platform) || IsVulkanMobileSM5Platform(Parameters.Platform);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(FVector4f, PreTransform)
	END_SHADER_PARAMETER_STRUCT()
};
