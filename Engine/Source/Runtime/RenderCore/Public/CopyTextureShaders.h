// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GlobalShader.h"
#include "Math/IntVector.h"
#include "Math/UnrealMathSSE.h"
#include "Misc/AssertionMacros.h"
#include "RHICommandList.h"
#include "RHIDefinitions.h"
#include "Serialization/MemoryLayout.h"
#include "Shader.h"
#include "ShaderCompilerCore.h"
#include "ShaderCore.h"
#include "ShaderParameterUtils.h"
#include "ShaderParameters.h"
#include "DataDrivenShaderPlatformInfo.h"

class FPointerTableBase;
class FRHIComputeShader;

enum class ECopyTextureResourceType
{
	Texture2D      = 0,
	Texture2DArray = 1,
	Texture3D      = 2
};

enum class ECopyTextureValueType
{
	Float,
	Int32,
	Uint32
};

namespace CopyTextureCS
{
	template <ECopyTextureResourceType> struct TThreadGroupSize              { static constexpr int32 X = 8, Y = 8, Z = 1; };
	template <> struct TThreadGroupSize<ECopyTextureResourceType::Texture3D> { static constexpr int32 X = 4, Y = 4, Z = 4; };

	struct DispatchContext
	{
		uint32 ThreadGroupSizeX = 0u;
		uint32 ThreadGroupSizeY = 0u;
		uint32 ThreadGroupSizeZ = 0u;
		ECopyTextureResourceType SrcType;
		ECopyTextureResourceType DstType;
		ECopyTextureValueType ValueType;
		uint32 NumChannels;
	};
}

class FCopyTextureCS : public FGlobalShader
{
	DECLARE_EXPORTED_TYPE_LAYOUT(FCopyTextureCS, RENDERCORE_API, NonVirtual);
protected:
	FCopyTextureCS() {}
	FCopyTextureCS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FGlobalShader(Initializer)
	{
		DstOffsetParam.Bind(Initializer.ParameterMap, TEXT("DstOffset"), SPF_Mandatory);
		SrcOffsetParam.Bind(Initializer.ParameterMap, TEXT("SrcOffset"), SPF_Mandatory);
		DimensionsParam.Bind(Initializer.ParameterMap, TEXT("Dimensions"), SPF_Mandatory);
		SrcResourceParam.Bind(Initializer.ParameterMap, TEXT("SrcResource"), SPF_Mandatory);
		DstResourceParam.Bind(Initializer.ParameterMap, TEXT("DstResource"), SPF_Mandatory);
	}

public:
	static inline void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);

		if (FDataDrivenShaderPlatformInfo::GetRequiresBindfulUtilityShaders(Parameters.Platform))
		{
			OutEnvironment.CompilerFlags.Add(CFLAG_ForceBindful);
		}
	}

	inline const FShaderResourceParameter& GetSrcResourceParam() { return SrcResourceParam; }
	inline const FShaderResourceParameter& GetDstResourceParam() { return DstResourceParam; }

	void Dispatch(
		FRHIComputeCommandList& RHICmdList,
		const CopyTextureCS::DispatchContext& Context,
		FIntVector const& SrcOffset,
		FIntVector const& DstOffset,
		FIntVector const& Dimensions
	)
	{
		check(SrcOffset.GetMin() >= 0 && DstOffset.GetMin() >= 0 && Dimensions.GetMin() >= 0);
		check(Context.DstType != ECopyTextureResourceType::Texture2D || Dimensions.Z <= 1);

		FRHIBatchedShaderParameters& BatchedParameters = RHICmdList.GetScratchShaderParameters();

		SetShaderValue(BatchedParameters, SrcOffsetParam, SrcOffset);
		SetShaderValue(BatchedParameters, DstOffsetParam, DstOffset);
		SetShaderValue(BatchedParameters, DimensionsParam, Dimensions);

		RHICmdList.SetBatchedShaderParameters(RHICmdList.GetBoundComputeShader(), BatchedParameters);

		RHICmdList.DispatchComputeShader(
			FMath::DivideAndRoundUp(uint32(Dimensions.X), Context.ThreadGroupSizeX),
			FMath::DivideAndRoundUp(uint32(Dimensions.Y), Context.ThreadGroupSizeY),
			FMath::DivideAndRoundUp(uint32(Dimensions.Z), Context.ThreadGroupSizeZ)
		);
	}

	static inline TShaderRef<FCopyTextureCS> SelectShader(FGlobalShaderMap* GlobalShaderMap, ECopyTextureResourceType SrcType, ECopyTextureResourceType DstType, ECopyTextureValueType ValueType, CopyTextureCS::DispatchContext& OutContext);

protected:
	template<typename ShaderType>
	static inline TShaderRef<FCopyTextureCS> SelectShader(FGlobalShaderMap* GlobalShaderMap, CopyTextureCS::DispatchContext& OutContext);

	template <ECopyTextureResourceType SrcType>
	static inline TShaderRef<FCopyTextureCS> SelectShader(FGlobalShaderMap* GlobalShaderMap, ECopyTextureResourceType DstType, ECopyTextureValueType ValueType, CopyTextureCS::DispatchContext& OutContext);

	template <ECopyTextureResourceType SrcType, ECopyTextureResourceType DstType>
	static inline TShaderRef<FCopyTextureCS> SelectShader(FGlobalShaderMap* GlobalShaderMap, ECopyTextureValueType ValueType, CopyTextureCS::DispatchContext& OutContext);

	LAYOUT_FIELD(FShaderParameter, DstOffsetParam);
	LAYOUT_FIELD(FShaderParameter, SrcOffsetParam);
	LAYOUT_FIELD(FShaderParameter, DimensionsParam);
	LAYOUT_FIELD(FShaderResourceParameter, SrcResourceParam);
	LAYOUT_FIELD(FShaderResourceParameter, DstResourceParam);
};

template <ECopyTextureResourceType SrcType, ECopyTextureResourceType DstType, ECopyTextureValueType ValueType, uint32 NumChannels>
class TCopyResourceCS : public FCopyTextureCS
{
	static_assert(NumChannels >= 1 && NumChannels <= 4, "Only 1 to 4 channels are supported.");

	DECLARE_EXPORTED_SHADER_TYPE(TCopyResourceCS, Global, RENDERCORE_API);

public:
	TCopyResourceCS() {}
	TCopyResourceCS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FCopyTextureCS(Initializer)
	{}

	static constexpr uint32 ThreadGroupSizeX = CopyTextureCS::TThreadGroupSize<DstType>::X;
	static constexpr uint32 ThreadGroupSizeY = CopyTextureCS::TThreadGroupSize<DstType>::Y;
	static constexpr uint32 ThreadGroupSizeZ = CopyTextureCS::TThreadGroupSize<DstType>::Z;

	static const TCHAR* GetSourceFilename() { return TEXT("/Engine/Private/CopyTextureShaders.usf"); }
	static const TCHAR* GetFunctionName() { return TEXT("CopyTextureCS"); }

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return true;
	}

	static inline void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FCopyTextureCS::ModifyCompilationEnvironment(Parameters, OutEnvironment);

		OutEnvironment.SetDefine(TEXT("THREADGROUPSIZE_X"), ThreadGroupSizeX);
		OutEnvironment.SetDefine(TEXT("THREADGROUPSIZE_Y"), ThreadGroupSizeY);
		OutEnvironment.SetDefine(TEXT("THREADGROUPSIZE_Z"), ThreadGroupSizeZ);
		OutEnvironment.SetDefine(TEXT("SRC_TYPE"), uint32(SrcType));
		OutEnvironment.SetDefine(TEXT("DST_TYPE"), uint32(DstType));

		switch (ValueType)
		{
		case ECopyTextureValueType::Float:
			switch (NumChannels)
			{
			case 1: OutEnvironment.SetDefine(TEXT("VALUE_TYPE"), TEXT("float"));  break;
			case 2: OutEnvironment.SetDefine(TEXT("VALUE_TYPE"), TEXT("float2")); break;
			case 3: OutEnvironment.SetDefine(TEXT("VALUE_TYPE"), TEXT("float3")); break;
			case 4: OutEnvironment.SetDefine(TEXT("VALUE_TYPE"), TEXT("float4")); break;
			}
			break;

		case ECopyTextureValueType::Int32:
			switch (NumChannels)
			{
			case 1: OutEnvironment.SetDefine(TEXT("VALUE_TYPE"), TEXT("int"));  break;
			case 2: OutEnvironment.SetDefine(TEXT("VALUE_TYPE"), TEXT("int2")); break;
			case 3: OutEnvironment.SetDefine(TEXT("VALUE_TYPE"), TEXT("int3")); break;
			case 4: OutEnvironment.SetDefine(TEXT("VALUE_TYPE"), TEXT("int4")); break;
			}
			break;

		case ECopyTextureValueType::Uint32:
			switch (NumChannels)
			{
			case 1: OutEnvironment.SetDefine(TEXT("VALUE_TYPE"), TEXT("uint"));  break;
			case 2: OutEnvironment.SetDefine(TEXT("VALUE_TYPE"), TEXT("uint2")); break;
			case 3: OutEnvironment.SetDefine(TEXT("VALUE_TYPE"), TEXT("uint3")); break;
			case 4: OutEnvironment.SetDefine(TEXT("VALUE_TYPE"), TEXT("uint4")); break;
			}
			break;
		}
	}

	void GetDispatchContext(CopyTextureCS::DispatchContext& OutContext)
	{
		OutContext.ThreadGroupSizeX = ThreadGroupSizeX;
		OutContext.ThreadGroupSizeY = ThreadGroupSizeY;
		OutContext.ThreadGroupSizeZ = ThreadGroupSizeZ;
		OutContext.SrcType = SrcType;
		OutContext.DstType = DstType;
		OutContext.ValueType = ValueType;
		OutContext.NumChannels = NumChannels;
	}
};

template <typename ShaderType>
inline TShaderRef<FCopyTextureCS> FCopyTextureCS::SelectShader(FGlobalShaderMap* GlobalShaderMap, CopyTextureCS::DispatchContext& OutContext)
{
	TShaderRef<ShaderType> Shader = GlobalShaderMap->GetShader<ShaderType>();
	Shader->GetDispatchContext(OutContext);
	return Shader;
}

template <ECopyTextureResourceType SrcType, ECopyTextureResourceType DstType>
inline TShaderRef<FCopyTextureCS> FCopyTextureCS::SelectShader(FGlobalShaderMap* GlobalShaderMap, ECopyTextureValueType ValueType, CopyTextureCS::DispatchContext& OutContext)
{
	switch (ValueType)
	{
	default: checkNoEntry();
	case ECopyTextureValueType::Float:  return SelectShader<TCopyResourceCS<SrcType, DstType, ECopyTextureValueType::Float,  4>>(GlobalShaderMap, OutContext);
	case ECopyTextureValueType::Int32:  return SelectShader<TCopyResourceCS<SrcType, DstType, ECopyTextureValueType::Int32,  4>>(GlobalShaderMap, OutContext);
	case ECopyTextureValueType::Uint32: return SelectShader<TCopyResourceCS<SrcType, DstType, ECopyTextureValueType::Uint32, 4>>(GlobalShaderMap, OutContext);
	}
}

template <ECopyTextureResourceType SrcType>
inline TShaderRef<FCopyTextureCS> FCopyTextureCS::SelectShader(FGlobalShaderMap* GlobalShaderMap, ECopyTextureResourceType DstType, ECopyTextureValueType ValueType, CopyTextureCS::DispatchContext& OutContext)
{
	switch (DstType)
	{
	default: checkNoEntry();
	case ECopyTextureResourceType::Texture2D:	   return FCopyTextureCS::SelectShader<SrcType, ECopyTextureResourceType::Texture2D     >(GlobalShaderMap, ValueType, OutContext);
	case ECopyTextureResourceType::Texture2DArray: return FCopyTextureCS::SelectShader<SrcType, ECopyTextureResourceType::Texture2DArray>(GlobalShaderMap, ValueType, OutContext);
	case ECopyTextureResourceType::Texture3D:      return FCopyTextureCS::SelectShader<SrcType, ECopyTextureResourceType::Texture3D     >(GlobalShaderMap, ValueType, OutContext);
	}
}

inline TShaderRef<FCopyTextureCS> FCopyTextureCS::SelectShader(FGlobalShaderMap* GlobalShaderMap, ECopyTextureResourceType SrcType, ECopyTextureResourceType DstType, ECopyTextureValueType ValueType, CopyTextureCS::DispatchContext& OutContext)
{
	switch (SrcType)
	{
	default: checkNoEntry();
	case ECopyTextureResourceType::Texture2D:	   return FCopyTextureCS::SelectShader<ECopyTextureResourceType::Texture2D     >(GlobalShaderMap, DstType, ValueType, OutContext);
	case ECopyTextureResourceType::Texture2DArray: return FCopyTextureCS::SelectShader<ECopyTextureResourceType::Texture2DArray>(GlobalShaderMap, DstType, ValueType, OutContext);
	case ECopyTextureResourceType::Texture3D:      return FCopyTextureCS::SelectShader<ECopyTextureResourceType::Texture3D     >(GlobalShaderMap, DstType, ValueType, OutContext);
	}
}
