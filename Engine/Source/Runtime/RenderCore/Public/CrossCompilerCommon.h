// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	CrossCompilerCommon.h: Common functionality between compiler & runtime.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "ShaderCore.h"
#include "ShaderCompilerCore.h"

namespace CrossCompiler
{
	enum EShaderStage
	{
		SHADER_STAGE_VERTEX = 0,
		SHADER_STAGE_PIXEL,
		SHADER_STAGE_GEOMETRY,
		SHADER_STAGE_MESH,
        SHADER_STAGE_AMPLIFICATION,
		NUM_NON_COMPUTE_SHADER_STAGES,
		SHADER_STAGE_COMPUTE = NUM_NON_COMPUTE_SHADER_STAGES,
		NUM_SHADER_STAGES,
	};

	// Make sure these enums match hlslcc EArray on Engine\Source\ThirdParty\hlslcc\hlslcc\src\hlslcc_lib\glsl_parser_extras.h
	enum class EPackedTypeName : int8
	{
		HighP	= 'h',
		MediumP	= 'm',
		LowP	= 'l',
		Int		= 'i',
		Uint	= 'u',
		Sampler	= 's',
		Image	= 'g',

		Invalid = ' ',
	};

	enum class EPackedTypeIndex : int8
	{
		HighP	= 0,
		MediumP	= 1,
		LowP	= 2,
		Int		= 3,
		Uint	= 4,
		Sampler	= 5,
		Image	= 6,

		Max		= 7,
		Invalid = -1,
	};

	enum
	{
		PACKED_TYPENAME_HIGHP	= (int32)EPackedTypeName::HighP,	// Make sure these enums match hlslcc
		PACKED_TYPENAME_MEDIUMP	= (int32)EPackedTypeName::MediumP,
		PACKED_TYPENAME_LOWP	= (int32)EPackedTypeName::LowP,
		PACKED_TYPENAME_INT		= (int32)EPackedTypeName::Int,
		PACKED_TYPENAME_UINT	= (int32)EPackedTypeName::Uint,
		PACKED_TYPENAME_SAMPLER	= (int32)EPackedTypeName::Sampler,
		PACKED_TYPENAME_IMAGE	= (int32)EPackedTypeName::Image,

		PACKED_TYPEINDEX_HIGHP		= (int32)EPackedTypeIndex::HighP,
		PACKED_TYPEINDEX_MEDIUMP	= (int32)EPackedTypeIndex::MediumP,
		PACKED_TYPEINDEX_LOWP		= (int32)EPackedTypeIndex::LowP,
		PACKED_TYPEINDEX_INT		= (int32)EPackedTypeIndex::Int,
		PACKED_TYPEINDEX_UINT		= (int32)EPackedTypeIndex::Uint,
		PACKED_TYPEINDEX_SAMPLER	= (int32)EPackedTypeIndex::Sampler,
		PACKED_TYPEINDEX_IMAGE		= (int32)EPackedTypeIndex::Image,
		PACKED_TYPEINDEX_MAX		= (int32)EPackedTypeIndex::Max,
	};

	static inline uint8 ShaderStageIndexToTypeName(uint8 ShaderStage)
	{
		switch (ShaderStage)
		{
		case SHADER_STAGE_VERTEX:	return 'v';
		case SHADER_STAGE_PIXEL:	return 'p';
		case SHADER_STAGE_GEOMETRY:	return 'g';
		case SHADER_STAGE_COMPUTE:	return 'c';
		default: break;
		}
		checkf(0, TEXT("invalid value: ShaderStageIndexToTypeName(ShaderStage = '%u')"), ShaderStage);
		return 0;
	}

	static inline uint8 PackedTypeIndexToTypeName(uint8 ArrayType)
	{
		switch (ArrayType)
		{
		case PACKED_TYPEINDEX_HIGHP:	return PACKED_TYPENAME_HIGHP;
		case PACKED_TYPEINDEX_MEDIUMP:	return PACKED_TYPENAME_MEDIUMP;
		case PACKED_TYPEINDEX_LOWP:		return PACKED_TYPENAME_LOWP;
		case PACKED_TYPEINDEX_INT:		return PACKED_TYPENAME_INT;
		case PACKED_TYPEINDEX_UINT:		return PACKED_TYPENAME_UINT;
		case PACKED_TYPEINDEX_SAMPLER:	return PACKED_TYPENAME_SAMPLER;
		case PACKED_TYPEINDEX_IMAGE:	return PACKED_TYPENAME_IMAGE;
		default: break;
		}
		checkf(0, TEXT("invalid value: PackedTypeIndexToTypeName(ArrayType = %d)"), (int32)ArrayType);
		return 0;
	}

	static inline uint8 PackedTypeNameToTypeIndex(uint8 ArrayName)
	{
		switch (ArrayName)
		{
		case 0: // Fallthrough; Used for bindless on some platforms
		case PACKED_TYPENAME_HIGHP:		return PACKED_TYPEINDEX_HIGHP;
		case PACKED_TYPENAME_MEDIUMP:	return PACKED_TYPEINDEX_MEDIUMP;
		case PACKED_TYPENAME_LOWP:		return PACKED_TYPEINDEX_LOWP;
		case PACKED_TYPENAME_INT:		return PACKED_TYPEINDEX_INT;
		case PACKED_TYPENAME_UINT:		return PACKED_TYPEINDEX_UINT;
		case PACKED_TYPENAME_SAMPLER:	return PACKED_TYPEINDEX_SAMPLER;
		case PACKED_TYPENAME_IMAGE:		return PACKED_TYPEINDEX_IMAGE;
		default: break;
		}
		checkf(0, TEXT("invalid value: PackedTypeNameToTypeIndex(ArrayName = %d)"), (int32)ArrayName);
		return 0;
	}

	static inline bool IsValidPackedTypeName(EPackedTypeName TypeName)
	{
		switch (TypeName)
		{
		case EPackedTypeName::HighP:
		case EPackedTypeName::MediumP:
		case EPackedTypeName::LowP:
		case EPackedTypeName::Int:
		case EPackedTypeName::Uint:
		case EPackedTypeName::Sampler:
		case EPackedTypeName::Image:
			return true;
		default: break;
		}

		return false;
	}

	static inline EPackedTypeName PackedTypeIndexToTypeName(EPackedTypeIndex TypeIndex)
	{
		switch (TypeIndex)
		{
		case EPackedTypeIndex::HighP:	return EPackedTypeName::HighP;
		case EPackedTypeIndex::MediumP:	return EPackedTypeName::MediumP;
		case EPackedTypeIndex::LowP:	return EPackedTypeName::LowP;
		case EPackedTypeIndex::Int:		return EPackedTypeName::Int;
		case EPackedTypeIndex::Uint:	return EPackedTypeName::Uint;
		case EPackedTypeIndex::Sampler:	return EPackedTypeName::Sampler;
		case EPackedTypeIndex::Image:	return EPackedTypeName::Image;
		default: break;
		}

		return EPackedTypeName::Invalid;
	}

	static inline EPackedTypeIndex PackedTypeNameToTypeIndex(EPackedTypeName TypeName)
	{
		switch (TypeName)
		{
		case EPackedTypeName::HighP:	return EPackedTypeIndex::HighP;
		case EPackedTypeName::MediumP:	return EPackedTypeIndex::MediumP;
		case EPackedTypeName::LowP:		return EPackedTypeIndex::LowP;
		case EPackedTypeName::Int:		return EPackedTypeIndex::Int;
		case EPackedTypeName::Uint:		return EPackedTypeIndex::Uint;
		case EPackedTypeName::Sampler:	return EPackedTypeIndex::Sampler;
		case EPackedTypeName::Image:	return EPackedTypeIndex::Image;
		default: break;
		}

		return EPackedTypeIndex::Invalid;
	}

	struct FPackedArrayInfo
	{
		uint16	Size;		// Bytes
		uint8	TypeName;	// PACKED_TYPENAME
		uint8	TypeIndex;	// PACKED_TYPE
	};

	inline FArchive& operator<<(FArchive& Ar, FPackedArrayInfo& Info)
	{
		Ar << Info.Size;
		Ar << Info.TypeName;
		Ar << Info.TypeIndex;
		return Ar;
	}

#pragma warning(push)
#pragma warning(error : 4596)
	struct FShaderBindingInOutMask
	{
		uint32 Bitmask = 0;

		/** Maximum value for a valid index in this bitmask. */
		static constexpr int32 MaxIndex = (sizeof(decltype(FShaderBindingInOutMask::Bitmask)) * 8u) - 1u;

		/** Index to mark the binding of a depth-stencil output resource. */
		static constexpr int32 DepthStencilMaskIndex = MaxIndex;

		/** Sets the specified bitfield in this bitmask and validates its index boundary. */
		inline void EnableField(int32 Index)
		{
			ensure(Index >= 0 && Index <= FShaderBindingInOutMask::MaxIndex);
			Bitmask |= (1u << Index);
		}

		/** Returns whether the specified bitfield in this bitmask is set and validates its index boundary. */
		inline bool IsFieldEnabled(int32 Index) const
		{
			ensure(Index >= 0 && Index <= FShaderBindingInOutMask::MaxIndex);
			return (Bitmask & (1u << Index)) != 0;
		}

		friend bool operator == (const FShaderBindingInOutMask& Lhs, const FShaderBindingInOutMask& Rhs)
		{
			return Lhs.Bitmask == Rhs.Bitmask;
		}

		friend bool operator != (const FShaderBindingInOutMask& Lhs, const FShaderBindingInOutMask& Rhs)
		{
			return !(Lhs == Rhs);
		}
	};
#pragma warning(pop)

	inline FArchive& operator<<(FArchive& Ar, FShaderBindingInOutMask& BindingInOutMask)
	{
		Ar << BindingInOutMask.Bitmask;
		return Ar;
	}

	struct FShaderBindings
	{
		TArray<TArray<FPackedArrayInfo>>	PackedUniformBuffers;
		TArray<FPackedArrayInfo>			PackedGlobalArrays;
		FShaderCompilerResourceTable		ShaderResourceTable;

		FShaderBindingInOutMask	InOutMask;
		uint8					NumSamplers;
		uint8					NumUniformBuffers;
		uint8					NumUAVs;
		uint8					NumAccelerationStructures;
		bool					bHasRegularUniformBuffers;
	};

	// Information for copying members from uniform buffers to packed
	struct FUniformBufferCopyInfo
	{
		uint16 SourceOffsetInFloats;
		uint8 SourceUBIndex;
		uint8 DestUBIndex;
		uint8 DestUBTypeName;
		uint8 DestUBTypeIndex;
		uint16 DestOffsetInFloats;
		uint16 SizeInFloats;
	};

	inline FArchive& operator<<(FArchive& Ar, FUniformBufferCopyInfo& Info)
	{
		Ar << Info.SourceOffsetInFloats;
		Ar << Info.SourceUBIndex;
		Ar << Info.DestUBIndex;
		Ar << Info.DestUBTypeName;
		if (Ar.IsLoading())
		{
			Info.DestUBTypeIndex = CrossCompiler::PackedTypeNameToTypeIndex(Info.DestUBTypeName);
		}
		Ar << Info.DestOffsetInFloats;
		Ar << Info.SizeInFloats;
		return Ar;
	}
}
