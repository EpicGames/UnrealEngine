// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"

class FRHICommandListImmediate;

class ICustomResourcePool
{
public:
	virtual ~ICustomResourcePool() = default;
	virtual void Tick(FRHICommandListImmediate& RHICmdList) = 0;

	static RENDERCORE_API void TickPoolElements(FRHICommandListImmediate& RHICmdList);
};
extern RENDERCORE_API ICustomResourcePool* GCustomResourcePool;
