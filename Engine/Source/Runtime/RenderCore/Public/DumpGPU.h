// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Build.h"
#include "Misc/WildcardString.h"
#include "UObject/NameTypes.h"

class IDumpGPUUploadServiceProvider
{
public:

	struct FDumpParameters
	{
		static constexpr const TCHAR* kServiceFileName = TEXT("Base/DumpService.json");

		FString Type;
		FString LocalPath;
		FString Time;

		FName CompressionName;
		FWildcardString CompressionFiles;

		RENDERCORE_API FString DumpServiceParametersFileContent() const;
		RENDERCORE_API bool DumpServiceParametersFile() const;
	};

	virtual void UploadDump(const FDumpParameters& Parameters) = 0;
	virtual ~IDumpGPUUploadServiceProvider() = default;

	static RENDERCORE_API IDumpGPUUploadServiceProvider* GProvider;
};

#if WITH_DUMPGPU

namespace UE::RenderCore::DumpGPU
{

RENDERCORE_API void TickEndFrame();

RENDERCORE_API bool IsDumpingFrame();

RENDERCORE_API bool ShouldCameraCut();

class FDumpScope
{
public:
	RENDERCORE_API FDumpScope(bool bInCondition = true, const TCHAR* Cmd = TEXT(""), const TCHAR* Context = TEXT(""));

	~FDumpScope()
	{
		if (bCondition)
		{
			// Tick the dump GPU system again, which will end the active dump.
			TickEndFrame();
		}
	}

private:
	bool bCondition;
};

} // namespace  UE::RenderCore::DumpGPU

#else

namespace UE::RenderCore::DumpGPU
{

class FDumpScope
{
public:
	FDumpScope(bool bCondition = true) {}
};

} // namespace  UE::RenderCore::DumpGPU

#endif // WITH_DUMPGPU