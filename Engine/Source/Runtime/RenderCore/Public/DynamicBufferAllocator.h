// Copyright Epic Games, Inc. All Rights Reserved.

/*==============================================================================
DynamicBufferAllocator.h: Classes for allocating transient rendering data.
==============================================================================*/

#pragma once

#include "Containers/Array.h"
#include "HAL/PlatformMath.h"
#include "RHI.h"
#include "RHIUtilities.h"
#include "RenderResource.h"
#include "Async/Mutex.h"

struct FDynamicReadBufferPool;

struct FDynamicAllocReadBuffer : public FDynamicReadBuffer
{
	int32 AllocatedByteCount = 0;
	/** Number of successive frames for which AllocatedByteCount == 0. Used as a metric to decide when to free the allocation. */
	int32 NumFramesUnused = 0;

	TArray<FShaderResourceViewRHIRef> SubAllocations;

	void Lock(FRHICommandListBase& RHICmdList)
	{
		SubAllocations.Reset();
		FDynamicReadBuffer::Lock(RHICmdList);
	}

	/**
	* Unocks the buffer so the GPU may read from it.
	*/
	void Unlock(FRHICommandListBase& RHICmdList)
	{
		FDynamicReadBuffer::Unlock(RHICmdList);
		AllocatedByteCount = 0;
		NumFramesUnused = 0;
	}
};

/**
* A system for dynamically allocating GPU memory for rendering. Note that this must derive from FRenderResource 
  so that we can safely free the shader resource views for OpenGL and other platforms. If we wait until the module is shutdown,
  the renderer RHI will have already been destroyed and we can execute code on invalid data. By making ourself a render resource, we
  clean up immediately before the renderer dies.
*/
class FGlobalDynamicReadBuffer : public FRenderResource
{
public:
	/**
	* Information regarding an allocation from this buffer.
	*/
	struct FAllocation
	{
		/** The location of the buffer in main memory. */
		uint8* Buffer;
		/** The read buffer to bind for draw calls. */
		FDynamicAllocReadBuffer* ReadBuffer;

		FRHIShaderResourceView* SRV;

		/** Default constructor. */
		FAllocation()
			: Buffer(NULL)
			, ReadBuffer(NULL)
		{
		}

		/** Returns true if the allocation is valid. */
		inline bool IsValid() const
		{
			return Buffer != NULL;
		}
	};

	RENDERCORE_API FGlobalDynamicReadBuffer();
	RENDERCORE_API ~FGlobalDynamicReadBuffer();
	
	RENDERCORE_API FAllocation AllocateHalf(uint32 Num);
	RENDERCORE_API FAllocation AllocateFloat(uint32 Num);
	RENDERCORE_API FAllocation AllocateInt32(uint32 Num);
	RENDERCORE_API FAllocation AllocateUInt32(uint32 Num);

	/**
	* Commits allocated memory to the GPU.
	*		WARNING: Once this buffer has been committed to the GPU, allocations
	*		remain valid only until the next call to Allocate!
	*/
	RENDERCORE_API void Commit(FRHICommandListImmediate& RHICmdList);

	/** Returns true if log statements should be made because we exceeded GMaxVertexBytesAllocatedPerFrame */
	RENDERCORE_API bool IsRenderAlarmLoggingEnabled() const;

protected:
	RENDERCORE_API virtual void InitRHI(FRHICommandListBase& RHICmdList) override;
	RENDERCORE_API virtual void ReleaseRHI() override;
	RENDERCORE_API void Cleanup();
	RENDERCORE_API void IncrementTotalAllocations(uint32 Num);

	template<EPixelFormat Format, typename Type>
	FAllocation AllocateInternal(FDynamicReadBufferPool* BufferPool, uint32 Num);

	UE::FMutex Mutex;
	FRHICommandListBase* RHICmdList = nullptr;

	/** The pools of read buffers from which allocations are made. */
	FDynamicReadBufferPool* HalfBufferPool;
	FDynamicReadBufferPool* FloatBufferPool;
	FDynamicReadBufferPool* Int32BufferPool;
	FDynamicReadBufferPool* UInt32BufferPool;

	/** A total of all allocations made since the last commit. Used to alert about spikes in memory usage. */
	size_t TotalAllocatedSinceLastCommit;
};

