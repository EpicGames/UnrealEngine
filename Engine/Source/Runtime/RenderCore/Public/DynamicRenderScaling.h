// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "Containers/List.h"
#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "HAL/IConsoleManager.h"
#include "Misc/AssertionMacros.h"
#include "Stats/Stats.h"

namespace DynamicRenderScaling
{

template<typename Type>
class TMap;

/* Model how the GPU costs of a budget scales with a ResolutionFraction. */
enum class EHeuristicModel
{
	Unknown,

	// GPU cost scales linearly to the ResolutionFraction
	Linear,

	// GPU cost scales quadratically to the ResolutionFraction
	Quadratic,
};

/* Represent an independent budget to dynamically scale by its own. */
struct FHeuristicSettings final
{
	static constexpr float kDefaultMinResolutionFraction = 0.5f;
	static constexpr float kDefaultMaxResolutionFraction = 1.0f;
	static constexpr float kDefaultThrottlingMaxResolutionFraction = 0.0f;
	static constexpr float kBudgetMsDisabled = 0.0f;
	static constexpr float kDefaultChangeThreshold = 0.02f;
	static constexpr float kDefaultTargetedHeadRoom = 0.05f;
	static constexpr float kDefaultIncreaseAmortizationFactor = 0.9f;
	static constexpr int32 kDefaultFractionQuantization = 0;
	static constexpr int32 kDefaultUpperBoundQuantization = 0;

	EHeuristicModel Model = EHeuristicModel::Unknown;
	bool bModelScalesWithPrimaryScreenPercentage = false;

	float MinResolutionFraction      = kDefaultMinResolutionFraction;
	float MaxResolutionFraction      = kDefaultMaxResolutionFraction;
	float ThrottlingMaxResolutionFraction = kDefaultThrottlingMaxResolutionFraction;
	float BudgetMs                   = kBudgetMsDisabled;
	float ChangeThreshold            = kDefaultChangeThreshold;
	float TargetedHeadRoom           = kDefaultTargetedHeadRoom;
	float IncreaseAmortizationFactor = kDefaultIncreaseAmortizationFactor;
	int32 FractionQuantization       = kDefaultFractionQuantization;
	int32 UpperBoundQuantization     = kDefaultUpperBoundQuantization;

	/** Returns whether the heuristic is enabled or not. */
	RENDERCORE_API bool IsEnabled() const;

	/** Returns the desired GPU cost to be targeted to have head room left to not go over budget. */
	RENDERCORE_API float GetTargetedMs(float BudgetMs) const;

	/** Returns how much the GPU cost scales for a given ResolutionFraction. */
	RENDERCORE_API float EstimateCostScale(float ResolutionFraction) const;

	/** Returns how much the ResolutionFraction should scale for a GPU timing to fit to target. */
	RENDERCORE_API float EstimateResolutionFactor(float TargetMs, float TimingMs) const;

	/** Returns how much the GPU time should scale between two different resolution fraction. */
	RENDERCORE_API float EstimateTimeFactor(float CurrentResolutionFraction, float NewResolutionFraction) const;

	/** Corrects new resolution fraction to ensure it's within bounds, honor AmortizationFactor and FractionQuantization. */
	RENDERCORE_API float CorrectNewResolutionFraction(float CurrentResolutionFraction, float NewResolutionFraction, float ResolutionFractionScale) const;

	/** Returns whether the ResolutionFraction is changing meaningfully enough. */
	RENDERCORE_API bool DoesResolutionChangeEnough(float CurrentResolutionFraction, float NewResolutionFraction, bool bCanChangeResolution) const;
};


/* Represent an independent budget to dynamically scale by its own. */
class FBudget final
{
public:
	RENDERCORE_API FBudget(const TCHAR* Name, FHeuristicSettings (*HeuristicSettingsGetter)(void));
	RENDERCORE_API ~FBudget();

	/* Returns the global list of all dynamic resolution budgets. */
	static RENDERCORE_API TLinkedList<FBudget*>*& GetGlobalList();
	static RENDERCORE_API int32 GetGlobalListSize();

	inline const TCHAR* GetName() const
	{
		return Name;
	}

	inline const char* GetAnsiName() const
	{
		return AnsiName.GetData();
	}

	/* Returns the settings of the heuristic for this budget. */
	inline const FHeuristicSettings& GetSettings() const
	{
		return CachedSettings;
	}

	inline int32 GetBudgetId() const
	{
		return BudgetId;
	}

	inline bool operator == (const FBudget& Other) const
	{
		return BudgetId == Other.BudgetId;
	}

#if STATS
	inline const TStatId& GetStatId_TargetMs() const
	{
		return StatId_TargetMs;
	}

	inline const TStatId& GetStatId_MeasuredMs() const
	{
		return StatId_MeasuredMs;
	}

	inline const TStatId& GetStatId_MinScaling() const
	{
		return StatId_MinScaling;
	}

	inline const TStatId& GetStatId_MaxScaling() const
	{
		return StatId_MaxScaling;
	}

	inline const TStatId& GetStatId_CurrentScaling() const
	{
		return StatId_CurrentScaling;
	}
#endif

private:
	const TCHAR* Name;
	TArray<char> AnsiName;
	FHeuristicSettings(*HeuristicSettingsGetter)(void);
	TLinkedList<FBudget*> GlobalListLink;
	FHeuristicSettings CachedSettings;
	int32 BudgetId = 0;

#if STATS
	TStatId StatId_TargetMs;
	TStatId StatId_MeasuredMs;
	TStatId StatId_MinScaling;
	TStatId StatId_MaxScaling;
	TStatId StatId_CurrentScaling;
#endif

	UE_NONCOPYABLE(FBudget);

	template<typename Type>
	friend class TMap;
	friend RENDERCORE_API void UpdateHeuristicsSettings();
};


/* Map of FBudget -> <Type>. */
template<typename Type>
class TMap final
{
public:
	static constexpr int32 kInlineAllocatedBudgets = 16;

	TMap()
	{
		Array.SetNum(FBudget::GetGlobalListSize());
#if !UE_BUILD_SHIPPING
		for (TLinkedList<FBudget*>::TIterator BudgetIt(FBudget::GetGlobalList()); BudgetIt; BudgetIt.Next())
		{
			Array[BudgetIt->BudgetId].Name = BudgetIt->GetName();
		}
#endif
	}

	TMap(const Type& Value)
		: TMap()
	{
		SetAll(Value);
	}

	TMap(const TMap<Type>& Map) = default;
	~TMap() = default;

	inline void SetAll(const Type& Value)
	{
		Array.SetNum(FBudget::GetGlobalListSize());
		for (int32 i = 0; i < Array.Num(); i++)
		{
			Array[i].Value = Value;
		}
	}

	inline const Type& operator [](const FBudget& Budget) const
	{
		check(Array.Num() == FBudget::GetGlobalListSize());
		return Array[Budget.BudgetId].Value;
	}

	inline Type& operator [](const FBudget& Budget)
	{
		check(Array.Num() == FBudget::GetGlobalListSize());
		return Array[Budget.BudgetId].Value;
	}

	inline const Type& operator [](const int32 BudgetId) const
	{
		check(Array.Num() == FBudget::GetGlobalListSize());
		return Array[BudgetId].Value;
	}

	inline Type& operator [](const int32 BudgetId)
	{
		check(Array.Num() == FBudget::GetGlobalListSize());
		return Array[BudgetId].Value;
	}

private:
	struct FItem
	{
#if !UE_BUILD_SHIPPING
		const TCHAR* Name = nullptr;
#endif
		Type Value;
	};
	TArray<FItem, TInlineAllocator<kInlineAllocatedBudgets>> Array;
};


constexpr float FractionToPercentage(float Fraction)
{
	return Fraction * 100.0f;
}

constexpr float PercentageToFraction(float Percentage)
{
	return Percentage / 100.0f;
}

inline float GetPercentageCVarToFraction(const TAutoConsoleVariable<float>& Percentage)
{
	return PercentageToFraction(Percentage.GetValueOnAnyThread());
}

/** Returns whether the DynamicRenderScaling API is supported */
RENDERCORE_API bool IsSupported();

/** Updates all FBudget::GetSettings() with their new FHeuristicSettings. */
RENDERCORE_API void UpdateHeuristicsSettings();

/** Begins recording GPU timings in RDG for all the different FBudgets. */
RENDERCORE_API void BeginFrame(const TMap<bool>& bIsBudgetEnabled);

/** Ends recording GPU timings in RDG for all the different FBudgets. */
RENDERCORE_API void EndFrame();

/** Returns the latest available timings. */
RENDERCORE_API const TMap<uint64>& GetLatestTimings();


} // namespace DynamicRenderScaling
