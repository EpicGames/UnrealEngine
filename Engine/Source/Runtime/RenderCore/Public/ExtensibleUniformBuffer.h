// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RenderGraphDefinitions.h"
#include "RenderGraphBuilder.h"
#include "ShaderParameterMacros.h"
#include "ShaderParameterMetadata.h"
#include "ShaderParameterMetadataBuilder.h"

// Construct a shader parameter struct with the specified type at Obj.
// The resulting struct must be valid for usage (i.e. all members initialized)
using FShaderParameterStructConstructor = TFunction<void(void* Obj, const FShaderParametersMetadata& Metadata, FRDGBuilder& GraphBuilder)>;
template<typename ParameterStructType>
using TShaderParameterStructConstructor = TFunction<void(ParameterStructType& Obj, FRDGBuilder& GraphBuilder)>;

/**
 * Collects UB members during static initialization,
 * and uses these to construct the buffer layout during Main.
 */
class FExtensibleUniformBufferTypeRegistry
{
public:
	using FMemberId = int;
	struct FMemberInfo;
	class FImpl;
	FImpl* Impl;

	template<typename ParameterStructType>
	static FExtensibleUniformBufferTypeRegistry New()
	{
		return FExtensibleUniformBufferTypeRegistry
		{
			ParameterStructType::FSourceDefinition::LayoutName,
			ParameterStructType::FSourceDefinition::TypeName,
			ParameterStructType::FSourceDefinition::ShaderVariableName,
			ParameterStructType::FSourceDefinition::StaticSlotName,
			ParameterStructType::FTypeInfo::FileName,
			ParameterStructType::FTypeInfo::FileLine,
			ParameterStructType::MaxSize 
		};
	}

	template<typename ParameterStructType>
	static FExtensibleUniformBufferTypeRegistry& Get();

	RENDERCORE_API FExtensibleUniformBufferTypeRegistry(
		const TCHAR* LayoutName,
		const TCHAR* StructTypeName,
		const TCHAR* ShaderVariableName,
		const TCHAR* StaticSlotName,
		const ANSICHAR* FileName,
		const int32 FileLine,
		const uint64 MaxStructSize);
	RENDERCORE_API ~FExtensibleUniformBufferTypeRegistry();

	RENDERCORE_API FMemberId RegisterMember(
		const TCHAR* Name,
		const FShaderParametersMetadata& StructMetadata,
		// Used to set default values for this member if no value is set explicitly
		FShaderParameterStructConstructor DefaultValueConstructor);

	RENDERCORE_API FShaderParametersMetadata* GetStructMetadata();
};

/**
 * Used to declare individual UB members.
 */
class FExtensibleUniformBufferMemberRegistration
{
public:
	using FMemberId = FExtensibleUniformBufferTypeRegistry::FMemberId;

	// Called at runtime
	virtual void Commit() = 0;
	static void CommitAll();

	FString Name;
	FMemberId MemberId = -1;
	FShaderParameterStructConstructor DefaultValueConstructor;

protected:
	RENDERCORE_API static void Register(FExtensibleUniformBufferMemberRegistration& Entry);

	// Called during static initialization
	FExtensibleUniformBufferMemberRegistration(const TCHAR* Name)
		: Name(Name)
	{
		Register(*this);
	}
	virtual ~FExtensibleUniformBufferMemberRegistration() = default;

private:
	static TArray<FExtensibleUniformBufferMemberRegistration*>& GetInstances();
};

template<typename TMember, typename ParameterStructType>
class TExtensibleUniformBufferMemberRegistration : public FExtensibleUniformBufferMemberRegistration
{
	using FRegistry = FExtensibleUniformBufferTypeRegistry;

public:
	TExtensibleUniformBufferMemberRegistration(const TCHAR* Name, TShaderParameterStructConstructor<TMember> TemplatedDefaultValueConstructor)
	: FExtensibleUniformBufferMemberRegistration(Name)
	{
		DefaultValueConstructor = [=](void* Obj, const FShaderParametersMetadata& Metadata, FRDGBuilder& GraphBuilder)
		{
			check(&Metadata == TMember::FTypeInfo::GetStructMetadata());
			TMember& Member = *(new(Obj) TMember());
			TemplatedDefaultValueConstructor(Member, GraphBuilder);
		};
	}

	void Commit() override
	{
		MemberId = FRegistry::Get<ParameterStructType>().RegisterMember(
			*Name,
			*TShaderParameterStructTypeInfo<TMember>::GetStructMetadata(),
			DefaultValueConstructor);
	}
};

/**
 * Shader parameter struct with a layout that is defined at runtime by FExtensibleUniformBufferTypeRegistry
 */
template<typename SelfType>
class TExtensibleUniformParametersBase
{
public:
	struct FTypeInfo
	{
		static constexpr int32 NumRows = 1;
		static constexpr int32 NumColumns = 1;
		static constexpr int32 NumElements = 0;
		static constexpr int32 Alignment = SHADER_PARAMETER_STRUCT_ALIGNMENT;
		static constexpr bool bIsStoredInConstantBuffer = true;
		static constexpr const ANSICHAR* const FileName = SelfType::FSourceDefinition::FileName;
		static constexpr int32 FileLine = SelfType::FSourceDefinition::FileLine;
		using TAlignedType = SelfType;
		static const FShaderParametersMetadata* GetStructMetadata()
		{
			FExtensibleUniformBufferTypeRegistry& Registry = FExtensibleUniformBufferTypeRegistry::Get<SelfType>();
			return Registry.GetStructMetadata();
		}
	};

	const void* GetContents() const
	{
		return Data;
	}

private:
	template<typename, template<typename> typename>
	friend class TExtensibleUniformBuffer;
	friend class FExtensibleUniformBufferTypeRegistry;

	// TODO: Decouple parameter struct and actual data, so the data can be allocated dynamically (variable size)
	// This requires some work in buffer binding/parameter macros/RHI, as these all assume the data to be inline.
	static constexpr size_t MaxSize = 2048;
	uint8_t Data[MaxSize];
};

#define DECLARE_EXTENSIBLE_UNIFORM_BUFFER_PARAMETER_STRUCT(PrefixKeywords, ParameterStructTypeName, ShaderVariableAndSlotName)						\
	DECLARE_UNIFORM_BUFFER_STRUCT(ParameterStructTypeName, PrefixKeywords)																			\
	MS_ALIGN(SHADER_PARAMETER_STRUCT_ALIGNMENT) class ParameterStructTypeName final : public TExtensibleUniformParametersBase<ParameterStructTypeName>	\
	{																																				\
	public:																																			\
		struct FSourceDefinition																													\
		{																																			\
			static constexpr const TCHAR* const LayoutName = TEXT(#ParameterStructTypeName);														\
			static constexpr const TCHAR* const TypeName = TEXT(#ParameterStructTypeName);                                         					\
			static constexpr const TCHAR* const ShaderVariableName = TEXT(#ShaderVariableAndSlotName);												\
			static constexpr const TCHAR* const StaticSlotName = TEXT(#ShaderVariableAndSlotName);													\
			static constexpr const ANSICHAR* const FileName = UE_LOG_SOURCE_FILE(__FILE__);															\
			static constexpr int32 FileLine = __LINE__;																								\
		};																																			\
	} GCC_ALIGN(SHADER_PARAMETER_STRUCT_ALIGNMENT);

/**
 * Used to create instances of an extensible buffer, set data, and build an RDG object
 */
class FExtensibleUniformBufferBase
{
public:
	// Get and re-create the UB if the cached parameters are modified.
	RENDERCORE_API FRDGUniformBuffer* GetBufferGeneric(FRDGBuilder& GraphBuilder);
	UE_DEPRECATED(5.8, "Use the RDG buffer accessor instead.")
	RENDERCORE_API FRHIUniformBuffer* GetBufferRHI(FRDGBuilder& GraphBuilder);

#if !UE_BUILD_SHIPPING
	struct FDebugMemberInfo
	{
		FString ShaderType;
		FString Name;
		const void* Data;
	};
	TArray<FDebugMemberInfo> GetDebugInfo() const;
#endif

protected:
	using FMemberId = FExtensibleUniformBufferTypeRegistry::FMemberId;
	using FRegistry = FExtensibleUniformBufferTypeRegistry;

	class FParameterTypeAdaptor
	{
	public:
		FParameterTypeAdaptor(const FShaderParametersMetadata& InStructMetadata, const FExtensibleUniformBufferTypeRegistry& InRegistry)
		: StructMetadata(InStructMetadata)
		, Registry(InRegistry)
		{ }
		virtual ~FParameterTypeAdaptor() = default;
		// Create and copy cached parameters into the RDG-lifetime struct
		virtual FRDGUniformBuffer* BuildRDGUniformBuffer(FRDGBuilder& GraphBuilder, FExtensibleUniformBufferBase& Buffer) const = 0;
		const FShaderParametersMetadata& StructMetadata;
		const FExtensibleUniformBufferTypeRegistry& Registry;
	};

	RENDERCORE_API FExtensibleUniformBufferBase(FParameterTypeAdaptor& InParameterTypeAdaptor);

	/**
	 * Set a field in the parameter struct.
	 * The change will be reflected in any buffer that GetBuffer() returns after this call.
	 * Returns true if anything actually changed.
	 */
	RENDERCORE_API bool Set(const FMemberId MemberId, const void* Value, const int32 ValueSize);

	/**
	 * Retrieve a field in the parameter struct.
	 * If the field has not been set, this will return null;
	 */
	RENDERCORE_API const void* Get(const FMemberId MemberId, const int32 ExpectedValueSize) const UE_LIFETIMEBOUND;

	/**
	 * Retrieve a field in the parameter struct.
	 * If the field has not been set, this will set a default value;
	 */
	RENDERCORE_API const void* GetOrDefault(const FMemberId MemberId, const int32 ExpectedValueSize, FRDGBuilder& GraphBuilder) UE_LIFETIMEBOUND;

protected:
	const FParameterTypeAdaptor& ParameterTypeAdaptor;

	FRDGUniformBuffer* Buffer = nullptr;

	bool bAnyMemberDirty = false;
	TBitArray<> MemberHasBeenSet;
	TArray<uint8_t, TSizedHeapAllocator<32>> CachedData;
};


template<typename ParameterStructType, template<typename> typename TMemberRegistration>
class TExtensibleUniformBuffer : public FExtensibleUniformBufferBase
{
	template<typename SelfType>
	friend class TExtensibleUniformParametersBase;

	class TParameterTypeAdaptor : public FParameterTypeAdaptor
	{
		using FDerivedExtensibleUniformBuffer = TExtensibleUniformBuffer<ParameterStructType, TMemberRegistration>;
	public:
		static TParameterTypeAdaptor& Get()
		{
			static TParameterTypeAdaptor Instance{};
			return Instance;
		}

		TParameterTypeAdaptor()
		: FParameterTypeAdaptor(*ParameterStructType::FTypeInfo::GetStructMetadata(), FExtensibleUniformBufferTypeRegistry::Get<ParameterStructType>())
		{ }

		virtual ~TParameterTypeAdaptor() = default;

		virtual FRDGUniformBuffer* BuildRDGUniformBuffer(FRDGBuilder& GraphBuilder, FExtensibleUniformBufferBase& Container) const override
		{
			ParameterStructType* ParameterStruct = GraphBuilder.AllocObject<ParameterStructType>();
			FDerivedExtensibleUniformBuffer& DerivedContainer = static_cast<FDerivedExtensibleUniformBuffer&>(Container);
			FPlatformMemory::Memcpy(ParameterStruct->Data, DerivedContainer.CachedData.GetData(), DerivedContainer.CachedData.Num());
			return GraphBuilder.CreateUniformBuffer(ParameterStruct);
		}
	};

public:
	TExtensibleUniformBuffer()
	: FExtensibleUniformBufferBase(TParameterTypeAdaptor::Get())
	{ }

	/**
	 * Set a field in the parameter struct.
	 * The change will be reflected in any buffer that GetBuffer() returns after this call.
	 * Returns true if anything actually changed.
	 */
	template<typename TMember>
	bool Set(const TMemberRegistration<TMember>& Registration, const TMember& Value)
	{
		return FExtensibleUniformBufferBase::Set(Registration.MemberId, &Value, TMember::FTypeInfo::GetStructMetadata()->GetSize());
	}

	/**
	 * Retrieve a field in the parameter struct.
	 * If the field has not been set, this will crash.
	 */
	template<typename TMember>
	const TMember& Get(const TMemberRegistration<TMember>& Registration) const UE_LIFETIMEBOUND
	{
		const void* Ptr = FExtensibleUniformBufferBase::Get(Registration.MemberId, TMember::FTypeInfo::GetStructMetadata()->GetSize());
		checkf(Ptr, TEXT("%s has not been set yet"), *Registration.Name);
		return *reinterpret_cast<const TMember*>(Ptr);
	}

	/**
	 * Retrieve a field in the parameter struct.
	 * If the field has not been set, this will set a default value.
	 */
	template<typename TMember>
	const TMember& GetOrDefault(const TMemberRegistration<TMember>& Registration, FRDGBuilder& GraphBuilder) const UE_LIFETIMEBOUND
	{
		return *reinterpret_cast<const TMember*>(FExtensibleUniformBufferBase::GetOrDefault(Registration.MemberId, TMember::FTypeInfo::GetStructMetadata()->GetSize(), GraphBuilder));
	}

	// Get and re-create the UB if the cached parameters are modified.
	TRDGUniformBufferRef<ParameterStructType> GetBuffer(FRDGBuilder& GraphBuilder)
	{
		return static_cast<TRDGUniformBufferRef<ParameterStructType>>(FExtensibleUniformBufferBase::GetBufferGeneric(GraphBuilder));
	}
};

#define IMPLEMENT_EXTENSIBLE_UNIFORM_BUFFER(PrefixKeywords, ParameterStructType)																	\
	static FUniformBufferStaticSlotRegistrar UniformBufferStaticSlot_##ParameterStructType(ParameterStructType::FSourceDefinition::StaticSlotName);	\
	template<> PrefixKeywords FExtensibleUniformBufferTypeRegistry& FExtensibleUniformBufferTypeRegistry::Get<ParameterStructType>()               \
	{                                                                                                                                       	\
		static FExtensibleUniformBufferTypeRegistry Instance = FExtensibleUniformBufferTypeRegistry::New<ParameterStructType>();               	\
		return Instance;                                                                                                                    	\
	}                                                                                                                                       	\
																																				\
	PrefixKeywords const FShaderParametersMetadata* GetForwardDeclaredShaderParametersStructMetadata(const ParameterStructType* DummyPtr)      	\
	{                                                                                                                                       	\
		return ParameterStructType::FTypeInfo::GetStructMetadata();                                                                            	\
	}                                                                                                                                       	\
																																				\
	FShaderParametersMetadataRegistration ParametersMetadataRegistration                                                        				\
	{ TFunctionRef<const ::FShaderParametersMetadata* ()>{ ParameterStructType::FTypeInfo::GetStructMetadata } };
