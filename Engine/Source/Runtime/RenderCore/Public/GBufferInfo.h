// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "PixelFormat.h"
#include "RHIDefinitions.h"
#include "RHIShaderPlatform.h"

enum EGBufferSlot
{
	GBS_Invalid,
	GBS_SceneColor, // RGB 11.11.10
	GBS_WorldNormal, // RGB 10.10.10
	GBS_PerObjectGBufferData, // 2
	GBS_Metallic, // R8
	GBS_Specular, // R8
	GBS_Roughness, // R8
	GBS_ShadingModelId, // 4 bits
	GBS_SelectiveOutputMask, // 4 bits
	GBS_BaseColor, // RGB8
	GBS_GenericAO, // R8
	GBS_IndirectIrradiance, // R8
	GBS_AO, // R8
	GBS_Velocity, // RG, float16
	GBS_PrecomputedShadowFactor, // RGBA8
	GBS_WorldTangent, // RGB8
	GBS_Anisotropy, // R8
	GBS_CustomData, // RGBA8, no compression
	GBS_SubsurfaceColor, // RGB8
	GBS_Opacity, // R8
	GBS_SubsurfaceProfile, //RGB8
	GBS_ClearCoat, // R8
	GBS_ClearCoatRoughness, // R8
	GBS_HairSecondaryWorldNormal, // RG8
	GBS_HairBacklit, // R8
	GBS_Cloth, // R8
	GBS_SubsurfaceProfileX, // R8
	GBS_IrisNormal, // RG8
	GBS_SeparatedMainDirLight, // RGB 11.11.10
	GBS_Num
};


enum EGBufferCompression
{
	GBC_Invalid,
	GBC_Raw_Float_16_16_16_16,
	GBC_Raw_Float_16_16_16,
	GBC_Raw_Float_11_11_10,
	GBC_Raw_Float_10_10_10,
	GBC_Raw_Unorm_8_8_8_8,
	GBC_Raw_Unorm_8_8_8,
	GBC_Raw_Unorm_8_8,
	GBC_Raw_Unorm_8,
	GBC_Raw_Unorm_2, // a float value normalized to use 2 bits
	GBC_Raw_Float_16_16,
	GBC_Raw_Float_16,
	GBC_Bits_4, // an int value, that is fundamentally 4 bits
	GBC_Bits_2, // an int value, that is fundamentally 2 bits

	GBC_Packed_Normal_Octahedral_8_8,
	GBC_EncodeNormal_Normal_16_16_16,
	GBC_EncodeNormal_Normal_10_10_10,
	GBC_EncodeNormal_Normal_8_8_8,

	GBC_Packed_Color_5_6_5, // a unorm value, packed to 565
	GBC_Packed_Color_5_6_5_Sqrt, // a unorm value, packed to 565, with sqrt
	GBC_Packed_Color_4_4_4, // a unorm value, packed to 444
	GBC_Packed_Color_4_4_4_Sqrt, // a unorm value, packed to 444, with sqrt
	GBC_Packed_Color_3_3_2, // a unorm value, packed to 332
	GBC_Packed_Color_3_3_2_Sqrt, // a unorm value, packed to 332, with sqrt
	GBC_Packed_Quantized_6, // a unorm value, quantized to 6 bits
	GBC_Packed_Quantized_4, // a unorm value, quantized to 4 bits
	GBC_Packed_Quantized_2, // a unorm value, quantized to 2 bits

	GBC_Num
};

// the actual format of the output texture
enum EGBufferType
{
	GBT_Invalid,
	GBT_Unorm_16_16,
	GBT_Unorm_8_8_8_8,
	GBT_Unorm_11_11_10,
	GBT_Unorm_10_10_10_2,
	GBT_Unorm_16_16_16_16,
	GBT_Float_16_16,
	GBT_Float_16_16_16_16,
	GBT_Float_32,
	GBT_Num
};

enum EGBufferChecker
{
	GBCH_Invalid,
	GBCH_Even,
	GBCH_Odd,
	GBCH_Both,
	GBCH_Num
};

enum EGBufferLayout
{
	GBL_Default, // Default GBuffer Layout
	GBL_ForceVelocity, // Force the inclusion of the velocity target (if it's not included in GBL_Default)

	GBL_Num
};


struct FGBufferCompressionInfo
{
	EGBufferCompression Type; //  compression type
	int32 SrcNumChan; // how many channels before compression
	int32 DstNumChan; // how many channels after copression
	int32 ChanBits[4]; // how many bits are each destination channel
	bool bIsPackedBits; //
	bool bIsConversion; //
	const TCHAR* EncodeFuncName; // name of the function to do this conversion
	const TCHAR* DecodeFuncName; // name of the function to do this conversion
};


struct FGBufferPacking
{
	FGBufferPacking()
	{
		bIsValid = false;
		bFull = false;
		TargetIndex = -1;
		DstChannelIndex = -1;
		SrcChannelIndex = -1;
		DstBitStart = -1;
		SrcBitStart = -1;
		BitNum = -1;
	}

	// if we pass in 2 values, it means we use the entire channel
	FGBufferPacking(int32 InTargetIndex, int32 InSrcChannelIndex, int32 InDstChannelIndex)
	{
		bIsValid = true;
		bFull = true;
		TargetIndex = InTargetIndex;
		DstChannelIndex = InDstChannelIndex;
		SrcChannelIndex = InSrcChannelIndex;
		DstBitStart = -1;
		SrcBitStart = -1;
		BitNum = -1;
	}

	// if we pass in 4 values, it means we also need to pack the bits
	FGBufferPacking(int32 InTargetIndex, int32 InSrcChannelIndex, int32 InDstChannelIndex, int32 InSrcBitStart, int32 InDstBitStart, int32 InBitNum)
	{
		bIsValid = true;
		bFull = false;
		TargetIndex = InTargetIndex;
		DstChannelIndex = InDstChannelIndex;
		SrcChannelIndex = InSrcChannelIndex;
		DstBitStart = InDstBitStart;
		SrcBitStart = InSrcBitStart;
		BitNum = InBitNum;
	}

	bool bIsValid;
	bool bFull;
	int32 TargetIndex;
	int32 DstChannelIndex;
	int32 DstBitStart;
	int32 SrcChannelIndex;
	int32 SrcBitStart;
	int32 BitNum;
};


// the texture positions in the GBuffer
struct FGBufferItem
{
	FGBufferItem()
	{
		bIsValid = false;
		bQuantizationBias = false;
		BufferSlot = GBS_Invalid;
		Compression = GBC_Invalid;
		Checker = GBCH_Invalid;

		for (int32 I = 0; I < 4; I++)
		{
			Packing[I] = {};
		}
	}

	FGBufferItem(EGBufferSlot InBufferSlot, EGBufferCompression InCompression, EGBufferChecker InChecker)
	{
		bIsValid = true;
		bQuantizationBias = false;
		BufferSlot = InBufferSlot;
		Compression = InCompression;
		Checker = InChecker;

		for (int32 I = 0; I < 4; I++)
		{
			Packing[I] = {};
		}
	}

	static const int MaxPacking = 8;

	bool bIsValid;
	bool bQuantizationBias;
	EGBufferSlot BufferSlot;
	EGBufferCompression Compression;
	EGBufferChecker Checker;
	FGBufferPacking Packing[MaxPacking]; // 8 should be plenty, can always make it larger
};

struct FGBufferTarget
{
	FGBufferTarget()
	{
		TargetType = GBT_Invalid;
		bIsSrgb = false;
		bIsRenderTargetable = false;
		bIsShaderResource = false;
		bIsUsingExtraFlags = false;
	}

	bool operator==(const FGBufferTarget & Rhs) const
	{
		return TargetType == Rhs.TargetType &&
			   TargetName == Rhs.TargetName &&
			   bIsSrgb == Rhs.bIsSrgb &&
			   bIsRenderTargetable == Rhs.bIsRenderTargetable &&
			   bIsShaderResource == Rhs.bIsShaderResource &&
			   bIsUsingExtraFlags == Rhs.bIsUsingExtraFlags;
	}

	void Init(EGBufferType InTargetType,
			FString InTargetName,
			bool bInIsSrgb,
			bool bInIsRenderTargetable,
			bool bInIsShaderResource,
			bool bInIsUsingExtraFlags)
	{
		TargetType = InTargetType;
		TargetName = InTargetName;
		bIsSrgb = bInIsSrgb;
		bIsRenderTargetable = bInIsRenderTargetable;
		bIsShaderResource = bInIsShaderResource;
		bIsUsingExtraFlags = bInIsUsingExtraFlags;
	}

	EGBufferType TargetType;
	FString TargetName;
	bool bIsSrgb;
	bool bIsRenderTargetable;
	bool bIsShaderResource;
	bool bIsUsingExtraFlags;
};

struct FGBufferBinding
{
	int32 Index = -1;
	EPixelFormat Format = PF_Unknown;
	ETextureCreateFlags Flags = TexCreate_None;
};

// Describes the bindings of the GBuffer for a given layout
struct FGBufferBindings
{
	FGBufferBinding GBufferA;
	FGBufferBinding GBufferB;
	FGBufferBinding GBufferC;
	FGBufferBinding GBufferD;
	FGBufferBinding GBufferE;
	FGBufferBinding GBufferVelocity;
};

struct FGBufferInfo
{
	static const int MaxTargets = 8;

	int32 NumTargets;
	FGBufferTarget Targets[MaxTargets];

	FGBufferItem Slots[GBS_Num];

};

struct FGBufferParams
{
	EShaderPlatform ShaderPlatform = SP_NumPlatforms;
	int32 LegacyFormatIndex = 0;
	bool bHasVelocity = false;
	bool bHasTangent = false;
	bool bHasPrecShadowFactor = false;
	bool bUsesVelocityDepth = false;
	bool bHasSingleLayerWaterSeparatedMainLight = false;

	bool operator == (const FGBufferParams& RHS) const
	{
		return
			ShaderPlatform == RHS.ShaderPlatform &&
			LegacyFormatIndex == RHS.LegacyFormatIndex &&
			bHasVelocity == RHS.bHasVelocity &&
			bHasTangent == RHS.bHasTangent &&
			bHasPrecShadowFactor == RHS.bHasPrecShadowFactor &&
			bUsesVelocityDepth == RHS.bUsesVelocityDepth &&
			bHasSingleLayerWaterSeparatedMainLight == RHS.bHasSingleLayerWaterSeparatedMainLight;
	}

	bool operator != (const FGBufferParams& RHS) const
	{
		return !(*this == RHS);
	}
};


int32 RENDERCORE_API FindGBufferTargetByName(const FGBufferInfo& GBufferInfo, const FString& Name);

FGBufferBinding RENDERCORE_API FindGBufferBindingByName(const FGBufferInfo& GBufferInfo, const FString& Name, EShaderPlatform ShaderPlatform);

FGBufferInfo RENDERCORE_API FetchFullGBufferInfo(const FGBufferParams& Params);

bool RENDERCORE_API IsGBufferInfoEqual(const FGBufferInfo& Lhs, const FGBufferInfo& Rhs);


