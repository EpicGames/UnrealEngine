// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

/*=============================================================================
	GPUDebugCrashUtils.h: Utilities for crashing the GPU in various ways on purpose.
	=============================================================================*/
#include "RenderGraphBuilder.h"

extern RENDERCORE_API void ScheduleGPUDebugCrash(FRDGBuilder& GraphBuilder);
