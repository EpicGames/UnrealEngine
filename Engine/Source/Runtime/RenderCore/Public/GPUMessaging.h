// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Misc/AssertionMacros.h"
#include "RenderGraphDefinitions.h"
#include "ShaderParameterMacros.h"
#include "Templates/SharedPointer.h"
#include "Templates/UnrealTemplate.h"

class FRDGBufferUAV;
class FRDGBuilder;

#define GPU_MESSAGE_SCOPE(GraphBuilder) GPUMessage::FScope GPUMessageScope_(GraphBuilder);

namespace GPUMessage
{

class FSocket;
class FSystem;

/** RAII object for controlling a new GPU message scope. */
class FScope
{
public:
	RENDERCORE_API FScope(FRDGBuilder& GraphBuilder);
	RENDERCORE_API ~FScope();

private:
	FRDGBuilder& GraphBuilder;

	static bool bRecursionCheck;
};

using FMessageId = TRDGHandle<FSocket, uint32>;

/** A socket object mapped to a persistently registered handler. Supports only move construction. Will unregister the handler on destruction. */
class FSocket
{
public:
	FSocket() = default;

	FSocket(FSocket&& Other)
	{
		*this = MoveTemp(Other);
	}

	FSocket& operator=(FSocket&& Other)
	{
		Reset();
		MessageId = Other.MessageId;
		Other.MessageId = FMessageId::Null;
		return *this;
	}

	~FSocket()
	{
		Reset();
	}

	const FMessageId GetMessageId() const { return MessageId; }

	RENDERCORE_API void Reset();

private:
	FSocket(FMessageId InMessageId)
		: MessageId(InMessageId)
	{}

	FMessageId MessageId;

	friend FSystem;
};

/** Interface for the handelr to read data from the GPU buffer during readback. */
class FReader
{
public:
	const FMessageId MessageId;

	template <typename DataType>
	DataType Read(const DataType& DefaultValue = {})
	{
		if (ensure(ReadOffset < PayloadSize))
		{
			return PayloadData[ReadOffset++];
		}
		return DefaultValue;
	}

	// Read up to N elements
	TConstArrayView<uint32> ReadCount(int32 Count)
	{
		TConstArrayView<uint32> Payload(PayloadData, PayloadSize);
		Payload.MidInline(ReadOffset, Count);
		ReadOffset += Payload.Num();
		return Payload;
	}

	uint32 GetPayloadSize() const { return PayloadSize; }

private:
	FReader(FMessageId InMessageId, uint32 InPayloadSize, const uint32* InPayloadData)
		: MessageId(InMessageId)
		, PayloadSize(InPayloadSize)
		, PayloadData(InPayloadData)
	{}

	const uint32 PayloadSize;
	const uint32* PayloadData;

	uint32 ReadOffset = 0;

	friend FSystem;
};

/** Base class persistent handler for receiving messages. */
class FHandler
{
public:
	FHandler(const TCHAR* InName)
		: Name(InName)
	{}

	virtual ~FHandler() = default;

	virtual void Execute(FReader Message) = 0;

	const TCHAR* GetName() const { return Name; }

private:
	const TCHAR* Name;
};

/** Lambda persistent handler which calls the provided lambda when a message is received. */
template <typename LambdaType>
class TLambdaHandler : public FHandler
{
public:
	TLambdaHandler(const TCHAR* InDebugId, LambdaType&& InLambda)
		: FHandler(InDebugId)
		, Lambda(MoveTemp(InLambda))
	{}

	virtual void Execute(FReader Message) override
	{
		Lambda(Message);
	}

private:
	LambdaType Lambda;
};

///////////////////////////////////////////////////////////////////////////////

/** Registers a persistent handler with the system. The FSocket object controls the lifetime of the handler. The handler
 *  execute function is called when a message is received. Messages may be delayed by several frames.
 */
extern RENDERCORE_API FSocket RegisterHandler(const TSharedPtr<FHandler>& Handler);

/** Registers a lambda function as a message handler. */
template <typename LambdaType>
FSocket RegisterHandler(const TCHAR* MessageDebugId, LambdaType&& Lambda)
{
	return RegisterHandler(TSharedPtr<FHandler>(new TLambdaHandler(MessageDebugId, MoveTemp(Lambda))));
}

/** Shader parameters used to write to write a GPU message. */
BEGIN_SHADER_PARAMETER_STRUCT(FParameters, RENDERCORE_API)
	SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<uint>, GPUMessageDataBuffer)
	SHADER_PARAMETER(uint32, GPUMessageDataBufferSize)
END_SHADER_PARAMETER_STRUCT()

/** Returns the shader parameters for the active GPU_MESSAGE_SCOPE. Will assert if called outside of that scope. */
extern RENDERCORE_API FParameters GetShaderParameters(FRDGBuilder& GraphBuilder);

///////////////////////////////////////////////////////////////////////////////

} // GPUMessage
