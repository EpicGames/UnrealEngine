// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "PixelFormat.h"
#include "RHIDefinitions.h"
#include "RenderGraphDefinitions.h"
#include "Templates/SharedPointer.h"

class FRDGBuilder;
class FRHICommandListImmediate;
class FRHISamplerState;
class FRHITexture;
struct FGenerateMipsStruct;

struct FGenerateMipsParams
{
	ESamplerFilter Filter = SF_Bilinear;
	ESamplerAddressMode AddressU = AM_Clamp;
	ESamplerAddressMode AddressV = AM_Clamp;
	ESamplerAddressMode AddressW = AM_Clamp;
};

enum class EGenerateMipsPass
{
	AutoDetect,
	Compute,
	Raster
};

class FGenerateMips
{
public:
	static RENDERCORE_API bool WillFormatSupportCompute(EPixelFormat InPixelFormat);

	/** (ES3.1+) Generates mips for the requested RHI texture using the feature-level appropriate means (Compute, Raster, or Fixed-Function). */
	static RENDERCORE_API void Execute(
		FRDGBuilder& GraphBuilder,
		ERHIFeatureLevel::Type FeatureLevel,
		FRDGTextureRef Texture,
		FGenerateMipsParams Params = {},
		EGenerateMipsPass Pass = EGenerateMipsPass::AutoDetect);

	/** (SM5+) Generates mips for the requested RDG texture using the requested compute / raster pass. */
	static RENDERCORE_API void Execute(
		FRDGBuilder& GraphBuilder,
		ERHIFeatureLevel::Type FeatureLevel,
		FRDGTextureRef Texture,
		FRHISamplerState* Sampler,
		EGenerateMipsPass Pass = EGenerateMipsPass::AutoDetect);

	static RENDERCORE_API void ExecuteCompute(
		FRDGBuilder& GraphBuilder, ERHIFeatureLevel::Type FeatureLevel,
		FRDGTextureRef Texture,
		FRHISamplerState* Sampler);
	
	/** (SM5+) Generate mips for the requested RDG texture using the compute pass conditionally.
		if( uint(ConditionBuffer[Offset]) > 0)
			Execute(...)
	*/
	static RENDERCORE_API void ExecuteCompute(
		FRDGBuilder& GraphBuilder,
		ERHIFeatureLevel::Type FeatureLevel,
		FRDGTextureRef Texture,
		FRHISamplerState* Sampler,
		FRDGBufferRef ConditionBuffer, uint32 Offset = 0);

	static RENDERCORE_API void ExecuteRaster(
		FRDGBuilder& GraphBuilder,
		ERHIFeatureLevel::Type FeatureLevel,
		FRDGTextureRef Texture,
		FRHISamplerState* Sampler);
};
