// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RenderResource.h"

// A global white texture.
extern RENDERCORE_API FTexture* GWhiteTexture;

// A global white texture with an SRV.
extern RENDERCORE_API FTextureWithSRV* GWhiteTextureWithSRV;

// A global black texture.
extern RENDERCORE_API FTexture* GBlackTexture;

// A global black texture with an SRV.
extern RENDERCORE_API FTextureWithSRV* GBlackTextureWithSRV;

// A global black transparent texture.
extern RENDERCORE_API FTexture* GTransparentBlackTexture;

// A global black transparent texture with an SRV
extern RENDERCORE_API FTextureWithSRV* GTransparentBlackTextureWithSRV;

// A global dummy texture used for manual transition (Ex: when the VS is sampling produce in a previous pass, and needs a barrier)
UE_DEPRECATED(5.7, "This is no longer being used.")
extern RENDERCORE_API FTexture* GDummyTransitionTexture;

// An empty vertex buffer with a UAV
extern RENDERCORE_API FVertexBufferWithSRV* GEmptyVertexBufferWithUAV;

// An empty uint4 vertex buffer with a UAV
extern RENDERCORE_API FVertexBufferWithSRV* GEmptyVertexBufferUInt4WithUAV;

// An empty structured buffer with a UAV
extern RENDERCORE_API FVertexBufferWithSRV* GEmptyStructuredBufferWithUAV;

// An empty float4 structured buffer
extern RENDERCORE_API FVertexBufferWithSRV* GBlackFloat4StructuredBufferWithSRV;

// An empty float4 vertex buffer
extern RENDERCORE_API FVertexBufferWithSRV* GBlackFloat4VertexBufferWithSRV;

// An white vertex buffer with a SRV
extern RENDERCORE_API FVertexBufferWithSRV* GWhiteVertexBufferWithSRV;

// An black vertex buffer with a SRV
extern RENDERCORE_API FVertexBufferWithSRV* GBlackVertexBufferWithSRV;

// A white vertex buffer used with RenderGraph.
extern RENDERCORE_API FBufferWithRDG* GWhiteVertexBufferWithRDG;

// A global black array texture
extern RENDERCORE_API FTexture* GBlackArrayTexture;

// A global black volume texture.
extern RENDERCORE_API FTexture* GBlackVolumeTexture;

// A global black volume texture, with alpha=1.
extern RENDERCORE_API FTexture* GBlackAlpha1VolumeTexture;

// A global black texture<uint>
extern RENDERCORE_API FTexture* GBlackUintTexture;

// A global black volume texture<uint> 
extern RENDERCORE_API FTexture* GBlackUintVolumeTexture;

// A global white cube texture.
extern RENDERCORE_API FTexture* GWhiteTextureCube;

// A global black cube texture.
extern RENDERCORE_API FTexture* GBlackTextureCube;

// A global black cube depth texture.
extern RENDERCORE_API FTexture* GBlackTextureDepthCube;

// A global black cube array texture.
extern RENDERCORE_API FTexture* GBlackCubeArrayTexture;

// A global texture that has a different solid color in each mip-level.
extern RENDERCORE_API FTexture* GMipColorTexture;

/** Number of mip-levels in 'GMipColorTexture' */
extern RENDERCORE_API int32 GMipColorTextureMipLevels;

// 4: 8x8 cubemap resolution, shader needs to use the same value as preprocessing
extern RENDERCORE_API const uint32 GDiffuseConvolveMipLevel;

/**
* A vertex buffer with a single color component.  This is used on meshes that don't have a color component
* to keep from needing a separate vertex factory to handle this case.
*/
class FNullColorVertexBuffer : public FVertexBuffer
{
public:
	RENDERCORE_API FNullColorVertexBuffer();
	RENDERCORE_API ~FNullColorVertexBuffer();

	RENDERCORE_API virtual void InitRHI(FRHICommandListBase& RHICmdList) override;
	RENDERCORE_API virtual void ReleaseRHI() override;

	FShaderResourceViewRHIRef VertexBufferSRV;
};

/** The global null color vertex buffer, which is set with a stride of 0 on meshes without a color component. */
extern RENDERCORE_API TGlobalResource<FNullColorVertexBuffer, FRenderResource::EInitPhase::Pre> GNullColorVertexBuffer;

/**
* A vertex buffer with a single zero float3 component.
*/
class FNullVertexBuffer : public FVertexBuffer
{
public:
	RENDERCORE_API FNullVertexBuffer();
	RENDERCORE_API ~FNullVertexBuffer();

	RENDERCORE_API virtual void InitRHI(FRHICommandListBase& RHICmdList) override;
	RENDERCORE_API virtual void ReleaseRHI() override;

	FShaderResourceViewRHIRef VertexBufferSRV;
};

/** The global null vertex buffer, which is set with a stride of 0 on meshes */
extern RENDERCORE_API TGlobalResource<FNullVertexBuffer, FRenderResource::EInitPhase::Pre> GNullVertexBuffer;

class FScreenSpaceVertexBuffer : public FVertexBuffer
{
public:
	/**
	* Initialize the RHI for this rendering resource
	*/
	RENDERCORE_API virtual void InitRHI(FRHICommandListBase& RHICmdList) override;
};

extern RENDERCORE_API TGlobalResource<FScreenSpaceVertexBuffer, FRenderResource::EInitPhase::Pre> GScreenSpaceVertexBuffer;

class FTileVertexDeclaration : public FRenderResource
{
public:
	RENDERCORE_API FTileVertexDeclaration();
	RENDERCORE_API virtual ~FTileVertexDeclaration();

	RENDERCORE_API virtual void InitRHI(FRHICommandListBase& RHICmdList) override;
	RENDERCORE_API virtual void ReleaseRHI() override;

	FVertexDeclarationRHIRef VertexDeclarationRHI;
};

extern RENDERCORE_API TGlobalResource<FTileVertexDeclaration, FRenderResource::EInitPhase::Pre> GTileVertexDeclaration;

class FCubeIndexBuffer : public FIndexBuffer
{
public:
	/**
	* Initialize the RHI for this rendering resource
	*/
	virtual void InitRHI(FRHICommandListBase& RHICmdList) override;
};
extern RENDERCORE_API TGlobalResource<FCubeIndexBuffer, FRenderResource::EInitPhase::Pre> GCubeIndexBuffer;

class FTwoTrianglesIndexBuffer : public FIndexBuffer
{
public:
	/**
	* Initialize the RHI for this rendering resource
	*/
	virtual void InitRHI(FRHICommandListBase& RHICmdList) override;
};
extern RENDERCORE_API TGlobalResource<FTwoTrianglesIndexBuffer, FRenderResource::EInitPhase::Pre> GTwoTrianglesIndexBuffer;

class FEmptyResourceCollection : public FRenderResource
{
public:
	FEmptyResourceCollection() = default;
	~FEmptyResourceCollection() = default;

	RENDERCORE_API virtual void InitRHI(FRHICommandListBase& RHICmdList) override;
	RENDERCORE_API virtual void ReleaseRHI() override;

	FRHIResourceCollectionRef ResourceCollection;
};

extern RENDERCORE_API TGlobalResource<FEmptyResourceCollection, FRenderResource::EInitPhase::Pre> GEmptyResourceCollection;

//////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// FGlobalDynamicVertexBuffer

template <typename BufferType>
class TDynamicBuffer;

using FDynamicVertexBuffer = TDynamicBuffer<FVertexBuffer>;
using FDynamicIndexBuffer  = TDynamicBuffer<FIndexBuffer>;

struct FGlobalDynamicVertexBufferAllocation
{
	/** The location of the buffer in main memory. */
	uint8* Buffer = nullptr;

	/** The vertex buffer to bind for draw calls. */
	FVertexBuffer* VertexBuffer = nullptr;

	/** The offset in to the vertex buffer. */
	uint32 VertexOffset = 0;

	/** Returns true if the allocation is valid. */
	inline bool IsValid() const
	{
		return Buffer != nullptr;
	}
};

/**
 * A system for dynamically allocating GPU memory for vertices.
 */
class FGlobalDynamicVertexBuffer
{
public:
	using FAllocation = FGlobalDynamicVertexBufferAllocation;

	FGlobalDynamicVertexBuffer() = default;

	FGlobalDynamicVertexBuffer(FRHICommandListBase& InRHICmdList)
		: RHICmdList(&InRHICmdList)
	{}

	~FGlobalDynamicVertexBuffer()
	{
		Commit();
	}

	void Init(FRHICommandListBase& InRHICmdList)
	{
		check(VertexBuffers.IsEmpty());
		RHICmdList = &InRHICmdList;
	}

	/**
	 * Allocates space in the global vertex buffer.
	 * @param SizeInBytes - The amount of memory to allocate in bytes.
	 * @returns An FAllocation with information regarding the allocated memory.
	 */
	RENDERCORE_API FAllocation Allocate(uint32 SizeInBytes);

	/**
	 * Commits allocated memory to the GPU.
	 *		WARNING: Once this buffer has been committed to the GPU, allocations
	 *		remain valid only until the next call to Allocate!
	 */
	RENDERCORE_API void Commit();

	/** Returns true if log statements should be made because we exceeded GMaxVertexBytesAllocatedPerFrame */
	RENDERCORE_API bool IsRenderAlarmLoggingEnabled() const;

private:
	FRHICommandListBase* RHICmdList = nullptr;
	TArray<FDynamicVertexBuffer*> VertexBuffers;
};

struct FGlobalDynamicIndexBufferAllocation
{
	/** The location of the buffer in main memory. */
	uint8* Buffer = nullptr;

	/** The vertex buffer to bind for draw calls. */
	FIndexBuffer* IndexBuffer = nullptr;

	/** The offset in to the index buffer. */
	uint32 FirstIndex = 0;

	/** Returns true if the allocation is valid. */
	inline bool IsValid() const
	{
		return Buffer != nullptr;
	}
};

struct FGlobalDynamicIndexBufferAllocationEx : public FGlobalDynamicIndexBufferAllocation
{
	FGlobalDynamicIndexBufferAllocationEx(const FGlobalDynamicIndexBufferAllocation& InRef, uint32 InNumIndices, uint32 InIndexStride)
		: FGlobalDynamicIndexBufferAllocation(InRef)
		, NumIndices(InNumIndices)
		, IndexStride(InIndexStride)
	{}

	/** The number of indices allocated. */
	uint32 NumIndices = 0;
	/** The allocation stride (2 or 4 bytes). */
	uint32 IndexStride = 0;
	/** The maximum value of the indices used. */
	uint32 MaxUsedIndex = 0;
};

/**
 * A system for dynamically allocating GPU memory for indices.
 */
class FGlobalDynamicIndexBuffer
{
public:
	using FAllocation = FGlobalDynamicIndexBufferAllocation;
	using FAllocationEx = FGlobalDynamicIndexBufferAllocationEx;

	FGlobalDynamicIndexBuffer() = default;

	FGlobalDynamicIndexBuffer(FRHICommandListBase& InRHICmdList)
		: RHICmdList(&InRHICmdList)
	{}

	~FGlobalDynamicIndexBuffer()
	{
		Commit();
	}

	void Init(FRHICommandListBase& InRHICmdList)
	{
		check(IndexBuffers16.IsEmpty() && IndexBuffers32.IsEmpty());
		RHICmdList = &InRHICmdList;
	}

	/**
	 * Allocates space in the global index buffer.
	 * @param NumIndices - The number of indices to allocate.
	 * @param IndexStride - The size of an index (2 or 4 bytes).
	 * @returns An FAllocation with information regarding the allocated memory.
	 */
	RENDERCORE_API FAllocation Allocate(uint32 NumIndices, uint32 IndexStride);

	/**
	 * Helper function to allocate.
	 * @param NumIndices - The number of indices to allocate.
	 * @returns an FAllocation with information regarding the allocated memory.
	 */
	template <typename IndexType>
	inline FAllocationEx Allocate(uint32 NumIndices)
	{
		return FAllocationEx(Allocate(NumIndices, sizeof(IndexType)), NumIndices, sizeof(IndexType));
	}

	/**
	 * Commits allocated memory to the GPU.
	 *		WARNING: Once this buffer has been committed to the GPU, allocations
	 *		remain valid only until the next call to Allocate!
	 */
	RENDERCORE_API void Commit();

private:
	FRHICommandListBase* RHICmdList = nullptr;
	TArray<FDynamicIndexBuffer*> IndexBuffers16;
	TArray<FDynamicIndexBuffer*> IndexBuffers32;
};

namespace GlobalDynamicBuffer
{
	RENDERCORE_API void GarbageCollect();
}