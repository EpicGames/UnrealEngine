// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	GlobalShader.h: Shader manager definitions.
=============================================================================*/

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Paths.h"
#include "RHI.h"
#include "RHIDefinitions.h"
#include "RHIResources.h"
#include "Serialization/MemoryImage.h"
#include "Serialization/MemoryLayout.h"
#include "Shader.h"
#include "ShaderParameterUtils.h"
#include "Templates/Tuple.h"
#include "Templates/UniquePtr.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealNames.h"

class FArchive;
class FShaderCommonCompileJob;
class FShaderCompileJob;
class FShaderParametersMetadata;
class FShaderUniformBufferParameter;
class FVertexFactoryType;
class ITargetPlatform;
struct FShaderCompilerEnvironment;
#if WITH_EDITOR
struct FShaderCacheLoadContext;
#endif

/** Used to identify the global shader map in compile queues. */
extern RENDERCORE_API const int32 GlobalShaderMapId;

/** Class that encapsulates logic to create a DDC key for the global shader map. */
class FGlobalShaderMapId
{
public:

	/** Create a global shader map Id for the given platform. */
	RENDERCORE_API FGlobalShaderMapId(EShaderPlatform Platform, const ITargetPlatform* TargetPlatform);

	/** Append to a string that will be used as a DDC key. */
	RENDERCORE_API void AppendKeyString(FString& KeyString, const TArray<FShaderTypeDependency>& Dependencies) const;

	const TMap<FString, TArray<FShaderTypeDependency>>& GetShaderFilenameToDependeciesMap() const { return ShaderFilenameToDependenciesMap; }

	RENDERCORE_API bool WithEditorOnly() const;

	inline EShaderPermutationFlags GetShaderPermutationFlags() const { return ::GetShaderPermutationFlags(LayoutParams); }

private:
	FPlatformTypeLayoutParameters LayoutParams;
	EShaderPlatform ShaderPlatform;
	FName IniPlatformName;

	/** Shader types that this shader map is dependent on and their stored state. Mapped by shader filename, so every filename can have it's own DDC key. */
	TMap<FString, TArray<FShaderTypeDependency>> ShaderFilenameToDependenciesMap;

	/** Shader pipeline types that this shader map is dependent on and their stored state. */
	TArray<FShaderPipelineTypeDependency> ShaderPipelineTypeDependencies;
};

struct FGlobalShaderPermutationParameters : public FShaderPermutationParameters
{
	const FName GlobalShaderName;

	FGlobalShaderPermutationParameters(FName InGlobalShaderName, EShaderPlatform InShaderPlatform, int32 InPermutationId = 0, EShaderPermutationFlags InFlags = EShaderPermutationFlags::HasEditorOnlyData)
		: FShaderPermutationParameters(InShaderPlatform, InPermutationId, InFlags)
		, GlobalShaderName(InGlobalShaderName)
	{}
};

/**
 * A shader meta type for the simplest shaders; shaders which are not material or vertex factory linked.
 * There should only a single instance of each simple shader type.
 */
class FGlobalShaderType : public FShaderType
{
	friend class FGlobalShaderTypeCompiler;
public:

	typedef FShader::CompiledShaderInitializerType CompiledShaderInitializerType;

	IMPLEMENT_SHADER_TYPE_CONSTRUCTOR(FGlobalShaderType, EShaderTypeForDynamicCast::Global, TEXT("global shader"));

	/**
	 * Checks if the shader type should be cached for a particular platform.
	 * @param Platform - The platform to check.
	 * @return True if this shader type should be cached.
	 */
	bool ShouldCompilePermutation(EShaderPlatform Platform, int32 PermutationId, EShaderPermutationFlags Flags) const
	{
		return FShaderType::ShouldCompilePermutation(FGlobalShaderPermutationParameters(GetFName(), Platform, PermutationId, Flags));
	}

	EShaderPermutationPrecacheRequest ShouldPrecachePermutation(EShaderPlatform Platform, int32 PermutationId, EShaderPermutationFlags Flags) const
	{
		return FShaderType::ShouldPrecachePermutation(FGlobalShaderPermutationParameters(GetFName(), Platform, PermutationId, Flags));
	}

#if WITH_EDITOR
	/**
	 * Sets up the environment used to compile an instance of this shader type.
	 * @param Platform - Platform to compile for.
	 * @param Environment - The shader compile environment that the function modifies.
	 */
	RENDERCORE_API void SetupCompileEnvironment(EShaderPlatform Platform, int32 PermutationId, EShaderPermutationFlags Flags, FShaderCompilerEnvironment& Environment) const;
#endif // WITH_EDITOR

	RENDERCORE_API static bool ShouldCompilePipeline(const FShaderPipelineType* ShaderPipelineType, EShaderPlatform Platform, EShaderPermutationFlags Flags);

	RENDERCORE_API bool ForEachCompiledPermutation(EShaderPlatform Platform, EShaderPermutationFlags PermutationFlags, TFunction<bool(int32)> Function);
};

class FGlobalShaderMapContent : public FShaderMapContent
{
	using Super = FShaderMapContent;
	friend class FGlobalShaderMap;
	friend class FGlobalShaderMapSection;
	DECLARE_TYPE_LAYOUT(FGlobalShaderMapContent, NonVirtual);
public:
	const FHashedName& GetHashedSourceFilename() const { return HashedSourceFilename; }

private:
	inline FGlobalShaderMapContent(EShaderPlatform InPlatform, const FHashedName& InHashedSourceFilename)
		: Super(InPlatform)
		, HashedSourceFilename(InHashedSourceFilename)
	{}

	LAYOUT_FIELD(FHashedName, HashedSourceFilename);
};

class FGlobalShaderMapSection : public TShaderMap<FGlobalShaderMapContent, FShaderMapPointerTable>
{
	using Super = TShaderMap<FGlobalShaderMapContent, FShaderMapPointerTable>;
	friend class FGlobalShaderMap;
public:
	RENDERCORE_API static FGlobalShaderMapSection* CreateFromArchive(FArchive& Ar);

#if WITH_EDITOR
	RENDERCORE_API static FGlobalShaderMapSection* CreateFromCache(FShaderCacheLoadContext& Ctx);
#endif

	RENDERCORE_API bool Serialize(FShaderSerializeContext& Ctx);
	RENDERCORE_API virtual void GetShaderList(TMap<FHashedName, TShaderRef<FShader>>& OutShaders) const override;
	RENDERCORE_API virtual void GetShaderPipelineList(TArray<FShaderPipelineRef>& OutShaderPipelines) const override;
private:
	inline FGlobalShaderMapSection() 
	{ 
#if WITH_EDITOR
		// associate with a non-empty asset name as to not have an exception in the library
		FShaderMapBase::AssociateWithAsset(NAME_None);
#endif
	}

	inline FGlobalShaderMapSection(EShaderPlatform InPlatform, const FHashedName& InHashedSourceFilename)
		: FGlobalShaderMapSection()
	{
		AssignContent(new FGlobalShaderMapContent(InPlatform, InHashedSourceFilename));
	}

	UE_DEPRECATED(5.8, "Specifying the shader platform is now required when getting a shader from the FGlobalShaderMapSection.")
		inline TShaderRef<FShader> GetShader(FShaderType* ShaderType, int32 PermutationId = 0) const
	{
		return GetShader(GetShaderPlatform(), ShaderType, PermutationId);
	}

	TShaderRef<FShader> GetShader(EShaderPlatform ShaderPlatform, FShaderType* ShaderType, int32 PermutationId = 0) const;
	FShaderPipelineRef GetShaderPipeline(const FShaderPipelineType* PipelineType) const;
};

class FGlobalShaderMap
{
public:
	RENDERCORE_API explicit FGlobalShaderMap(EShaderPlatform InPlatform);
	RENDERCORE_API ~FGlobalShaderMap();

	RENDERCORE_API TShaderRef<FShader> GetShader(FShaderType* ShaderType, int32 PermutationId = 0) const;
	RENDERCORE_API FShaderPipelineRef GetShaderPipeline(const FShaderPipelineType* PipelineType) const;

	template<typename ShaderType>
	TShaderRef<ShaderType> GetShader() const
	{
		ensureMsgf(ShaderType::GetStaticType().GetPermutationCount() == 1, TEXT("Failed to provide PermutationId for shader type %s with %u permutations"), ShaderType::GetStaticType().GetName(), ShaderType::GetStaticType().GetPermutationCount());
		return GetShader<ShaderType>(0);
	}

	template<typename ShaderType>
	TShaderRef<ShaderType> GetShader(int32 PermutationId) const
	{
		TShaderRef<FShader> Shader = GetShader(&ShaderType::GetStaticType(), PermutationId);
		checkf(Shader.IsValid(), TEXT("Failed to find permutation %d of shader type %s in Platform %s"), PermutationId, ShaderType::GetStaticType().GetName(), *LegacyShaderPlatformToShaderFormat(Platform).ToString());

		// Validate that the permutation has been precached
		ensureMsgf(ShaderType::ShouldPrecachePermutation(FGlobalShaderPermutationParameters(ShaderType::GetStaticType().GetFName(), Platform, PermutationId)) != EShaderPermutationPrecacheRequest::NotUsed,
			TEXT("Using a global shader permutation of %s which hasn't been requested for precaching at runtime. Check the implementation of ShouldPrecachePermutation on the global shader and make sure all required or development only permutations are requested for PSO precaching"), ShaderType::GetStaticType().GetName());

		return TShaderRef<ShaderType>::Cast(Shader);
	}

	/** Finds the shader with the given type.  Asserts on failure. */
	template<typename ShaderType>
	TShaderRef<ShaderType> GetShader(const typename ShaderType::FPermutationDomain& PermutationVector) const
	{
		return GetShader<ShaderType>(PermutationVector.ToDimensionValueId());
	}

	bool HasShader(FShaderType* Type, int32 PermutationId) const
	{
		return GetShader(Type, PermutationId).IsValid();
	}

	bool HasShaderPipeline(const FShaderPipelineType* ShaderPipelineType) const
	{
		return GetShaderPipeline(ShaderPipelineType).IsValid();
	}

	EShaderPlatform GetPlatform() const
	{
		return Platform;
	}

	RENDERCORE_API bool IsEmpty() const;

	/** Whether the shadermap has all the shader types it needs (i.e. ones that returned ShouldCompilePermutation) */
	RENDERCORE_API bool IsComplete(const ITargetPlatform* TargetPlatform) const;

	RENDERCORE_API void Empty();
	RENDERCORE_API void ReleaseAllSections();

	RENDERCORE_API FShader* FindOrAddShader(const FShaderType* ShaderType, int32 PermutationId, FShader* Shader);
	RENDERCORE_API FShaderPipeline* FindOrAddShaderPipeline(const FShaderPipelineType* ShaderPipelineType, FShaderPipeline* ShaderPipeline);

	RENDERCORE_API void RemoveShaderTypePermutaion(const FShaderType* Type, int32 PermutationId);
	RENDERCORE_API void RemoveShaderPipelineType(const FShaderPipelineType* ShaderPipelineType);

	RENDERCORE_API void AddSection(FGlobalShaderMapSection* InSection);
	RENDERCORE_API FGlobalShaderMapSection* FindSection(const FHashedName& HashedShaderFilename);
	RENDERCORE_API FGlobalShaderMapSection* FindOrAddSection(const FShaderType* ShaderType);
	
	RENDERCORE_API void LoadFromGlobalArchive(FArchive& Ar);
	RENDERCORE_API void SaveToGlobalArchive(FArchive& Ar);

	RENDERCORE_API void BeginCreateAllShaders();

#if WITH_EDITOR
	RENDERCORE_API void GetOutdatedTypes(TArray<const FShaderType*>& OutdatedShaderTypes, TArray<const FShaderPipelineType*>& OutdatedShaderPipelineTypes, TArray<const FVertexFactoryType*>& OutdatedFactoryTypes) const;
	RENDERCORE_API void SaveShaderStableKeys(EShaderPlatform TargetShaderPlatform);
#endif // WITH_EDITOR

	const FGlobalShaderMapSection* GetFirstSection()
	{
		if (const auto It = SectionMap.CreateConstIterator())
		{
			return It.Value();
		}
		return nullptr;
	}

private:
	TMap<FHashedName, FGlobalShaderMapSection*> SectionMap;
	EShaderPlatform Platform;
};

extern RENDERCORE_API FGlobalShaderMap* GGlobalShaderMap[SP_NumPlatforms];


/**
 * FGlobalShader
 * 
 * Global shaders derive from this class to set their default recompile group as a global one
 */
class FGlobalShader : public FShader
{
	DECLARE_EXPORTED_TYPE_LAYOUT(FGlobalShader, RENDERCORE_API, NonVirtual);
public:
	using ShaderMetaType = FGlobalShaderType;
	using FPermutationParameters = FGlobalShaderPermutationParameters;

	FGlobalShader() = default;

	RENDERCORE_API FGlobalShader(const ShaderMetaType::CompiledShaderInitializerType& Initializer);
	
	template<typename TViewUniformShaderParameters>
	inline void SetParameters(FRHIBatchedShaderParameters& BatchedParameters, FRHIUniformBuffer* ViewUniformBuffer)
	{
		const auto& ViewUniformBufferParameter = static_cast<const FShaderUniformBufferParameter&>(GetUniformBufferParameter<TViewUniformShaderParameters>());
		SetUniformBufferParameter(BatchedParameters, ViewUniformBufferParameter, ViewUniformBuffer);
	}

	using FShader::ShouldCompilePermutation;
	using FShader::ShouldPrecachePermutation;
	
	static inline void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& Environment) { };
};

/**
 * An internal dummy pixel shader to use when the user calls RHISetPixelShader(NULL).
 */
class FNULLPS : public FGlobalShader
{
	DECLARE_EXPORTED_SHADER_TYPE(FNULLPS,Global, RENDERCORE_API);
public:

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return true;
	}

	FNULLPS( )	{ }
	FNULLPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer):
		FGlobalShader(Initializer)
	{
	}
};

/**
 * Accesses the global shader map.  This is a global FGlobalShaderMap which contains an instance of each global shader type.
 *
 * @param Platform Which platform's global shader map to use
 * @param bRefreshShaderMap If true, the existing global shader map will be tossed first
 * @return A reference to the global shader map.
 */
extern RENDERCORE_API FGlobalShaderMap* GetGlobalShaderMap(EShaderPlatform Platform);

RENDERCORE_API void PrimeGlobalShaderSourceHashCache(EShaderPlatform Platform, const ITargetPlatform* TargetPlatform);

inline void PrimeGlobalShaderSourceHashCache()
{
	EShaderPlatform Platform = GShaderPlatformForFeatureLevel[GMaxRHIFeatureLevel];
	PrimeGlobalShaderSourceHashCache(Platform, nullptr);
}

/**
  * Overload for the above GetGlobalShaderMap which takes a feature level and translates to the appropriate shader platform
  *
  * @param FeatureLevel - Which feature levels shader map to use
  * @param bRefreshShaderMap If true, the existing global shader map will be tossed first
  * @return A reference to the global shader map.
  *
  **/
inline FGlobalShaderMap* GetGlobalShaderMap(ERHIFeatureLevel::Type FeatureLevel)
{ 
	return GetGlobalShaderMap(GShaderPlatformForFeatureLevel[FeatureLevel]); 
}


/** DECLARE_GLOBAL_SHADER and IMPLEMENT_GLOBAL_SHADER setup a global shader class's boiler plate. They are meant to be used like so:
 *
 * class FMyGlobalShaderPS : public FGlobalShader
 * {
 *		// Setup the shader's boiler plate.
 *		DECLARE_GLOBAL_SHADER(FMyGlobalShaderPS);
 *
 *		// Setup the shader's permutation domain. If no dimensions, can do FPermutationDomain = FShaderPermutationNone.
 *		using FPermutationDomain = TShaderPermutationDomain<DIMENSIONS...>;
 *
 *		// ...
 * };
 *
 * // Instantiates global shader's global variable that will take care of compilation process of the shader. This needs imperatively to be
 * done in a .cpp file regardless of whether FMyGlobalShaderPS is in a header or not.
 * IMPLEMENT_GLOBAL_SHADER(FMyGlobalShaderPS, "/Engine/Private/MyShaderFile.usf", "MainPS", SF_Pixel);
 *
 * When the shader class is a public header, let say in RenderCore module public header, the class declaration macro then should have the RENDERCORE_API
 * like this:
 *
 * class FMyGlobalShaderPS : public FGlobalShader
 * {
 *		// Setup the shader's boiler plate.
 *		DECLARE_EXPORTED_GLOBAL_SHADER(FMyGlobalShaderPS, RENDERCORE_API);
 *
 *		// ...
 * };
 */
/*#define DECLARE_GLOBAL_SHADER(ShaderClass) \
	public: \
	\
	using ShaderMetaType = FGlobalShaderType; \
	\
	using ShaderMapType = FGlobalShaderMap; \
	\
	static ShaderMetaType StaticType; \
	\
	static FShader* ConstructSerializedInstance() { return new ShaderClass(); } \
	static FShader* ConstructCompiledInstance(const ShaderMetaType::CompiledShaderInitializerType& Initializer) \
	{ return new ShaderClass(Initializer); } \
	\
	static void ModifyCompilationEnvironmentImpl( \
		const FShaderPermutationParameters& Parameters, \
		FShaderCompilerEnvironment& OutEnvironment) \
	{ \
		FPermutationDomain PermutationVector(Parameters.PermutationId); \
		PermutationVector.ModifyCompilationEnvironment(OutEnvironment); \
		ShaderClass::ModifyCompilationEnvironment(Parameters, OutEnvironment); \
	}

#define IMPLEMENT_GLOBAL_SHADER(ShaderClass,SourceFilename,FunctionName,Frequency) \
	ShaderClass::ShaderMetaType ShaderClass::StaticType( \
		TEXT(#ShaderClass), \
		TEXT(SourceFilename), \
		TEXT(FunctionName), \
		Frequency, \
		ShaderClass::FPermutationDomain::PermutationCount, \
		ShaderClass::ConstructSerializedInstance, \
		ShaderClass::ConstructCompiledInstance, \
		ShaderClass::ConstructEditorContent, \
		ShaderClass::DestroyInstance, \
		ShaderClass::DestroyEditorContent, \
		ShaderClass::ModifyCompilationEnvironmentImpl, \
		ShaderClass::ShouldCompilePermutation, \
		ShaderClass::ValidateCompiledResult, \
		ShaderClass::FreezeMemoryImageImpl, \
		ShaderClass::FreezeMemoryImageEditorImpl, \
		sizeof(ShaderClass), \
		sizeof(ShaderClass::FEditorContent), \
		ShaderClass::GetRootParametersMetadata() \
		ShaderClass::FPermutationDomain::SpecializationCount, \
		)
		*/

#define DECLARE_GLOBAL_SHADER(ShaderClass) DECLARE_SHADER_TYPE(ShaderClass, Global)
#define DECLARE_EXPORTED_GLOBAL_SHADER(ShaderClass, RequiredAPI) DECLARE_EXPORTED_SHADER_TYPE(ShaderClass, Global, RequiredAPI)
#define IMPLEMENT_GLOBAL_SHADER(ShaderClass,SourceFilename,FunctionName,Frequency) IMPLEMENT_SHADER_TYPE(,ShaderClass,TEXT(SourceFilename),TEXT(FunctionName),Frequency)

/**
* Groups of global shaders which can be used in recursive
* RHI command lists by platform RHI implementations.
*/
enum class ERecursiveShader
{
	None = 0,

	// Shaders used for resolving MSAA textures (defined in ResolveShader.h)
	Resolve = 1 << 0,

	// Shaders used for clearing textures and buffers (defined in ClearReplacementShaders.h)
	Clear = 1 << 1,

	// A null pixel shader (defined above)
	Null = 1 << 2
};
ENUM_CLASS_FLAGS(ERecursiveShader)

/*
* Set by the platform RHI implementations to indicate which groups of global shaders will be used recursively.
*/
extern RENDERCORE_API ERecursiveShader GRequiredRecursiveShaders;

/*
* Called on the render thread when the global shader map is available to force-init certain global shaders as specified by GRequiredRecursiveShaders.
*/
extern RENDERCORE_API void CreateRecursiveShaders();

void ForceInitGlobalShaderType(FShaderType& ShaderType);

template <typename TShaderType>
inline void ForceInitGlobalShaderType()
{
	ForceInitGlobalShaderType(TShaderType::GetStaticType());
}
