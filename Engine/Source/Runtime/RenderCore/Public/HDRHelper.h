// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ColorManagement/ColorSpace.h"

RENDERCORE_API bool IsHDREnabled();
RENDERCORE_API bool IsHDRAllowed();

RENDERCORE_API bool IsDefaultBackBufferLinearSDR();

RENDERCORE_API void HDRGetMetaData(EDisplayOutputFormat& OutDisplayOutputFormat, EDisplayColorGamut& OutDisplayColorGamut, bool& OutbHDRSupported,
								   const FVector2D& WindowTopLeft, const FVector2D& WindowBottomRight, void* OSWindow);

struct FHDRMetaData
{
	EDisplayOutputFormat DisplayOutputFormat;
	EDisplayColorGamut DisplayColorGamut;
	float MinimumLuminanceInNits;
	float MaximumLuminanceInNits;
	float MaximumFullFrameLuminanceInNits;
	float HDRPaperWhiteInNits;
	UE::Color::FColorSpace LimitingColorSpace;
	bool bHDRSupported;
};
RENDERCORE_API void HDRGetMetaDataForWindow(FHDRMetaData& OutHDRMetaData, const FVector2D& WindowTopLeft, const FVector2D& WindowBottomRight, void* OSWindow);

RENDERCORE_API void HDRConfigureCVars(bool bIsHDREnabled, uint32 DisplayNits, bool bFromGameSettings);
RENDERCORE_API EDisplayOutputFormat HDRGetDefaultDisplayOutputFormat();
RENDERCORE_API EDisplayColorGamut HDRGetDefaultDisplayColorGamut();
RENDERCORE_API float HDRGetDisplayMaximumLuminance();
RENDERCORE_API void HDRAddCustomMetaData(void* OSWindow, EDisplayOutputFormat DisplayOutputFormat, EDisplayColorGamut DisplayColorGamut, bool bHDREnabled);
RENDERCORE_API void HDRRemoveCustomMetaData(void* OSWindow);
RENDERCORE_API FMatrix44f GamutToXYZMatrix(EDisplayColorGamut ColorGamut);
RENDERCORE_API FMatrix44f XYZToGamutMatrix(EDisplayColorGamut ColorGamut);
RENDERCORE_API void ConvertPixelDataToSCRGB(TArray<FLinearColor>& InOutRawPixels, EDisplayOutputFormat Pixelformat);

struct FDisplayInformation
{
	FIntRect DesktopCoordinates = FIntRect(0, 0, 0, 0);
	bool bHDRSupported = false;
	float MinimumLuminanceInNits = 0.f;
	float MaximumLuminanceInNits = 100.f;
	float MaximumFullFrameLuminanceInNits = 100.f;
	float HDRPaperWhiteInNits = 203.f;
	UE::Color::FColorSpace LimitingColorSpace = UE::Color::FColorSpace::GetSRGB();
};

using FDisplayInformationArray = TArray<FDisplayInformation>;

struct FACESTonemapParams
{
	// Default Values such as MinLuminanceLog10=-4, MidLuminance=15, MaxLuminance=1000, SceneColorMultiplier=1.5
	FACESTonemapParams()
	: ACESMinMaxData(2.50898620e-06f, 1e-4f, 692.651123f, 1000.0f)
	, ACESMidData(0.0822144598f, 4.80000019f, 1.55f, 1.0f)
	, ACESCoefsLow_0(-4.00000000f, -4.00000000f, -3.15737653f, -0.485249996f)
	, ACESCoefsHigh_0(-0.332863420f, 1.69534576f, 2.75812411f, 3.00000000f)
	, ACESCoefsLow_4(1.84773231f)
	, ACESCoefsHigh_4(3.0f)
	, ACESSceneColorMultiplier(1.5f)
	, ACESGamutCompression(0.0f)
	{

	}
	FVector4f ACESMinMaxData;
	FVector4f ACESMidData;
	FVector4f ACESCoefsLow_0;
	FVector4f ACESCoefsHigh_0;
	float ACESCoefsLow_4;
	float ACESCoefsHigh_4;
	float ACESSceneColorMultiplier;
	float ACESGamutCompression;
};

UE_DEPRECATED(all, "For internal use only.")
RENDERCORE_API FACESTonemapParams HDRGetACESTonemapParams();

template<class FACESTonemapParametersType>
inline void GetACESTonemapParameters(FACESTonemapParametersType& ACESTonemapParameters)
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FACESTonemapParams TonemapperParams = HDRGetACESTonemapParams();
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	
	ACESTonemapParameters.ACESMinMaxData = TonemapperParams.ACESMinMaxData;
	ACESTonemapParameters.ACESMidData = TonemapperParams.ACESMidData;
	ACESTonemapParameters.ACESCoefsLow_0 = TonemapperParams.ACESCoefsLow_0;
	ACESTonemapParameters.ACESCoefsHigh_0 = TonemapperParams.ACESCoefsHigh_0;
	ACESTonemapParameters.ACESCoefsLow_4 = TonemapperParams.ACESCoefsLow_4;
	ACESTonemapParameters.ACESCoefsHigh_4 = TonemapperParams.ACESCoefsHigh_4;
	ACESTonemapParameters.ACESSceneColorMultiplier = TonemapperParams.ACESSceneColorMultiplier;
	ACESTonemapParameters.ACESGamutCompression = FMath::Clamp(TonemapperParams.ACESGamutCompression, 0.0f, 1.0f);
}
