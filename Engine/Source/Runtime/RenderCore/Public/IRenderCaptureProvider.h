// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Features/IModularFeatures.h"

class FRHICommandListImmediate;
class FViewport;

class IRenderCaptureProvider : public IModularFeature
{
public:
	/**
	 * Get the feature name used for module resolution.
	 *
	 * @return	The feature name.
	 */
	static FName GetModularFeatureName()
	{
		static FName FeatureName = FName("RenderCaptureProvider");
		return FeatureName;
	}

	/**
	 * Checks to see if the specified feature is available.
	 * 
	 * @return	True if the feature is available right now and it is safe to call Get().
	 */
	static inline bool IsAvailable()
	{
		// This function is accessible from the render and RHI threads, so lock the list of modular features before calling GetModularFeature
		IModularFeatures::FScopedLockModularFeatureList ScopedLockModularFeatureList;
		return IModularFeatures::Get().IsModularFeatureAvailable(GetModularFeatureName());
	}

	/**
	 * Gets the first registered implementation of this feature. Will assert or crash
	 * if the specified feature is not available!  You should call IsAvailable() first!
	 *
	 * @return	The feature.
	 */
	static inline IRenderCaptureProvider& Get()
	{
		// This function is accessible from the render and RHI threads, so lock the list of modular features before calling GetModularFeature
		IModularFeatures::FScopedLockModularFeatureList ScopedLockModularFeatureList;
		return IModularFeatures::Get().GetModularFeature<IRenderCaptureProvider>(GetModularFeatureName());
	}

	/** Derived classes return false if they cannot support an RHI submission thread. */
	virtual bool CanSupportSubmissionThread() const { return true; }

	/**
	 * Flags to pass to capture API.
	 */
	enum ECaptureFlags
	{
		// Set to launch the capture viewing application. Whether this is supported depends on the underlying capture tool.
		ECaptureFlags_Launch = 1, 
	};

	/**
	 * Capture the next full frame of rendering information.
	 * Currently any capture details (number of frames etc.) must be set up by CVars exposed in the underlying capture tool.
	 * Call from main thread only.
	 * 
	 * @param	Viewport		The specific viewport to capture. (Optional).
	 * @param	bLaunch			
	 * @param	DestFileName	The destination file name for saving the capture. (Optional).
	 *
	 */
	virtual void CaptureFrame(FViewport* InViewport = nullptr, uint32 InFlags = 0, FString const& InDestFileName = FString()) = 0;

	/**
	 * Start capturing rendering information.
	 * Call from render thread only.
	 * 
	 * @param	RHICommandList	The command list to capture on.
	 * @param	DestFileName	The destination file name for saving the capture. (Optional).
	 */
	virtual void BeginCapture(FRHICommandListImmediate* InRHICommandList, uint32 InFlags = 0, FString const& InDestFileName = FString()) = 0;
	
	/**
	 * Stop capturing rendering information and save the captured data.
	 * Call from render thread only.
	 *
	 * @param	RHICommandList	The command list to capture on.
	 */
	virtual void EndCapture(FRHICommandListImmediate* InRHICommandList) = 0;
};
