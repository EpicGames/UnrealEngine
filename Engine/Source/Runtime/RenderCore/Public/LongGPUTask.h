// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HAL/Platform.h"

class FRHICommandListImmediate;

extern RENDERCORE_API void IssueScalableLongGPUTask(FRHICommandListImmediate& RHICmdList, int32 NumIteration = -1);
extern RENDERCORE_API void MeasureLongGPUTaskExecutionTime(FRHICommandListImmediate& RHICmdList);
