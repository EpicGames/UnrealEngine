// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GlobalShader.h"
#include "ShaderParameterStruct.h"

class FPointerTableBase;
class FRDGBuilder;
class FRDGTexture;

namespace UE {
	namespace Color {
		enum class EEncoding : uint8;
	}
}

/** MAX number of conversion operations. Reflects MAX in EMediaCaptureConversionOperation */
#define NUM_MEDIA_SHADERS_CONVERSION_OP 5


namespace MediaShaders
{
	enum class EToneMapMethod : uint8
	{
		None = 0,
		Hable = 1,
		SimpleReinhard = 2,

		MAX
	};

	/**
	 * Controls chroma interpolation in 4:2:2 YUV conversion shaders.
	 *
	 * 4:2:2 formats share one chroma pair per two luma samples, so naive decoding produces
	 * hard chroma steps at 2-pixel boundaries. `Enabled` requests a 50/50 blend toward the
	 * next chroma pair at odd pixels (co-sited left / BT.601/709). `Disabled` preserves the
	 * nearest-chroma behavior preferred by broadcast QC workflows that rely on pixel-exact
	 * chroma values for waveform monitors or test patterns.
	 *
	 * Can be globally force-disabled via the `r.Media.ChromaInterpolation` CVar.
	 */
	enum class EChromaInterpolation : uint8
	{
		Disabled = 0,
		Enabled = 1,
	};

	/** Color transform from YUV to Rec601 without range scaling. */
	RENDERCORE_API extern const FMatrix YuvToRgbRec601Unscaled;

	/** Color transform from YUV Video Range to Rec601 Full Range. */
	RENDERCORE_API extern const FMatrix YuvToRgbRec601Scaled;

	/** Color transform from YUV to Rec709 without range scaling. */
	RENDERCORE_API extern const FMatrix YuvToRgbRec709Unscaled;

	/** Color transform from YUV Video Range to Rec709 Full Range. */
	RENDERCORE_API extern const FMatrix YuvToRgbRec709Scaled;

	/** Color transform from YUV to Rec2020 without range scaling. */
	RENDERCORE_API extern const FMatrix YuvToRgbRec2020Unscaled;

	/** Color transform from YUV Video Range to Rec2020 Full Range. */
	RENDERCORE_API extern const FMatrix YuvToRgbRec2020Scaled;

	/** Color transform from YUV to sRGB (using rounded values from PS4 AvPlayer codec). */
	RENDERCORE_API extern const FMatrix YuvToSrgbPs4;

	/** Color transform from RGB to YUV (in Rec. 709 color space, including inversion of range scaling) */
	RENDERCORE_API extern const FMatrix RgbToYuvRec709Scaled;

	/** Color transform from RGB to YUV (in Rec. 2020 color space, including inversion of range scaling) */
	RENDERCORE_API extern const FMatrix RgbToYuvRec2020Scaled;

	/** YUV Offset for 8 bit conversion (Computed as 16/255, 128/255, 128/255) */
	RENDERCORE_API extern const FVector YUVOffset8bits;

	/** YUV Offset for 8 bit conversion (Computed as 0/255, 128/255, 128/255) */
	RENDERCORE_API extern const FVector YUVOffsetNoScale8bits;

	/** YUV Offset for 10 bit conversion (Computed as 64/1023, 512/1023, 512/1023) */
	RENDERCORE_API extern const FVector YUVOffset10bits;

	/** YUV Offset for 10 bit conversion (Computed as 0/1023, 512/1023, 512/1023) */
	RENDERCORE_API extern const FVector YUVOffsetNoScale10bits;

	/** YUV Offset for 16 bit conversion (Computed as 4096/65535, 32768/65535, 32768/65535) */
	RENDERCORE_API extern const FVector YUVOffset16bits;

	/** YUV Offset for 16 bit conversion (Computed as 0/65535, 32768/65535, 32768/65535) */
	RENDERCORE_API extern const FVector YUVOffsetNoScale16bits;

	/** YUV Offset for float conversion (Computed as 1/16, 1/2, 1/2) */
	RENDERCORE_API extern const FVector YUVOffsetFloat;

	/** YUV Offset for float conversion (Computed as 0, 1/2, 1/2) */
	RENDERCORE_API extern const FVector YUVOffsetNoScaleFloat;

	/** Combine color transform matrix with yuv offset in a single matrix */
	RENDERCORE_API FMatrix CombineColorTransformAndOffset(const FMatrix& InMatrix, const FVector& InYUVOffset);
}


/**
 * Stores media drawing vertices.
 */
struct FMediaElementVertex
{
	FVector4f Position;
	FVector2f TextureCoordinate;

	FMediaElementVertex() { }

	FMediaElementVertex(const FVector4f& InPosition, const FVector2f& InTextureCoordinate)
		: Position(InPosition)
		, TextureCoordinate(InTextureCoordinate)
	{ }
};

inline FBufferRHIRef CreateTempMediaVertexBuffer(FRHICommandListBase& RHICmdList, float ULeft = 0.0f, float URight = 1.0f, float VTop = 0.0f, float VBottom = 1.0f)
{
	const FRHIBufferCreateDesc CreateDesc =
		FRHIBufferCreateDesc::CreateVertex<FMediaElementVertex>(TEXT("TempMediaVertexBuffer"), 4)
		.AddUsage(EBufferUsageFlags::Volatile)
		.DetermineInitialState()
		.SetInitActionInitializer();

	TRHIBufferInitializer<FMediaElementVertex> Vertices = RHICmdList.CreateBufferInitializer(CreateDesc);

	Vertices[0].Position.Set(-1.0f, 1.0f, 1.0f, 1.0f); // Top Left
	Vertices[1].Position.Set(1.0f, 1.0f, 1.0f, 1.0f); // Top Right
	Vertices[2].Position.Set(-1.0f, -1.0f, 1.0f, 1.0f); // Bottom Left
	Vertices[3].Position.Set(1.0f, -1.0f, 1.0f, 1.0f); // Bottom Right

	Vertices[0].TextureCoordinate.Set(ULeft, VTop);
	Vertices[1].TextureCoordinate.Set(URight, VTop);
	Vertices[2].TextureCoordinate.Set(ULeft, VBottom);
	Vertices[3].TextureCoordinate.Set(URight, VBottom);

	return Vertices.Finalize();
}

/**
 * The simple element vertex declaration resource type.
 */
class FMediaVertexDeclaration : public FRenderResource
{
public:
	RENDERCORE_API FMediaVertexDeclaration();
	RENDERCORE_API virtual ~FMediaVertexDeclaration();

	RENDERCORE_API virtual void InitRHI(FRHICommandListBase& RHICmdList) override;
	RENDERCORE_API virtual void ReleaseRHI() override;

	FVertexDeclarationRHIRef VertexDeclarationRHI;
};

RENDERCORE_API extern TGlobalResource<FMediaVertexDeclaration> GMediaVertexDeclaration;


/**
 * Media vertex shader (shared by all media shaders).
 */
class FMediaShadersVS
	: public FGlobalShader
{
	DECLARE_EXPORTED_SHADER_TYPE(FMediaShadersVS, Global, RENDERCORE_API);

public:
	/** Default constructor. */
	FMediaShadersVS() { }

	/** Initialization constructor. */
	FMediaShadersVS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FGlobalShader(Initializer)
	{ }
};


/**
 * Pixel shader to convert an AYUV texture to RGBA.
 *
 * This shader expects a single texture consisting of a N x M array of pixels
 * in AYUV format. Each pixel is encoded as four consecutive unsigned chars
 * with the following layout: [V0 U0 Y0 A0][V1 U1 Y1 A1]..
 */
class FAYUVConvertPS
	: public FGlobalShader
{
	DECLARE_EXPORTED_SHADER_TYPE(FAYUVConvertPS, Global, RENDERCORE_API);

public:
	FAYUVConvertPS() { }

	FAYUVConvertPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FGlobalShader(Initializer)
	{ }

	RENDERCORE_API void SetParameters(FRHIBatchedShaderParameters& BatchedParameters, TRefCountPtr<FRHITexture> AYUVTexture, const FMatrix& ColorTransform, const FVector& YUVOffset, bool SrgbToLinear);
};


/**
 * Pixel shader to convert a Windows Bitmap texture.
 *
 * This shader expects a BMP frame packed into a single texture in PF_B8G8R8A8 format.
 */
class FBMPConvertPS
	: public FGlobalShader
{
	DECLARE_EXPORTED_SHADER_TYPE(FBMPConvertPS, Global, RENDERCORE_API);

public:
	FBMPConvertPS() { }

	FBMPConvertPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FGlobalShader(Initializer)
	{ }

	RENDERCORE_API void SetParameters(FRHIBatchedShaderParameters& BatchedParameters, TRefCountPtr<FRHITexture> BMPTexture, const FIntPoint& OutputDimensions, bool SrgbToLinear);
};


/**
 * Pixel shader to convert a NV12 frame to RGBA. (from NV12 texture)
 *
 * This shader expects an NV12 frame packed into a single texture in PF_G8 format.
 *
 * @see http://www.fourcc.org/yuv.php#NV12
 */
class FNV12ConvertPS
	: public FGlobalShader
{
	DECLARE_EXPORTED_SHADER_TYPE(FNV12ConvertPS, Global, RENDERCORE_API);

public:
	FNV12ConvertPS() { }

	FNV12ConvertPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FGlobalShader(Initializer)
	{ }

	RENDERCORE_API void SetParameters(FRHIBatchedShaderParameters& BatchedParameters, const FIntPoint & TexDim, FShaderResourceViewRHIRef SRV_Y, FShaderResourceViewRHIRef SRV_UV, const FIntPoint& OutputDimensions, const FMatrix44f& ColorTransform, UE::Color::EEncoding Encoding, const FMatrix44f& CSTransform, bool bSwapChroma, MediaShaders::EToneMapMethod ToneMapMethod = MediaShaders::EToneMapMethod::None);
};


/**
 * Pixel shader to convert a NV12 frame to RGBA (NV12 data; texture viewed as G8)
 *
 * This shader expects an NV12 frame packed into a single texture in PF_G8 format.
 *
 * @see http://www.fourcc.org/yuv.php#NV12
 */
class FNV12ConvertAsBytesPS
	: public FGlobalShader
{
	DECLARE_EXPORTED_SHADER_TYPE(FNV12ConvertPS, Global, RENDERCORE_API);

public:

	FNV12ConvertAsBytesPS() { }

	FNV12ConvertAsBytesPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FGlobalShader(Initializer)
	{ }

	RENDERCORE_API void SetParameters(FRHIBatchedShaderParameters& BatchedParameters, TRefCountPtr<FRHITexture> NV12Texture, const FIntPoint& OutputDimensions, const FMatrix44f& ColorTransform, UE::Color::EEncoding Encoding, const FMatrix44f& CSTransform, bool bSwapChroma, MediaShaders::EToneMapMethod ToneMapMethod = MediaShaders::EToneMapMethod::None);
};


/**
 * Pixel shader to convert a NV21 frame to RGBA.
 *
 * This shader expects an NV21 frame packed into a single texture in PF_G8 format.
 *
 * @see http://www.fourcc.org/yuv.php#NV21
 */
class FNV21ConvertPS
	: public FGlobalShader
{
	DECLARE_EXPORTED_SHADER_TYPE(FNV21ConvertPS, Global, RENDERCORE_API);

public:
	FNV21ConvertPS() { }

	FNV21ConvertPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FGlobalShader(Initializer)
	{ }

	RENDERCORE_API void SetParameters(FRHIBatchedShaderParameters& BatchedParameters, TRefCountPtr<FRHITexture> NV21Texture, const FIntPoint& OutputDimensions, const FMatrix& ColorTransform, const FVector& YUVOffset, bool SrgbToLinear);
};

/**
 * Pixel shader to convert a P010 frame to RGBA.
 */
class FP010ConvertPS
	: public FGlobalShader
{
	DECLARE_EXPORTED_SHADER_TYPE(FP010ConvertPS, Global, RENDERCORE_API);

public:

	FP010ConvertPS() { }

	FP010ConvertPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FGlobalShader(Initializer)
	{ }

	RENDERCORE_API void SetParameters(FRHIBatchedShaderParameters& BatchedParameters, const FIntPoint& TexDim, FShaderResourceViewRHIRef SRV_Y, FShaderResourceViewRHIRef SRV_UV, const FIntPoint& OutputDimensions, const FMatrix44f& ColorTransform, const FMatrix44f& CSTransform, UE::Color::EEncoding Encoding, MediaShaders::EToneMapMethod ToneMapMethod = MediaShaders::EToneMapMethod::None);
};

/**
 * Pixel shader to convert a P010 frame inside a G16 texture to RGBA.
 */
class FP010ConvertAsUINT16sPS
	: public FGlobalShader
{
	DECLARE_EXPORTED_SHADER_TYPE(FP010ConvertAsUINT16sPS, Global, RENDERCORE_API);

public:

	FP010ConvertAsUINT16sPS() { }

	FP010ConvertAsUINT16sPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FGlobalShader(Initializer)
	{ }

	RENDERCORE_API void SetParameters(FRHIBatchedShaderParameters& BatchedParameters, const FIntPoint& TexDim, TRefCountPtr<FRHITexture> NV12Texture, const FIntPoint& OutputDimensions, const FMatrix44f& ColorTransform, const FMatrix44f& CSTransform, UE::Color::EEncoding Encoding, MediaShaders::EToneMapMethod ToneMapMethod = MediaShaders::EToneMapMethod::None);
};

/**
 * Pixel shader to convert a P010 frame stored as 3 plane RGB2101010 to RGBA.
 */
class FP010_2101010ConvertPS
	: public FGlobalShader
{
	DECLARE_EXPORTED_SHADER_TYPE(FP010_2101010ConvertPS, Global, RENDERCORE_API);

public:

	FP010_2101010ConvertPS() { }

	FP010_2101010ConvertPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FGlobalShader(Initializer)
	{ }

	RENDERCORE_API void SetParameters(FRHIBatchedShaderParameters& BatchedParameters, const FIntPoint& TexDim, FShaderResourceViewRHIRef SRV_Y, FShaderResourceViewRHIRef SRV_U, FShaderResourceViewRHIRef SRV_V, const FIntPoint& OutputDimensions, const FMatrix44f& ColorTransform, const FMatrix44f& CSTransform, UE::Color::EEncoding Encoding, MediaShaders::EToneMapMethod ToneMapMethod = MediaShaders::EToneMapMethod::None);
};

/**
 * Pixel shader to convert an RGB texture.
 *
 * This shader expects an RGB or RGBA frame packed into a single texture
 * in PF_B8G8R8A8, PF_A2B10G10R10 or PF_FloatRGB format.
 */
class FRGBConvertPS
	: public FGlobalShader
{
	DECLARE_EXPORTED_SHADER_TYPE(FRGBConvertPS, Global, RENDERCORE_API);

public:
	FRGBConvertPS() { }

	FRGBConvertPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FGlobalShader(Initializer)
	{ }


	RENDERCORE_API void SetParameters(FRHIBatchedShaderParameters& BatchedParameters, TRefCountPtr<FRHITexture> RGBTexture, const FIntPoint& OutputDimensions, UE::Color::EEncoding Encoding, const FMatrix44f& CSTransform, MediaShaders::EToneMapMethod ToneMapMethod = MediaShaders::EToneMapMethod::None);
};

/**
 * Pixel convert to resize an YCoCg texture.
 *
 * This shader expects an YCoCg frame packed into a single texture
 * in PF_DXT5 format.
 */
class FYCoCgConvertPS
	: public FGlobalShader
{
	DECLARE_EXPORTED_SHADER_TYPE(FYCoCgConvertPS, Global, RENDERCORE_API);

public:
	FYCoCgConvertPS() { }

	FYCoCgConvertPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FGlobalShader(Initializer)
	{ }


	RENDERCORE_API void SetParameters(FRHIBatchedShaderParameters& BatchedParameters, TRefCountPtr<FRHITexture> YCoCgTexture, const FIntPoint& OutputDimensions, UE::Color::EEncoding Encoding, const FMatrix44f& CSTransform, MediaShaders::EToneMapMethod ToneMapMethod = MediaShaders::EToneMapMethod::None);
};


/**
 * Pixel shader to convert a PS4 YCbCr texture to RGBA.
 *
 * This shader expects a separate chroma and luma plane stored in two textures
 * in PF_B8G8R8A8 format. The full-size luma plane contains the Y-components.
 * The half-size chroma plane contains the UV components in the following
 * memory layout: [U0, V0][U1, V1]
 *
 */
class FYCbCrConvertPS
	: public FGlobalShader
{
	DECLARE_EXPORTED_SHADER_TYPE(FYCbCrConvertPS, Global, RENDERCORE_API);

public:
	FYCbCrConvertPS() { }

	FYCbCrConvertPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FGlobalShader(Initializer)
	{ }

	RENDERCORE_API void SetParameters(FRHIBatchedShaderParameters& BatchedParameters, TRefCountPtr<FRHITexture> LumaTexture, TRefCountPtr<FRHITexture> CbCrTexture, const FMatrix& ColorTransform, const FVector& YUVOffset, bool SrgbToLinear);
};

class FYCbCrConvertPS_4x4Matrix : public FYCbCrConvertPS
{
    DECLARE_EXPORTED_SHADER_TYPE(FYCbCrConvertPS_4x4Matrix, Global, RENDERCORE_API);

public:
    FYCbCrConvertPS_4x4Matrix() { }

    FYCbCrConvertPS_4x4Matrix(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
    : FYCbCrConvertPS(Initializer)
    { }
};


/**
 * Pixel shader to convert a UYVY (Y422, UYNV) frame to RGBA.
 *
 * This shader expects a UYVY frame packed into a single texture in PF_B8G8R8A8
 * format with the following memory layout: [U0, Y0, V1, Y1][U1, Y2, V1, Y3]..
 *
 * @see http://www.fourcc.org/yuv.php#UYVY
 */
class FUYVYConvertPS
	: public FGlobalShader
{
	DECLARE_EXPORTED_SHADER_TYPE(FUYVYConvertPS, Global, RENDERCORE_API);

public:
	FUYVYConvertPS() { }

	FUYVYConvertPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FGlobalShader(Initializer)
	{ }

	RENDERCORE_API void SetParameters(FRHIBatchedShaderParameters& BatchedParameters, TRefCountPtr<FRHITexture> UYVYTexture, const FMatrix& ColorTransform, const FVector& YUVOffset, bool SrgbToLinear);
};


/**
 * Pixel shader to convert Y, U, and V planes to RGBA.
 *
 * This shader expects three textures in PF_G8 format,
 * one for each plane of Y, U, and V components.
 */
class FYUVConvertPS
	: public FGlobalShader
{
	DECLARE_EXPORTED_SHADER_TYPE(FYUVConvertPS, Global, RENDERCORE_API);

public:
	FYUVConvertPS() { }

	FYUVConvertPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FGlobalShader(Initializer)
	{ }


	RENDERCORE_API void SetParameters(FRHIBatchedShaderParameters& BatchedParameters, TRefCountPtr<FRHITexture> YTexture, TRefCountPtr<FRHITexture> UTexture, TRefCountPtr<FRHITexture> VTexture, const FIntPoint& OutputDimensions, const FMatrix& ColorTransform, const FVector& YUVOffset, bool SrgbToLinear);
};


/**
 * Pixel shader to convert Y, U, and V planes to RGBA.
 *
 * This shader expects textures in PF_G8 or PF_G16 format.
 */
class FYUVScaledConvertPS
	: public FGlobalShader
{
	DECLARE_EXPORTED_SHADER_TYPE(FYUVScaledConvertPS, Global, RENDERCORE_API);

public:
	FYUVScaledConvertPS() { }

	FYUVScaledConvertPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FGlobalShader(Initializer)
	{ }


	RENDERCORE_API void SetParameters(FRHIBatchedShaderParameters& BatchedParameters, TRefCountPtr<FRHITexture> YTexture, TRefCountPtr<FRHITexture> UTexture, TRefCountPtr<FRHITexture> VTexture, const FIntPoint& OutputDimensions, const FMatrix& ColorTransform, const FVector& YUVOffset, bool SrgbToLinear, float YUVScale);
};


/**
 * Pixel shader to convert YUV v216/2vuy/yuv2 to RGB
 *
 * This shader expects a single texture in PF_R16G16B16A16 or PF_R8G8B8A8 format.
 */
class FYUVv216ConvertPS
	: public FGlobalShader
{
	DECLARE_EXPORTED_SHADER_TYPE(FYUVv216ConvertPS, Global, RENDERCORE_API);

public:
	FYUVv216ConvertPS() { }

	FYUVv216ConvertPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FGlobalShader(Initializer)
	{ }

	RENDERCORE_API void SetParameters(FRHIBatchedShaderParameters& BatchedParameters, TRefCountPtr<FRHITexture> YUVTexture, const FIntPoint& OutputDimensions, const FMatrix44f& ColorTransform, UE::Color::EEncoding Encoding, const FMatrix44f& CSTransform, bool bIsCbY0CrY1, bool bIsARGBFmt, bool bSwapChroma, MediaShaders::EToneMapMethod ToneMapMethod = MediaShaders::EToneMapMethod::None, bool bUseBilinearSamplingForChroma = false);
};


/**
 * Pixel shader to convert YUV v210 to RGB
 *
 * This shader expects a single texture in PF_R32G32B32A32_UINT format.
 * (from which it decodes a R10G10B10A2 encoded data stream, in which RGB components represent the v210 data)
 */
class FYUVv210ConvertPS
	: public FGlobalShader
{
	DECLARE_EXPORTED_SHADER_TYPE(FYUVv210ConvertPS, Global, RENDERCORE_API);

public:
	FYUVv210ConvertPS() { }

	FYUVv210ConvertPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FGlobalShader(Initializer)
	{ }

	RENDERCORE_API void SetParameters(FRHIBatchedShaderParameters& BatchedParameters, TRefCountPtr<FRHITexture> YUVTexture, const FIntPoint& OutputDimensions, const FMatrix44f& ColorTransform, UE::Color::EEncoding Encoding, const FMatrix44f& CSTransform, bool bIsCbY0CrY1, MediaShaders::EToneMapMethod ToneMapMethod = MediaShaders::EToneMapMethod::None, MediaShaders::EChromaInterpolation ChromaInterpolation = MediaShaders::EChromaInterpolation::Enabled);
};


/**
 * Pixel shader to convert YUV Y416 to RGB
 *
 * This shader expects a single texture in PF_A16B16G16R16 format.
 */
class FYUVY416ConvertPS
	: public FGlobalShader
{
	DECLARE_EXPORTED_SHADER_TYPE(FYUVY416ConvertPS, Global, RENDERCORE_API);

public:
	FYUVY416ConvertPS() { }

	FYUVY416ConvertPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FGlobalShader(Initializer)
	{ }

	RENDERCORE_API void SetParameters(FRHIBatchedShaderParameters& BatchedParameters, FShaderResourceViewRHIRef SRV_Y, const FMatrix44f& ColorTransform, UE::Color::EEncoding Encoding, const FMatrix44f& CSTransform, bool bIsARGBFmt, MediaShaders::EToneMapMethod ToneMapMethod = MediaShaders::EToneMapMethod::None);
};

/**
 * Pixel shader to convert a YUY2 frame to RGBA.
 *
 * This shader expects an YUY2 frame packed into a single texture in PF_B8G8R8A8
 * format with the following memory layout: [Y0, U0, Y1, V0][Y2, U1, Y3, V1]...
 *
 * @see http://www.fourcc.org/yuv.php#YUY2
 */
class FYUY2ConvertPS
	: public FGlobalShader
{
	DECLARE_EXPORTED_SHADER_TYPE(FYUY2ConvertPS, Global, RENDERCORE_API);

public:
	FYUY2ConvertPS() { }

	FYUY2ConvertPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FGlobalShader(Initializer)
	{ }

	RENDERCORE_API void SetParameters(FRHIBatchedShaderParameters& BatchedParameters, TRefCountPtr<FRHITexture> YUY2Texture, const FIntPoint& OutputDimensions, const FMatrix& ColorTransform, const FVector& YUVOffset, bool SrgbToLinear, MediaShaders::EToneMapMethod ToneMapMethod = MediaShaders::EToneMapMethod::None);
};

/**
 * Pixel shader to convert ARGB16, big endian to RGB
 *
 * This shader expects a single texture in PF_R16G16B16A16_UINT format.
 */
class FARGB16BigConvertPS
	: public FGlobalShader
{
	DECLARE_EXPORTED_SHADER_TYPE(FARGB16BigConvertPS, Global, RENDERCORE_API);

public:
	FARGB16BigConvertPS() { }

	FARGB16BigConvertPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FGlobalShader(Initializer)
	{ }

	RENDERCORE_API void SetParameters(FRHIBatchedShaderParameters& BatchedParameters, FShaderResourceViewRHIRef SRV, const FIntPoint& OutputDimensions, UE::Color::EEncoding Encoding, const FMatrix44f& CSTransform, MediaShaders::EToneMapMethod ToneMapMethod = MediaShaders::EToneMapMethod::None);
};


/**
 * Pixel shader to convert a YVYU frame to RGBA.
 *
 * This shader expects a YVYU frame packed into a single texture in PF_B8G8R8A8
 * format with the following memory layout: [Y0, V0, Y1, U0][Y2, V1, Y3, U1]..
 *
 * @see http://www.fourcc.org/yuv.php#YVYU
 */
class FYVYUConvertPS
	: public FGlobalShader
{
	DECLARE_EXPORTED_SHADER_TYPE(FYVYUConvertPS, Global, RENDERCORE_API);

public:
	FYVYUConvertPS() { }

	FYVYUConvertPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FGlobalShader(Initializer)
	{ }

	RENDERCORE_API void SetParameters(FRHIBatchedShaderParameters& BatchedParameters, TRefCountPtr<FRHITexture> YVYUTexture, const FMatrix& ColorTransform, const FVector& YUVOffset, bool SrgbToLinear);
};


/**
 * Pixel shader to convert RGB 8 bits to Y 8 bits
 *
 * This shader expects a single texture in PF_B8G8R8A8 format.
 */
class FRGB8toY8ConvertPS
	: public FGlobalShader
{
	DECLARE_EXPORTED_SHADER_TYPE(FRGB8toY8ConvertPS, Global, RENDERCORE_API);

public:
	FRGB8toY8ConvertPS() { }

	FRGB8toY8ConvertPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FGlobalShader(Initializer)
	{ }

	RENDERCORE_API void SetParameters(FRHIBatchedShaderParameters& BatchedParameters, TRefCountPtr<FRHITexture> RGBATexture, const FVector4f& ColorTransform, bool LinearToSrgb);
};


/**
 * Pixel shader to read from TextureExternal source
 */
class FReadTextureExternalPS
	: public FGlobalShader
{
	DECLARE_EXPORTED_SHADER_TYPE(FReadTextureExternalPS, Global, RENDERCORE_API);

public:
	FReadTextureExternalPS() { }

	FReadTextureExternalPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FGlobalShader(Initializer)
	{
		InTexture.Bind(Initializer.ParameterMap, TEXT("InExternalTexture"), SPF_Mandatory);
	}

	RENDERCORE_API void SetParameters(FRHIBatchedShaderParameters& BatchedParameters, FTextureRHIRef TextureExt, FSamplerStateRHIRef SamplerState, const FLinearColor & ScaleRotation, const FLinearColor & Offset);

	LAYOUT_FIELD(FShaderResourceParameter, InTexture);
};


/** Struct of common parameters used in media capture shaders to do RGB to YUV conversions */
BEGIN_SHADER_PARAMETER_STRUCT(FRGBToYUVConversion, RENDERCORE_API)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, InputTexture)
	SHADER_PARAMETER_SAMPLER(SamplerState, InputSampler)
	SHADER_PARAMETER(FMatrix44f, ColorTransform)
	SHADER_PARAMETER(uint32, DoLinearToSrgb)
	SHADER_PARAMETER(float, OnePixelDeltaX)
END_SHADER_PARAMETER_STRUCT()

/**
 * Pixel shader to convert RGB 8 bits to UYVY 8 bits
 */
class FRGB8toUYVY8ConvertPS : public FGlobalShader
{
public:
	DECLARE_EXPORTED_GLOBAL_SHADER(FRGB8toUYVY8ConvertPS, RENDERCORE_API);

	SHADER_USE_PARAMETER_STRUCT(FRGB8toUYVY8ConvertPS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FRGBToYUVConversion, RGBToYUVConversion)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

public:
	/** Allocates and setup shader parameter in the incoming graph builder */
	RENDERCORE_API FRGB8toUYVY8ConvertPS::FParameters* AllocateAndSetParameters(FRDGBuilder& GraphBuilder, FRDGTextureRef RGBATexture, const FMatrix& ColorTransform, const FVector& YUVOffset, bool bDoLinearToSrgb, FRDGTextureRef OutputTexture);
};


/**
 * Pixel shader to convert RGB 10 bits to YUV v210
 */
class FRGB10toYUVv210ConvertPS : public FGlobalShader
{
public:
	DECLARE_EXPORTED_GLOBAL_SHADER(FRGB10toYUVv210ConvertPS, RENDERCORE_API);

	SHADER_USE_PARAMETER_STRUCT(FRGB10toYUVv210ConvertPS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FRGBToYUVConversion, RGBToYUVConversion)
		SHADER_PARAMETER(float, PaddingScale)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

public:
	/** Allocates and setup shader parameter in the incoming graph builder */
	RENDERCORE_API FRGB10toYUVv210ConvertPS::FParameters* AllocateAndSetParameters(FRDGBuilder& GraphBuilder, FRDGTextureRef RGBATexture, const FMatrix& ColorTransform, const FVector& YUVOffset, bool bDoLinearToSrgb, FRDGTextureRef OutputTexture);
};

/**
 * Pixel shader to swizzle R G B A components, set alpha to 1 or inverts alpha
 *
 * General conversion shader that is only used to swizzle shaders that do not require any color conversion. RGB to BGR, RGB10A2 to RGBA8 etc
 */
class FModifyAlphaSwizzleRgbaPS	: public FGlobalShader
{
public:
	DECLARE_EXPORTED_GLOBAL_SHADER(FModifyAlphaSwizzleRgbaPS, RENDERCORE_API);

	SHADER_USE_PARAMETER_STRUCT(FModifyAlphaSwizzleRgbaPS, FGlobalShader);

	class FConversionOp : SHADER_PERMUTATION_INT("CONVERSION_OP", NUM_MEDIA_SHADERS_CONVERSION_OP);
	using FPermutationDomain = TShaderPermutationDomain<FConversionOp>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, InputTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, InputSampler)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

public:
	/** Allocates and setup shader parameter in the incoming graph builder */
	RENDERCORE_API FModifyAlphaSwizzleRgbaPS::FParameters* AllocateAndSetParameters(FRDGBuilder& GraphBuilder, FRDGTextureRef RGBATexture, FRDGTextureRef OutputTexture);
};

/**
 * Pixel shader to convert a VYU frame to RGBA.
 *
 * This shader expects a VYU frame packed into a single texture in PF_R8G8B8A8
 * format with the following memory layout: [V0, Y0, U0, unused][V1, Y1, U1, unused]..
 *
 * @see https://registry.khronos.org/vulkan/specs/1.3/html/vkspec.html#VkSamplerYcbcrModelConversion
 */
class FVYUConvertPS
	: public FGlobalShader
{
	DECLARE_EXPORTED_SHADER_TYPE(FVYUConvertPS, Global, RENDERCORE_API);

public:
	FVYUConvertPS() { }

	FVYUConvertPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FGlobalShader(Initializer)
	{ }

	RENDERCORE_API void SetParameters(FRHIBatchedShaderParameters& BatchedParameters, TRefCountPtr<FRHITexture> VYUTexture, const FIntPoint& OutputDimensions, const FMatrix44f& ColorTransform, UE::Color::EEncoding Encoding, const FMatrix44f& CSTransform,
		MediaShaders::EToneMapMethod ToneMapMethod = MediaShaders::EToneMapMethod::None,
		const FLinearColor& UVRotation = FLinearColor(1.0f, 0.0f, 0.0f, 1.0f),
		const FLinearColor& UVRotOffset = FLinearColor(0.0f, 0.0f, 0.0f, 0.0f));
};
