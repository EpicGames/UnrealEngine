// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GlobalShader.h"
#include "HAL/Platform.h"
#include "RHICommandList.h"
#include "RenderResource.h"
#include "Serialization/MemoryLayout.h"
#include "Shader.h"
#include "ShaderParameterUtils.h"
#include "ShaderParameters.h"

class FPointerTableBase;
class FRHISamplerState;
class FRHITexture;
class FTexture;

class UE_DEPRECATED(5.8, "Oculus shaders no longer supported") FOculusVertexShader : public FGlobalShader
{
	DECLARE_EXPORTED_SHADER_TYPE(FOculusVertexShader, Global, RENDERCORE_API);
public:

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) { return true; }

	FOculusVertexShader(const ShaderMetaType::CompiledShaderInitializerType& Initializer) :
		FGlobalShader(Initializer)
	{}
	FOculusVertexShader() {}
};

class FOculusWhiteShader : public FGlobalShader
{
	DECLARE_EXPORTED_SHADER_TYPE(FOculusWhiteShader, Global, RENDERCORE_API);
public:

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) { return true; }

	FOculusWhiteShader(const ShaderMetaType::CompiledShaderInitializerType& Initializer) :
		FGlobalShader(Initializer)
	{
	}
	
	FOculusWhiteShader() {}
};

class UE_DEPRECATED(5.8, "Oculus shaders no longer supported") FOculusBlackShader : public FGlobalShader
{
	DECLARE_EXPORTED_SHADER_TYPE(FOculusBlackShader, Global, RENDERCORE_API);
public:

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) { return true; }

	FOculusBlackShader(const ShaderMetaType::CompiledShaderInitializerType& Initializer) :
		FGlobalShader(Initializer)
	{
	}

	FOculusBlackShader() {}
};

class UE_DEPRECATED(5.8, "Oculus shaders no longer supported") FOculusAlphaInverseShader : public FGlobalShader
{
	DECLARE_EXPORTED_SHADER_TYPE(FOculusAlphaInverseShader, Global, RENDERCORE_API);
public:

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) { return true; }

	FOculusAlphaInverseShader(const ShaderMetaType::CompiledShaderInitializerType& Initializer) :
		FGlobalShader(Initializer)
	{
		InTexture.Bind(Initializer.ParameterMap, TEXT("InTexture"), SPF_Mandatory);
		InTextureSampler.Bind(Initializer.ParameterMap, TEXT("InTextureSampler"));
	}
	FOculusAlphaInverseShader() {}

	void SetParameters(FRHIBatchedShaderParameters& BatchedParameters, const FTexture* Texture)
	{
		SetTextureParameter(BatchedParameters, InTexture, InTextureSampler, Texture);
	}

	void SetParameters(FRHIBatchedShaderParameters& BatchedParameters, FRHISamplerState* SamplerStateRHI, FRHITexture* TextureRHI)
	{
		SetTextureParameter(BatchedParameters, InTexture, InTextureSampler, SamplerStateRHI, TextureRHI);
	}

private:
	LAYOUT_FIELD(FShaderResourceParameter, InTexture);
	LAYOUT_FIELD(FShaderResourceParameter, InTextureSampler);
};


/**
* A pixel shader for rendering a textured screen element.
*/

class UE_DEPRECATED(5.8, "Use FOculusOpenXRCubemapPS() instead.") FOculusCubemapPS : public FGlobalShader
{
	DECLARE_EXPORTED_SHADER_TYPE(FOculusCubemapPS, Global, RENDERCORE_API);
public:

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) { return true; }

	FOculusCubemapPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer) :
		FGlobalShader(Initializer)
	{
		InTexture.Bind(Initializer.ParameterMap, TEXT("InTextureCube"), SPF_Mandatory);
		InTextureSampler.Bind(Initializer.ParameterMap, TEXT("InTextureSampler"));
		InFaceIndexParameter.Bind(Initializer.ParameterMap, TEXT("CubeFaceIndex"));
	}
	FOculusCubemapPS() {}

	void SetParameters(FRHIBatchedShaderParameters& BatchedParameters, const FTexture* Texture, int FaceIndex)
	{
		SetTextureParameter(BatchedParameters, InTexture, InTextureSampler, Texture);
		SetShaderValue(BatchedParameters, InFaceIndexParameter, FaceIndex);
	}

	void SetParameters(FRHIBatchedShaderParameters& BatchedParameters, FRHISamplerState* SamplerStateRHI, FRHITexture* TextureRHI, int FaceIndex)
	{
		SetTextureParameter(BatchedParameters, InTexture, InTextureSampler, SamplerStateRHI, TextureRHI);
		SetShaderValue(BatchedParameters, InFaceIndexParameter, FaceIndex);
	}

private:
	LAYOUT_FIELD(FShaderResourceParameter, InTexture);
	LAYOUT_FIELD(FShaderResourceParameter, InTextureSampler);
	LAYOUT_FIELD(FShaderParameter, InFaceIndexParameter);
};