// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "GlobalShader.h"
#include "HAL/Platform.h"
#include "Math/Color.h"
#include "Misc/AssertionMacros.h"
#include "PixelFormat.h"
#include "RHICommandList.h"
#include "RHIDefinitions.h"
#include "Serialization/MemoryLayout.h"
#include "Shader.h"
#include "ShaderCore.h"
#include "ShaderParameterMacros.h"
#include "ShaderParameterStruct.h"
#include "ShaderParameterUtils.h"
#include "ShaderParameters.h"
#include "ShaderPermutation.h"

class FPointerTableBase;

/**
 * Vertex shader for rendering a single, constant color.
 */
template<bool bUsingNDCPositions=true, bool bUsingVertexLayers=false>
class TOneColorVS : public FGlobalShader
{
	DECLARE_EXPORTED_SHADER_TYPE(TOneColorVS, Global, RENDERCORE_API);

	/** Default constructor. */
	TOneColorVS() {}

public:

	TOneColorVS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FGlobalShader(Initializer)
	{
		DepthParameter.Bind(Initializer.ParameterMap, TEXT("InputDepth"), SPF_Mandatory);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("USING_NDC_POSITIONS"), (uint32)(bUsingNDCPositions ? 1 : 0));
		OutEnvironment.SetDefine(TEXT("USING_LAYERS"), (uint32)(bUsingVertexLayers ? 1 : 0));
	}

	void SetParameters(FRHIBatchedShaderParameters& BatchedParameters, float Depth)
	{
		SetShaderValue(BatchedParameters, DepthParameter, Depth);
	}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return true;
	}
	
	static const TCHAR* GetSourceFilename()
	{
		return TEXT("/Engine/Private/OneColorShader.usf");
	}
	
	static const TCHAR* GetFunctionName()
	{
		return TEXT("MainVertexShader");
	}

private:
	LAYOUT_FIELD(FShaderParameter, DepthParameter);
};

/**
 * Pixel shader for rendering a single, constant color.
 */
class FOneColorPS : public FGlobalShader
{
	DECLARE_EXPORTED_GLOBAL_SHADER(FOneColorPS, RENDERCORE_API);
public:
	SHADER_USE_PARAMETER_STRUCT(FOneColorPS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, RENDERCORE_API)
		SHADER_PARAMETER_ARRAY(FLinearColor, DrawColorMRT, [MaxSimultaneousRenderTargets])
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

	RENDERCORE_API void FillParameters(FParameters& Parameters, const FLinearColor* Colors, int32 NumColors);
};

/**
 * Pixel shader for rendering a single, constant color to MRTs.
 */
class TOneColorPixelShaderMRT : public FOneColorPS
{
	DECLARE_EXPORTED_GLOBAL_SHADER(TOneColorPixelShaderMRT, RENDERCORE_API);
public:
	class TOneColorPixelShader128bitRT : SHADER_PERMUTATION_BOOL("b128BITRENDERTARGET");
	class TOneColorPixelShaderNumOutputs : SHADER_PERMUTATION_RANGE_INT("NUM_OUTPUTS", 1, 8);
	class TOneColorPixelNumUintOutputs : SHADER_PERMUTATION_RANGE_INT("NUM_UINT_OUTPUTS", 0, 8);
	using FPermutationDomain = TShaderPermutationDomain<TOneColorPixelShaderNumOutputs, TOneColorPixelShader128bitRT, TOneColorPixelNumUintOutputs>;

	TOneColorPixelShaderMRT();
	TOneColorPixelShaderMRT(const ShaderMetaType::CompiledShaderInitializerType& Initializer);

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters);
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment);
};

/**
 * Compute shader for writing values
 */
class FFillTextureCS : public FGlobalShader
{
	DECLARE_EXPORTED_GLOBAL_SHADER(FFillTextureCS, RENDERCORE_API);
public:
	FFillTextureCS();
	FFillTextureCS(const ShaderMetaType::CompiledShaderInitializerType& Initializer);

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters);

	LAYOUT_FIELD(FShaderParameter, FillValue);
	LAYOUT_FIELD(FShaderParameter, Params0);	// Texture Width,Height (.xy); Use Exclude Rect 1 : 0 (.z)
	LAYOUT_FIELD(FShaderParameter, Params1);	// Include X0,Y0 (.xy) - X1,Y1 (.zw)
	LAYOUT_FIELD(FShaderParameter, Params2);	// ExcludeRect X0,Y0 (.xy) - X1,Y1 (.zw)
	LAYOUT_FIELD(FShaderResourceParameter, FillTexture);
};

class FLongGPUTaskPS : public FGlobalShader
{
	DECLARE_EXPORTED_GLOBAL_SHADER(FLongGPUTaskPS, RENDERCORE_API);
public:
	FLongGPUTaskPS();
	FLongGPUTaskPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer);
};

