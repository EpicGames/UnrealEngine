// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "Serialization/MemoryLayout.h"

// LWC_TODO: Packed normals need proper double vector support.

/** A normal vector, quantized and packed into 32-bits. */
struct FPackedNormal
{
	union
	{
		struct
		{
			int8	X, Y, Z, W;

		};
		uint32		Packed;
	}				Vector;

	// Constructors.

	FPackedNormal() { Vector.Packed = 0; }
	FPackedNormal(const FVector3f& InVector) { *this = InVector; }
	FPackedNormal(const FVector3d& InVector) { *this = InVector; }
	FPackedNormal(const FVector4f& InVector) { *this = InVector; }
	FPackedNormal(const FVector4d& InVector) { *this = InVector; }

	// Conversion operators.

	void operator=(const FVector3f& InVector);
	void operator=(const FVector3d& InVector);
	void operator=(const FVector4f& InVector);
	void operator=(const FVector4d& InVector);
	VectorRegister4Float GetVectorRegister() const;

	FVector ToFVector() const;
	FVector3f ToFVector3f() const;
	FVector4 ToFVector4() const;
	FVector4f ToFVector4f() const;

	// Set functions.
	void Set(const FVector& InVector) { *this = InVector; }

	// Equality operator.

	bool operator==(const FPackedNormal& B) const;
	bool operator!=(const FPackedNormal& B) const;

	// Serializer.

	FString ToString() const
	{
		return FString::Printf(TEXT("X=%d Y=%d Z=%d W=%d"), Vector.X, Vector.Y, Vector.Z, Vector.W);
	}

	friend RENDERCORE_API FArchive& operator<<(FArchive& Ar, FPackedNormal& N);
};

DECLARE_INTRINSIC_TYPE_LAYOUT(FPackedNormal);

class FDeprecatedSerializedPackedNormal
{
	union
	{
		struct
		{
			uint8 X, Y, Z, W;
		};
		uint32		Packed;
	} Vector;

public:
	operator FVector3f() const
	{
		// Rescale [0..255] range to [-1..1]
		VectorRegister4Float VectorToUnpack = VectorLoadByte4(this);
		VectorToUnpack = VectorMultiplyAdd(VectorToUnpack, VectorSetFloat1(1.0f / 127.5f), VectorSetFloat1(-1.0f));

		FVector3f UnpackedVector;
		VectorStoreFloat3(VectorToUnpack, &UnpackedVector);

		return UnpackedVector;
	}

	operator FVector3d() const
	{
		FVector3f AsFloat = *this;
		return FVector3d(AsFloat);
	}

	operator FVector4f() const
	{
		// Rescale [0..255] range to [-1..1]
		VectorRegister4Float VectorToUnpack = VectorLoadByte4(this);
		VectorToUnpack = VectorMultiplyAdd(VectorToUnpack, VectorSetFloat1(1.0f / 127.5f), VectorSetFloat1(-1.0f));

		FVector4f UnpackedVector;
		VectorStore(VectorToUnpack, &UnpackedVector);

		return UnpackedVector;
	}

	operator FVector4d() const
	{
		FVector4f AsFloat = *this;
		return FVector4d(AsFloat);
	}

	friend RENDERCORE_API FArchive& operator<<(FArchive& Ar, FDeprecatedSerializedPackedNormal& N);
};

inline void FPackedNormal::operator=(const FVector3f& InVector)
{
	const float Scale = MAX_int8;
	Vector.X = (int8)FMath::Clamp<int32>(FMath::RoundToInt32(InVector.X * Scale), MIN_int8, MAX_int8);
	Vector.Y = (int8)FMath::Clamp<int32>(FMath::RoundToInt32(InVector.Y * Scale), MIN_int8, MAX_int8);
	Vector.Z = (int8)FMath::Clamp<int32>(FMath::RoundToInt32(InVector.Z * Scale), MIN_int8, MAX_int8);
	Vector.W = MAX_int8;
}

inline void FPackedNormal::operator=(const FVector3d& InVector)
{
	const float Scale = MAX_int8;
	Vector.X = (int8)FMath::Clamp<int32>(FMath::RoundToInt32(InVector.X * Scale), MIN_int8, MAX_int8);
	Vector.Y = (int8)FMath::Clamp<int32>(FMath::RoundToInt32(InVector.Y * Scale), MIN_int8, MAX_int8);
	Vector.Z = (int8)FMath::Clamp<int32>(FMath::RoundToInt32(InVector.Z * Scale), MIN_int8, MAX_int8);
	Vector.W = MAX_int8;
}

inline void FPackedNormal::operator=(const FVector4f& InVector)
{
	const float Scale = MAX_int8;
	Vector.X = (int8)FMath::Clamp<int32>(FMath::RoundToInt32(InVector.X * Scale), MIN_int8, MAX_int8);
	Vector.Y = (int8)FMath::Clamp<int32>(FMath::RoundToInt32(InVector.Y * Scale), MIN_int8, MAX_int8);
	Vector.Z = (int8)FMath::Clamp<int32>(FMath::RoundToInt32(InVector.Z * Scale), MIN_int8, MAX_int8);
	Vector.W = (int8)FMath::Clamp<int32>(FMath::RoundToInt32(InVector.W * Scale), MIN_int8, MAX_int8);
}

inline void FPackedNormal::operator=(const FVector4d& InVector)
{
	const float Scale = MAX_int8;
	Vector.X = (int8)FMath::Clamp<int32>(FMath::RoundToInt32(InVector.X * Scale), MIN_int8, MAX_int8);
	Vector.Y = (int8)FMath::Clamp<int32>(FMath::RoundToInt32(InVector.Y * Scale), MIN_int8, MAX_int8);
	Vector.Z = (int8)FMath::Clamp<int32>(FMath::RoundToInt32(InVector.Z * Scale), MIN_int8, MAX_int8);
	Vector.W = (int8)FMath::Clamp<int32>(FMath::RoundToInt32(InVector.W * Scale), MIN_int8, MAX_int8);
}

inline bool FPackedNormal::operator==(const FPackedNormal& B) const
{
	return Vector.Packed == B.Vector.Packed;
}

inline bool FPackedNormal::operator!=(const FPackedNormal& B) const
{
	return !(*this == B);
}

inline FVector FPackedNormal::ToFVector() const
{
	// LWC_TODO: Support FVector3d
	return FVector(ToFVector3f());
}

inline FVector3f FPackedNormal::ToFVector3f() const
{
	VectorRegister4Float VectorToUnpack = GetVectorRegister();
	// Write to FVector and return it.
	FVector3f UnpackedVector;
	VectorStoreFloat3(VectorToUnpack, &UnpackedVector);
	return UnpackedVector;
}

inline FVector4 FPackedNormal::ToFVector4() const
{
	// LWC_TODO: Support FVector4d
	return FVector4(ToFVector4f());
}

inline FVector4f FPackedNormal::ToFVector4f() const
{
	VectorRegister4Float VectorToUnpack = GetVectorRegister();
	// Write to FVector4f and return it.
	FVector4f UnpackedVector;
	VectorStore(VectorToUnpack, &UnpackedVector);
	return UnpackedVector;
}

inline VectorRegister4Float FPackedNormal::GetVectorRegister() const
{
	// Rescale [0..255] range to [-1..1]
	VectorRegister4Float VectorToUnpack		= VectorLoadSignedByte4( this );
	VectorToUnpack = VectorMultiply(VectorToUnpack, VectorSetFloat1(1.0f / 127.0f));
	// Return unpacked vector register.
	return VectorToUnpack;
}

/** A vector, quantized and packed into 32-bits. */
struct FPackedRGB10A2N
{
	union
	{
		struct
		{
#if PLATFORM_LITTLE_ENDIAN
			uint32 X : 10;
			uint32 Y : 10;
			uint32 Z : 10;
			uint32 W : 2;
#else
			uint32 W : 2;
			uint32 Z : 10;
			uint32 Y : 10;
			uint32 X : 10;
#endif
		};

		struct  
		{
			uint32 Packed;
		};
	} Vector;

	// Constructors.

	FPackedRGB10A2N() { Vector.Packed = 0; }
	FPackedRGB10A2N(const FVector3f& InVector) { *this = InVector; }
	FPackedRGB10A2N(const FVector3d& InVector) { *this = InVector; }
	FPackedRGB10A2N(const FVector4f& InVector) { *this = InVector; }

	// Conversion operators.

	void operator=(const FVector3f& InVector);
	void operator=(const FVector3d& InVector);
	void operator=(const FVector4f& InVector);

	VectorRegister4Float GetVectorRegister() const;

	// Set functions.
	void Set(const FVector3f& InVector) { *this = InVector; }
	void Set(const FVector3d& InVector) { *this = InVector; }
	void Set(const FVector4f& InVector) { *this = InVector; }

	// Equality operator.

	bool operator==(const FPackedRGB10A2N& B) const;
	bool operator!=(const FPackedRGB10A2N& B) const;

	// Serializer.

	friend RENDERCORE_API FArchive& operator<<(FArchive& Ar, FPackedRGB10A2N& N);

	FString ToString() const
	{
		return FString::Printf(TEXT("X=%d Y=%d Z=%d W=%d"), Vector.X, Vector.Y, Vector.Z, Vector.W);
	}

	static RENDERCORE_API FPackedRGB10A2N ZeroVector;
};

inline void FPackedRGB10A2N::operator=(const FVector3f& InVector)
{
	Vector.X = FMath::Clamp(FMath::TruncToInt32(InVector.X * 511.5f + 511.5f), 0, 1023);
	Vector.Y = FMath::Clamp(FMath::TruncToInt32(InVector.Y * 511.5f + 511.5f), 0, 1023);
	Vector.Z = FMath::Clamp(FMath::TruncToInt32(InVector.Z * 511.5f + 511.5f), 0, 1023);
	Vector.W = 3;
}

inline void FPackedRGB10A2N::operator=(const FVector3d& InVector)
{
	Vector.X = FMath::Clamp(FMath::TruncToInt32(InVector.X * 511.5f + 511.5f), 0, 1023);
	Vector.Y = FMath::Clamp(FMath::TruncToInt32(InVector.Y * 511.5f + 511.5f), 0, 1023);
	Vector.Z = FMath::Clamp(FMath::TruncToInt32(InVector.Z * 511.5f + 511.5f), 0, 1023);
	Vector.W = 3;
}

inline void FPackedRGB10A2N::operator=(const FVector4f& InVector)
{
	Vector.X = FMath::Clamp(FMath::TruncToInt32(InVector.X * 511.5f + 511.5f), 0, 1023);
	Vector.Y = FMath::Clamp(FMath::TruncToInt32(InVector.Y * 511.5f + 511.5f), 0, 1023);
	Vector.Z = FMath::Clamp(FMath::TruncToInt32(InVector.Z * 511.5f + 511.5f), 0, 1023);
	Vector.W = FMath::Clamp(FMath::TruncToInt32(InVector.W * 1.5f   + 1.5f),   0, 3);
}

inline bool FPackedRGB10A2N::operator==(const FPackedRGB10A2N& B) const
{
	return Vector.Packed == B.Vector.Packed;
}

inline bool FPackedRGB10A2N::operator!=(const FPackedRGB10A2N& B) const
{
	return !(*this == B);
}

inline VectorRegister4Float FPackedRGB10A2N::GetVectorRegister() const
{
	VectorRegister4Float VectorToUnpack = VectorLoadURGB10A2N((void*)this);
	VectorToUnpack = VectorMultiplyAdd(VectorToUnpack, MakeVectorRegister(2.0f, 2.0f, 2.0f, 2.0f), MakeVectorRegister(-1.0f, -1.0f, -1.0f, -1.0f));
	VectorResetFloatRegisters();
	// Return unpacked vector register.
	return VectorToUnpack;
}

/** A vector, quantized and packed into 32-bits. */
struct FPackedRGBA16N
{
	struct
	{
		int16 X;
		int16 Y;
		int16 Z;
		int16 W;
	};

	// Constructors.

	FPackedRGBA16N() { X = 0; Y = 0; Z = 0; W = 0; }
	FPackedRGBA16N(const FVector3d& InVector) { *this = InVector; }
	FPackedRGBA16N(const FVector3f& InVector) { *this = InVector; }
	FPackedRGBA16N(const FVector4f& InVector) { *this = InVector; }
	//FPackedRGBA16N(uint16 InX, uint16 InY, uint16 InZ, uint16 InW) { X = InX; Y = InY; Z = InZ; W = InW; }

	// Conversion operators.

	void operator=(const FVector3d& InVector);
	void operator=(const FVector3f& InVector);
	void operator=(const FVector4f& InVector);
	void operator=(const FVector4d& InVector);

	FVector ToFVector() const;
	FVector3f ToFVector3f() const;
	FVector4 ToFVector4() const;
	FVector4f ToFVector4f() const;

	VectorRegister4Float GetVectorRegister() const;

	// Set functions.
	void Set(const FVector3d& InVector) { *this = InVector; }
	void Set(const FVector3f& InVector) { *this = InVector; }
	void Set(const FVector4f& InVector) { *this = InVector; }

	// Equality operator.

	bool operator==(const FPackedRGBA16N& B) const;
	bool operator!=(const FPackedRGBA16N& B) const;

	FString ToString() const
	{
		return FString::Printf(TEXT("X=%d Y=%d Z=%d W=%d"), X, Y, Z, W);
	}

	friend RENDERCORE_API FArchive& operator<<(FArchive& Ar, FPackedRGBA16N& N);
};

DECLARE_INTRINSIC_TYPE_LAYOUT(FPackedRGBA16N);

inline void FPackedRGBA16N::operator=(const FVector3f& InVector)
{
	const float Scale = MAX_int16;
	X = (int16)FMath::Clamp<int32>(FMath::RoundToInt32(InVector.X * Scale), MIN_int16, MAX_int16);
	Y = (int16)FMath::Clamp<int32>(FMath::RoundToInt32(InVector.Y * Scale), MIN_int16, MAX_int16);
	Z = (int16)FMath::Clamp<int32>(FMath::RoundToInt32(InVector.Z * Scale), MIN_int16, MAX_int16);
	W = MAX_int16;
}

inline void FPackedRGBA16N::operator=(const FVector3d& InVector)
{
	const float Scale = MAX_int16;
	X = (int16)FMath::Clamp<int32>(FMath::RoundToInt32(InVector.X * Scale), MIN_int16, MAX_int16);
	Y = (int16)FMath::Clamp<int32>(FMath::RoundToInt32(InVector.Y * Scale), MIN_int16, MAX_int16);
	Z = (int16)FMath::Clamp<int32>(FMath::RoundToInt32(InVector.Z * Scale), MIN_int16, MAX_int16);
	W = MAX_int16;
}

inline void FPackedRGBA16N::operator=(const FVector4f& InVector)
{
	const float Scale = MAX_int16;
	X = (int16)FMath::Clamp<int32>(FMath::RoundToInt32(InVector.X * Scale), MIN_int16, MAX_int16);
	Y = (int16)FMath::Clamp<int32>(FMath::RoundToInt32(InVector.Y * Scale), MIN_int16, MAX_int16);
	Z = (int16)FMath::Clamp<int32>(FMath::RoundToInt32(InVector.Z * Scale), MIN_int16, MAX_int16);
	W = (int16)FMath::Clamp<int32>(FMath::RoundToInt32(InVector.W * Scale), MIN_int16, MAX_int16);
}

inline void FPackedRGBA16N::operator=(const FVector4d& InVector)
{
	const float Scale = MAX_int16;
	X = (int16)FMath::Clamp<int32>(FMath::RoundToInt32(InVector.X * Scale), MIN_int16, MAX_int16);
	Y = (int16)FMath::Clamp<int32>(FMath::RoundToInt32(InVector.Y * Scale), MIN_int16, MAX_int16);
	Z = (int16)FMath::Clamp<int32>(FMath::RoundToInt32(InVector.Z * Scale), MIN_int16, MAX_int16);
	W = (int16)FMath::Clamp<int32>(FMath::RoundToInt32(InVector.W * Scale), MIN_int16, MAX_int16);
}

inline bool FPackedRGBA16N::operator==(const FPackedRGBA16N& B) const
{
	return (X == B.X) && (Y == B.Y) && (Z == B.Z) && (W == B.W);
}

inline bool FPackedRGBA16N::operator!=(const FPackedRGBA16N& B) const
{
	return !(*this == B);
}

inline FVector FPackedRGBA16N::ToFVector() const
{
	// LWC_TODO: Support FVector3d
	return FVector(ToFVector3f());
}

inline FVector3f FPackedRGBA16N::ToFVector3f() const
{
	VectorRegister4Float VectorToUnpack = GetVectorRegister();
	// Write to FVector and return it.
	FVector3f UnpackedVector;
	VectorStoreFloat3(VectorToUnpack, &UnpackedVector);
	return UnpackedVector;
}

inline FVector4 FPackedRGBA16N::ToFVector4() const
{
	// LWC_TODO: Support FVector4d
	return FVector4(ToFVector4f());
}

inline FVector4f FPackedRGBA16N::ToFVector4f() const
{
	VectorRegister4Float VectorToUnpack = GetVectorRegister();
	// Write to FVector4f and return it.
	FVector4f UnpackedVector;
	VectorStore(VectorToUnpack, &UnpackedVector);
	return UnpackedVector;
}

inline VectorRegister4Float FPackedRGBA16N::GetVectorRegister() const
{
	VectorRegister4Float VectorToUnpack = VectorLoadSRGBA16N((void*)this);
	VectorToUnpack = VectorMultiply(VectorToUnpack, VectorSetFloat1(1.0f / 32767.0f));
	// Return unpacked vector register.
	return VectorToUnpack;
}
