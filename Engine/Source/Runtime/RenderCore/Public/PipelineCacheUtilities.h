// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Build.h"

// cooked games may want to load key files for extra information about the PSOs
#define UE_WITH_PIPELINE_CACHE_UTILITIES			(WITH_EDITOR || !UE_BUILD_SHIPPING)
#if UE_WITH_PIPELINE_CACHE_UTILITIES

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Containers/StringFwd.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "RHIDefinitions.h"
#include "ShaderCodeLibrary.h"

class FName;
class ITargetPlatform;
struct FPipelineCacheFileFormatPSO;
struct FStableShaderKeyAndValue;

namespace UE
{
namespace PipelineCacheUtilities
{
#if WITH_EDITOR	// limit what's compiled for the cooked games

	/** Describes a particular combination of shaders. */
	struct FPermutation
	{
		/** Each frequency holds an index of shaders in StableArray. */
		int32 Slots[SF_NumFrequencies];
	};

	/** Describes a PSO with an array of other stable shaders that could be used with it. */
	struct FPermsPerPSO
	{
		/** Original PSO (as recorded during the collection run). */
		const FPipelineCacheFileFormatPSO* PSO;
		/** Boolean table describing which frequencies are active (i.e. have valid shaders). */
		bool ActivePerSlot[SF_NumFrequencies];
		/** Array of other stable shaders whose hashes were the same, so they could potentially be used in this PSO. */
		TArray<FPermutation> Permutations;

		FPermsPerPSO()
			: PSO(nullptr)
		{
			for (int32 Index = 0; Index < SF_NumFrequencies; Index++)
			{
				ActivePerSlot[Index] = false;
			}
		}
	};

#endif // WITH_EDITOR

	/** 
	 * Loads stable shader keys file (using a proprietary format). Stable key is a way to identify a shader independently of its output hash
	 * 
	 * @param Filename filename (with path if needed)
	 * @param InOutArray array to put the file contents. Existing array contents will be preserved and appended to
	 * @return true if successful
	 */
	RENDERCORE_API bool LoadStableKeysFile(const FStringView& Filename, TArray<FStableShaderKeyAndValue>& InOutArray);

#if WITH_EDITOR	// limit what's compiled for the cooked games

	/** 
	* Load Multiple stable shader keys file (using a proprietary format). Stable key is a way to identify a shader independently of its output hash
	* 
	* @param FileNames array of filenames (with path if needed)
	* @param Values destination for merged file values
	* @return Count of files read in and merged
	*/
	RENDERCORE_API uint32 LoadMultipleStableKeysFiles(TArrayView<const FStringView> FileNames, FStableShaderSet& Values);

	/** 
	 * Saves stable shader keys file (using a proprietary format). Stable key is a way to identify a shader independently of its output hash
	 * 
	 * @param Filename filename (with path if needed)
	 * @param Values values to be saved
	 * @return true if successful
	 */
	RENDERCORE_API bool SaveStableKeysFile(const FStringView& Filename, const FStableShaderSet& Values);

	/**
	 * Saves stable pipeline cache file.
	 * 
	 * The cache file is saved together with the stable shader keys file that were used to map its hashes to the build-agnostic ("stable") shader identifier.
	 * 
	 * @param OutputFilename file name for the binary file
	 * @param StableResults an array of PSOs together with all permutations allowed for it
	 * @param StableShaderKeyIndexTable the table of build-agnostic shader keys
	 */
	RENDERCORE_API bool SaveStablePipelineCacheFile(const FString& OutputFilename, const TArray<FPermsPerPSO>& StableResults, const TArray<FStableShaderKeyAndValue>& StableShaderKeyIndexTable);

	/**
	 * Loads stable pipeline cache file.
	 * 
	 * @param Filename file to be loaded
	 * @param StableMap Mapping of the stable (build-agnostic) shader keys to the shader code hashes as of the current moment
	 * @param OutPSOs the PSOs loaded from that file
	 * @param OutTargetPlatform target platform for this file
	 * @param OutPSOsRejected number of PSOs that were rejected during loading (usually because the stable key it used is no longer present in StableMap)
	 * @param OutPSOsMerged number of PSOs that mapped to the same shader code hashes despite using different build-agnostic ("stable") shader keys.
	 */
	RENDERCORE_API bool LoadStablePipelineCacheFile(const FString& Filename, const FStableShaderSet& StableMap, TSet<FPipelineCacheFileFormatPSO>& OutPSOs, FName& OutTargetPlatform, int32& OutPSOsRejected, int32& OutPSOsMerged);

	/**
	 * Saves description of assets in the chunk.
	 *
	 * This file is later used to split the cache into several files. 
	 * Note that it is assumed that all possible shader format of the target platform share the same subdivision into chunks (which is the current behavior of other code, too).
	 *
	 * @param ShaderLibraryName shader library this info is associated with (usually either a project name or a DLC name)
	 * @param InChunkId chunk id
	 * @param InPackagesInChunk list of package names in chunk
	 * @param TargetPlatform target platform for which we're cooking
	 * @param PathToSaveTo directory where to save the info
	 * @param OutChunkFilenames files that we need to include in this chunk
	 * @return true if successfully saved
	 */
	RENDERCORE_API bool SaveChunkInfo(const FString& ShaderLibraryName, const int32 InChunkId, const TSet<FName>& InPackagesInChunk, const ITargetPlatform* TargetPlatform, const FString& PathToSaveTo, TArray<FString>& OutChunkFilenames);

	/**
	 * Finds all saved chunk info files for the given shaderlibrary and targetplatform.
	 *
	 * Note that it is assumed that all possible shader format of the target platform share the same subdivision into chunks (which is the current behavior of other code, too).
	 *
	 * @param ShaderLibraryName shader library that we chose to associate the info with (usually either a project name or a DLC name)
	 * @param TargetPlatformName target platform name (ITargetPlatform::PlatformName()) of our target. Game and Client could have different packages
	 * @param PathToSearchIn path where the search needs to be done (perhaps the same as where they were saved by SaveChunkInfo).
	 * @param OutInfoFilenames found filenames, which can be loaded by LoadChunkInfo function when passed as is
	 */
	RENDERCORE_API void FindAllChunkInfos(const FString& ShaderLibraryName, const FString& TargetPlatformName, const FString& PathToSearchIn, TArray<FString>& OutInfoFilenames);

	/**
	 * Loads description of assets in the chunk
	 *
	 * Note that it is assumed that all possible shader format of the target platform share the same subdivision into chunks (which is the current behavior of other code, too).
	 *
	 * @param Filename file to be loaded
	 * @param InShaderFormat shader format we're interested in (info file may have several, even if its contents aren't dependent on that)
	 * @param OutChunkId chunk id this info is describing
	 * @param OutPSOs the PSOs loaded from that file
	 * @param OutChunkedCacheFilename filename of the PSO cache file that needs to be produced for this chunk.
	 * @param OutPackages packages in this chunk
	 */
	RENDERCORE_API bool LoadChunkInfo(const FString& Filename, const FString& InShaderFormat, int32 &OutChunkId, FString& OutChunkedCacheFilename, TSet<FName>& OutPackages);

#endif // WITH_EDITOR

}
};

#endif // UE_WITH_PIPELINE_CACHE_UTILITIES
