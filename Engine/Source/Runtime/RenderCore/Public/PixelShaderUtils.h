// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	PixelShaderUtils.h: Utilities for pixel shaders.
=============================================================================*/

#pragma once

#include "CommonRenderResources.h"
#include "GlobalRenderResources.h"
#include "GlobalShader.h"
#include "Math/IntPoint.h"
#include "Math/IntRect.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector2D.h"
#include "Misc/AssertionMacros.h"
#include "PipelineStateCache.h"
#include "RHI.h"
#include "RHICommandList.h"
#include "RHIDefinitions.h"
#include "RHIStaticStates.h"
#include "RenderGraphDefinitions.h"
#include "RenderGraphEvent.h"
#include "RenderGraphResources.h"
#include "RenderGraphUtils.h"
#include "RenderResource.h"
#include "RenderUtils.h"
#include "Serialization/MemoryLayout.h"
#include "Shader.h"
#include "ShaderParameterMacros.h"
#include "ShaderParameterStruct.h"
#include "ShaderPermutation.h"
#include "Templates/UnrealTemplate.h"

class FPointerTableBase;
class FRDGBuilder;


/** All utils for pixel shaders. */
struct FPixelShaderUtils
{
	/** Utility vertex shader for rect array based operations. For example for clearing specified parts of an atlas. */
	class FRasterizeToRectsVS : public FGlobalShader
	{
		DECLARE_EXPORTED_SHADER_TYPE(FRasterizeToRectsVS, Global, RENDERCORE_API);
		SHADER_USE_PARAMETER_STRUCT(FRasterizeToRectsVS, FGlobalShader);

		BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
			SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint4>, RectCoordBuffer)
			SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float4>, RectUVBuffer)
			SHADER_PARAMETER(FVector2f, InvViewSize)
			SHADER_PARAMETER(FVector2f, InvTextureSize)
			SHADER_PARAMETER(float, DownsampleFactor)
			SHADER_PARAMETER(float, DepthIncrement)
		END_SHADER_PARAMETER_STRUCT()

		class FRectUV : SHADER_PERMUTATION_BOOL("RECT_UV");
		using FPermutationDomain = TShaderPermutationDomain<FRectUV>;

		static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters);
	};

	/** Draw a single triangle on the entire viewport. */
	static RENDERCORE_API void DrawFullscreenTriangle(FRHICommandList& RHICmdList, uint32 InstanceCount = 1);

	/** Draw a two triangle on the entire viewport. */
	static RENDERCORE_API void DrawFullscreenQuad(FRHICommandList& RHICmdList, uint32 InstanceCount = 1);

	/** Initialize a pipeline state object initializer with almost all the basics required to do a full viewport pass. */
	static RENDERCORE_API void InitFullscreenPipelineState(
		FRHICommandList& RHICmdList,
		const FGlobalShaderMap* GlobalShaderMap,
		const TShaderRef<FShader>& PixelShader,
		FGraphicsPipelineStateInitializer& GraphicsPSOInit);

	/** Initialize a pipeline state object initializer with almost all the basics required to do a full multi-viewport pass. */
	static RENDERCORE_API void InitFullscreenMultiviewportPipelineState(
		FRHICommandList& RHICmdList,
		const FGlobalShaderMap* GlobalShaderMap,
		const TShaderRef<FShader>& PixelShader,
		FGraphicsPipelineStateInitializer& GraphicsPSOInit);

	/** Dispatch a full screen pixel shader to rhi command list with its parameters. */
	template<typename TShaderClass>
	static inline void DrawFullscreenPixelShader(
		FRHICommandList& RHICmdList, 
		const FGlobalShaderMap* GlobalShaderMap,
		const TShaderRef<TShaderClass>& PixelShader,
		const typename TShaderClass::FParameters& Parameters,
		const FIntRect& Viewport,
		FRHIBlendState* BlendState = nullptr,
		FRHIRasterizerState* RasterizerState = nullptr,
		FRHIDepthStencilState* DepthStencilState = nullptr,
		uint32 StencilRef = 0)
	{
		check(PixelShader.IsValid());
		RHICmdList.SetViewport((float)Viewport.Min.X, (float)Viewport.Min.Y, 0.0f, (float)Viewport.Max.X, (float)Viewport.Max.Y, 1.0f);
		
		FGraphicsPipelineStateInitializer GraphicsPSOInit;
		InitFullscreenPipelineState(RHICmdList, GlobalShaderMap, PixelShader, /* out */ GraphicsPSOInit);
		GraphicsPSOInit.BlendState = BlendState ? BlendState : GraphicsPSOInit.BlendState;
		GraphicsPSOInit.RasterizerState = RasterizerState ? RasterizerState : GraphicsPSOInit.RasterizerState;
		GraphicsPSOInit.DepthStencilState = DepthStencilState ? DepthStencilState : GraphicsPSOInit.DepthStencilState;

		SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, StencilRef);

		SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), Parameters);

		DrawFullscreenTriangle(RHICmdList);
	}

	/** Dispatch a full screen pixel shader to rhi command list with its parameters, covering several views at once. */
	template<typename TShaderClass>
	static inline void DrawFullscreenInstancedMultiViewportPixelShader(
		FRHICommandList& RHICmdList,
		const FGlobalShaderMap* GlobalShaderMap,
		const TShaderRef<TShaderClass>& PixelShader,
		const typename TShaderClass::FParameters& Parameters,
		TArrayView<FIntRect const> Viewports,
		FRHIBlendState* BlendState = nullptr,
		FRHIRasterizerState* RasterizerState = nullptr,
		FRHIDepthStencilState* DepthStencilState = nullptr,
		uint32 StencilRef = 0)
	{
		check(PixelShader.IsValid());
		checkf(Viewports.Num() == 2, TEXT("Only two instanced viewports are currently supported"));
		{
			const FIntRect& LeftViewRect = Viewports[0];
			const int32 LeftMinX = LeftViewRect.Min.X;
			const int32 LeftMaxX = LeftViewRect.Max.X;
			const int32 LeftMaxY = LeftViewRect.Max.Y;

			const FIntRect& RightViewRect = Viewports[1];
			const int32 RightMinX = RightViewRect.Min.X;
			const int32 RightMaxX = RightViewRect.Max.X;
			const int32 RightMaxY = RightViewRect.Max.Y;
			RHICmdList.SetStereoViewport((float)LeftMinX, (float)RightMinX, 0.0f, 0.0f, 0.0f, (float)LeftMaxX, (float)RightMaxX, (float)LeftMaxY, (float)RightMaxY, 1.0f);
		}

		FGraphicsPipelineStateInitializer GraphicsPSOInit;
		InitFullscreenMultiviewportPipelineState(RHICmdList, GlobalShaderMap, PixelShader, /* out */ GraphicsPSOInit);
		GraphicsPSOInit.BlendState = BlendState ? BlendState : GraphicsPSOInit.BlendState;
		GraphicsPSOInit.RasterizerState = RasterizerState ? RasterizerState : GraphicsPSOInit.RasterizerState;
		GraphicsPSOInit.DepthStencilState = DepthStencilState ? DepthStencilState : GraphicsPSOInit.DepthStencilState;

		SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, StencilRef);

		SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), Parameters);

		DrawFullscreenTriangle(RHICmdList, Viewports.Num());
	}

	/** Dispatch a pixel shader to render graph builder with its parameters. */
	template<typename TShaderClass>
	static inline void AddFullscreenPass(
		FRDGBuilder& GraphBuilder,
		const FGlobalShaderMap* GlobalShaderMap,
		FRDGEventName&& PassName,
		const TShaderRef<TShaderClass>& PixelShader,
		typename TShaderClass::FParameters* Parameters,
		const FIntRect& Viewport,
		FRHIBlendState* BlendState = nullptr,
		FRHIRasterizerState* RasterizerState = nullptr,
		FRHIDepthStencilState* DepthStencilState = nullptr,
		uint32 StencilRef = 0,
		ERDGPassFlags AdditionalPassFlags = ERDGPassFlags::None)
	{
		check(PixelShader.IsValid());
		ClearUnusedGraphResources(PixelShader, Parameters);

		ERDGPassFlags PassFlags = ERDGPassFlags::Raster;
		PassFlags |= AdditionalPassFlags;

		GraphBuilder.AddPass(
			Forward<FRDGEventName>(PassName),
			Parameters,
			PassFlags,
			[Parameters, GlobalShaderMap, PixelShader, Viewport, BlendState, RasterizerState, DepthStencilState, StencilRef](FRDGAsyncTask, FRHICommandList& RHICmdList)
		{
			FPixelShaderUtils::DrawFullscreenPixelShader(RHICmdList, GlobalShaderMap, PixelShader, *Parameters, Viewport,
			BlendState, RasterizerState, DepthStencilState, StencilRef);
		});
	}

	/** Dispatch a pixel shader to render graph builder with its parameters. */
	template<typename TShaderClass>
	static inline void AddFullscreenInstancedMultiViewportPass(
		FRDGBuilder& GraphBuilder,
		const FGlobalShaderMap* GlobalShaderMap,
		FRDGEventName&& PassName,
		const TShaderRef<TShaderClass>& PixelShader,
		typename TShaderClass::FParameters* Parameters,
		TArray<FIntRect>&& Viewports,
		FRHIBlendState* BlendState = nullptr,
		FRHIRasterizerState* RasterizerState = nullptr,
		FRHIDepthStencilState* DepthStencilState = nullptr,
		uint32 StencilRef = 0,
		ERDGPassFlags AdditionalPassFlags = ERDGPassFlags::None)
	{
		check(PixelShader.IsValid());
		ClearUnusedGraphResources(PixelShader, Parameters);

		ERDGPassFlags PassFlags = ERDGPassFlags::Raster;
		PassFlags |= AdditionalPassFlags;

		GraphBuilder.AddPass(
			Forward<FRDGEventName>(PassName),
			Parameters,
			PassFlags,
			[Parameters, GlobalShaderMap, PixelShader, Viewports, BlendState, RasterizerState, DepthStencilState, StencilRef](FRDGAsyncTask, FRHICommandList& RHICmdList)
		{
			FPixelShaderUtils::DrawFullscreenInstancedMultiViewportPixelShader(RHICmdList, GlobalShaderMap, PixelShader, *Parameters, Viewports,
				BlendState, RasterizerState, DepthStencilState, StencilRef);
		});
	}

	/** Rect based pixel shader pass. */
	template<typename TPixelShaderClass, typename TPassParameters>
	static inline void AddRasterizeToRectsPass(
		FRDGBuilder& GraphBuilder,
		const FGlobalShaderMap* GlobalShaderMap,
		FRDGEventName&& PassName,
		const TShaderRef<TPixelShaderClass>& PixelShader,
		TPassParameters* Parameters,
		FIntPoint ViewportSize,
		FRDGBufferSRVRef RectCoordBufferSRV,
		uint32 NumRects,
		FRHIBlendState* BlendState = nullptr,
		FRHIRasterizerState* RasterizerState = nullptr,
		FRHIDepthStencilState* DepthStencilState = nullptr,
		uint32 StencilRef = 0,
		FIntPoint TextureSize = FIntPoint(1, 1),
		FRDGBufferSRVRef RectUVBufferSRV = nullptr,
		uint32 DownsampleFactor = 1,
		ERDGPassFlags AdditionalPassFlags = ERDGPassFlags::None,
		float DepthIncrement = 0.0f)
	{
		FRasterizeToRectsVS::FPermutationDomain PermutationVector;
		PermutationVector.Set<FRasterizeToRectsVS::FRectUV>(RectUVBufferSRV != nullptr);
		auto VertexShader = GlobalShaderMap->GetShader<FRasterizeToRectsVS>(PermutationVector);

		Parameters->VS.InvViewSize = FVector2f(1.0f / ViewportSize.X, 1.0f / ViewportSize.Y);
		Parameters->VS.InvTextureSize = FVector2f(1.0f / TextureSize.X, 1.0f / TextureSize.Y);
		Parameters->VS.DownsampleFactor = 1.0f / DownsampleFactor;
		Parameters->VS.RectCoordBuffer = RectCoordBufferSRV;
		Parameters->VS.RectUVBuffer = RectUVBufferSRV;
		Parameters->VS.DepthIncrement = DepthIncrement;

		ClearUnusedGraphResources(PixelShader, &Parameters->PS);

		ERDGPassFlags PassFlags = ERDGPassFlags::Raster;
		PassFlags |= AdditionalPassFlags;
		const bool bSkipRenderPass = EnumHasAnyFlags(PassFlags, ERDGPassFlags::SkipRenderPass);

		GraphBuilder.AddPass(
			Forward<FRDGEventName>(PassName),
			Parameters,
			PassFlags,
			[Parameters, GlobalShaderMap, VertexShader, PixelShader, ViewportSize, BlendState, RasterizerState, DepthStencilState, StencilRef, bSkipRenderPass, NumRects](FRDGAsyncTask, FRHICommandList& RHICmdList)
		{
			if (bSkipRenderPass)
			{
				FRHIRenderPassInfo RPInfo;
				RPInfo.ResolveRect.X1 = 0;
				RPInfo.ResolveRect.Y1 = 0;
				RPInfo.ResolveRect.X2 = ViewportSize.X;
				RPInfo.ResolveRect.Y2 = ViewportSize.Y;
				RHICmdList.BeginRenderPass(RPInfo, TEXT("RasterizeToRects"));
			}

			FGraphicsPipelineStateInitializer GraphicsPSOInit;
			RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);

			RHICmdList.SetViewport(0.0f, 0.0f, 0.0f, (float)ViewportSize.X, (float)ViewportSize.Y, 1.0f);

			GraphicsPSOInit.BlendState = BlendState ? BlendState : TStaticBlendState<>::GetRHI();
			GraphicsPSOInit.RasterizerState = RasterizerState ? RasterizerState : TStaticRasterizerState<>::GetRHI();
			GraphicsPSOInit.DepthStencilState = DepthStencilState ? DepthStencilState : TStaticDepthStencilState<false, CF_Always>::GetRHI();

			GraphicsPSOInit.PrimitiveType = GRHISupportsRectTopology ? PT_RectList : PT_TriangleList;

			GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GTileVertexDeclaration.VertexDeclarationRHI;
			GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
			GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();

			SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, StencilRef);

			SetShaderParameters(RHICmdList, VertexShader, VertexShader.GetVertexShader(), Parameters->VS);
			SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), Parameters->PS);

			const uint32 NumPrimitives = GRHISupportsRectTopology ? 1 : 2;
			const uint32 NumInstances = NumRects;
			RHICmdList.DrawPrimitive(0, NumPrimitives, NumInstances);

			if (bSkipRenderPass)
			{
				RHICmdList.EndRenderPass();
			}
		});
	}
};
