// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/AssertionMacros.h"
#include "Math/MathFwd.h"
#include "Containers/UnrealString.h"
#include "GenericPlatform/GenericPlatformMemory.h"
#include "RHIFwd.h"
#include "RHIDefinitions.h"
#include "RHIResources.h"
#include "RHIFeatureLevel.h"
#include "Misc/EnumClassFlags.h"
#include "PixelFormat.h"
#include "Templates/RefCounting.h"

class FRHITransientTexture;

/** All necessary data to create a render target from the pooled render targets. */
struct FPooledRenderTargetDesc
{
public:

	/** Default constructor, use one of the factory functions below to make a valid description */
	FPooledRenderTargetDesc()
		: PackedBits(0)
	{
		check(!IsValid());
	}

	/**
	 * Factory function to create 2D texture description
	 * @param InFlags bit mask combined from elements of ETextureCreateFlags e.g. TexCreate_UAV
	 */
	static FPooledRenderTargetDesc Create2DDesc(
		FIntPoint InExtent,
		EPixelFormat InFormat,
		const FClearValueBinding& InClearValue,
		ETextureCreateFlags InFlags,
		ETextureCreateFlags InTargetableFlags,
		bool bInForceSeparateTargetAndShaderResource,
		uint8 InNumMips = 1,
		bool InAutowritable = true,
		bool InCreateRTWriteMask = false,
		bool InCreateFmask = false)
	{
		check(InExtent.X);
		check(InExtent.Y);

		FPooledRenderTargetDesc NewDesc;
		NewDesc.ClearValue = InClearValue;
		NewDesc.Extent = InExtent;
		NewDesc.Depth = 0;
		NewDesc.ArraySize = 1;
		NewDesc.bIsArray = false;
		NewDesc.bIsCubemap = false;
		NewDesc.NumMips = InNumMips;
		NewDesc.NumSamples = 1;
		NewDesc.Format = InFormat;
		NewDesc.Flags = InFlags | InTargetableFlags;
		NewDesc.DebugName = TEXT("UnknownTexture2D");
		check(NewDesc.Is2DTexture());
		return NewDesc;
	}

	/**
 * Factory function to create 2D array texture description
 * @param InFlags bit mask combined from elements of ETextureCreateFlags e.g. TexCreate_UAV
 */
	static FPooledRenderTargetDesc Create2DArrayDesc(
		FIntPoint InExtent,
		EPixelFormat InFormat,
		const FClearValueBinding& InClearValue,
		ETextureCreateFlags InFlags,
		ETextureCreateFlags InTargetableFlags,
		bool bInForceSeparateTargetAndShaderResource,
		uint16 InArraySize,
		uint8 InNumMips = 1,
		bool InAutowritable = true,
		bool InCreateRTWriteMask = false,
		bool InCreateFmask = false)
	{
		check(InExtent.X);
		check(InExtent.Y);

		FPooledRenderTargetDesc NewDesc;
		NewDesc.ClearValue = InClearValue;
		NewDesc.Extent = InExtent;
		NewDesc.Depth = 0;
		NewDesc.ArraySize = InArraySize;
		NewDesc.bIsArray = true;
		NewDesc.bIsCubemap = false;
		NewDesc.NumMips = InNumMips;
		NewDesc.NumSamples = 1;
		NewDesc.Format = InFormat;
		NewDesc.Flags = InFlags | InTargetableFlags;
		NewDesc.DebugName = TEXT("UnknownTexture2DArray");
		check(NewDesc.Is2DTexture());

		return NewDesc;
	}

	/**
	 * Factory function to create 3D texture description
	 * @param InFlags bit mask combined from elements of ETextureCreateFlags e.g. TexCreate_UAV
	 */
	static FPooledRenderTargetDesc CreateVolumeDesc(
		uint32 InSizeX,
		uint32 InSizeY,
		uint16 InSizeZ,
		EPixelFormat InFormat,
		const FClearValueBinding& InClearValue,
		ETextureCreateFlags InFlags,
		ETextureCreateFlags InTargetableFlags,
		bool bInForceSeparateTargetAndShaderResource,
		uint8 InNumMips = 1,
		bool InAutowritable = true)
	{
		check(InSizeX);
		check(InSizeY);

		FPooledRenderTargetDesc NewDesc;
		NewDesc.ClearValue = InClearValue;
		NewDesc.Extent = FIntPoint(InSizeX,InSizeY);
		NewDesc.Depth = InSizeZ;
		NewDesc.ArraySize = 1;
		NewDesc.bIsArray = false;
		NewDesc.bIsCubemap = false;
		NewDesc.NumMips = InNumMips;
		NewDesc.NumSamples = 1;
		NewDesc.Format = InFormat;
		NewDesc.Flags = InFlags | InTargetableFlags;
		NewDesc.DebugName = TEXT("UnknownTextureVolume");
		check(NewDesc.Is3DTexture());
		return NewDesc;
	}

	/**
	 * Factory function to create cube map texture description
	 * @param InFlags bit mask combined from elements of ETextureCreateFlags e.g. TexCreate_UAV
	 */
	static FPooledRenderTargetDesc CreateCubemapDesc(
		uint32 InExtent,
		EPixelFormat InFormat,
		const FClearValueBinding& InClearValue,
		ETextureCreateFlags InFlags,
		ETextureCreateFlags InTargetableFlags,
		bool bInForceSeparateTargetAndShaderResource,
		uint16 InArraySize = 1,
		uint8 InNumMips = 1,
		bool InAutowritable = true)
	{
		check(InExtent);

		FPooledRenderTargetDesc NewDesc;
		NewDesc.ClearValue = InClearValue;
		NewDesc.Extent = FIntPoint(InExtent, InExtent);
		NewDesc.Depth = 0;
		NewDesc.ArraySize = InArraySize;
		// Note: this doesn't allow an array of size 1
		NewDesc.bIsArray = InArraySize > 1;
		NewDesc.bIsCubemap = true;
		NewDesc.NumMips = InNumMips;
		NewDesc.NumSamples = 1;
		NewDesc.Format = InFormat;
		NewDesc.Flags = InFlags | InTargetableFlags;
		NewDesc.DebugName = TEXT("UnknownTextureCube");
		check(NewDesc.IsCubemap());

		return NewDesc;
	}

	/**
	 * Factory function to create cube map array texture description
	 * @param InFlags bit mask combined from elements of ETextureCreateFlags e.g. TexCreate_UAV
	 */
	static FPooledRenderTargetDesc CreateCubemapArrayDesc(
		uint32 InExtent,
		EPixelFormat InFormat,
		const FClearValueBinding& InClearValue,
		ETextureCreateFlags InFlags,
		ETextureCreateFlags InTargetableFlags,
		bool bInForceSeparateTargetAndShaderResource,
		uint16 InArraySize,
		uint8 InNumMips = 1,
		bool InAutowritable = true)
	{
		check(InExtent);

		FPooledRenderTargetDesc NewDesc;
		NewDesc.ClearValue = InClearValue;
		NewDesc.Extent = FIntPoint(InExtent, InExtent);
		NewDesc.Depth = 0;
		NewDesc.ArraySize = InArraySize;
		NewDesc.bIsArray = true;
		NewDesc.bIsCubemap = true;
		NewDesc.NumMips = InNumMips;
		NewDesc.NumSamples = 1;
		NewDesc.Format = InFormat;
		NewDesc.Flags = InFlags | InTargetableFlags;
		NewDesc.DebugName = TEXT("UnknownTextureCubeArray");
		check(NewDesc.IsCubemap());

		return NewDesc;
	}

	/** Comparison operator to test if a render target can be reused */
	bool Compare(const FPooledRenderTargetDesc& rhs, bool bExact) const
	{
		auto LhsFlags = Flags;
		auto RhsFlags = rhs.Flags;

		if (!bExact || !FPlatformMemory::SupportsFastVRAMMemory())
		{
			LhsFlags &= (~TexCreate_FastVRAM);
			RhsFlags &= (~TexCreate_FastVRAM);
		}
		
		return ClearValue == rhs.ClearValue
			&& LhsFlags == RhsFlags
			&& Format == rhs.Format
			&& UAVFormat == rhs.UAVFormat
			&& Extent == rhs.Extent
			&& Depth == rhs.Depth
			&& ArraySize == rhs.ArraySize
			&& NumMips == rhs.NumMips
			&& NumSamples == rhs.NumSamples
			&& PackedBits == rhs.PackedBits
			&& FastVRAMPercentage == rhs.FastVRAMPercentage 
			&& AliasableFormats == rhs.AliasableFormats;
	}

	bool IsCubemap() const
	{
		return bIsCubemap;
	}

	bool Is2DTexture() const
	{
		return Extent.X != 0 && Extent.Y != 0 && Depth == 0 && !bIsCubemap;
	}

	bool Is3DTexture() const
	{
		return Extent.X != 0 && Extent.Y != 0 && Depth != 0 && !bIsCubemap;
	}

	// @return true if this texture is a texture array
	bool IsArray() const
	{
		return bIsArray;
	}

	bool IsValid() const
	{
		if(NumSamples != 1)
		{
			if(NumSamples < 1 || NumSamples > 8)
			{
				// D3D11 limitations
				return false;
			}

			if(!Is2DTexture())
			{
				return false;
			}
		}

		return Extent.X != 0 && NumMips != 0 && NumSamples >=1 && NumSamples <=16 && Format != PF_Unknown
			&& (EnumHasAnyFlags(Flags, TexCreate_UAV) || GMaxRHIFeatureLevel >= ERHIFeatureLevel::SM5 || GMaxRHIFeatureLevel == ERHIFeatureLevel::ES3_1);
	}

	FIntVector GetSize() const
	{
		return FIntVector(Extent.X, Extent.Y, Depth);
	}

	/** 
	 * for debugging purpose
	 * @return e.g. (2D 128x64 PF_R8)
	 */
	FString GenerateInfoString() const
	{
		const TCHAR* FormatString = GetPixelFormatString(Format);

		FString FlagsString;

		ETextureCreateFlags LocalFlags = Flags;

		if(EnumHasAnyFlags(LocalFlags, TexCreate_RenderTargetable))
		{
			FlagsString += TEXT(" RT");
		}
		if(EnumHasAnyFlags(LocalFlags, TexCreate_SRGB))
		{
			FlagsString += TEXT(" sRGB");
		}
		if(NumSamples > 1)
		{
			FlagsString += FString::Printf(TEXT(" %dxMSAA"), NumSamples);
		}
		if(EnumHasAnyFlags(LocalFlags, TexCreate_UAV))
		{
			FlagsString += TEXT(" UAV");
		}

		if(EnumHasAnyFlags(LocalFlags, TexCreate_FastVRAM))
		{
			FlagsString += TEXT(" VRam");
		}

		FString ArrayString;

		if(IsArray())
		{
			ArrayString = FString::Printf(TEXT("[%d]"), ArraySize);
		}

		if(Is2DTexture())
		{
			return FString::Printf(TEXT("(2D%s %dx%d %s%s)"), *ArrayString, Extent.X, Extent.Y, FormatString, *FlagsString);
		}
		else if(Is3DTexture())
		{
			return FString::Printf(TEXT("(3D%s %dx%dx%d %s%s)"), *ArrayString, Extent.X, Extent.Y, Depth, FormatString, *FlagsString);
		}
		else if(IsCubemap())
		{
			return FString::Printf(TEXT("(Cube%s %d %s%s)"), *ArrayString, Extent.X, FormatString, *FlagsString);
		}
		else
		{
			return TEXT("(INVALID)");
		}
	}

	// useful when compositing graph takes an input as output format
	void Reset()
	{
		// Usually we don't want to propagate MSAA samples.
		NumSamples = 1;

		// Remove UAV flag for rendertargets that don't need it (some formats are incompatible)
		Flags |= TexCreate_RenderTargetable;
		Flags &= (~TexCreate_UAV);
	}

	/** only set a pointer to memory that never gets released */
	const TCHAR* DebugName = TEXT("UnknownTexture");
	/** Value allowed for fast clears for this target. */
	FClearValueBinding ClearValue;
	/** The flags that must be set on both the shader-resource and the targetable texture. bit mask combined from elements of ETextureCreateFlags e.g. TexCreate_UAV */
	ETextureCreateFlags Flags = TexCreate_None;
	/** Texture format e.g. PF_B8G8R8A8 */
	EPixelFormat Format = PF_Unknown;
	/** Texture format used when creating the UAV (if TexCreate_UAV is also passed in TargetableFlags, ignored otherwise). PF_Unknown == use default (same as Format) */
	EPixelFormat UAVFormat = PF_Unknown;
	/** In pixels, (0,0) if not set, (x,0) for cube maps, todo: make 3d int vector for volume textures */
	FIntPoint Extent = FIntPoint::ZeroValue;
	/** 0, unless it's texture array or volume texture */
	uint16 Depth = 0;
	/** >1 if a texture array should be used (not supported on DX9) */
	uint16 ArraySize = 1;
	/** Number of mips */
	uint8 NumMips = 0;
	/** Number of MSAA samples, default: 1  */
	uint8 NumSamples = 1;
	/** Resource memory percentage which should be allocated onto fast VRAM (hint-only). (encoding into 8bits, 0..255 -> 0%..100%) */
	uint8 FastVRAMPercentage = 0xFF;
	/** Formats this texture is aliasable to */
	TArray<EPixelFormat, TInlineAllocator<1>> AliasableFormats;

	union
	{
		struct
		{
			/** true if an array texture. Note that ArraySize still can be 1 */
			uint8 bIsArray : 1;
			/** true if a cubemap texture */
			uint8 bIsCubemap : 1;
			/** Unused flags. */
			uint8 bReserved0 : 6;
		};
		uint8 PackedBits;
	};
};

/**
 * Single render target item consists of a render surface and its resolve texture, Render thread side
 */
struct FSceneRenderTargetItem
{
	/** default constructor */
	FSceneRenderTargetItem() {}

	/** constructor */
	FSceneRenderTargetItem(FRHITexture* InTargetableTexture, FRHITexture* InShaderResourceTexture, FUnorderedAccessViewRHIRef InUAV)
		:	TargetableTexture(InTargetableTexture)
		,	ShaderResourceTexture(InShaderResourceTexture)
		,	UAV(InUAV)
	{}

	void SafeRelease()
	{
		TargetableTexture.SafeRelease();
		ShaderResourceTexture.SafeRelease();
		UAV.SafeRelease();
	}

	bool IsValid() const
	{
		return TargetableTexture != 0
			|| ShaderResourceTexture != 0
			|| UAV != 0;
	}

	FRHITexture* GetRHI() const { return TargetableTexture; }

	/** The 2D or cubemap texture that may be used as a render or depth-stencil target. */
	FTextureRHIRef TargetableTexture;

	/** The 2D or cubemap shader-resource 2D texture that the targetable textures may be resolved to. */
	FTextureRHIRef ShaderResourceTexture;
	
	/** only created if requested through the flag. */
	FUnorderedAccessViewRHIRef UAV;
};

/**
 * Render thread side, use TRefCountPtr<IPooledRenderTarget>, allows sharing and VisualizeTexture
 */
struct IPooledRenderTarget
{
	virtual ~IPooledRenderTarget() = default;

	/** Checks if the reference count indicated that the rendertarget is unused and can be reused. */
	virtual bool IsFree() const = 0;
	
	/** Get all the data that is needed to create the render target. */
	virtual const FPooledRenderTargetDesc& GetDesc() const = 0;
	
	/**
	 * Only for debugging purpose
	 * @return in bytes
	 **/
	virtual uint32 ComputeMemorySize() const = 0;

	/** Returns if the render target is tracked by a pool. */
	virtual bool IsTracked() const = 0;

	/** Returns a transient texture if this is a container for one. */
	virtual FRHITransientTexture* GetTransientTexture() const { return nullptr; }

	// Refcounting
	virtual uint32 AddRef() const = 0;
	virtual uint32 Release() = 0;
	virtual uint32 GetRefCount() const = 0;

	FRHITexture* GetRHI() const { return RenderTargetItem.TargetableTexture; }

protected:
	/** The internal references to the created render target */
	FSceneRenderTargetItem RenderTargetItem;
};

/** Creates an untracked pooled render target from an RHI texture. */
extern RENDERCORE_API TRefCountPtr<IPooledRenderTarget> CreateRenderTarget(FRHITexture* Texture, const TCHAR* Name);

/** Creates an untracked pooled render target from the RHI texture, but only if the pooled render target is
 *  empty or doesn't match the input texture. If the pointer already exists and points at the input texture,
 *  the function just returns. Useful to cache a pooled render target for an RHI texture. Returns true if the
 *  render target was created, or false if it was reused.
 */
extern RENDERCORE_API bool CacheRenderTarget(FRHITexture* Texture, const TCHAR* Name, TRefCountPtr<IPooledRenderTarget>& OutPooledRenderTarget);
