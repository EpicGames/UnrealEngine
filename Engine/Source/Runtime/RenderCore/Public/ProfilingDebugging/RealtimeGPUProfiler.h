// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/StaticArray.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "GpuProfilerTrace.h"
#include "HAL/CriticalSection.h"

#include "RHI.h"
#include "RHIDefinitions.h"
#include "RHICommandList.h"
#include "RHIBreadcrumbs.h"
#include "RenderingThread.h"
#include "Misc/Build.h"

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8
#include "MultiGPU.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "ProfilingDebugging/CsvProfilerConfig.h"
#include "Stats/Stats.h"
#include "UObject/NameTypes.h"
#include <tuple>
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8

#define WANTS_DRAW_MESH_EVENTS (WITH_PROFILEGPU && WITH_RHI_BREADCRUMBS)

#if WITH_RHI_BREADCRUMBS

	struct FRHIBreadcrumbScope_GameThread
	{
	private:
		TOptional<FRHIBreadcrumbScope>* Event;

	public:
		template <typename TDesc, typename... TValues>
		FRHIBreadcrumbScope_GameThread(TRHIBreadcrumbInitializer<TDesc, TValues...>&& Args)
			: Event(new TOptional<FRHIBreadcrumbScope>)
		{
			check(IsInGameThread());

			ENQUEUE_RENDER_COMMAND(FRHIBreadcrumbScope_GameThread_Begin)(
			[
				Event = Event,
				Args = MoveTemp(Args)
			](FRHICommandListImmediate& RHICmdList) mutable
			{
				Event->Emplace(RHICmdList, MoveTemp(Args));
			});
		}

		~FRHIBreadcrumbScope_GameThread()
		{
			check(IsInGameThread());

			ENQUEUE_RENDER_COMMAND(FRHIBreadcrumbScope_GameThread_End)([Event = Event](FRHICommandListImmediate&)
			{
				delete Event;
			});
		}
	};

	#define RHI_BREADCRUMB_EVENT_GAMETHREAD_PRIVATE_IMPL(Stat, Condition, StaticName, Format, ...)	\
		TOptional<FRHIBreadcrumbScope_GameThread> UE_JOIN(BreadcrumbScope, __LINE__);				\
		do																							\
		{																							\
			if (Condition)																			\
			{																						\
				UE_JOIN(BreadcrumbScope, __LINE__).Emplace(											\
					RHI_BREADCRUMB_DESC_COPY_VALUES(												\
						  StaticName																\
						, Format																	\
						, Stat																		\
					)(__VA_ARGS__)																	\
				);																					\
			}																						\
		} while(false)

	// Note, the varargs are deprecated and ignored in these two macros.
	#define RHI_BREADCRUMB_EVENT_GAMETHREAD(                         StaticName,         ...) RHI_BREADCRUMB_EVENT_GAMETHREAD_PRIVATE_IMPL(RHI_GPU_STAT_ARGS_NONE,      true, TEXT(StaticName),      nullptr, ##__VA_ARGS__)
	#define RHI_BREADCRUMB_EVENT_CONDITIONAL_GAMETHREAD(  Condition, StaticName,         ...) RHI_BREADCRUMB_EVENT_GAMETHREAD_PRIVATE_IMPL(RHI_GPU_STAT_ARGS_NONE, Condition, TEXT(StaticName),      nullptr, ##__VA_ARGS__)

	// Format versions of the breadcrumb macros.
	#define RHI_BREADCRUMB_EVENT_GAMETHREAD_F(                       StaticName, Format, ...) RHI_BREADCRUMB_EVENT_GAMETHREAD_PRIVATE_IMPL(RHI_GPU_STAT_ARGS_NONE,      true, TEXT(StaticName), TEXT(Format), ##__VA_ARGS__)
	#define RHI_BREADCRUMB_EVENT_CONDITIONAL_GAMETHREAD_F(Condition, StaticName, Format, ...) RHI_BREADCRUMB_EVENT_GAMETHREAD_PRIVATE_IMPL(RHI_GPU_STAT_ARGS_NONE, Condition, TEXT(StaticName), TEXT(Format), ##__VA_ARGS__)

#else

	#define RHI_BREADCRUMB_EVENT_GAMETHREAD(...)                do { } while(0)
	#define RHI_BREADCRUMB_EVENT_CONDITIONAL_GAMETHREAD(...)    do { } while(0)
	#define RHI_BREADCRUMB_EVENT_GAMETHREAD_F(...)			    do { } while(0)
	#define RHI_BREADCRUMB_EVENT_CONDITIONAL_GAMETHREAD_F(...)  do { } while(0)

#endif

// Macros to allow for scoping of draw events outside of RHI function implementations
// Render-thread event macros:
#define SCOPED_DRAW_EVENT(RHICmdList, Name)                                      RHI_BREADCRUMB_EVENT(RHICmdList, #Name);
#define SCOPED_DRAW_EVENTF(RHICmdList, Name, Format, ...)                        RHI_BREADCRUMB_EVENT_F_STR_DEPRECATED(RHICmdList, #Name, Format, ##__VA_ARGS__);
#define SCOPED_CONDITIONAL_DRAW_EVENT(RHICmdList, Name, Condition)               RHI_BREADCRUMB_EVENT_CONDITIONAL(RHICmdList, Condition, #Name);
#define SCOPED_CONDITIONAL_DRAW_EVENTF(RHICmdList, Name, Condition, Format, ...) RHI_BREADCRUMB_EVENT_F_CONDITIONAL_STR_DEPRECATED(RHICmdList, Condition, #Name, Format, ##__VA_ARGS__);

#if HAS_GPU_STATS

	#define DECLARE_GPU_STAT_NAME_TYPE(StatName, NameString) \
		struct TRHIGPUStatNameProvider_##StatName            \
		{                                                    \
			static constexpr TCHAR const* GetDisplayName()   \
			{                                                \
				return NameString;                           \
			}                                                \
			static constexpr TCHAR const* GetStatName()      \
			{                                                \
				return TEXT(#StatName);                      \
			}                                                \
		}

	// Extern GPU stats are needed where a stat is used in multiple CPPs. Use the DECLARE_GPU_STAT_NAMED_EXTERN in the header and DEFINE_GPU_STAT in the CPPs
	#define DECLARE_GPU_STAT_NAMED(                StatName, NameString) DECLARE_GPU_STAT_NAME_TYPE(StatName, NameString); static UE::RHI::GPUProfiler::TGPUStat                    <TRHIGPUStatNameProvider_##StatName> GPUStat_##StatName
	#define DECLARE_GPU_STAT_NAMED_EXTERN(         StatName, NameString) DECLARE_GPU_STAT_NAME_TYPE(StatName, NameString); extern UE::RHI::GPUProfiler::TGPUStat                    <TRHIGPUStatNameProvider_##StatName> GPUStat_##StatName
	#define DECLARE_GPU_DRAWCALL_STAT_NAMED(       StatName, NameString) DECLARE_GPU_STAT_NAME_TYPE(StatName, NameString); static UE::RHI::GPUProfiler::TGPUStatWithDrawcallCategory<TRHIGPUStatNameProvider_##StatName> GPUStat_##StatName
	#define DECLARE_GPU_DRAWCALL_STAT_NAMED_EXTERN(StatName, NameString) DECLARE_GPU_STAT_NAME_TYPE(StatName, NameString); extern UE::RHI::GPUProfiler::TGPUStatWithDrawcallCategory<TRHIGPUStatNameProvider_##StatName> GPUStat_##StatName

	#define DECLARE_GPU_STAT(                StatName) DECLARE_GPU_STAT_NAMED(                StatName, TEXT(#StatName))
	#define DECLARE_GPU_STAT_EXTERN(         StatName) DECLARE_GPU_STAT_NAMED_EXTERN(         StatName, TEXT(#StatName))
	#define DECLARE_GPU_DRAWCALL_STAT(       StatName) DECLARE_GPU_DRAWCALL_STAT_NAMED(       StatName, TEXT(#StatName))
	#define DECLARE_GPU_DRAWCALL_STAT_EXTERN(StatName) DECLARE_GPU_DRAWCALL_STAT_NAMED_EXTERN(StatName, TEXT(#StatName))

	#define DEFINE_GPU_STAT(         StatName) UE::RHI::GPUProfiler::TGPUStat                    <TRHIGPUStatNameProvider_##StatName> GPUStat_##StatName
	#define DEFINE_GPU_DRAWCALL_STAT(StatName) UE::RHI::GPUProfiler::TGPUStatWithDrawcallCategory<TRHIGPUStatNameProvider_##StatName> GPUStat_##StatName

#else

	#define DECLARE_GPU_STAT_NAMED(...)
	#define DECLARE_GPU_STAT_NAMED_EXTERN(...)
	#define DECLARE_GPU_DRAWCALL_STAT_NAMED(...)
	#define DECLARE_GPU_DRAWCALL_STAT_NAMED_EXTERN(...)

	#define DECLARE_GPU_STAT(...)
	#define DECLARE_GPU_STAT_EXTERN(...)
	#define DECLARE_GPU_DRAWCALL_STAT(...)
	#define DECLARE_GPU_DRAWCALL_STAT_EXTERN(...)

	#define DEFINE_GPU_STAT(...)
	#define DEFINE_GPU_DRAWCALL_STAT(...)

#endif

#define SCOPED_GPU_STAT_VERBOSE(...) UE_DEPRECATED_MACRO(5.8, "The legacy GPU profiler has been removed from the engine. SCOPED_GPU_STAT_VERBOSE has been deprecated, and now does nothing. GPU stats are handled via the _STAT versions of RHI_BREADCRUMB_EVENT macros, e.g. RHI_BREADCRUMB_EVENT_STAT.")
#define SCOPED_GPU_STAT(...)         UE_DEPRECATED_MACRO(5.8, "The legacy GPU profiler has been removed from the engine. SCOPED_GPU_STAT"      " has been deprecated, and now does nothing. GPU stats are handled via the _STAT versions of RHI_BREADCRUMB_EVENT macros, e.g. RHI_BREADCRUMB_EVENT_STAT.")

#define GPU_STATS_BEGINFRAME(...)    UE_DEPRECATED_MACRO(5.8, "The legacy GPU profiler has been removed from the engine. GPU_STATS_BEGINFRAME"" does nothing. Remove uses of this macro. There is no replacement.")
#define GPU_STATS_ENDFRAME(...)      UE_DEPRECATED_MACRO(5.8, "The legacy GPU profiler has been removed from the engine. GPU_STATS_ENDFRAME"  " does nothing. Remove uses of this macro. There is no replacement.")
#define GPU_STATS_SUSPENDFRAME(...)  UE_DEPRECATED_MACRO(5.8, "The legacy GPU profiler has been removed from the engine. GPU_STATS_SUSPENDFRAME does nothing. Remove uses of this macro. There is no replacement.")
