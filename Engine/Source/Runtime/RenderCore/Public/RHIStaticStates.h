// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	RHIStaticStates.h: RHI static state template definition.
=============================================================================*/

#pragma once

#include "Stats/Stats.h"
#include "Async/TaskGraphInterfaces.h"
#include "Containers/StaticArray.h"
#include "RHI.h"
#include "RenderResource.h"
#include "Misc/ScopedEvent.h"
#include "Templates/UnrealTypeTraits.h"

/**
 * The base class of the static RHI state classes.
 */
template<typename TInitializerType, typename TRHIResourceType>
class TStaticStateRHI
{
	static_assert(TIsDerivedFrom<TRHIResourceType, FRHIResource>::IsDerived, "Argument TRHIResourceType must derive from FRHIResource");

public:
	static TRHIResourceType* GetRHI()
	{
		checkSlow(StaticResource.StateRHI);
		return StaticResource.StateRHI;
	};

private:
	class FStaticStateResource : public FRenderResource
	{
	public:
		TRefCountPtr<TRHIResourceType> StateRHI;

		virtual void InitRHI(FRHICommandListBase& RHICmdList) override
		{
			StateRHI = TInitializerType::CreateRHI();
		}

		virtual void ReleaseRHI() override
		{
			StateRHI.SafeRelease();
		}
	};

	static TGlobalResource<FStaticStateResource, FRenderResource::EInitPhase::Pre> StaticResource;
};

template<typename TInitializerType, typename TRHIResourceType>
TGlobalResource<typename TStaticStateRHI<TInitializerType, TRHIResourceType>::FStaticStateResource, FRenderResource::EInitPhase::Pre> TStaticStateRHI<TInitializerType, TRHIResourceType>::StaticResource;

/**
 * A static RHI sampler state resource.
 * TStaticSamplerState<...>::GetRHI() will return a sampler state with the desired settings.
 */
template<
	ESamplerFilter      Filter   = SF_Point,
	ESamplerAddressMode AddressU = AM_Clamp,
	ESamplerAddressMode AddressV = AM_Clamp,
	ESamplerAddressMode AddressW = AM_Clamp, 
	int32 MipBias = 0,
	// Note: setting to a different value than GSystemSettings.MaxAnisotropy is only supported in D3D11 (is that still true?)
	// A value of 0 will use GSystemSettings.MaxAnisotropy
	int32 MaxAnisotropy = 1,
	uint32 BorderColor = 0,
	/** Only supported in D3D11 */
	ESamplerCompareFunction SamplerComparisonFunction = SCF_Never
>
class TStaticSamplerState : public TStaticStateRHI<TStaticSamplerState<Filter, AddressU, AddressV, AddressW, MipBias, MaxAnisotropy, BorderColor, SamplerComparisonFunction>, FRHISamplerState>
{
public:
	static FSamplerStateRHIRef CreateRHI()
	{
		FSamplerStateInitializerRHI Initializer(Filter, AddressU, AddressV, AddressW, MipBias, MaxAnisotropy, 0, FLT_MAX, BorderColor, SamplerComparisonFunction);
		return RHICreateSamplerState(Initializer);
	}
};

/**
 * A static RHI rasterizer state resource.
 * TStaticRasterizerState<...>::GetRHI() will return a rasterizer state with the desired settings.
 */
template<ERasterizerFillMode FillMode = FM_Solid, ERasterizerCullMode CullMode = CM_None, ERasterizerDepthClipMode DepthClipMode = ERasterizerDepthClipMode::DepthClip, bool bEnableMSAA = true>
class TStaticRasterizerState : public TStaticStateRHI<TStaticRasterizerState<FillMode, CullMode, DepthClipMode, bEnableMSAA>, FRHIRasterizerState>
{
public:
	inline static FRasterizerStateRHIRef CreateRHI()
	{
		const FRasterizerStateInitializerRHI Initializer(FillMode, CullMode, 0.0f, 0.0f, DepthClipMode, bEnableMSAA);
		return RHICreateRasterizerState(Initializer);
	}
};

/** Given a fill and cull mode, returns a static rasterizer state. */
template<bool bEnableMSAA>
inline FRHIRasterizerState* GetStaticRasterizerState(ERasterizerFillMode FillMode,ERasterizerCullMode CullMode)
{
	constexpr ERasterizerDepthClipMode DepthClipMode = ERasterizerDepthClipMode::DepthClip;
	switch(FillMode)
	{
	default:
	case FM_Solid:
		switch(CullMode)
		{
		default:
		case CM_CW:   return TStaticRasterizerState<FM_Solid,CM_CW  , DepthClipMode, bEnableMSAA>::GetRHI();
		case CM_CCW:  return TStaticRasterizerState<FM_Solid,CM_CCW , DepthClipMode, bEnableMSAA>::GetRHI();
		case CM_None: return TStaticRasterizerState<FM_Solid,CM_None, DepthClipMode, bEnableMSAA>::GetRHI();
		};
		break;
	case FM_Wireframe:
		switch(CullMode)
		{
		default:
		case CM_CW:   return TStaticRasterizerState<FM_Wireframe,CM_CW  , DepthClipMode, bEnableMSAA>::GetRHI();
		case CM_CCW:  return TStaticRasterizerState<FM_Wireframe,CM_CCW , DepthClipMode, bEnableMSAA>::GetRHI();
		case CM_None: return TStaticRasterizerState<FM_Wireframe,CM_None, DepthClipMode, bEnableMSAA>::GetRHI();
		};
		break;
	case FM_Point:
		switch(CullMode)
		{
		default:
		case CM_CW:   return TStaticRasterizerState<FM_Point,CM_CW  , DepthClipMode, bEnableMSAA>::GetRHI();
		case CM_CCW:  return TStaticRasterizerState<FM_Point,CM_CCW , DepthClipMode, bEnableMSAA>::GetRHI();
		case CM_None: return TStaticRasterizerState<FM_Point,CM_None, DepthClipMode, bEnableMSAA>::GetRHI();
		};
		break;
	}
}

/**
 * A static RHI depth stencil state resource.
 * TStaticDepthStencilState<...>::GetRHI() will return a depth stencil state with the desired settings.
 */
template<
	bool             bEnableDepthWrite             = true,
	ECompareFunction DepthTest                     = CF_DepthNearOrEqual,
	bool             bEnableFrontFaceStencil       = false,
	ECompareFunction FrontFaceStencilTest          = CF_Always,
	EStencilOp       FrontFaceStencilFailStencilOp = SO_Keep,
	EStencilOp       FrontFaceDepthFailStencilOp   = SO_Keep,
	EStencilOp       FrontFacePassStencilOp        = SO_Keep,
	bool             bEnableBackFaceStencil        = false,
	ECompareFunction BackFaceStencilTest           = CF_Always,
	EStencilOp       BackFaceStencilFailStencilOp  = SO_Keep,
	EStencilOp       BackFaceDepthFailStencilOp    = SO_Keep,
	EStencilOp       BackFacePassStencilOp         = SO_Keep,
	uint8            StencilReadMask               = 0xFF,
	uint8            StencilWriteMask              = 0xFF
	>
class TStaticDepthStencilState : public TStaticStateRHI<
	TStaticDepthStencilState<
		bEnableDepthWrite,
		DepthTest,
		bEnableFrontFaceStencil,
		FrontFaceStencilTest,
		FrontFaceStencilFailStencilOp,
		FrontFaceDepthFailStencilOp,
		FrontFacePassStencilOp,
		bEnableBackFaceStencil,
		BackFaceStencilTest,
		BackFaceStencilFailStencilOp,
		BackFaceDepthFailStencilOp,
		BackFacePassStencilOp,
		StencilReadMask,
		StencilWriteMask
		>,
	FRHIDepthStencilState
	>
{
public:
	static FDepthStencilStateRHIRef CreateRHI()
	{
		FDepthStencilStateInitializerRHI Initializer(
			bEnableDepthWrite,
			DepthTest,
			bEnableFrontFaceStencil,
			FrontFaceStencilTest,
			FrontFaceStencilFailStencilOp,
			FrontFaceDepthFailStencilOp,
			FrontFacePassStencilOp,
			bEnableBackFaceStencil,
			BackFaceStencilTest,
			BackFaceStencilFailStencilOp,
			BackFaceDepthFailStencilOp,
			BackFacePassStencilOp,
			StencilReadMask,
			StencilWriteMask);

		return RHICreateDepthStencilState(Initializer);
	}
};

/**
 * A static RHI blend state resource.
 * TStaticBlendState<...>::GetRHI() will return a blend state with the desired settings.
 * 
 * Alpha blending happens on GPU's as:
 * FinalColor.rgb = SourceColor * ColorSrcBlend (ColorBlendOp) DestColor * ColorDestBlend;
 * if (BlendState->bSeparateAlphaBlendEnable)
 *		FinalColor.a = SourceAlpha * AlphaSrcBlend (AlphaBlendOp) DestAlpha * AlphaDestBlend;
 * else
 *		Alpha blended the same way as rgb
 * 
 * Where source is the color coming from the pixel shader, and target is the color in the render target.
 *
 * So for example, TStaticBlendState<BO_Add,BF_SourceAlpha,BF_InverseSourceAlpha,BO_Add,BF_Zero,BF_One> produces:
 * FinalColor.rgb = SourceColor * SourceAlpha + DestColor * (1 - SourceAlpha);
 * FinalColor.a = SourceAlpha * 0 + DestAlpha * 1;
 */
template<
	EColorWriteMask RT0ColorWriteMask   = CW_RGBA,
	EBlendOperation RT0ColorBlendOp     = BO_Add,
	EBlendFactor    RT0ColorSrcBlend    = BF_One,
	EBlendFactor    RT0ColorDestBlend   = BF_Zero,
	EBlendOperation RT0AlphaBlendOp     = BO_Add,
	EBlendFactor    RT0AlphaSrcBlend    = BF_One,
	EBlendFactor    RT0AlphaDestBlend   = BF_Zero,
	EColorWriteMask RT1ColorWriteMask   = CW_RGBA,
	EBlendOperation RT1ColorBlendOp     = BO_Add,
	EBlendFactor    RT1ColorSrcBlend    = BF_One,
	EBlendFactor    RT1ColorDestBlend   = BF_Zero,
	EBlendOperation RT1AlphaBlendOp     = BO_Add,
	EBlendFactor    RT1AlphaSrcBlend    = BF_One,
	EBlendFactor    RT1AlphaDestBlend   = BF_Zero,
	EColorWriteMask RT2ColorWriteMask   = CW_RGBA,
	EBlendOperation RT2ColorBlendOp     = BO_Add,
	EBlendFactor    RT2ColorSrcBlend    = BF_One,
	EBlendFactor    RT2ColorDestBlend   = BF_Zero,
	EBlendOperation RT2AlphaBlendOp     = BO_Add,
	EBlendFactor    RT2AlphaSrcBlend    = BF_One,
	EBlendFactor    RT2AlphaDestBlend   = BF_Zero,
	EColorWriteMask RT3ColorWriteMask   = CW_RGBA,
	EBlendOperation RT3ColorBlendOp     = BO_Add,
	EBlendFactor    RT3ColorSrcBlend    = BF_One,
	EBlendFactor    RT3ColorDestBlend   = BF_Zero,
	EBlendOperation RT3AlphaBlendOp     = BO_Add,
	EBlendFactor    RT3AlphaSrcBlend    = BF_One,
	EBlendFactor    RT3AlphaDestBlend   = BF_Zero,
	EColorWriteMask RT4ColorWriteMask   = CW_RGBA,
	EBlendOperation RT4ColorBlendOp     = BO_Add,
	EBlendFactor    RT4ColorSrcBlend    = BF_One,
	EBlendFactor    RT4ColorDestBlend   = BF_Zero,
	EBlendOperation RT4AlphaBlendOp     = BO_Add,
	EBlendFactor    RT4AlphaSrcBlend    = BF_One,
	EBlendFactor    RT4AlphaDestBlend   = BF_Zero,
	EColorWriteMask RT5ColorWriteMask   = CW_RGBA,
	EBlendOperation RT5ColorBlendOp     = BO_Add,
	EBlendFactor    RT5ColorSrcBlend    = BF_One,
	EBlendFactor    RT5ColorDestBlend   = BF_Zero,
	EBlendOperation RT5AlphaBlendOp     = BO_Add,
	EBlendFactor    RT5AlphaSrcBlend    = BF_One,
	EBlendFactor    RT5AlphaDestBlend   = BF_Zero,
	EColorWriteMask RT6ColorWriteMask   = CW_RGBA,
	EBlendOperation RT6ColorBlendOp     = BO_Add,
	EBlendFactor    RT6ColorSrcBlend    = BF_One,
	EBlendFactor    RT6ColorDestBlend   = BF_Zero,
	EBlendOperation RT6AlphaBlendOp     = BO_Add,
	EBlendFactor    RT6AlphaSrcBlend    = BF_One,
	EBlendFactor    RT6AlphaDestBlend   = BF_Zero,
	EColorWriteMask RT7ColorWriteMask   = CW_RGBA,
	EBlendOperation RT7ColorBlendOp     = BO_Add,
	EBlendFactor    RT7ColorSrcBlend    = BF_One,
	EBlendFactor    RT7ColorDestBlend   = BF_Zero,
	EBlendOperation RT7AlphaBlendOp     = BO_Add,
	EBlendFactor    RT7AlphaSrcBlend    = BF_One,
	EBlendFactor    RT7AlphaDestBlend   = BF_Zero,
	bool			bUseAlphaToCoverage = false
	>
class TStaticBlendState : public TStaticStateRHI<
	TStaticBlendState<
		RT0ColorWriteMask, RT0ColorBlendOp, RT0ColorSrcBlend, RT0ColorDestBlend, RT0AlphaBlendOp, RT0AlphaSrcBlend, RT0AlphaDestBlend,
		RT1ColorWriteMask, RT1ColorBlendOp, RT1ColorSrcBlend, RT1ColorDestBlend, RT1AlphaBlendOp, RT1AlphaSrcBlend, RT1AlphaDestBlend,
		RT2ColorWriteMask, RT2ColorBlendOp, RT2ColorSrcBlend, RT2ColorDestBlend, RT2AlphaBlendOp, RT2AlphaSrcBlend, RT2AlphaDestBlend,
		RT3ColorWriteMask, RT3ColorBlendOp, RT3ColorSrcBlend, RT3ColorDestBlend, RT3AlphaBlendOp, RT3AlphaSrcBlend, RT3AlphaDestBlend,
		RT4ColorWriteMask, RT4ColorBlendOp, RT4ColorSrcBlend, RT4ColorDestBlend, RT4AlphaBlendOp, RT4AlphaSrcBlend, RT4AlphaDestBlend,
		RT5ColorWriteMask, RT5ColorBlendOp, RT5ColorSrcBlend, RT5ColorDestBlend, RT5AlphaBlendOp, RT5AlphaSrcBlend, RT5AlphaDestBlend,
		RT6ColorWriteMask, RT6ColorBlendOp, RT6ColorSrcBlend, RT6ColorDestBlend, RT6AlphaBlendOp, RT6AlphaSrcBlend, RT6AlphaDestBlend,
		RT7ColorWriteMask, RT7ColorBlendOp, RT7ColorSrcBlend, RT7ColorDestBlend, RT7AlphaBlendOp, RT7AlphaSrcBlend, RT7AlphaDestBlend,
		bUseAlphaToCoverage
		>,
	FRHIBlendState
	>
{
public:
	static FBlendStateRHIRef CreateRHI()
	{
		TStaticArray<FBlendStateInitializerRHI::FRenderTarget, 8> RenderTargetBlendStates;
		RenderTargetBlendStates[0] = FBlendStateInitializerRHI::FRenderTarget(RT0ColorBlendOp, RT0ColorSrcBlend, RT0ColorDestBlend, RT0AlphaBlendOp, RT0AlphaSrcBlend, RT0AlphaDestBlend, RT0ColorWriteMask);
		RenderTargetBlendStates[1] = FBlendStateInitializerRHI::FRenderTarget(RT1ColorBlendOp, RT1ColorSrcBlend, RT1ColorDestBlend, RT1AlphaBlendOp, RT1AlphaSrcBlend, RT1AlphaDestBlend, RT1ColorWriteMask);
		RenderTargetBlendStates[2] = FBlendStateInitializerRHI::FRenderTarget(RT2ColorBlendOp, RT2ColorSrcBlend, RT2ColorDestBlend, RT2AlphaBlendOp, RT2AlphaSrcBlend, RT2AlphaDestBlend, RT2ColorWriteMask);
		RenderTargetBlendStates[3] = FBlendStateInitializerRHI::FRenderTarget(RT3ColorBlendOp, RT3ColorSrcBlend, RT3ColorDestBlend, RT3AlphaBlendOp, RT3AlphaSrcBlend, RT3AlphaDestBlend, RT3ColorWriteMask);
		RenderTargetBlendStates[4] = FBlendStateInitializerRHI::FRenderTarget(RT4ColorBlendOp, RT4ColorSrcBlend, RT4ColorDestBlend, RT4AlphaBlendOp, RT4AlphaSrcBlend, RT4AlphaDestBlend, RT4ColorWriteMask);
		RenderTargetBlendStates[5] = FBlendStateInitializerRHI::FRenderTarget(RT5ColorBlendOp, RT5ColorSrcBlend, RT5ColorDestBlend, RT5AlphaBlendOp, RT5AlphaSrcBlend, RT5AlphaDestBlend, RT5ColorWriteMask);
		RenderTargetBlendStates[6] = FBlendStateInitializerRHI::FRenderTarget(RT6ColorBlendOp, RT6ColorSrcBlend, RT6ColorDestBlend, RT6AlphaBlendOp, RT6AlphaSrcBlend, RT6AlphaDestBlend, RT6ColorWriteMask);
		RenderTargetBlendStates[7] = FBlendStateInitializerRHI::FRenderTarget(RT7ColorBlendOp, RT7ColorSrcBlend, RT7ColorDestBlend, RT7AlphaBlendOp, RT7AlphaSrcBlend, RT7AlphaDestBlend, RT7ColorWriteMask);

		return RHICreateBlendState(FBlendStateInitializerRHI(RenderTargetBlendStates, bUseAlphaToCoverage));
	}
};


/**
 * A static RHI blend state resource which only allows controlling MRT write masks, for use when only opaque blending is needed.
 */
template<
	EColorWriteMask RT0ColorWriteMask   = CW_RGBA,
	EColorWriteMask RT1ColorWriteMask   = CW_RGBA,
	EColorWriteMask RT2ColorWriteMask   = CW_RGBA,
	EColorWriteMask RT3ColorWriteMask   = CW_RGBA,
	EColorWriteMask RT4ColorWriteMask   = CW_RGBA,
	EColorWriteMask RT5ColorWriteMask   = CW_RGBA,
	EColorWriteMask RT6ColorWriteMask   = CW_RGBA,
	EColorWriteMask RT7ColorWriteMask   = CW_RGBA,
	bool			bUseAlphaToCoverage = false
	>
class TStaticBlendStateWriteMask : public TStaticBlendState<
	RT0ColorWriteMask, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero,
	RT1ColorWriteMask, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero,
	RT2ColorWriteMask, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero,
	RT3ColorWriteMask, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero,
	RT4ColorWriteMask, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero,
	RT5ColorWriteMask, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero,
	RT6ColorWriteMask, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero,
	RT7ColorWriteMask, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero,
	bUseAlphaToCoverage
	>
{
public:
	static FBlendStateRHIRef CreateRHI()
	{
		return TStaticBlendState<
			RT0ColorWriteMask, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero,
			RT1ColorWriteMask, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero,
			RT2ColorWriteMask, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero,
			RT3ColorWriteMask, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero,
			RT4ColorWriteMask, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero,
			RT5ColorWriteMask, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero,
			RT6ColorWriteMask, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero,
			RT7ColorWriteMask, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero,
			bUseAlphaToCoverage
			>
			::CreateRHI();
	}
};
