// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	RayGenShaderUtils.h: Utilities for ray generation shaders shaders.
=============================================================================*/

#pragma once

#include "RHIDefinitions.h"

#if RHI_RAYTRACING
#include "RenderResource.h"
#include "RenderGraphUtils.h"
#include "PipelineStateCache.h"
#include "ShaderParameters.h"

/** All utils for ray generation shaders. */
struct FRayGenShaderUtils
{
	/** Dispatch a ray generation shader to render graph builder with its parameters. */
	template<typename TShaderClass>
	static inline void AddRayTraceDispatchPass(
		FRDGBuilder& GraphBuilder,
		FRDGEventName&& PassName,
		const TShaderRef<TShaderClass>& RayGenerationShader,
		typename TShaderClass::FParameters* Parameters,
		FIntPoint Resolution)
	{
		ClearUnusedGraphResources(RayGenerationShader, Parameters);

		GraphBuilder.AddPass(
			Forward<FRDGEventName>(PassName),
			Parameters,
			ERDGPassFlags::Compute,
			[RayGenerationShader, Parameters, Resolution](FRDGAsyncTask, FRHICommandList& RHICmdList)
		{
			FRHIBatchedShaderParameters& GlobalResources = RHICmdList.GetScratchShaderParameters();
			SetShaderParameters(GlobalResources, RayGenerationShader, *Parameters);

			FRayTracingPipelineStateInitializer Initializer;
			FRHIRayTracingShader* RayGenShaderTable[] = { RayGenerationShader.GetRayTracingShader() };
			Initializer.SetRayGenShaderTable(RayGenShaderTable);

			FRayTracingPipelineState* Pipeline = PipelineStateCache::GetAndOrCreateRayTracingPipelineState(RHICmdList, Initializer);
			RHICmdList.RayTraceDispatch(Pipeline, RayGenerationShader.GetRayTracingShader(), GlobalResources, Resolution.X, Resolution.Y);
		});
	}
};
#endif //RHI_RAYTRACINGO