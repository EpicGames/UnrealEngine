// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RayTracingGeometryManagerTypes.h"
#include "Containers/DynamicRHIResourceArray.h"
#include "RenderResource.h"
#include "RHI.h"

class FRHICommandListBase;

namespace RayTracing
{
	RENDERCORE_API bool ShouldForceRuntimeBLAS();
}

enum class ERTAccelerationStructureBuildPriority : uint8
{
	Immediate,
	High,
	Normal,
	Low,
	Skip
};

/** A ray tracing geometry resource */
class FRayTracingGeometry : public FRenderResource
{
public:
	FRayTracingGeometryOfflineDataHeader RawDataHeader;
	TResourceArray<uint8> RawData;

#if RHI_RAYTRACING

	/** When set to NonSharedVertexBuffers, then shared vertex buffers are not used  */
	static constexpr int64 NonSharedVertexBuffers = -1;

	/**
	Vertex buffers for dynamic geometries may be sub-allocated from a shared pool, which is periodically reset and its generation ID is incremented.
	Geometries that use the shared buffer must be updated (rebuilt or refit) before they are used for rendering after the pool is reset.
	This is validated by comparing the current shared pool generation ID against generation IDs stored in FRayTracingGeometry during latest update.
	*/
	int64 DynamicGeometrySharedBufferGenerationID = NonSharedVertexBuffers;

	// Last frame when geometry was updated (only dynamic geometry)
	uint64 LastUpdatedFrame = 0;

	// How many updates since the last build (only dynamic geometry)
	uint32 NumUpdatesSinceLastBuild = 0;

	UE_DEPRECATED(5.8, "Use SetInitializer(...) / GetInitializer() instead.")
	FRayTracingGeometryInitializer Initializer;

	FRHIRayTracingGeometry* GetRHI() const
	{
		return RayTracingGeometryRHI;
	}

	/**
	When LODIndex is set to AdditionalGeometryLODIndex, the geometry is added to the group as an "additional geometry".
	"Additional geometries" skips LOD-based streaming/eviction logic but still contribute to reference tracking and invalidation of cached ray tracing state in proxies.
	*/
	static constexpr int8 AdditionalGeometryLODIndex = INT8_MIN;

	RayTracing::FGeometryGroupHandle GroupHandle;

	/** LOD of the mesh associated with this ray tracing geometry object (-1 if unknown) */
	int8 LODIndex = -1;

	// Flags for tracking the state of RayTracingGeometryRHI.
	enum class EGeometryStateFlags : uint32
	{
		// Initial state when the geometry was not created or was created for streaming but not yet streamed in.
		Invalid = 0,

		// If the geometry needs to be built.
		RequiresBuild = 1 << 0,

		// If the geometry was successfully created or streamed in.
		Valid = 1 << 1,

		// Special flag that is used when ray tracing is dynamic to mark the streamed geometry to be recreated when ray tracing is switched on.
		// Only set when mesh streaming is used.
		StreamedIn = 1 << 2,

		// If the geometry is initialized but was evicted
		Evicted = 1 << 3,

		// If geometry requires an update (dynamic geometry only)
		RequiresUpdate = 1 << 4
	};
	FRIEND_ENUM_CLASS_FLAGS(EGeometryStateFlags);

	RENDERCORE_API void SetInitializer(FRayTracingGeometryInitializer InInitializer);
	const FRayTracingGeometryInitializer& GetInitializer() const
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return Initializer;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	RENDERCORE_API bool HasValidInitializer() const;
	RENDERCORE_API bool IsValid() const;
	RENDERCORE_API bool IsEvicted() const;

	void SetAsStreamedIn()
	{
		EnumAddFlags(GeometryState, EGeometryStateFlags::StreamedIn);
	}

	bool GetRequiresBuild() const
	{
		return EnumHasAnyFlags(GeometryState, EGeometryStateFlags::RequiresBuild);
	}

	void SetRequiresBuild(bool bBuild)
	{
		if (bBuild)
		{
			EnumAddFlags(GeometryState, EGeometryStateFlags::RequiresBuild);
		}
		else
		{
			EnumRemoveFlags(GeometryState, EGeometryStateFlags::RequiresBuild);
		}
	}

	bool GetRequiresUpdate() const
	{
		return EnumHasAnyFlags(GeometryState, EGeometryStateFlags::RequiresUpdate);
	}

	void SetRequiresUpdate(bool bUpdate)
	{
		if (bUpdate)
		{
			EnumAddFlags(GeometryState, EGeometryStateFlags::RequiresUpdate);
		}
		else
		{
			EnumRemoveFlags(GeometryState, EGeometryStateFlags::RequiresUpdate);
		}
	}

	EGeometryStateFlags GetGeometryState() const
	{
		return GeometryState;
	}

	RENDERCORE_API void InitRHIForStreaming(FRHIRayTracingGeometry* IntermediateGeometry, FRHIResourceReplaceBatcher& Batcher);
	RENDERCORE_API void ReleaseRHIForStreaming(FRHIResourceReplaceBatcher& Batcher);

	RENDERCORE_API void RequestBuildIfNeeded(FRHICommandListBase& RHICmdList, ERTAccelerationStructureBuildPriority InBuildPriority);

	RENDERCORE_API void CreateRayTracingGeometry(FRHICommandListBase& RHICmdList, ERTAccelerationStructureBuildPriority InBuildPriority);

	RENDERCORE_API void MakeResident(FRHICommandList& RHICmdList);
	RENDERCORE_API void Evict();
	
	RENDERCORE_API bool HasPendingBuildRequest() const;

	RENDERCORE_API void BoostBuildPriority(float InBoostValue = 0.01f) const;

	// FRenderResource interface

	virtual FString GetFriendlyName() const override { return TEXT("FRayTracingGeometry"); }

	RENDERCORE_API virtual void InitRHI(FRHICommandListBase& RHICmdList) override;
	RENDERCORE_API virtual void ReleaseRHI() override;

	RENDERCORE_API virtual void InitResource(FRHICommandListBase& RHICmdList) override;
	RENDERCORE_API virtual void ReleaseResource() override;

	RayTracing::FGeometryHandle GetGeometryHandle() const { return RayTracingGeometryHandle; }

protected:
	RENDERCORE_API void RemoveBuildRequest();

	FRayTracingGeometryRHIRef RayTracingGeometryRHI;

	EGeometryStateFlags GeometryState = EGeometryStateFlags::Invalid;

	RayTracing::FBuildRequestHandle RayTracingBuildRequestIndex;
	RayTracing::FGeometryHandle RayTracingGeometryHandle; // Only valid when ray tracing is dynamic

	friend class FRayTracingGeometryManager;
#endif
};

#if RHI_RAYTRACING
ENUM_CLASS_FLAGS(FRayTracingGeometry::EGeometryStateFlags);
#endif
