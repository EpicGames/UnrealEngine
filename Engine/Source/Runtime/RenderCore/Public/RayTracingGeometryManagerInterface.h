// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "RayTracingGeometryManagerTypes.h"
#include "RHI.h"
#include "Containers/ArrayView.h"

#if RHI_RAYTRACING

class FRayTracingGeometry;
class FRHICommandList;
class FRHIComputeCommandList;
enum class ERTAccelerationStructureBuildPriority : uint8;

class IRayTracingGeometryManager
{
public:

	using FBuildRequestIndex UE_DEPRECATED(5.8, "Use RayTracing::FBuildRequestHandle instead.") = RayTracing::FBuildRequestHandle;
	using FGeometryHandle  UE_DEPRECATED(5.8, "Use RayTracing::FGeometryHandle instead.") = RayTracing::FGeometryHandle;

	virtual ~IRayTracingGeometryManager() = default;

	virtual RayTracing::FBuildRequestHandle RequestBuildAccelerationStructure(FRayTracingGeometry* InGeometry, ERTAccelerationStructureBuildPriority InPriority, EAccelerationStructureBuildMode InBuildMode) = 0;

	RayTracing::FBuildRequestHandle RequestBuildAccelerationStructure(FRayTracingGeometry* InGeometry, ERTAccelerationStructureBuildPriority InPriority)
	{
		return RequestBuildAccelerationStructure(InGeometry, InPriority, EAccelerationStructureBuildMode::Build);
	}

	virtual void RemoveBuildRequest(RayTracing::FBuildRequestHandle RequestHandle) = 0;
	virtual void BoostPriority(RayTracing::FBuildRequestHandle RequestHandle, float InBoostValue) = 0;
	virtual void ForceBuildIfPending(FRHIComputeCommandList& InCmdList, const TArrayView<const FRayTracingGeometry*> InGeometries) = 0;
	virtual void ProcessBuildRequests(FRHIComputeCommandList& InCmdList, bool bInBuildAll = false) = 0;

	virtual RayTracing::FGeometryHandle RegisterRayTracingGeometry(FRayTracingGeometry* InGeometry) = 0;
	virtual void ReleaseRayTracingGeometryHandle(RayTracing::FGeometryHandle Handle) = 0;

	/*
	* RayTracing::FGeometryGroupHandle is used to group multiple FRayTracingGeometry that are associated with the same asset.
	* For example, the FRayTracingGeometry of all the LODs of UStaticMesh should use the same RayTracing::FGeometryGroupHandle.
	* This grouping is useful to keep track which proxies need to be invalidated when a FRayTracingGeometry is built or made resident.
	*/
	virtual RayTracing::FGeometryGroupHandle RegisterRayTracingGeometryGroup(uint32 NumLODs, uint32 CurrentFirstLODIdx = 0) = 0;
	virtual void ReleaseRayTracingGeometryGroup(RayTracing::FGeometryGroupHandle Handle) = 0;

	virtual void RequestUpdateCachedRenderState(RayTracing::FGeometryGroupHandle InRayTracingGeometryGroupHandle) = 0;

	virtual void RefreshRegisteredGeometry(RayTracing::FGeometryHandle Handle) = 0;

	virtual void PreRender() = 0;
	virtual void Update(FRHICommandList& RHICmdList) = 0;
	virtual void Tick(FRHICommandList& RHICmdList) = 0;

	virtual bool IsGeometryVisible(RayTracing::FGeometryHandle GeometryHandle) const = 0;
	virtual void AddVisibleGeometry(RayTracing::FGeometryHandle GeometryHandle) = 0;

protected:
	static const uint32 HandleIndexNumBits = 22;
	static const uint32 HandleIndexMask = (1 << HandleIndexNumBits) - 1;
	static const uint32 HandleVersionMask = (1 << (32 - HandleIndexNumBits)) - 1;

	static uint32 GetHandleVersion(const RayTracing::FGeometryManagerHandle& Handle)
	{
		return Handle.Internal >> HandleIndexNumBits;
	}

	static uint32 GetHandleIndex(const RayTracing::FGeometryManagerHandle& Handle)
	{
		return Handle.Internal & HandleIndexMask;
	}

	static RayTracing::FGeometryHandle MakeGeometryHandle(uint32 Version, uint32 Index)
	{
		check(Version <= HandleVersionMask && Index < HandleIndexMask);
		return RayTracing::FGeometryHandle((Version << HandleIndexNumBits) | (Index & HandleIndexMask));
	}

	static RayTracing::FGeometryGroupHandle MakeGeometryGroupHandle(uint32 Version, uint32 Index)
	{
		check(Version <= HandleVersionMask && Index < HandleIndexMask);
		return RayTracing::FGeometryGroupHandle((Version << HandleIndexNumBits) | (Index & HandleIndexMask));
	}

	static RayTracing::FBuildRequestHandle MakeBuildRequestHandle(uint32 Version, uint32 Index)
	{
		check(Version <= HandleVersionMask && Index < HandleIndexMask);
		return RayTracing::FBuildRequestHandle((Version << HandleIndexNumBits) | (Index & HandleIndexMask));
	}
};

extern RENDERCORE_API IRayTracingGeometryManager* GRayTracingGeometryManager;

#endif // RHI_RAYTRACING
