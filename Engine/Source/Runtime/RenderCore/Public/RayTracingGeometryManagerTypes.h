// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "RHIDefinitions.h"

#if RHI_RAYTRACING

class IRayTracingGeometryManager;
class FRayTracingScene;

namespace RayTracing
{
	struct FGeometryManagerHandle
	{
		FGeometryManagerHandle()
		{
		}

		bool IsNull() const
		{
			return Internal == UINT32_MAX;
		}

		inline friend uint32 GetTypeHash(const FGeometryManagerHandle& Handle)
		{
			return Handle.Internal;
		}

	protected:
		FGeometryManagerHandle(uint32 InInternal)
			: Internal(InInternal)
		{
		}

		friend bool operator==(const FGeometryManagerHandle&, const FGeometryManagerHandle&) = default;

		uint32 Internal = UINT32_MAX;

		friend class ::IRayTracingGeometryManager;
	};

	struct FGeometryHandle : public FGeometryManagerHandle
	{
		FGeometryHandle()
			: FGeometryManagerHandle()
		{
		}

		UE_DEPRECATED(5.8, "Use RayTracing::FGeometryHandle instead of int32.")
		FGeometryHandle(int32 InInternal)
			: FGeometryManagerHandle(InInternal)
		{
		}

		UE_DEPRECATED(5.8, "Use RayTracing::FGeometryHandle instead of int32.")
		bool operator==(int32 Handle) const
		{
			return Internal == Handle;
		}

		friend bool operator==(const FGeometryHandle&, const FGeometryHandle&) = default;
		bool operator==(const FGeometryManagerHandle&) const = delete;

	private:
		FGeometryHandle(uint32 InInternal)
			: FGeometryManagerHandle(InInternal)
		{
		}

		friend class ::IRayTracingGeometryManager;
		friend class ::FRayTracingScene;
	};

	struct FGeometryGroupHandle : public FGeometryManagerHandle
	{
		FGeometryGroupHandle()
			: FGeometryManagerHandle()
		{
		}

		UE_DEPRECATED(5.8, "Use RayTracing::FGeometryGroupHandle instead of int32.")
		FGeometryGroupHandle(int32 InInternal)
			: FGeometryManagerHandle(InInternal)
		{
		}

		UE_DEPRECATED(5.8, "Use RayTracing::FGeometryGroupHandle instead of int32.")
		bool operator==(int32 Handle) const
		{
			return Internal == Handle;
		}

		friend bool operator==(const FGeometryGroupHandle&, const FGeometryGroupHandle&) = default;
		bool operator==(const FGeometryManagerHandle&) const = delete;

	private:
		FGeometryGroupHandle(uint32 InInternal)
			: FGeometryManagerHandle(InInternal)
		{
		}

		friend class ::IRayTracingGeometryManager;
	};

	struct FBuildRequestHandle : public FGeometryManagerHandle
	{
		FBuildRequestHandle()
			: FGeometryManagerHandle()
		{
		}

		UE_DEPRECATED(5.8, "Use RayTracing::FBuildRequestHandle instead of int32.")
		FBuildRequestHandle(int32 InInternal)
			: FGeometryManagerHandle(InInternal)
		{
		}

		UE_DEPRECATED(5.8, "Use RayTracing::FBuildRequestHandle instead of int32.")
		bool operator==(int32 Handle) const
		{
			return Internal == Handle;
		}

		friend bool operator==(const FBuildRequestHandle&, const FBuildRequestHandle&) = default;
		bool operator==(const FGeometryManagerHandle&) const = delete;

	private:
		FBuildRequestHandle(uint32 InInternal)
			: FGeometryManagerHandle(InInternal)
		{
		}

		friend class ::IRayTracingGeometryManager;
	};
}

#endif // RHI_RAYTRACING
