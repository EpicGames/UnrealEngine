// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BuiltInRayTracingShaders.h"
#include "RHIDefinitions.h"
#include "ShaderCompilerCore.h"

#if RHI_RAYTRACING

class FRayTracingValidateGeometryBuildParamsCS : public FBuiltInRayTracingShader
{
	DECLARE_GLOBAL_SHADER(FRayTracingValidateGeometryBuildParamsCS);

public:
	FRayTracingValidateGeometryBuildParamsCS() = default;
	FRayTracingValidateGeometryBuildParamsCS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FBuiltInRayTracingShader(Initializer)
	{
		IndexBufferParam.Bind(Initializer.ParameterMap, TEXT("IndexBuffer"), SPF_Optional);
		VertexBufferParam.Bind(Initializer.ParameterMap, TEXT("VertexBuffer"), SPF_Optional);
		VertexBufferStrideParam.Bind(Initializer.ParameterMap, TEXT("VertexBufferStride"), SPF_Optional);
		VertexBufferOffsetInBytesParam.Bind(Initializer.ParameterMap, TEXT("VertexBufferOffsetInBytes"), SPF_Optional);
		IndexBufferOffsetInBytesParam.Bind(Initializer.ParameterMap, TEXT("IndexBufferOffsetInBytes"), SPF_Optional);
		IndexBufferStrideParam.Bind(Initializer.ParameterMap, TEXT("IndexBufferStride"), SPF_Optional);
		NumPrimitivesParam.Bind(Initializer.ParameterMap, TEXT("NumPrimitives"), SPF_Optional);
		MaxVerticesParam.Bind(Initializer.ParameterMap, TEXT("MaxVertices"), SPF_Optional);
	}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return FBuiltInRayTracingShader::ShouldCompilePermutation(Parameters) && RHISupportsWaveOperations(Parameters.Platform);
	}

	// Large thread group to handle large meshes with a single 1D dispatch
	static const uint32 NumThreadsX = 1024;

	static inline void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		OutEnvironment.CompilerFlags.Add(CFLAG_ForceDXC);
		OutEnvironment.SetDefine(TEXT("NUM_THREADS_X"), NumThreadsX);
		FBuiltInRayTracingShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
	}

	static RENDERCORE_API void Dispatch(FRHICommandList& RHICmdList, const FRayTracingGeometryBuildParams& Params);

	LAYOUT_FIELD(FShaderResourceParameter, IndexBufferParam);
	LAYOUT_FIELD(FShaderResourceParameter, VertexBufferParam);
	LAYOUT_FIELD(FShaderParameter, VertexBufferStrideParam);
	LAYOUT_FIELD(FShaderParameter, VertexBufferOffsetInBytesParam);
	LAYOUT_FIELD(FShaderParameter, IndexBufferOffsetInBytesParam);
	LAYOUT_FIELD(FShaderParameter, IndexBufferStrideParam);
	LAYOUT_FIELD(FShaderParameter, NumPrimitivesParam);
	LAYOUT_FIELD(FShaderParameter, MaxVerticesParam);

};

class FRayTracingValidateSceneBuildParamsCS : public FBuiltInRayTracingShader
{
	DECLARE_GLOBAL_SHADER(FRayTracingValidateSceneBuildParamsCS);

public:
	FRayTracingValidateSceneBuildParamsCS() = default;
	FRayTracingValidateSceneBuildParamsCS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FBuiltInRayTracingShader(Initializer)
	{
		InstanceBufferParam.Bind(Initializer.ParameterMap, TEXT("InstanceBuffer"), SPF_Optional);
		InstanceBufferOffsetInBytesParam.Bind(Initializer.ParameterMap, TEXT("InstanceBufferOffsetInBytes"), SPF_Optional);
		InstanceBufferStrideInBytesParam.Bind(Initializer.ParameterMap, TEXT("InstanceBufferStrideInBytes"), SPF_Optional);
		NumInstancesParam.Bind(Initializer.ParameterMap, TEXT("NumInstances"), SPF_Optional);
		NumHitGroupsParam.Bind(Initializer.ParameterMap, TEXT("NumHitGroups"), SPF_Optional);
	}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return FBuiltInRayTracingShader::ShouldCompilePermutation(Parameters) && RHISupportsWaveOperations(Parameters.Platform);
	}

	// Large thread group to handle large meshes with a single 1D dispatch
	static const uint32 NumThreadsX = 1024;

	static inline void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		OutEnvironment.CompilerFlags.Add(CFLAG_ForceDXC);
		OutEnvironment.SetDefine(TEXT("NUM_THREADS_X"), NumThreadsX);
		FBuiltInRayTracingShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
	}

	static RENDERCORE_API void Dispatch(FRHICommandList& RHICmdList, 
		uint32 NumHitGroups, uint32 NumInstances, 
		FRHIBuffer* InstanceBuffer, uint32 InstanceBufferOffset, uint32 InstanceBufferStride);

	LAYOUT_FIELD(FShaderResourceParameter, InstanceBufferParam);
	LAYOUT_FIELD(FShaderParameter, InstanceBufferOffsetInBytesParam);
	LAYOUT_FIELD(FShaderParameter, InstanceBufferStrideInBytesParam);
	LAYOUT_FIELD(FShaderParameter, NumInstancesParam);
	LAYOUT_FIELD(FShaderParameter, NumHitGroupsParam);
};


#endif // RHI_RAYTRACING
