// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ReadOnlyCVarCache.h: Cache of read-only console variables used by the renderer
=============================================================================*/

#pragma once
#include "RHIShaderPlatform.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "ShaderPlatformCachedIniValue.h"


struct FReadOnlyCVARCache
{
	static void Initialize();
		
	static inline bool AllowStaticLighting()
	{
		checkSlow(bInitialized);
		#ifdef PROJECT_CVAR_ALLOW_STATIC_LIGHTING
			return PROJECT_CVAR_ALLOW_STATIC_LIGHTING;
		#else	
			return bAllowStaticLighting;
		#endif	
	}

	static inline bool EnablePointLightShadows(const FStaticShaderPlatform Platform)
	{
		if (!IsMobilePlatform(Platform))
		{
			return bEnablePointLightShadows;
		}
		else
		{
			static FShaderPlatformCachedIniValue<bool> MobileMovablePointLightShadowsIniValue(TEXT("r.Mobile.EnableMovablePointLightsShadows"));
			return MobileMovablePointLightShadowsIniValue.Get(Platform) && FReadOnlyCVARCache::MobileSupportsGPUScene();
		}
	}
	
	static inline bool EnableStationarySkylight()
	{
		return bEnableStationarySkylight;
	}
	
	static inline bool EnableLowQualityLightmaps()
	{
		return bEnableLowQualityLightmaps;
	}

	static inline bool SupportSkyAtmosphere()
	{
		return bSupportSkyAtmosphere;
	}

	// Mobile specific
	static inline bool MobileHDR()
	{
		checkSlow(bInitialized);
		#ifdef PROJECT_CVAR_MOBILE_HDR
			return PROJECT_CVAR_MOBILE_HDR;
		#else
			return bMobileHDR;
		#endif	
	}

	static inline bool MobileSupportsGPUScene()
	{
		checkSlow(bInitialized);
		#ifdef PROJECT_CVAR_MOBILE_SUPPORTS_GPUSCENE
			return PROJECT_CVAR_MOBILE_SUPPORTS_GPUSCENE;
		#else
			return bMobileSupportsGPUScene;
		#endif
	}

	static inline bool MobileAllowDistanceFieldShadows()
	{
		return bMobileAllowDistanceFieldShadows;
	}
	
	static inline bool MobileEnableStaticAndCSMShadowReceivers()
	{
		return bMobileEnableStaticAndCSMShadowReceivers;
	}

	static inline bool MobileEnableMovableLightCSMShaderCulling()
	{
		return bMobileEnableMovableLightCSMShaderCulling;
	}
	
	static inline int32 MobileForwardDecalLighting()
	{
		return MobileForwardDecalLightingValue;
	}

	static inline int32 MobileEarlyZPass(EShaderPlatform Platform)
	{
		#if WITH_EDITOR
			return MobileEarlyZPassIniValue(Platform);
		#else
			return MobileEarlyZPassValue;
		#endif
	}
	
	static inline int32 MobileForwardLocalLights(EShaderPlatform Platform)
	{
		#if WITH_EDITOR
			return MobileForwardLocalLightsIniValue(Platform);
		#elif defined PROJECT_CVAR_MOBILE_FORWARD_LOCALLIGHTS
			return PROJECT_CVAR_MOBILE_FORWARD_LOCALLIGHTS;
		#else
			return MobileForwardLocalLightsValue;
		#endif
	}

	static inline int32 MobileForwardParticleLights(EShaderPlatform Platform)
	{
		#if WITH_EDITOR
				return MobileForwardParticleLightsIniValue(Platform);
		#elif defined PROJECT_CVAR_MOBILE_FORWARD_PARTICLELIGHTS
				return PROJECT_CVAR_MOBILE_FORWARD_PARTICLELIGHTS;
		#else
				return bMobileForwardParticleLights;
		#endif
	}
	
	static inline bool MobileDeferredShading(EShaderPlatform Platform)
	{
		#if WITH_EDITOR
			return MobileDeferredShadingIniValue(Platform);
		#elif defined PROJECT_CVAR_MOBILE_DEFERRED_SHADING
			return PROJECT_CVAR_MOBILE_DEFERRED_SHADING;
		#else
			return bMobileDeferredShadingValue;
		#endif
	}

	static inline bool MobileEnableMovableSpotlightsShadow(EShaderPlatform Platform)
	{
		#if WITH_EDITOR
			return MobileEnableMovableSpotlightsShadowIniValue(Platform);
		#else
			return bMobileEnableMovableSpotlightsShadowValue;
		#endif
	}

	static inline bool MobileAllowDitheredLODTransition(EShaderPlatform Platform)
	{
		#if WITH_EDITOR
			return MobileAllowDitheredLODTransitionValueIniValue(Platform);
		#else
			return bMobileAllowDitheredLODTransitionValue;
		#endif
	}

	static inline bool MobileAllowFramebufferFetch(EShaderPlatform Platform)
	{
		#if WITH_EDITOR
			return MobileAllowFramebufferFetchIniValue(Platform);
		#else
			return bMobileAllowFramebufferFetchValue;
		#endif
	}

private:
	RENDERCORE_API static bool bInitialized;

	RENDERCORE_API static bool bAllowStaticLighting;
	RENDERCORE_API static bool bEnablePointLightShadows;
	RENDERCORE_API static bool bEnableStationarySkylight;
	RENDERCORE_API static bool bEnableLowQualityLightmaps;
	RENDERCORE_API static bool bSupportSkyAtmosphere;

	// Mobile specific
	RENDERCORE_API static bool bMobileHDR;
	RENDERCORE_API static bool bMobileAllowDistanceFieldShadows;
	RENDERCORE_API static bool bMobileEnableStaticAndCSMShadowReceivers;
	RENDERCORE_API static bool bMobileEnableMovableLightCSMShaderCulling;
	RENDERCORE_API static bool bMobileSupportsGPUScene;
	RENDERCORE_API static int32 MobileEarlyZPassValue;
	RENDERCORE_API static int32 MobileForwardLocalLightsValue;
	RENDERCORE_API static bool bMobileForwardParticleLights;
	RENDERCORE_API static int32 MobileForwardDecalLightingValue;
	RENDERCORE_API static bool bMobileDeferredShadingValue;
	RENDERCORE_API static bool bMobileEnableMovableSpotlightsShadowValue;
	RENDERCORE_API static bool bMobileAllowDitheredLODTransitionValue;
	RENDERCORE_API static bool bMobileAllowFramebufferFetchValue;

private:
	RENDERCORE_API static int32 MobileEarlyZPassIniValue(EShaderPlatform Platform);
	RENDERCORE_API static int32 MobileForwardLocalLightsIniValue(EShaderPlatform Platform);
	RENDERCORE_API static bool MobileForwardParticleLightsIniValue(EShaderPlatform Platform);
	RENDERCORE_API static bool MobileDeferredShadingIniValue(EShaderPlatform Platform);
	RENDERCORE_API static bool MobileEnableMovableSpotlightsShadowIniValue(EShaderPlatform Platform);
	RENDERCORE_API static bool MobileAllowDitheredLODTransitionValueIniValue(EShaderPlatform Platform);
	RENDERCORE_API static bool MobileAllowFramebufferFetchIniValue(EShaderPlatform Platform);
};

