// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "RenderGraphEvent.h"
#include "RHIBreadcrumbs.h"

class FRDGBuilder;
class FRHICommandList;

/** Easy to use interface for IRenderCaptureProvider. */
namespace RenderCaptureInterface
{
	/** 
	 * Helper for capturing within a scope. 
	 * Handles both game and render thread. Fails gracefully if no IRenderCaptureProvider exists.
	 */
	class FScopedCapture
	{
	public:
		/** Use this constructor if not on rendering thread. Use bEnable to allow control over the capture frequency. */
		RENDERCORE_API FScopedCapture(bool bEnable, TCHAR const* InEventName = nullptr, TCHAR const* InFileName = nullptr);
		/** Use this constructor if on rendering thread. Use bEnable to allow control over the capture frequency. */
		RENDERCORE_API FScopedCapture(bool bEnable, FRHICommandList* InRHICommandList, TCHAR const* InEventName = nullptr, TCHAR const* InFileName = nullptr);
		/** Use this constructor if using RenderGraph to schedule work. Use bEnable to allow control over the capture frequency. */
		RENDERCORE_API FScopedCapture(bool bEnable, FRDGBuilder& InGraphBuilder, TCHAR const* InEventName = nullptr, TCHAR const* InFileName = nullptr);

		RENDERCORE_API ~FScopedCapture();
	
	private:
		bool bCapture;
		bool bEvent;
		FRHICommandList* RHICommandList;
		FRDGBuilder* GraphBuilder;
#if RDG_EVENTS
		TOptional<TRDGEventScopeGuard<FRDGScope_RHI>> RDGEvent;
#endif
#if WITH_RHI_BREADCRUMBS
		TUniquePtr<TOptional<FRHIBreadcrumbEventManual>> RHIBreadcrumb;
#endif
	};
}
