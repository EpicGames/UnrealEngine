// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Tasks/Task.h"

////////////////////////////////////
// Render fences
////////////////////////////////////

 /**
 * Used to track pending rendering commands from the game thread.
 */
class FRenderCommandFence
{
public:
	enum class ESyncDepth
	{
		// The fence will be signalled by the render thread.
		RenderThread,

		// The fence will be enqueued to the RHI thread via a command on the immediate command list
		// and signalled once all prior parallel translation and submission is complete.
		RHIThread,

		// The fence will be signalled according to the rate of flips in the swapchain.
		// This is only supported on some platforms. On unsupported platforms, this behaves like RHIThread mode.
		Swapchain
	};

	/**
	 * Inserts this fence in the rendering pipeline.
	 * @param SyncDepth, determines which stage of the pipeline will signal the fence.
	 */
	RENDERCORE_API void BeginFence(ESyncDepth SyncDepth = ESyncDepth::RenderThread);

	/**
	 * Waits for pending fence commands to retire.
	 * @param bProcessGameThreadTasks, if true we are on a short callstack where it is safe to process arbitrary game thread tasks while we wait
	 */
	RENDERCORE_API void Wait(bool bProcessGameThreadTasks = false) const;

	// return true if the fence is complete
	RENDERCORE_API bool IsFenceComplete() const;

	// Ctor/dtor
	RENDERCORE_API FRenderCommandFence();
	RENDERCORE_API ~FRenderCommandFence();

private:
	/** Task that represents completion of this fence **/
	mutable UE::Tasks::FTask CompletionTask;
};
