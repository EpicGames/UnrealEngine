// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Stats/Stats.h"

////////////////////////////////////////////////////////////////////////////////////////////////////

enum class ERenderCommandCategory : uint8
{
	EnterTick, // Only allowed to execute when entering a tick from previous tick. Typically creating resources
	LoopTick,  // Allowed to be executed multiple times if rendering is looping the same tick
	LeaveTick, // Only allowed to be executed when leaving a tick to a new one. Typically releasing resources

	Unknown,   // Unknown category. Will not work with state stream path. ALWAYS LAST
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FRenderCommandTag
{
public:
	const TCHAR* GetName() const
	{
		return Name;
	}

	TStatId GetStatId() const
	{
		return StatId;
	}

	uint32& GetSpecId() const
	{
		return SpecId;
	}

	ERenderCommandCategory GetCategory() const
	{
		return Category;
	}

protected:
	FRenderCommandTag(const TCHAR* InName, TStatId InStatId, ERenderCommandCategory InCategory)
		: Name(InName)
		, StatId(InStatId)
		, Category(InCategory)
	{}

private:
	const TCHAR* Name;
	TStatId StatId;
	ERenderCommandCategory Category;
	mutable uint32 SpecId;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

DECLARE_STATS_GROUP(TEXT("Render Thread Commands"), STATGROUP_RenderThreadCommands, STATCAT_Advanced);

////////////////////////////////////////////////////////////////////////////////////////////////////
/** Type that contains profiler data necessary to mark up render commands for various profilers. */

template <typename TSTR>
class TRenderCommandTag : public FRenderCommandTag
{
public:
	static const TRenderCommandTag& Get()
	{
#if STATS
		struct FStatData
		{
			typedef FStatGroup_STATGROUP_RenderThreadCommands TGroup;
			static inline const char* GetStatName()
			{
				return TSTR::CStr();
			}
			static inline const TCHAR* GetDescription()
			{
				return TSTR::TStr();
			}
			static inline EStatDataType::Type GetStatType()
			{
				return EStatDataType::ST_int64;
			}
			static inline bool IsClearEveryFrame()
			{
				return true;
			}
			static inline bool IsCycleStat()
			{
				return true;
			}
			static inline FPlatformMemory::EMemoryCounterRegion GetMemoryRegion()
			{
				return FPlatformMemory::MCR_Invalid;
			}
		};
		static FThreadSafeStaticStat<FStatData> Stat;
		static TRenderCommandTag Tag(Stat.GetStatId());
#else
		static TRenderCommandTag Tag({});
#endif

		return Tag;
	}

private:
	TRenderCommandTag(TStatId InStatId)
		: FRenderCommandTag(TSTR::TStr(), InStatId, TSTR::GetCategory())
	{}
};

////////////////////////////////////////////////////////////////////////////////////////////////////
/** Type that contains profiler data necessary to mark up render commands for various profilers. */

template <typename TSTR>
class TRenderCommandTagNoMarker : public FRenderCommandTag
{
public:
	static const TRenderCommandTagNoMarker& Get()
	{
		static TRenderCommandTagNoMarker Tag;
		return Tag;
	}

private:
	TRenderCommandTagNoMarker()
		: FRenderCommandTag(TSTR::TStr(), TStatId(), TSTR::GetCategory())
	{
	}
};

/** Declares a new render command tag type from a name. */
#define DECLARE_RENDER_COMMAND_TAG(Type, Name, ...) \
	struct UE_JOIN(TSTR_, Name, __LINE__) \
	{  \
		static const char* CStr() { return #Name; } \
		static const TCHAR* TStr() { return TEXT(#Name); } \
		static constexpr ERenderCommandCategory GetCategory() { return __VA_OPT__(ERenderCommandCategory::__VA_ARGS__;) ERenderCommandCategory::Unknown; } \
	}; \
	using Type = TRenderCommandTag<UE_JOIN(TSTR_, Name, __LINE__)>;

#define DECLARE_RENDER_COMMAND_TAG_NOMARKER(Type, Name, ...) \
	struct UE_JOIN(TSTR_, Name, __LINE__) \
	{  \
		static const char* CStr() { return #Name; } \
		static const TCHAR* TStr() { return TEXT(#Name); } \
		static constexpr ERenderCommandCategory GetCategory() { return __VA_OPT__(ERenderCommandCategory::__VA_ARGS__;) ERenderCommandCategory::Unknown; } \
	}; \
	using Type = TRenderCommandTagNoMarker<UE_JOIN(TSTR_, Name, __LINE__)>;

////////////////////////////////////////////////////////////////////////////////////////////////////
