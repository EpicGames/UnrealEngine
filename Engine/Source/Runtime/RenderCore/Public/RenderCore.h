// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	RenderCore.h: Render core module definitions.
=============================================================================*/

#pragma once

#include "Stats/Stats.h" // IWYU pragma: keep

DECLARE_LOG_CATEGORY_EXTERN(LogRendererCore, Log, All);

/**
 * The scene rendering stats.
 */

DECLARE_CYCLE_STAT_EXTERN(TEXT("BeginOcclusionTests"),STAT_BeginOcclusionTestsTime,STATGROUP_SceneRendering, RENDERCORE_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Waiting for GPU - check ProfileGPU"),STAT_RenderQueryResultTime,STATGROUP_SceneRendering, RENDERCORE_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("InitViews"),STAT_InitViewsTime,STATGROUP_SceneRendering, RENDERCORE_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Finish Gather Ray Tracing Instances"), STAT_RayTracing_FinishGatherInstances, STATGROUP_SceneRendering, RENDERCORE_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("InitViewsPossiblyAfterPrepass"), STAT_InitViewsPossiblyAfterPrepass, STATGROUP_SceneRendering, RENDERCORE_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Dynamic shadow setup"), STAT_DynamicShadowSetupTime, STATGROUP_SceneRendering, RENDERCORE_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Total GPU rendering"),STAT_TotalGPUFrameTime,STATGROUP_SceneRendering, RENDERCORE_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("RenderViewFamily"),STAT_TotalSceneRenderingTime,STATGROUP_SceneRendering, RENDERCORE_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("FinishRenderViewTarget"),STAT_FinishRenderViewTargetTime,STATGROUP_SceneRendering, RENDERCORE_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("RenderVelocities"),STAT_RenderVelocities,STATGROUP_SceneRendering, RENDERCORE_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Depth drawing"),STAT_DepthDrawTime,STATGROUP_SceneRendering, RENDERCORE_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Base pass drawing"),STAT_BasePassDrawTime,STATGROUP_SceneRendering, RENDERCORE_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Anisotropy pass drawing"), STAT_AnisotropyPassDrawTime, STATGROUP_SceneRendering, RENDERCORE_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Water pass drawing"), STAT_WaterPassDrawTime, STATGROUP_SceneRendering, RENDERCORE_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Lighting drawing"),STAT_LightingDrawTime,STATGROUP_SceneRendering, RENDERCORE_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Proj Shadow drawing"),STAT_ProjectedShadowDrawTime,STATGROUP_SceneRendering, RENDERCORE_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Reflective Shadow map drawing"),STAT_ReflectiveShadowMapDrawTime,STATGROUP_SceneRendering, RENDERCORE_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Translucency drawing"),STAT_TranslucencyDrawTime,STATGROUP_SceneRendering, RENDERCORE_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Dynamic Primitive drawing"),STAT_DynamicPrimitiveDrawTime,STATGROUP_SceneRendering, RENDERCORE_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("StaticDrawList drawing"),STAT_StaticDrawListDrawTime,STATGROUP_SceneRendering, RENDERCORE_API);
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Lights in scene"),STAT_SceneLights,STATGROUP_SceneRendering, RENDERCORE_API);
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Mesh draw calls"),STAT_MeshDrawCalls,STATGROUP_SceneRendering, RENDERCORE_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Present time"),STAT_PresentTime,STATGROUP_SceneRendering, RENDERCORE_API);
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Decals in scene"),STAT_SceneDecals,STATGROUP_SceneRendering, RENDERCORE_API);
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Decals in view"),STAT_Decals,STATGROUP_SceneRendering, RENDERCORE_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Decal drawing"),STAT_DecalsDrawTime,STATGROUP_SceneRendering, RENDERCORE_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Cache Uniform Expressions"),STAT_CacheUniformExpressions,STATGROUP_SceneRendering, RENDERCORE_API);
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Lights using light shafts"), STAT_LightShaftsLights, STATGROUP_SceneRendering, RENDERCORE_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Create ray tracing pipeline"), STAT_CreateRayTracingPipeline, STATGROUP_SceneRendering, RENDERCORE_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Create Lumen ray tracing pipeline"), STAT_CreateLumenRayTracingPipeline, STATGROUP_SceneRendering, RENDERCORE_API);
// Number of instances sent to the TLAS build command, including active and inactive (culled) instances
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Ray tracing total instances"), STAT_RayTracingTotalInstances, STATGROUP_SceneRendering, RENDERCORE_API);
// Number of valid instances, taking into account per-instance culling / activation mask
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Ray tracing active instances"), STAT_RayTracingActiveInstances, STATGROUP_SceneRendering, RENDERCORE_API);

DECLARE_CYCLE_STAT_EXTERN(TEXT("Always Visible"), STAT_UpdateAlwaysVisible, STATGROUP_InitViews, RENDERCORE_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("View Visibility"),STAT_ViewVisibilityTime,STATGROUP_InitViews, RENDERCORE_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Decompress Occlusion"),STAT_DecompressPrecomputedOcclusion,STATGROUP_InitViews, RENDERCORE_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Update Fading"),STAT_UpdatePrimitiveFading,STATGROUP_InitViews, RENDERCORE_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Occlusion Cull"),STAT_OcclusionCull,STATGROUP_InitViews, RENDERCORE_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("View Relevance"),STAT_ViewRelevance,STATGROUP_InitViews, RENDERCORE_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Compute View Relevance"),STAT_ComputeViewRelevance,STATGROUP_InitViews, RENDERCORE_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Static Mesh Relevance"),STAT_StaticRelevance,STATGROUP_InitViews, RENDERCORE_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("UpdateStaticMeshes"),STAT_UpdateStaticMeshesTime,STATGROUP_InitViews, RENDERCORE_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("GetDynamicMeshElements"),STAT_GetDynamicMeshElements,STATGROUP_InitViews, RENDERCORE_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("SetupMeshPass"), STAT_SetupMeshPass, STATGROUP_InitViews, RENDERCORE_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Init dynamic shadows"),STAT_InitDynamicShadowsTime,STATGROUP_InitViews, RENDERCORE_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Update Preshadow Cache"),STAT_UpdatePreshadowCache,STATGROUP_InitViews, RENDERCORE_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Init Projected Shadow"),STAT_InitProjectedShadowVisibility,STATGROUP_InitViews, RENDERCORE_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Create WholeScene Shadow"),STAT_CreateWholeSceneProjectedShadow,STATGROUP_InitViews, RENDERCORE_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Add View Whole Scene Shadows"),STAT_AddViewDependentWholeSceneShadowsForView,STATGROUP_InitViews, RENDERCORE_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Setup Interaction Shadows"),STAT_SetupInteractionShadows,STATGROUP_InitViews, RENDERCORE_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("SortStaticDrawLists"),STAT_SortStaticDrawLists,STATGROUP_InitViews, RENDERCORE_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Update Indirect Lighting Cache"),STAT_UpdateIndirectLightingCache,STATGROUP_InitViews, RENDERCORE_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Update Indirect Lighting Cache Prims"), STAT_UpdateIndirectLightingCachePrims, STATGROUP_InitViews, RENDERCORE_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Update Indirect Lighting Cache Blocks"), STAT_UpdateIndirectLightingCacheBlocks, STATGROUP_InitViews, RENDERCORE_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Update Indirect Lighting Cache Transitions"), STAT_UpdateIndirectLightingCacheTransitions, STATGROUP_InitViews, RENDERCORE_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Update Indirect Lighting Cache Finalize"), STAT_UpdateIndirectLightingCacheFinalize, STATGROUP_InitViews, RENDERCORE_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Interpolate Volumetric Lightmap"), STAT_InterpolateVolumetricLightmapOnCPU, STATGROUP_InitViews, RENDERCORE_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("GatherShadowPrimitives"),STAT_GatherShadowPrimitivesTime,STATGROUP_InitViews, RENDERCORE_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("BuildCSMVisibilityState"), STAT_BuildCSMVisibilityState, STATGROUP_InitViews, RENDERCORE_API);
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Processed primitives"),STAT_ProcessedPrimitives,STATGROUP_InitViews, RENDERCORE_API);
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Frustum Culled primitives"),STAT_CulledPrimitives,STATGROUP_InitViews, RENDERCORE_API);
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Visible Raytracing Primitives"), STAT_VisibleRayTracingPrimitives, STATGROUP_InitViews, RENDERCORE_API);
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Statically occluded primitives"),STAT_StaticallyOccludedPrimitives,STATGROUP_InitViews, RENDERCORE_API);
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Occluded primitives"),STAT_OccludedPrimitives,STATGROUP_InitViews, RENDERCORE_API);
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Occlusion queries"),STAT_OcclusionQueries,STATGROUP_InitViews, RENDERCORE_API);
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Visible static mesh elements"),STAT_VisibleStaticMeshElements,STATGROUP_InitViews, RENDERCORE_API);
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Visible dynamic primitives"),STAT_VisibleDynamicPrimitives,STATGROUP_InitViews, RENDERCORE_API);
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Indirect Lighting Cache updates"),STAT_IndirectLightingCacheUpdates,STATGROUP_InitViews, RENDERCORE_API);
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Precomputed Lighting Buffer updates"),STAT_PrecomputedLightingBufferUpdates,STATGROUP_InitViews, RENDERCORE_API);
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("CSM Subject Primitives"), STAT_CSMSubjects, STATGROUP_InitViews, RENDERCORE_API);
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Static+CSM Static Mesh Receivers"), STAT_CSMStaticMeshReceivers, STATGROUP_InitViews, RENDERCORE_API);
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Static+CSM Static Primitive Receivers"), STAT_CSMStaticPrimitiveReceivers, STATGROUP_InitViews, RENDERCORE_API);

DECLARE_CYCLE_STAT_EXTERN(TEXT("WholeScene Shadow Projections"),STAT_RenderWholeSceneShadowProjectionsTime,STATGROUP_ShadowRendering, RENDERCORE_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("WholeScene Shadow Depths"),STAT_RenderWholeSceneShadowDepthsTime,STATGROUP_ShadowRendering, RENDERCORE_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("PerObject Shadow Projections"),STAT_RenderPerObjectShadowProjectionsTime,STATGROUP_ShadowRendering, RENDERCORE_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("PerObject Shadow Depths"),STAT_RenderPerObjectShadowDepthsTime,STATGROUP_ShadowRendering, RENDERCORE_API);
DECLARE_MEMORY_STAT_EXTERN(TEXT("Per-Frame Shadowmap Atlases"),STAT_ShadowmapAtlasMemory,STATGROUP_ShadowRendering, RENDERCORE_API); 
DECLARE_MEMORY_STAT_EXTERN(TEXT("Cached Shadowmaps"),STAT_CachedShadowmapMemory,STATGROUP_ShadowRendering, RENDERCORE_API); 
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Whole Scene shadows"),STAT_WholeSceneShadows,STATGROUP_ShadowRendering, RENDERCORE_API);
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Cached Whole Scene shadows"),STAT_CachedWholeSceneShadows,STATGROUP_ShadowRendering, RENDERCORE_API);
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("PreShadows"),STAT_PreShadows,STATGROUP_ShadowRendering, RENDERCORE_API);
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Reflective Shadow Maps"),STAT_ReflectiveShadowMaps,STATGROUP_ShadowRendering, RENDERCORE_API);
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Cached PreShadows"),STAT_CachedPreShadows,STATGROUP_ShadowRendering, RENDERCORE_API);
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Per Object shadows"),STAT_PerObjectShadows,STATGROUP_ShadowRendering, RENDERCORE_API);
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Dynamic path draw calls"),STAT_ShadowDynamicPathDrawCalls,STATGROUP_ShadowRendering, RENDERCORE_API);

DECLARE_CYCLE_STAT_EXTERN(TEXT("Render Lights and Shadows"),STAT_LightRendering,STATGROUP_LightRendering, RENDERCORE_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Direct lighting"),STAT_DirectLightRenderingTime,STATGROUP_LightRendering, RENDERCORE_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Translucent injection"),STAT_TranslucentInjectTime,STATGROUP_LightRendering, RENDERCORE_API);
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Num using standard deferred"),STAT_NumLightsUsingStandardDeferred,STATGROUP_LightRendering, RENDERCORE_API);
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Num injected into translucency (unbatched)"),STAT_NumLightsInjectedIntoTranslucency,STATGROUP_LightRendering, RENDERCORE_API);
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Num injected into translucency (batched)"), STAT_NumLightsInjectedIntoTranslucencyBatched, STATGROUP_LightRendering, RENDERCORE_API);
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Num batched"),STAT_NumBatchedLights,STATGROUP_LightRendering, RENDERCORE_API);
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Num light function only"),STAT_NumLightFunctionOnlyLights,STATGROUP_LightRendering, RENDERCORE_API);
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Num shadowed"),STAT_NumShadowedLights,STATGROUP_LightRendering, RENDERCORE_API);

/** The scene update stats. */
// The purpose of the SceneUpdate stat group is to show where rendering thread time is going from a high level outside of RenderViewFamily.
// It should only contain stats that are likely to track a lot of time in a typical scene, not edge case stats that are rarely non-zero.
// Use a more detailed profiler (like an instruction trace or sampling capture on Xbox 360) to track down where time is going in more detail.
DECLARE_CYCLE_STAT_EXTERN(TEXT("Update GPU Scene"), STAT_UpdateGPUSceneTime, STATGROUP_SceneUpdate, RENDERCORE_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("AddPrimitive"),STAT_AddScenePrimitiveRenderThreadTime,STATGROUP_SceneUpdate, RENDERCORE_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("UpdatePrimitive"), STAT_UpdateScenePrimitiveRenderThreadTime, STATGROUP_SceneUpdate, RENDERCORE_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("AddLight"),STAT_AddSceneLightTime,STATGROUP_SceneUpdate, RENDERCORE_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("RemovePrimitive"),STAT_RemoveScenePrimitiveTime,STATGROUP_SceneUpdate, RENDERCORE_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("RemoveLight"),STAT_RemoveSceneLightTime,STATGROUP_SceneUpdate, RENDERCORE_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("UpdateLight"),STAT_UpdateSceneLightTime,STATGROUP_SceneUpdate, RENDERCORE_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("UpdatePrimitiveTransform"),STAT_UpdatePrimitiveTransformRenderThreadTime,STATGROUP_SceneUpdate, RENDERCORE_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("UpdatePrimitiveInstance"), STAT_UpdatePrimitiveInstanceRenderThreadTime, STATGROUP_SceneUpdate, RENDERCORE_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Update CPU Skin"),STAT_CPUSkinUpdateRTTime,STATGROUP_SceneUpdate, RENDERCORE_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Update GPU Skin"),STAT_GPUSkinUpdateRTTime,STATGROUP_SceneUpdate, RENDERCORE_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Update particles"),STAT_ParticleUpdateRTTime,STATGROUP_SceneUpdate, RENDERCORE_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Update Influence Weights"),STAT_InfluenceWeightsUpdateRTTime,STATGROUP_SceneUpdate, RENDERCORE_API);

DECLARE_CYCLE_STAT_EXTERN(TEXT("RemovePrimitive (GT)"),STAT_RemoveScenePrimitiveGT,STATGROUP_SceneUpdate, RENDERCORE_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("AddPrimitive (GT)"),STAT_AddScenePrimitiveGT,STATGROUP_SceneUpdate, RENDERCORE_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("UpdatePrimitiveTransform (GT)"),STAT_UpdatePrimitiveTransformGT,STATGROUP_SceneUpdate, RENDERCORE_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("UpdatePrimitiveInstance (GT)"), STAT_UpdatePrimitiveInstanceGT, STATGROUP_SceneUpdate, RENDERCORE_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("UpdatePrimitiveInstanceFromCompute (GT)"), STAT_UpdatePrimitiveInstanceFromComputeGT, STATGROUP_SceneUpdate, RENDERCORE_API);

DECLARE_CYCLE_STAT_EXTERN(TEXT("Set Shader Maps On Material Resources (RT)"),STAT_Scene_SetShaderMapsOnMaterialResources_RT,STATGROUP_SceneUpdate, RENDERCORE_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Update Static Draw Lists (RT)"), STAT_Scene_UpdateStaticDrawLists_RT, STATGROUP_SceneUpdate, RENDERCORE_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Update Static Draw Lists For Materials (RT)"),STAT_Scene_UpdateStaticDrawListsForMaterials_RT,STATGROUP_SceneUpdate, RENDERCORE_API);
DECLARE_DWORD_COUNTER_STAT_EXTERN( TEXT( "Proxy Total" ), STAT_GameToRendererMallocTotal, STATGROUP_Memory, RENDERCORE_API );

/** Memory stats for tracking virtual allocations used by the renderer to represent the scene. */
DECLARE_MEMORY_STAT_EXTERN(TEXT("Primitive memory"),STAT_PrimitiveInfoMemory,STATGROUP_SceneMemory, RENDERCORE_API);
DECLARE_MEMORY_STAT_EXTERN(TEXT("Scene memory"),STAT_RenderingSceneMemory,STATGROUP_SceneMemory, RENDERCORE_API);
DECLARE_MEMORY_STAT_EXTERN(TEXT("ViewState memory"),STAT_ViewStateMemory,STATGROUP_SceneMemory, RENDERCORE_API);
DECLARE_MEMORY_STAT_EXTERN(TEXT("Light interaction memory"),STAT_LightInteractionMemory,STATGROUP_SceneMemory, RENDERCORE_API);

DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Num reflective shadow map lights"),STAT_NumReflectiveShadowMapLights,STATGROUP_LightRendering, RENDERCORE_API);

DECLARE_MEMORY_STAT_EXTERN(TEXT("Pool Size"), STAT_RenderTargetPoolSize, STATGROUP_RenderTargetPool, RENDERCORE_API);
DECLARE_MEMORY_STAT_EXTERN(TEXT("Pool Used"), STAT_RenderTargetPoolUsed, STATGROUP_RenderTargetPool, RENDERCORE_API);
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Pool Count"), STAT_RenderTargetPoolCount, STATGROUP_RenderTargetPool, RENDERCORE_API);

// shared by renderer and engine, compiles down to a constant in final release
RENDERCORE_API int32 GetCVarForceLOD();
RENDERCORE_API int32 GetCVarForceLOD_AnyThread();

RENDERCORE_API int32 GetCVarForceLODShadow();
RENDERCORE_API int32 GetCVarForceLODShadow_AnyThread();

RENDERCORE_API void SetNearClipPlaneGlobals(float NewNearClipPlane);
