// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class FPixelRenderCounters
{
public:
	float GetResolutionFraction() const
	{
		return ResolutionFraction;
	}

	FIntPoint GetRenderResolution() const
	{
		return RenderResolution;
	}

	uint32 GetPixelRenderCount() const
	{
		return PrevPixelRenderCount;
	}

	uint32 GetPixelDisplayCount() const
	{
		return PrevPixelDisplayCount;
	}

	void AddViewStatistics(float InResolutionFraction, FIntPoint InRenderResolution, uint32 PixelDisplayCount)
	{
		ResolutionFraction = InResolutionFraction;
		RenderResolution = InRenderResolution;
		CurrentPixelRenderCount += InRenderResolution.X * InRenderResolution.Y;
		CurrentPixelDisplayCount += PixelDisplayCount;
	}

private:
	float ResolutionFraction = 0.0f;
	FIntPoint RenderResolution = FIntPoint(0, 0);
	uint32 PrevPixelRenderCount = 0;
	uint32 PrevPixelDisplayCount = 0;
	uint32 CurrentPixelRenderCount = 0;
	uint32 CurrentPixelDisplayCount = 0;

	friend void TickPixelRenderCounters();
};

extern RENDERCORE_API FPixelRenderCounters GPixelRenderCounters;
