// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"

/**
 * The base class of objects that need to defer deletion until the render command queue has been flushed.
 */
class FDeferredCleanupInterface
{
public:
	virtual ~FDeferredCleanupInterface() = default;
};

/**
 * A set of cleanup objects which are pending deletion.
 */
class FPendingCleanupObjects
{
	TArray<FDeferredCleanupInterface*> CleanupArray;

public:
	inline bool IsEmpty() const { return CleanupArray.IsEmpty(); }
	FPendingCleanupObjects();
	RENDERCORE_API ~FPendingCleanupObjects();
};

/**
 * Adds the specified deferred cleanup object to the current set of pending cleanup objects.
 */
extern RENDERCORE_API void BeginCleanup(FDeferredCleanupInterface* CleanupObject);

/**
 * Transfers ownership of the current set of pending cleanup objects to the caller.  A new set is created for subsequent BeginCleanup calls.
 * @return A pointer to the set of pending cleanup objects.  The called is responsible for deletion.
 */
extern RENDERCORE_API FPendingCleanupObjects* GetPendingCleanupObjects();
