// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/** Rendering Dependency Graph framework to setup lambda scopes designed as passes to issue GPU commands to the RHI
 * through a deferred execution. They are created with FRDGBuilder::AddPass(). When creating a pass,
 * it needs to have shader parameters. Can be any shader parameters, but the framework is mostly interested about the
 * render graph resources.
 *
 * The structure that hold all the pass parameter should be allocated using FRDGBuilder::AllocParameters() to ensure
 * correct lifetime due to the lambda's execution being deferred.
 *
 * A render graph resource, created with FRDGBuilder::CreateTexture() or FRDGBuilder::CreateBuffer(), only record the
 * resource descriptor. Allocation will be done by the graph if/when the resource is needed. The graph will track the
 * resource's lifetime and can free and reuse the memory when the remaining passes no longer reference it.
 *
 * All render graph resources used in a pass, must be in the parameters of pass given on the FRDGBuilder::AddPass(). It
 * allows render graph to know what resource are being used by a given pass.
 * The resources are only guaranteed to be allocated when executing passes. Therefor accessing them should only done
 * within the lambda scope of the pass created with FRDGBuilder::AddPass(). Failing to list some resource used by the pass
 * may result in problems.
 *
 * It is important to not reference more graph resources in the parameters than a pass needs, since this artificially
 * increases the graphs knowledge of that resource's lifetime. This can cause an increase in memory use or prevent passes
 * from overlapping their execution. An example is ClearUnusedGraphResources() that can automatically null out resource
 * references unused by a shader for passes that have only on shader. Warnings will be emitted if a resource is not used
 * in a pass (thanks to FRDGResource::MarkResourceAsUsed())
 *
 * The lambda scope of a pass execution may happen any time after FRDGBuilder::AddPass(). For debugging purpose, it may
 * happen directly in the AddPass() with the immediate mode. When a bug is happening during pass execution, immediate mode
 * allows you to have the callstack of the pass setup that may have the source cause of the bug. The immediate mode can be
 * enabled from command line with -rdgimmediate, or with the cvar r.RDG.ImmediateMode=1.
 *
 * Pooled managed resource texture FPooledRenderTarget generated by legacy code can be used in render graph by using
 * FRDGBuilder::RegisterExternalTexture().
 *
 * With the knowledge of pass dependencies, the execution may prioritize different hardware goal, such as prioritizing memory
 * pressure or pass GPU execution concurrency. As such, the execution order of passes is not guaranteed. The execution
 * order of the pass only guarantees that the execution will perform the work on the intermediary resource exactly as the
 * immediate mode does on the GPU.
 * 
 * A render graph pass should not modify the state of a external data structure, as this may lead to edges case according to
 * the execution order of the pass. Render graph resources that should survive the render graph execution (For instance, a
 * viewport back buffer, temporal AA history for next frame...) should be extracted using
 * FRDGBuilder::QueueTextureExtraction(). If a pass is detected not useful to produce any of this resources scheduled for
 * extraction, or modifying an external texture, this pass may not even be executed with a warning (TODO).
 *
 * Unless exception motivated with strong technical reasons (like stereo rendering rendering multiple views at once for VR),
 * do not bundle multiple work on different resources in the same pass. This will end up creating more dependencies on a
 * bundle of work that individually may only require a subset of those dependencies. The scheduler may be able to overlap a
 * part of that with other GPU work. This may also retain allocated transient resources longer, potentially increasing
 * highest memory pressure peak of your entire frame.
 *
 * Although the AddPass() only wants a lambda scope to have deferred execution, it does not mean you nead to write one.
 * A pass may already be fully implementable using an even simpler utility such as the ones in FComputeShaderUtils
 * or FPixelShaderUtils.
 */

#include "RenderGraphDefinitions.h"
#include "RenderGraphResources.h"
#include "RenderGraphPass.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphUtils.h"
#include "ShaderParameterStruct.h"
#include "ShaderParameterMacros.h"
