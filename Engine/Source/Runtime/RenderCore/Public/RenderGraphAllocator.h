// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/StaticArray.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "CoreGlobals.h"
#include "CoreMinimal.h"
#include "HAL/MemoryBase.h"
#include "HAL/UnrealMemory.h"
#include "Math/UnrealMathSSE.h"
#include "Misc/AssertionMacros.h"
#include "Misc/MemStack.h"
#include "Templates/Function.h"
#include "Templates/UnrealTemplate.h"

#define RDG_USE_MALLOC USING_ADDRESS_SANITISER
#define RDG_ALLOCATOR_DEBUG USING_ADDRESS_SANITISER || UE_BUILD_DEBUG

class FRDGAllocator;

class FRDGAllocator
{
public:
	class FObject
	{
	public:
		virtual ~FObject() = default;
	};

	template <typename T>
	class TObject final : FObject
	{
		friend class FRDGAllocator;
	private:
		template <typename... TArgs>
		inline TObject(TArgs&&... Args)
			: Alloc(Forward<TArgs&&>(Args)...)
		{}

		T Alloc;
	};

	RENDERCORE_API static FRDGAllocator& GetTLS();

	FRDGAllocator();
	FRDGAllocator(FRDGAllocator&&);
	FRDGAllocator& operator=(FRDGAllocator&&);
	~FRDGAllocator();

	/** Allocates raw memory. */
	inline void* Alloc(uint64 SizeInBytes, uint32 AlignInBytes)
	{
		void* Memory;

#if RDG_ALLOCATOR_DEBUG
		AcquireAccess();
#endif

#if RDG_USE_MALLOC
		Memory = FMemory::Malloc(SizeInBytes, AlignInBytes);
		Mallocs.Emplace(Memory);
		NumMallocBytes += SizeInBytes;
#else
		Memory = MemStack.Alloc(SizeInBytes, AlignInBytes);
#endif

#if RDG_ALLOCATOR_DEBUG
		ReleaseAccess();
#endif

		return Memory;
	}

	/** Allocates an uninitialized type without destructor tracking. */
	template <typename PODType>
	inline PODType* AllocUninitialized(uint64 Count = 1)
	{
		return reinterpret_cast<PODType*>(Alloc(sizeof(PODType) * Count, alignof(PODType)));
	}

	/** Allocates and constructs an object and tracks it for destruction. */
	template <typename T, typename... TArgs>
	inline T* Alloc(TArgs&&... Args)
	{
#if RDG_ALLOCATOR_DEBUG
		AcquireAccess();
#endif

#if RDG_USE_MALLOC
		TObject<T>* Object = new TObject<T>(Forward<TArgs&&>(Args)...);
		NumMallocBytes += sizeof(Object);
#else
		TObject<T>* Object = new(MemStack) TObject<T>(Forward<TArgs&&>(Args)...);
#endif
		check(Object);
		Objects.Add(Object);

#if RDG_ALLOCATOR_DEBUG
		ReleaseAccess();
#endif
		return &Object->Alloc;
	}

	/** Allocates a C++ object with no destructor tracking (dangerous!). */
	template <typename T, typename... TArgs>
	inline T* AllocNoDestruct(TArgs&&... Args)
	{
		return new (AllocUninitialized<T>(1)) T(Forward<TArgs&&>(Args)...);
	}

	inline int32 GetByteCount() const
	{
#if RDG_USE_MALLOC
		return static_cast<int32>(NumMallocBytes);
#else
		return MemStack.GetByteCount();
#endif
	}

	void ReleaseAll();

private:
#if RDG_USE_MALLOC
	TArray<void*> Mallocs;
	uint64 NumMallocBytes = 0;
#else
	FMemStackBase MemStack;
#endif
	TArray<FObject*> Objects;

#if RDG_ALLOCATOR_DEBUG
	RENDERCORE_API void AcquireAccess();
	RENDERCORE_API void ReleaseAccess();

	std::atomic_int32_t NumAccesses{0};
	static thread_local int32 NumAccessesTLS;
#endif

	friend class FRDGAllocatorScope;
	static thread_local FRDGAllocator* AllocatorTLS;
};

class FRDGAllocatorScope
{
public:
	RENDERCORE_API FRDGAllocatorScope(FRDGAllocator& Allocator);
	RENDERCORE_API ~FRDGAllocatorScope();

private:
	FRDGAllocator* AllocatorToRestore;
};

#define RDG_FRIEND_ALLOCATOR_FRIEND(Type) friend class FRDGAllocator::TObject<Type>

namespace UE::RenderCore::Private
{
	[[noreturn]] RENDERCORE_API void OnInvalidRDGAllocatorNum(int32 NewNum, SIZE_T NumBytesPerElement);
}

/** A container allocator that allocates from a global RDG allocator instance. */
template<uint32 Alignment = DEFAULT_ALIGNMENT>
class TRDGArrayAllocator
{
public:
	using SizeType = int32;

	enum { NeedsElementType = true };
	enum { RequireRangeCheck = true };

	template<typename ElementType>
	class ForElementType
	{
	public:
		ForElementType() = default;

		inline void MoveToEmpty(ForElementType& Other)
		{
			checkSlow(this != &Other);

			Data = Other.Data;
			Other.Data = nullptr;
		}

		inline ElementType* GetAllocation() const
		{
			return Data;
		}

		void ResizeAllocation(SizeType CurrentNum, SizeType NewMax, SIZE_T NumBytesPerElement)
		{
			void* OldData = Data;
			if (NewMax)
			{
				static_assert(sizeof(int32) <= sizeof(SIZE_T), "SIZE_T is expected to be larger than int32");

				// Check for under/overflow
				if (UNLIKELY(NewMax < 0 || NumBytesPerElement < 1 || NumBytesPerElement > (SIZE_T)MAX_int32))
				{
					UE::RenderCore::Private::OnInvalidRDGAllocatorNum(NewMax, NumBytesPerElement);
				}

				// Allocate memory from the allocator.
				const int32 AllocSize = (int32)(NewMax * NumBytesPerElement);
				const int32 AllocAlignment = FMath::Max(Alignment, (uint32)alignof(ElementType));
				Data = (ElementType*)FRDGAllocator::GetTLS().Alloc(AllocSize, FMath::Max(AllocSize >= 16 ? (int32)16 : (int32)8, AllocAlignment));

				// If the container previously held elements, copy them into the new allocation.
				if (OldData && CurrentNum)
				{
					const SizeType NumCopiedElements = FMath::Min(NewMax, CurrentNum);
					FMemory::Memcpy(Data, OldData, NumCopiedElements * NumBytesPerElement);
				}
			}
		}
		inline SizeType CalculateSlackReserve(SizeType NewMax, SIZE_T NumBytesPerElement) const
		{
			return DefaultCalculateSlackReserve(NewMax, NumBytesPerElement, false, Alignment);
		}

		inline SizeType CalculateSlackShrink(SizeType NewMax, SizeType CurrentMax, SIZE_T NumBytesPerElement) const
		{
			return DefaultCalculateSlackShrink(NewMax, CurrentMax, NumBytesPerElement, false, Alignment);
		}

		inline SizeType CalculateSlackGrow(SizeType NewMax, SizeType CurrentMax, SIZE_T NumBytesPerElement) const
		{
			return DefaultCalculateSlackGrow(NewMax, CurrentMax, NumBytesPerElement, false, Alignment);
		}

		inline SIZE_T GetAllocatedSize(SizeType CurrentMax, SIZE_T NumBytesPerElement) const
		{
			return CurrentMax * NumBytesPerElement;
		}

		bool HasAllocation() const
		{
			return !!Data;
		}

		SizeType GetInitialCapacity() const
		{
			return 0;
		}

	private:
		ElementType* Data = nullptr;
	};

	typedef ForElementType<FScriptContainerElement> ForAnyElementType;
};

template <uint32 Alignment>
struct TAllocatorTraits<TRDGArrayAllocator<Alignment>> : TAllocatorTraitsBase<TRDGArrayAllocator<Alignment>>
{
	enum { IsZeroConstruct = true };
};

using FRDGArrayAllocator = TRDGArrayAllocator<>;
using FRDGBitArrayAllocator = TInlineAllocator<4, FRDGArrayAllocator>;
using FRDGSparseArrayAllocator = TSparseArrayAllocator<FRDGArrayAllocator, FRDGBitArrayAllocator>;
using FRDGSetAllocator = TSetAllocator<FRDGSparseArrayAllocator, TInlineAllocator<1, FRDGBitArrayAllocator>>;
