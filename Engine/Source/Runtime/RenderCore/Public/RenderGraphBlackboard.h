// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "HAL/PlatformMath.h"
#include "Misc/AssertionMacros.h"
#include "Misc/GeneratedTypeName.h"
#include "RenderGraphAllocator.h"
#include "RenderGraphDefinitions.h"
#include "Templates/UnrealTemplate.h"

/** Declares a struct for use by the RDG blackboard. */
#define RDG_REGISTER_BLACKBOARD_STRUCT(StructType)										\
	template <>																			\
	inline FString FRDGBlackboard::GetTypeName<StructType>()							\
	{																					\
		return GetTypeName(TEXT(#StructType), TEXT(__FILE__), __LINE__);				\
	}

/** The blackboard is a map of struct instances with a lifetime tied to a render graph allocator. It is designed
 *  to solve cases where explicit marshaling of immutable data is undesirable. Structures are created once and the mutable
 *  reference is returned.
 *
 *  Good candidates for the blackboard would be data that is created once and immutably fetched across the entire
 *  renderer pipeline, where marshaling would create more maintenance burden than benefit. More constrained data
 *  structures should be marshaled through function calls instead.
 *
 *  Example of Usage:
 *
 *	class FMyStruct
 *	{
 *	public:
 *		FRDGTextureRef TextureA = nullptr;
 *		FRDGTextureRef TextureB = nullptr;
 *		FRDGTextureRef TextureC = nullptr;
 *	};
 *
 *	RDG_REGISTER_BLACKBOARD_STRUCT(FMyStruct);
 *
 *	static void InitStruct(FRDGBlackboard& GraphBlackboard)
 *	{
 *		auto& MyStruct = GraphBlackboard.Create<FMyStruct>();
 *
 *		//...
 *	}
 *
 *	static void UseStruct(const FRDGBlackboard& GraphBlackboard)
 *	{
 *		const auto& MyStruct = GraphBlackboard.GetChecked<FMyStruct>();
 *
 *		//...
 *	}
 */
class FRDGBlackboard
{
public:
	/** Creates a new instance of a struct. Asserts if one already existed. */
	template <typename StructType, typename... ArgsType>
	StructType& Create(ArgsType&&... Args)
	{
		using HelperStructType = TStruct<StructType>;

		const int32 StructIndex = GetStructIndex<StructType>();
		if (StructIndex >= Blackboard.Num())
		{
			Blackboard.SetNumZeroed(StructIndex + 1);
		}

		checkf(!Blackboard[StructIndex], TEXT("RDGBlackboard duplicate Create called on struct '%s'. Only one Create call per struct is allowed."), GetGeneratedTypeName<StructType>());
		FStruct* Result = Allocator.Alloc<HelperStructType>(Forward<ArgsType&&>(Args)...);
		check(Result);
		Blackboard[StructIndex] = Result;
		return static_cast<HelperStructType*>(Result)->Struct;
	}

	/** Gets a mutable instance of the struct. Returns null if not present in the blackboard. */
	template <typename StructType>
	StructType* GetMutable() const
	{
		using HelperStructType = TStruct<StructType>;

		const int32 StructIndex = GetStructIndex<StructType>();
		if (StructIndex < Blackboard.Num())
		{
			if (HelperStructType* Element = static_cast<HelperStructType*>(Blackboard[StructIndex]))
			{
				return &Element->Struct;
			}
		}
		return nullptr;
	}

	/** Gets an immutable instance of the struct. Returns null if not present in the blackboard. */
	template <typename StructType>
	const StructType* Get() const
	{
		return GetMutable<StructType>();
	}
	
	template <typename StructType, typename... ArgsType>
	StructType& GetOrCreate(ArgsType&&... Args)
	{
		if (StructType* Struct = GetMutable<StructType>())
		{
			return *Struct;
		}
		return Create<StructType>(Forward<ArgsType&&>(Args)...);
	}

	/** Gets a mutable instance of the struct. Asserts if not present in the blackboard. */
	template <typename StructType>
	StructType& GetMutableChecked() const
	{
		StructType* Struct = GetMutable<StructType>();
		checkf(Struct, TEXT("RDGBlackboard Get failed to find instance of struct '%s' in the blackboard."), GetGeneratedTypeName<StructType>());
		return *Struct;
	}
	
	/** Gets an immutable instance of the struct. Asserts if not present in the blackboard. */
	template <typename StructType>
	const StructType& GetChecked() const
	{
		return GetMutableChecked<StructType>();
	}

private:
	FRDGBlackboard(FRDGAllocator& InAllocator)
		: Allocator(InAllocator)
	{}

	void Clear()
	{
		Blackboard.Empty();
	}

	struct FStruct
	{
		virtual ~FStruct() = default;
	};

	template <typename StructType>
	struct TStruct final : public FStruct
	{
		template <typename... TArgs>
		inline TStruct(TArgs&&... Args)
			: Struct(Forward<TArgs&&>(Args)...)
		{}

		StructType Struct;
	};

	template <typename StructType>
	static FString GetTypeName()
	{
		// Forces the compiler to only evaluate the assert on a concrete type.
		static_assert(sizeof(StructType) == 0, "Struct has not been registered with the RDG blackboard. Use RDG_REGISTER_BLACKBOARD_STRUCT to do this.");
		return FString();
	}

	static RENDERCORE_API FString GetTypeName(const TCHAR* ClassName, const TCHAR* FileName, uint32 LineNumber);
	static RENDERCORE_API uint32 AllocateIndex(FString&& TypeName);

	template <typename StructType>
	static uint32 GetStructIndex()
	{
		static uint32 Index = UINT_MAX;
		if (Index == UINT_MAX)
		{
			Index = AllocateIndex(GetTypeName<StructType>());
		}
		return Index;
	}

	FRDGAllocator& Allocator;
	TArray<FStruct*, FRDGArrayAllocator> Blackboard;

	friend class FRDGBuilder;
};