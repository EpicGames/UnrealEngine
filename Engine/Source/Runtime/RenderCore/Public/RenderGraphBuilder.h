// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Containers/SortedMap.h"
#include "Containers/UnrealString.h"
#include "Containers/StridedView.h"
#include "CoreTypes.h"
#include "Misc/AssertionMacros.h"
#include "MultiGPU.h"
#include "PixelFormat.h"
#include "RHIBreadcrumbs.h"
#include "RHIDefinitions.h"
#include "RenderGraphAllocator.h"
#include "RenderGraphBlackboard.h"
#include "RenderGraphDefinitions.h"
#include "RenderGraphEvent.h"
#include "RenderGraphPass.h"
#include "RenderGraphResources.h"
#include "RenderGraphTrace.h"
#include "RenderGraphValidation.h"
#include "RendererInterface.h"
#include "ShaderParameterMacros.h"
#include "Templates/RefCounting.h"
#include "Templates/UnrealTemplate.h"
#include "Tasks/Pipe.h"
#include "Experimental/Containers/RobinHoodHashTable.h"

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8
#include "ProfilingDebugging/CsvProfiler.h"
#include "Stats/Stats.h"
#endif //UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8

enum class ERenderTargetTexture : uint8;
struct FParallelPassSet;
struct FRHIRenderPassInfo;
struct FRHITrackedAccessInfo;
struct FRHITransientAliasingInfo;
struct TStatId;

/** Use the render graph builder to build up a graph of passes and then call Execute() to process them. Resource barriers
 *  and lifetimes are derived from _RDG_ parameters in the pass parameter struct provided to each AddPass call. The resulting
 *  graph is compiled, culled, and executed in Execute(). The builder should be created on the stack and executed prior to
 *  destruction.
 */
class FRDGBuilder
	: public FRDGScopeState
{
	struct FAsyncDeleter
	{
		TUniqueFunction<void()> Function;
		UE::Tasks::FTask Prerequisites;
		static UE::Tasks::FTask LastTask;

		RENDERCORE_API ~FAsyncDeleter();

	} AsyncDeleter;

	FRDGAllocatorScope RootAllocatorScope;

public:
	RENDERCORE_API FRDGBuilder(FRHICommandListImmediate& RHICmdList, FRDGEventName Name = {}, ERDGBuilderFlags Flags = ERDGBuilderFlags::None, EShaderPlatform ShaderPlatform = GMaxRHIShaderPlatform);
	FRDGBuilder(const FRDGBuilder&) = delete;
	RENDERCORE_API ~FRDGBuilder();

	/** Finds an RDG texture associated with the external texture, or returns null if none is found. */
	FRDGTexture* FindExternalTexture(FRHITexture* Texture) const;
	FRDGTexture* FindExternalTexture(IPooledRenderTarget* ExternalPooledTexture) const;

	/** Finds an RDG buffer associated with the external buffer, or returns null if none is found. */
	FRDGBuffer* FindExternalBuffer(FRHIBuffer* Buffer) const;
	FRDGBuffer* FindExternalBuffer(FRDGPooledBuffer* ExternalPooledBuffer) const;

	/** Registers a external pooled render target texture to be tracked by the render graph. The name of the registered RDG texture is pulled from the pooled render target. */
	RENDERCORE_API FRDGTextureRef RegisterExternalTexture(
		const TRefCountPtr<IPooledRenderTarget>& ExternalPooledTexture,
		ERDGTextureFlags Flags = ERDGTextureFlags::None);

	/** Register an external texture with a custom name. The name is only used if the texture has not already been registered. */
	RENDERCORE_API FRDGTextureRef RegisterExternalTexture(
		const TRefCountPtr<IPooledRenderTarget>& ExternalPooledTexture,
		const TCHAR* NameIfNotRegistered,
		ERDGTextureFlags Flags = ERDGTextureFlags::None);

	/** Register a external buffer to be tracked by the render graph. */
	RENDERCORE_API FRDGBufferRef RegisterExternalBuffer(const TRefCountPtr<FRDGPooledBuffer>& ExternalPooledBuffer, ERDGBufferFlags Flags = ERDGBufferFlags::None);
	RENDERCORE_API FRDGBufferRef RegisterExternalBuffer(const TRefCountPtr<FRDGPooledBuffer>& ExternalPooledBuffer, ERDGBufferFlags Flags, ERHIAccess AccessFinal);

	/** Register an external buffer with a custom name. The name is only used if the buffer has not already been registered. */
	RENDERCORE_API FRDGBufferRef RegisterExternalBuffer(
		const TRefCountPtr<FRDGPooledBuffer>& ExternalPooledBuffer,
		const TCHAR* NameIfNotRegistered,
		ERDGBufferFlags Flags = ERDGBufferFlags::None);

	/** Create graph tracked texture from a descriptor. The CPU memory is guaranteed to be valid through execution of
	 *  the graph, at which point it is released. The underlying RHI texture lifetime is only guaranteed for passes which
	 *  declare the texture in the pass parameter struct. The name is the name used for GPU debugging tools and the the
	 *  VisualizeTexture/Vis command.
	 */
	FRDGTextureRef CreateTexture(const FRDGTextureDesc& Desc, const TCHAR* Name, ERDGTextureFlags Flags = ERDGTextureFlags::None);

	/** Create graph tracked buffer from a descriptor. The CPU memory is guaranteed to be valid through execution of
	 *  the graph, at which point it is released. The underlying RHI buffer lifetime is only guaranteed for passes which
	 *  declare the buffer in the pass parameter struct. The name is the name used for GPU debugging tools.
	 */
	FRDGBufferRef CreateBuffer(const FRDGBufferDesc& Desc, const TCHAR* Name, ERDGBufferFlags Flags = ERDGBufferFlags::None);

	/** A variant of CreateBuffer where users supply NumElements through a callback. This allows creating buffers with
	 *  sizes unknown at creation time. The callback is called before executing the most recent RDG pass that references
	 *  the buffer so data must be ready before that.
	 */
	FRDGBufferRef CreateBuffer(const FRDGBufferDesc& Desc, const TCHAR* Name, FRDGBufferNumElementsCallback&& NumElementsCallback, ERDGBufferFlags Flags = ERDGBufferFlags::None);

	/** Create graph tracked SRV for a texture from a descriptor. */
	FRDGTextureSRVRef CreateSRV(const FRDGTextureSRVDesc& Desc);

	/** Create graph tracked SRV for a buffer from a descriptor. */
	FRDGBufferSRVRef CreateSRV(const FRDGBufferSRVDesc& Desc);

	inline FRDGBufferSRVRef CreateSRV(FRDGBufferRef Buffer, EPixelFormat Format)
	{
		return CreateSRV(FRDGBufferSRVDesc(Buffer, Format));
	}

	/** Create graph tracked UAV for a texture from a descriptor. */
	FRDGTextureUAVRef CreateUAV(const FRDGTextureUAVDesc& Desc, ERDGUnorderedAccessViewFlags Flags = ERDGUnorderedAccessViewFlags::None);

	inline FRDGTextureUAVRef CreateUAV(FRDGTextureRef Texture, ERDGUnorderedAccessViewFlags Flags = ERDGUnorderedAccessViewFlags::None, EPixelFormat Format = PF_Unknown)
	{
		return CreateUAV(FRDGTextureUAVDesc(Texture, /* MipLevel */ 0, Format), Flags);
	}

	/** Create graph tracked UAV for a buffer from a descriptor. */
	FRDGBufferUAVRef CreateUAV(const FRDGBufferUAVDesc& Desc, ERDGUnorderedAccessViewFlags Flags = ERDGUnorderedAccessViewFlags::None);

	inline FRDGBufferUAVRef CreateUAV(FRDGBufferRef Buffer, EPixelFormat Format, ERDGUnorderedAccessViewFlags Flags = ERDGUnorderedAccessViewFlags::None)
	{
		return CreateUAV(FRDGBufferUAVDesc(Buffer, Format), Flags);
	}

	/** Creates a graph tracked uniform buffer which can be attached to passes. These uniform buffers require some care
	 *  because they will bulk transition all resources. The graph will only transition resources which are not also
	 *  bound for write access by the pass.
	 */
	template <typename ParameterStructType>
	TRDGUniformBufferRef<ParameterStructType> CreateUniformBuffer(const ParameterStructType* ParameterStruct);

	//////////////////////////////////////////////////////////////////////////
	// Allocation Methods

	/** Allocates raw memory using an allocator tied to the lifetime of the graph. */
	void* Alloc(uint64 SizeInBytes, uint32 AlignInBytes = 16);

	/** Allocates POD memory using an allocator tied to the lifetime of the graph. Does not construct / destruct. */
	template <typename PODType>
	PODType* AllocPOD();

	/** Allocates POD memory using an allocator tied to the lifetime of the graph. Does not construct / destruct. */
	template <typename PODType>
	PODType* AllocPODArray(uint32 Count);

	/** Allocates POD memory using an allocator tied to the lifetime of the graph. Does not construct / destruct. */
	template <typename PODType>
	TArrayView<PODType> AllocPODArrayView(uint32 Count);

	/** Allocates a C++ object using an allocator tied to the lifetime of the graph. Will destruct the object. */
	template <typename ObjectType, typename... TArgs>
	ObjectType* AllocObject(TArgs&&... Args);

	/** Allocates a C++ array where both the array and the data are tied to the lifetime of the graph. The array itself is safe to pass into an RDG lambda. */
	template <typename ObjectType>
	TArray<ObjectType, SceneRenderingAllocator>& AllocArray();

	/** Allocates a parameter struct with a lifetime tied to graph execution. */
	template <typename ParameterStructType>
	ParameterStructType* AllocParameters();

	/** Allocates a parameter struct with a lifetime tied to graph execution, and copies contents from an existing parameters struct. */
	template <typename ParameterStructType>
	ParameterStructType* AllocParameters(const ParameterStructType* StructToCopy);

	/** Allocates a data-driven parameter struct with a lifetime tied to graph execution. */
	template <typename BaseParameterStructType>
	BaseParameterStructType* AllocParameters(const FShaderParametersMetadata* ParametersMetadata);

	/** Allocates an array of data-driven parameter structs with a lifetime tied to graph execution. */
	template <typename BaseParameterStructType>
	TStridedView<BaseParameterStructType> AllocParameters(const FShaderParametersMetadata* ParametersMetadata, uint32 NumStructs);

	//////////////////////////////////////////////////////////////////////////

	/** Adds a callback that is called after pass execution is complete. */
	void AddPostExecuteCallback(TUniqueFunction<void()>&& Callback)
	{
		check(Callback);
		PostExecuteCallbacks.Emplace(Forward<TUniqueFunction<void()>&&>(Callback));
	}

	/** Adds a lambda pass to the graph with an accompanied pass parameter struct.
	 *
	 *  RDG resources declared in the struct (via _RDG parameter macros) are safe to access in the lambda. The pass parameter struct
	 *  should be allocated by AllocParameters(), and once passed in, should not be mutated. It is safe to provide the same parameter
	 *  struct to multiple passes, so long as it is kept immutable. The lambda is deferred until execution unless the immediate debug
	 *  mode is enabled. All lambda captures should assume deferral of execution.
	 *
	 *  The lambda must include a single RHI command list as its parameter. The exact type of command list depends on the workload.
	 *  For example, use FRHIComputeCommandList& for Compute / AsyncCompute workloads. Raster passes should use FRHICommandList&.
	 *  Prefer not to use FRHICommandListImmediate& unless actually required.
	 *
	 *  Declare the type of GPU workload (i.e. Copy, Compute / AsyncCompute, Graphics) to the pass via the Flags argument. This is
	 *  used to determine async compute regions, render pass setup / merging, RHI transition accesses, etc. Other flags exist for
	 *  specialized purposes, like forcing a pass to never be culled (NeverCull). See ERDGPassFlags for more info.
	 *
	 *  The pass name is used by debugging / profiling tools.
	 */
	template <typename ParameterStructType, typename ExecuteLambdaType>
	FRDGPassRef AddPass(FRDGEventName&& Name, const ParameterStructType* ParameterStruct, ERDGPassFlags Flags, ExecuteLambdaType&& ExecuteLambda);

	/** Adds a lambda pass to the graph with a runtime-generated parameter struct. */
	template <typename ExecuteLambdaType>
	FRDGPassRef AddPass(FRDGEventName&& Name, const FShaderParametersMetadata* ParametersMetadata, const void* ParameterStruct, ERDGPassFlags Flags, ExecuteLambdaType&& ExecuteLambda);

	/** Adds a lambda pass to the graph without any parameters. This useful for deferring RHI work onto the graph timeline,
	 *  or incrementally porting code to use the graph system. NeverCull and SkipRenderPass (if Raster) are implicitly added
	 *  to Flags. AsyncCompute is not allowed. It is never permitted to access a created (i.e. not externally registered) RDG
	 *  resource outside of passes it is registered with, as the RHI lifetime is not guaranteed.
	 */
	template <typename ExecuteLambdaType>
	FRDGPassRef AddPass(FRDGEventName&& Name, ERDGPassFlags Flags, ExecuteLambdaType&& ExecuteLambda);

	/** Adds a pass that takes a FRDGDispatchPassBuilder instead of a RHI command list. The lambda should create command lists
     *  and launch tasks to record commands into them. Each task should call EndRenderPass() (if raster) and FinishRecording()
     *  to complete each command list. The user is responsible for task management of dispatched command lists.
	 */
	template <typename ParameterStructType, typename LaunchLambdaType>
	FRDGPassRef AddDispatchPass(FRDGEventName&& Name, const ParameterStructType* ParameterStruct, ERDGPassFlags Flags, LaunchLambdaType&& LaunchLambda);

	/** Sets the expected workload of the pass execution lambda. The default workload is 1 and is more or less the 'average cost' of a pass.
	 *  Recommended usage is to set a workload equal to the number of complex draw / dispatch calls (each with its own parameters, etc), and
	 *  only as a performance tweak if a particular pass is very expensive relative to other passes.
	 */
	void SetPassWorkload(FRDGPass* Pass, uint32 Workload);

	/** Adds a user-defined dependency between two passes. This can be used to fine-tune async compute overlap by forcing a sync point. */
	RENDERCORE_API void AddPassDependency(FRDGPass* Producer, FRDGPass* Consumer);

	/** A hint to the builder to flush work to the RHI thread after the last queued pass on the execution timeline. */
	void AddDispatchHint();

	/** Launches a task that is synced prior to graph execution. If parallel execution is not enabled, the lambda is run immediately. */
	template <typename TaskLambda>
	UE::Tasks::FTask AddSetupTask(TaskLambda&& Task, bool bCondition = true, ERDGSetupTaskWaitPoint WaitPoint = ERDGSetupTaskWaitPoint::Compile);

	template <typename TaskLambda>
	UE::Tasks::FTask AddSetupTask(TaskLambda&& Task, UE::Tasks::ETaskPriority Priority, bool bCondition = true, ERDGSetupTaskWaitPoint WaitPoint = ERDGSetupTaskWaitPoint::Compile);

	template <typename TaskLambda>
	UE::Tasks::FTask AddSetupTask(TaskLambda&& Task, UE::Tasks::FPipe* Pipe, UE::Tasks::ETaskPriority Priority = UE::Tasks::ETaskPriority::Normal, bool bCondition = true, ERDGSetupTaskWaitPoint WaitPoint = ERDGSetupTaskWaitPoint::Compile);

	template <typename TaskLambda, typename PrerequisitesCollectionType>
	UE::Tasks::FTask AddSetupTask(TaskLambda&& Task, PrerequisitesCollectionType&& Prerequisites, UE::Tasks::ETaskPriority Priority = UE::Tasks::ETaskPriority::Normal, bool bCondition = true, ERDGSetupTaskWaitPoint WaitPoint = ERDGSetupTaskWaitPoint::Compile);

	template <typename TaskLambda, typename PrerequisitesCollectionType>
	UE::Tasks::FTask AddSetupTask(TaskLambda&& Task, UE::Tasks::FPipe* Pipe, PrerequisitesCollectionType&& Prerequisites, UE::Tasks::ETaskPriority Priority = UE::Tasks::ETaskPriority::Normal, bool bCondition = true, ERDGSetupTaskWaitPoint WaitPoint = ERDGSetupTaskWaitPoint::Compile);

	/** Launches a task that is synced prior to graph execution. If parallel execution is not enabled, the lambda is run immediately. */
	template <typename TaskLambda>
	UE::Tasks::FTask AddCommandListSetupTask(TaskLambda&& Task, bool bCondition = true, ERDGSetupTaskWaitPoint WaitPoint = ERDGSetupTaskWaitPoint::Compile);

	template <typename TaskLambda>
	UE::Tasks::FTask AddCommandListSetupTask(TaskLambda&& Task, UE::Tasks::ETaskPriority Priority, bool bCondition = true, ERDGSetupTaskWaitPoint WaitPoint = ERDGSetupTaskWaitPoint::Compile);

	template <typename TaskLambda>
	UE::Tasks::FTask AddCommandListSetupTask(TaskLambda&& Task, UE::Tasks::FPipe* Pipe, UE::Tasks::ETaskPriority Priority = UE::Tasks::ETaskPriority::Normal, bool bCondition = true, ERDGSetupTaskWaitPoint WaitPoint = ERDGSetupTaskWaitPoint::Compile);

	template <typename TaskLambda, typename PrerequisitesCollectionType>
	UE::Tasks::FTask AddCommandListSetupTask(TaskLambda&& Task, PrerequisitesCollectionType&& Prerequisites, UE::Tasks::ETaskPriority Priority = UE::Tasks::ETaskPriority::Normal, bool bCondition = true, ERDGSetupTaskWaitPoint WaitPoint = ERDGSetupTaskWaitPoint::Compile);

	template <typename TaskLambda, typename PrerequisitesCollectionType>
	UE::Tasks::FTask AddCommandListSetupTask(TaskLambda&& Task, UE::Tasks::FPipe* Pipe, PrerequisitesCollectionType&& Prerequisites, UE::Tasks::ETaskPriority Priority = UE::Tasks::ETaskPriority::Normal, bool bCondition = true, ERDGSetupTaskWaitPoint WaitPoint = ERDGSetupTaskWaitPoint::Compile);

	/** Whether RDG will launch async tasks when AddSetup{CommandList}Task is called. */
	inline bool IsParallelSetupEnabled() const
	{
		return ParallelSetup.bEnabled;
	}

	inline bool IsAsyncComputeEnabled() const
	{
		return bSupportsAsyncCompute;
	}

	/** Tells the builder to delete unused RHI resources. The behavior of this method depends on whether RDG immediate mode is enabled:
	 *   Deferred:  RHI resource flushes are performed prior to execution.
	 *   Immediate: RHI resource flushes are performed immediately.
	 */
	RENDERCORE_API void SetFlushResourcesRHI();

	/** Tells the RDG Builder that it is safe not to issue a fence from Async Compute -> Graphics at the start of the graph.
	 *  By default RDG will issue the fence to keep new work from overlapping with work from prior builders.
	 */
	void SkipInitialAsyncComputeFence();

	/** Queues a buffer upload operation prior to execution. The resource lifetime is extended and the data is uploaded prior to executing passes. */
	void QueueBufferUpload(FRDGBufferRef Buffer, const void* InitialData, uint64 InitialDataSize, ERDGInitialDataFlags InitialDataFlags = ERDGInitialDataFlags::None);

	template <typename ElementType>
	inline void QueueBufferUpload(FRDGBufferRef Buffer, TArrayView<ElementType, int32> Container, ERDGInitialDataFlags InitialDataFlags = ERDGInitialDataFlags::None)
	{
		QueueBufferUpload(Buffer, Container.GetData(), Container.Num() * sizeof(ElementType), InitialDataFlags);
	}

	/** Queues a buffer upload operation prior to execution. The resource lifetime is extended and the data is uploaded prior to executing passes. */
	void QueueBufferUpload(FRDGBufferRef Buffer, const void* InitialData, uint64 InitialDataSize, FRDGBufferInitialDataFreeCallback&& InitialDataFreeCallback);

	template <typename ElementType>
	inline void QueueBufferUpload(FRDGBufferRef Buffer, TArrayView<ElementType, int32> Container, FRDGBufferInitialDataFreeCallback&& InitialDataFreeCallback)
	{
		QueueBufferUpload(Buffer, Container.GetData(), Container.Num() * sizeof(ElementType), InitialDataFreeCallback);
	}

	/** A variant where the buffer is mapped and the pointer / size is provided to the callback to fill the buffer pointer. */
	void QueueBufferUpload(FRDGBufferRef Buffer, FRDGBufferInitialDataFillCallback&& InitialDataFillCallback);

	/** A variant where InitialData and InitialDataSize are supplied through callbacks. This allows queuing an upload with information unknown at
	 *  creation time. The callbacks are called before RDG pass execution so data must be ready before that.
	 */
	void QueueBufferUpload(FRDGBufferRef Buffer, FRDGBufferInitialDataCallback&& InitialDataCallback, FRDGBufferInitialDataSizeCallback&& InitialDataSizeCallback);
	void QueueBufferUpload(FRDGBufferRef Buffer, FRDGBufferInitialDataCallback&& InitialDataCallback, FRDGBufferInitialDataSizeCallback&& InitialDataSizeCallback, FRDGBufferInitialDataFreeCallback&& InitialDataFreeCallback);

	/** Queues a reserved buffer commit on first use of the buffer in the graph. The commit is applied at the start of the graph and
	 *  synced at the pass when the resource is first used, or at the end of the graph if the resource is unused and external. A resource
	 *  may only be assigned a commit size once.
	 */
	void QueueCommitReservedBuffer(FRDGBufferRef Buffer, uint64 CommitSizeInBytes);

	/** Queues a pooled render target extraction to happen at the end of graph execution. For graph-created textures, this extends
	 *  the lifetime of the GPU resource until execution, at which point the pointer is filled. If specified, the texture is transitioned
	 *  to the AccessFinal state, or kDefaultAccessFinal otherwise.
	 */
	void QueueTextureExtraction(FRDGTextureRef Texture, TRefCountPtr<IPooledRenderTarget>* OutPooledTexturePtr, ERDGResourceExtractionFlags Flags = ERDGResourceExtractionFlags::None);
	void QueueTextureExtraction(FRDGTextureRef Texture, TRefCountPtr<IPooledRenderTarget>* OutPooledTexturePtr, ERHIAccess AccessFinal, ERDGResourceExtractionFlags Flags = ERDGResourceExtractionFlags::None);

	/** Queues a pooled buffer extraction to happen at the end of graph execution. For graph-created buffers, this extends the lifetime
	 *  of the GPU resource until execution, at which point the pointer is filled. If specified, the buffer is transitioned to the
	 *  AccessFinal state, or kDefaultAccessFinal otherwise.
	 */
	void QueueBufferExtraction(FRDGBufferRef Buffer, TRefCountPtr<FRDGPooledBuffer>* OutPooledBufferPtr);
	void QueueBufferExtraction(FRDGBufferRef Buffer, TRefCountPtr<FRDGPooledBuffer>* OutPooledBufferPtr, ERHIAccess AccessFinal);

	/** For graph-created resources, this forces immediate allocation of the underlying pooled resource, effectively promoting it
	 *  to an external resource. This will increase memory pressure, but allows for querying the pooled resource with GetPooled{Texture, Buffer}.
	 *  This is primarily used as an aid for porting code incrementally to RDG.
	 */
	RENDERCORE_API const TRefCountPtr<IPooledRenderTarget>& ConvertToExternalTexture(FRDGTextureRef Texture);
	RENDERCORE_API const TRefCountPtr<FRDGPooledBuffer>& ConvertToExternalBuffer(FRDGBufferRef Buffer);
	/** For a graph-created uniform buffer, this forces immediate allocation of the underlying resource, effectively promoting it
	 *  to an external resource. This will increase memory pressure, but allows access to the RHI resource.
	 *  Graph resources that are referenced in the buffer will be converted to external.
	 *  This is primarily used as an aid for porting code incrementally to RDG.
	 */
	RENDERCORE_API FRHIUniformBuffer* ConvertToExternalUniformBuffer(FRDGUniformBufferRef UniformBuffer);

	/** Performs an immediate query for the underlying pooled resource. This is only allowed for external or extracted resources. */
	const TRefCountPtr<IPooledRenderTarget>& GetPooledTexture(FRDGTextureRef Texture) const;
	const TRefCountPtr<FRDGPooledBuffer>& GetPooledBuffer(FRDGBufferRef Buffer) const;

	/** (External | Extracted only) Sets the access to transition to after execution at the end of the graph. Overwrites any previously set final access. */
	void SetTextureAccessFinal(FRDGTextureRef Texture, ERHIAccess Access);

	/** (External | Extracted only) Sets the access to transition to after execution at the end of the graph. Overwrites any previously set final access. */
	void SetBufferAccessFinal(FRDGBufferRef Buffer, ERHIAccess Access);

	/** Configures the resource for external access for all subsequent passes, or until UseInternalAccessMode is called.
	 *  Only read-only access states are allowed. When in external access mode, it is safe to access the underlying RHI
	 *  resource directly in later RDG passes. This method is only allowed for registered or externally converted resources.
	 *  The method effectively guarantees that RDG will transition the resource into the desired state for all subsequent
	 *  passes so long as the resource remains externally accessible.
	 */
	RENDERCORE_API void UseExternalAccessMode(FRDGViewableResource* Resource, ERHIAccess ReadOnlyAccess, ERHIPipeline Pipelines = ERHIPipeline::Graphics);

	void UseExternalAccessMode(TArrayView<FRDGViewableResource* const> Resources, ERHIAccess ReadOnlyAccess, ERHIPipeline Pipelines = ERHIPipeline::Graphics)
	{
		for (FRDGViewableResource* Resource : Resources)
		{
			UseExternalAccessMode(Resource, ReadOnlyAccess, Pipelines);
		}
	}

	/** Use this method to resume tracking of a resource after calling UseExternalAccessMode. It is safe to call this method
	 *  even if external access mode was not enabled (it will simply no-op). It is not valid to access the underlying RHI
	 *  resource in any pass added after calling this method.
	 */
	RENDERCORE_API void UseInternalAccessMode(FRDGViewableResource* Resource);

	inline void UseInternalAccessMode(TArrayView<FRDGViewableResource* const> Resources)
	{
		for (FRDGViewableResource* Resource : Resources)
		{
			UseInternalAccessMode(Resource);
		}
	}

	/** Flushes all queued passes to an async task to perform setup work. */
	RENDERCORE_API void FlushSetupQueue();

	/** Executes the queued passes, managing setting of render targets (RHI RenderPasses), resource transitions and queued texture extraction. */
	RENDERCORE_API void Execute();

	/** Per-frame update of the render graph resource pool. */
	static RENDERCORE_API void TickPoolElements();

	/** Whether RDG is running in immediate mode. */
	static RENDERCORE_API bool IsImmediateMode();

	/** Waits for the last RDG async delete task that was launched. If the event is valid it is consumed. Thus, it is not thread safe to wait from multiple threads at once. */
	static RENDERCORE_API void WaitForAsyncDeleteTask();
	
	/** Returns the last RDG chained deletion task that was launched. */
	static RENDERCORE_API const UE::Tasks::FTask& GetAsyncDeleteTask();

	/** Waits for the last RDG chained execution task that was launched. */
	static RENDERCORE_API void WaitForAsyncExecuteTask();

	/** Returns the last RDG chained execution task that was launched. */
	static RENDERCORE_API const UE::Tasks::FTask& GetAsyncExecuteTask();

	/** The blackboard used to hold common data tied to the graph lifetime. */
	FRDGBlackboard Blackboard;

#if RDG_DUMP_RESOURCES
	static RENDERCORE_API FString BeginResourceDump(const TCHAR* Cmd = TEXT(""), const TCHAR* Context = TEXT(""));
	static RENDERCORE_API bool IsDumpingFrame();
#else
	static bool IsDumpingFrame() { return false; }
#endif

#if WITH_MGPU
	/** Copy all cross GPU external resources (not marked MultiGPUGraphIgnore) at the end of execution (bad for perf, but useful for debugging). */
	void EnableForceCopyCrossGPU()
	{
		bForceCopyCrossGPU = true;
	}
#endif

private:
	static const char* const kDefaultUnaccountedCSVStat;

	const FRDGEventName BuilderName;

	//////////////////////////////////////////////////////////////////////////////
	// Passes

	/** The epilogue and prologue passes are sentinels that are used to simplify graph logic around barriers
	*  and traversal. The prologue pass is used exclusively for barriers before the graph executes, while the
	*  epilogue pass is used for resource extraction barriers--a property that also makes it the main root of
	*  the graph for culling purposes. The epilogue pass is added to the very end of the pass array for traversal
	*  purposes. The prologue does not need to participate in any graph traversal behavior.
	*/
	FRDGPass* ProloguePass = nullptr;
	FRDGPass* EpiloguePass = nullptr;

	bool bInitialAsyncComputeFence = GSupportsEfficientAsyncCompute;
	bool bSupportsAsyncCompute = false;
	bool bSupportsRenderPassMerge = false;

	uint32 AsyncComputePassCount = 0;
	uint32 RasterPassCount = 0;

	/** Tracks dispatch passes that need to launch tasks. */
	TArray<FRDGDispatchPass*, FRDGArrayAllocator> DispatchPasses;

	RENDERCORE_API ERDGPassFlags OverridePassFlags(const TCHAR* PassName, ERDGPassFlags Flags) const;

	inline FRDGPass* GetProloguePass() const
	{
		return ProloguePass;
	}

	/** Returns the graph prologue pass handle. */
	inline FRDGPassHandle GetProloguePassHandle() const
	{
		return FRDGPassHandle(0);
	}

	/** Returns the graph epilogue pass handle. */
	inline FRDGPassHandle GetEpiloguePassHandle() const
	{
		checkf(EpiloguePass, TEXT("The handle is not valid until the epilogue has been added to the graph during execution."));
		return Passes.Last();
	}

	FRHIRenderPassInfo GetRenderPassInfo(const FRDGPass* Pass) const;

	template <typename ParameterStructType, typename ExecuteLambdaType>
	FRDGPass* AddPassInternal(
		FRDGEventName&& Name,
		const FShaderParametersMetadata* ParametersMetadata,
		const ParameterStructType* ParameterStruct,
		ERDGPassFlags Flags,
		ExecuteLambdaType&& ExecuteLambda);

	void MarkResourcesAsProduced(FRDGPass* Pass);

	RENDERCORE_API FRDGPass* SetupEmptyPass(FRDGPass* Pass);
	RENDERCORE_API FRDGPass* SetupParameterPass(FRDGPass* Pass);

	void SetupPassInternals(FRDGPass* Pass);
	void SetupPassResources(FRDGPass* Pass);
	void SetupPassDependencies(FRDGPass* Pass);

	void Compile();
	void CompilePassOps(FRDGPass* Pass);

	void ExecuteSerialPass(FRHIComputeCommandList& RHICmdListPass, FRDGPass* Pass);

	static void ExecutePass(FRHIComputeCommandList& RHICmdListPass, FRDGPass* Pass);
	static void ExecutePassPrologue(FRHIComputeCommandList& RHICmdListPass, FRDGPass* Pass);
	static void ExecutePassEpilogue(FRHIComputeCommandList& RHICmdListPass, FRDGPass* Pass);

	static void PushPreScopes (FRHIComputeCommandList& RHICmdListPass, FRDGPass* FirstPass);
	static void PushPassScopes(FRHIComputeCommandList& RHICmdListPass, FRDGPass* Pass);
	static void PopPassScopes (FRHIComputeCommandList& RHICmdListPass, FRDGPass* Pass);
	static void PopPreScopes  (FRHIComputeCommandList& RHICmdListPass, FRDGPass* LastPass);
	//////////////////////////////////////////////////////////////////////////////
	// Resource Registries

	/** Registry of graph objects. */
	FRDGPassRegistry Passes;
	FRDGTextureRegistry Textures;
	FRDGBufferRegistry Buffers;
	FRDGViewRegistry Views;
	FRDGUniformBufferRegistry UniformBuffers;

	struct FExtractedTexture
	{
		FExtractedTexture() = default;

		FExtractedTexture(FRDGTexture* InTexture, TRefCountPtr<IPooledRenderTarget>* InPooledTexture)
			: Texture(InTexture)
			, PooledTexture(InPooledTexture)
		{}

		FRDGTexture* Texture{};
		TRefCountPtr<IPooledRenderTarget>* PooledTexture{};
	};

	TArray<FExtractedTexture, FRDGArrayAllocator> ExtractedTextures;

	struct FExtractedBuffer
	{
		FExtractedBuffer() = default;

		FExtractedBuffer(FRDGBuffer* InBuffer, TRefCountPtr<FRDGPooledBuffer>* InPooledBuffer)
			: Buffer(InBuffer)
			, PooledBuffer(InPooledBuffer)
		{}

		FRDGBuffer* Buffer{};
		TRefCountPtr<FRDGPooledBuffer>* PooledBuffer{};
	};

	TArray<FExtractedBuffer, FRDGArrayAllocator> ExtractedBuffers;

	/** Tracks external resources to their registered render graph counterparts for de-duplication. */
	Experimental::TRobinHoodHashMap<FRHITexture*, FRDGTexture*, DefaultKeyFuncs<FRHITexture*>, FRDGArrayAllocator> ExternalTextures;
	Experimental::TRobinHoodHashMap<FRHIBuffer*,  FRDGBuffer*,  DefaultKeyFuncs<FRHIBuffer*>,  FRDGArrayAllocator> ExternalBuffers;

	/** Tracks buffers that have a defered num elements callback. */
	TArray<FRDGBuffer*, FRDGArrayAllocator> NumElementsCallbackBuffers;

	//////////////////////////////////////////////////////////////////////////////
	// Resource Collection and Allocation

	IRHITransientResourceAllocator* TransientResourceAllocator = nullptr; 
	bool bSupportsTransientTextures = false;
	bool bSupportsTransientBuffers = false;

	bool IsTransient(FRDGTextureRef Texture) const;
	bool IsTransient(FRDGBufferRef Buffer) const;
	bool IsTransientInternal(FRDGViewableResource* Resource, bool bFastVRAM) const;

	struct FCollectResourceOp
	{
		enum class EOp : uint8
		{
			Allocate,
			Deallocate
		};

		static FCollectResourceOp Allocate(FRDGBufferHandle BufferHandle)
		{
			return FCollectResourceOp(BufferHandle.GetIndex(), ERDGViewableResourceType::Buffer, EOp::Allocate);
		}

		static FCollectResourceOp Allocate(FRDGTextureHandle TextureHandle)
		{
			return FCollectResourceOp(TextureHandle.GetIndex(), ERDGViewableResourceType::Texture, EOp::Allocate);
		}

		static FCollectResourceOp Deallocate(FRDGBufferHandle BufferHandle)
		{
			return FCollectResourceOp(BufferHandle.GetIndex(), ERDGViewableResourceType::Buffer, EOp::Deallocate);
		}

		static FCollectResourceOp Deallocate(FRDGTextureHandle TextureHandle)
		{
			return FCollectResourceOp(TextureHandle.GetIndex(), ERDGViewableResourceType::Texture, EOp::Deallocate);
		}

		FCollectResourceOp() = default;
		FCollectResourceOp(uint32 InResourceIndex, ERDGViewableResourceType InResourceType, EOp InOp)
			: ResourceIndex(InResourceIndex)
			, ResourceType(static_cast<uint32>(InResourceType))
			, Op(static_cast<uint32>(InOp))
		{}

		EOp GetOp() const
		{
			return static_cast<EOp>(Op);
		}

		ERDGViewableResourceType GetResourceType() const
		{
			return static_cast<ERDGViewableResourceType>(ResourceType);
		}

		FRDGTextureHandle GetTextureHandle() const
		{
			check(GetResourceType() == ERDGViewableResourceType::Texture);
			return FRDGTextureHandle(ResourceIndex);
		}

		FRDGBufferHandle GetBufferHandle() const
		{
			check(GetResourceType() == ERDGViewableResourceType::Buffer);
			return FRDGBufferHandle(ResourceIndex);
		}

		uint32 ResourceIndex : 30;
		uint32 ResourceType : 1;
		uint32 Op : 1;
	};

	using FCollectResourceOpArray = TArray<FCollectResourceOp, FRDGArrayAllocator>;

	/** A temporary context used to collect resources for allocation. */
	struct FCollectResourceContext
	{
		FCollectResourceOpArray TransientResources;
		FCollectResourceOpArray PooledTextures;
		FCollectResourceOpArray PooledBuffers;
		TArray<FRDGUniformBufferHandle, FRDGArrayAllocator> UniformBuffers;
		TArray<FRDGViewHandle, FRDGArrayAllocator> Views;
		FRDGUniformBufferBitArray UniformBufferMap;
		FRDGViewBitArray ViewMap;
	};

	/** Tracks the latest RDG resource to own an alias of a pooled resource (multiple RDG resources can reference the same pooled resource). */
	Experimental::TRobinHoodHashMap<FRDGPooledTexture*, FRDGTexture*, DefaultKeyFuncs<FRDGPooledTexture*>, FConcurrentLinearArrayAllocator> PooledTextureOwnershipMap;
	Experimental::TRobinHoodHashMap<FRDGPooledBuffer*, FRDGBuffer*, DefaultKeyFuncs<FRDGPooledBuffer*>, FConcurrentLinearArrayAllocator> PooledBufferOwnershipMap;

	/** Finalizes the resource descriptors by calling callbacks to gather resource sizes. */
	void FinalizeDescs();

	/** Collects new resource allocations for the pass into the provided context. */
	void CollectAllocations(FCollectResourceContext& Context, FRDGPass* Pass);
	void CollectAllocateTexture(FCollectResourceContext& Context, ERHIPipeline PassPipeline, FRDGPassHandle PassHandle, FRDGTexture* Texture);
	void CollectAllocateBuffer(FCollectResourceContext& Context, ERHIPipeline PassPipeline, FRDGPassHandle PassHandle, FRDGBuffer* Buffer);

	/** Collects new resource deallocations for the pass into the provided context. */
	void CollectDeallocations(FCollectResourceContext& Context, FRDGPass* Pass);
	void CollectDeallocateTexture(FCollectResourceContext& Context, ERHIPipeline PassPipeline, FRDGPassHandle PassHandle, FRDGTexture* Texture, uint32 ReferenceCount);
	void CollectDeallocateBuffer(FCollectResourceContext& Context, ERHIPipeline PassPipeline, FRDGPassHandle PassHandle, FRDGBuffer* Buffer, uint32 ReferenceCount);

	/** Allocates resources using the provided lifetime op arrays. */
	void AllocateTransientResources(TConstArrayView<FCollectResourceOp> Ops);
	void AllocatePooledTextures(FRHICommandListBase& RHICmdList, TConstArrayView<FCollectResourceOp> Ops);
	void AllocatePooledBuffers(FRHICommandListBase& RHICmdList, TConstArrayView<FCollectResourceOp> Ops);

	/** Creates resources for the provided handles. */
	void CreateViews(FRHICommandListBase& RHICmdList, TConstArrayView<FRDGViewHandle> ViewsToCreate);
	void CreateUniformBuffers(TConstArrayView<FRDGUniformBufferHandle> UniformBuffersToCreate);

	/** Allocates and returns a pooled resource for the RDG resource. Does not assign it. */
	TRefCountPtr<IPooledRenderTarget> AllocatePooledRenderTargetRHI(FRHICommandListBase& RHICmdList, FRDGTextureRef Texture);
	TRefCountPtr<FRDGPooledBuffer> AllocatePooledBufferRHI(FRHICommandListBase& RHICmdList, FRDGBufferRef Buffer);

	/** Assigns an underlying RHI resource to an RDG resource. */
	void SetExternalPooledRenderTargetRHI(FRDGTexture* Texture, IPooledRenderTarget* RenderTarget);
	void SetPooledTextureRHI(FRDGTexture* Texture, FRDGPooledTexture* PooledTexture);
	void SetTransientTextureRHI(FRDGTexture* Texture, FRHITransientTexture* TransientTexture);
	void SetDiscardPass(FRDGTexture* Texture, FRHITransientTexture* TransientTexture);
	void SetExternalPooledBufferRHI(FRDGBuffer* Buffer, const TRefCountPtr<FRDGPooledBuffer>& PooledBuffer);
	void SetPooledBufferRHI(FRDGBuffer* Buffer, FRDGPooledBuffer* PooledBuffer);
	void SetTransientBufferRHI(FRDGBuffer* Buffer, FRHITransientBuffer* TransientBuffer);

	/** Initializes various view types. Assumes that the underlying RHI viewable resource type is assigned. */
	void InitViewRHI(FRHICommandListBase& RHICmdList, FRDGView* View);
	void InitBufferViewRHI(FRHICommandListBase& RHICmdList, FRDGBufferSRV* SRV);
	void InitBufferViewRHI(FRHICommandListBase& RHICmdList, FRDGBufferUAV* UAV);
	void InitTextureViewRHI(FRHICommandListBase& RHICmdList, FRDGTextureSRV* SRV);
	void InitTextureViewRHI(FRHICommandListBase& RHICmdList, FRDGTextureUAV* UAV);

	//////////////////////////////////////////////////////////////////////////////
	// Resource Transitions and State Tracking

	/** Map of barrier batches begun from more than one pipe. */
	TMap<FRDGBarrierBatchBeginId, FRDGBarrierBatchBegin*, FRDGSetAllocator> BarrierBatchMap;

	/** Tracks the final access used on resources in order to call SetTrackedAccess. */
	TArray<FRHITrackedAccessInfo, FRDGArrayAllocator> EpilogueResourceAccesses;

	/** Array of all pooled references held during execution. */
	TArray<TRefCountPtr<IPooledRenderTarget>, FRDGArrayAllocator> ActivePooledTextures;
	TArray<TRefCountPtr<FRDGPooledBuffer>, FRDGArrayAllocator> ActivePooledBuffers;

	/** Set of all active barrier batch begin instances; used to create transitions. */
	FRDGTransitionCreateQueue TransitionCreateQueue;

	/** Texture state used for intermediate operations. Held here to avoid re-allocating. */
	FRDGTextureSubresourceState ScratchTextureState;

	/** Subresource state representing the graph prologue. Used for immediate mode. */
	FRDGSubresourceState PrologueSubresourceState;

	void CompilePassBarriers();
	void CollectPassBarriers();
	void CollectPassBarriers(FRDGPassHandle PassHandle);
	void CreatePassBarriers();
	void FinalizeResources();

	void AddFirstTextureTransition(FRDGTextureRef Texture);
	void AddFirstBufferTransition(FRDGBufferRef Buffer);

	void AddLastTextureTransition(FRDGTextureRef Texture);
	void AddLastBufferTransition(FRDGBufferRef Buffer);

	void AddCulledReservedCommitTransition(FRDGBufferRef Buffer);

	template <typename FilterSubresourceLambdaType>
	void AddTextureTransition(
		FRDGTextureRef Texture,
		FRDGTextureSubresourceState& StateBefore,
		FRDGTextureSubresourceState& StateAfter,
		FilterSubresourceLambdaType&& FilterSubresourceLambda);

	void AddTextureTransition(
		FRDGTextureRef Texture,
		FRDGTextureSubresourceState& StateBefore,
		FRDGTextureSubresourceState& StateAfter)
	{
		AddTextureTransition(Texture, StateBefore, StateAfter, [](FRDGSubresourceState*, int32) { return true; });
	}

	template <typename FilterSubresourceLambdaType>
	void AddBufferTransition(
		FRDGBufferRef Buffer,
		FRDGSubresourceState*& StateBefore,
		FRDGSubresourceState* StateAfter,
		FilterSubresourceLambdaType&& FilterSubresourceLambda);

	void AddBufferTransition(
		FRDGBufferRef Buffer,
		FRDGSubresourceState*& StateBefore,
		FRDGSubresourceState* StateAfter)
	{
		AddBufferTransition(Buffer, StateBefore, StateAfter, [](FRDGSubresourceState*) { return true; });
	}

	void AddTransition(
		FRDGViewableResource* Resource,
		FRDGSubresourceState StateBefore,
		FRDGSubresourceState StateAfter,
		FRDGTransitionInfo TransitionInfo);

	void AddAliasingTransition(
		FRDGPassHandle BeginPassHandle,
		FRDGPassHandle EndPassHandle,
		FRDGViewableResource* Resource,
		const FRHITransientAliasingInfo& Info);

	/** Prologue and Epilogue barrier passes are used to plan transitions around RHI render pass merging,
	*  as it is illegal to issue a barrier during a render pass. If passes [A, B, C] are merged together,
	*  'A' becomes 'B's prologue pass and 'C' becomes 'A's epilogue pass. This way, any transitions that
	*  need to happen before the merged pass (i.e. in the prologue) are done in A. Any transitions after
	*  the render pass merge are done in C.
	*/
	FRDGPassHandle GetEpilogueBarrierPassHandle(FRDGPassHandle Handle)
	{
		return Passes[Handle]->EpilogueBarrierPass;
	}

	FRDGPassHandle GetPrologueBarrierPassHandle(FRDGPassHandle Handle)
	{
		return Passes[Handle]->PrologueBarrierPass;
	}

	FRDGPass* GetEpilogueBarrierPass(FRDGPassHandle Handle)
	{
		return Passes[GetEpilogueBarrierPassHandle(Handle)];
	}

	FRDGPass* GetPrologueBarrierPass(FRDGPassHandle Handle)
	{
		return Passes[GetPrologueBarrierPassHandle(Handle)];
	}

	/** Ends the barrier batch in the prologue of the provided pass. */
	void AddToPrologueBarriersToEnd(FRDGPassHandle Handle, FRDGBarrierBatchBegin& BarriersToBegin)
	{
		FRDGPass* Pass = GetPrologueBarrierPass(Handle);
		Pass->GetPrologueBarriersToEnd(Allocators.Transition).AddDependency(&BarriersToBegin);
	}

	/** Ends the barrier batch in the epilogue of the provided pass. */
	void AddToEpilogueBarriersToEnd(FRDGPassHandle Handle, FRDGBarrierBatchBegin& BarriersToBegin)
	{
		FRDGPass* Pass = GetEpilogueBarrierPass(Handle);
		Pass->GetEpilogueBarriersToEnd(Allocators.Transition).AddDependency(&BarriersToBegin);
	}

	/** Utility function to add an immediate barrier dependency in the prologue of the provided pass. */
	template <typename FunctionType>
	void AddToPrologueBarriers(FRDGPassHandle PassHandle, FunctionType Function)
	{
		FRDGPass* Pass = GetPrologueBarrierPass(PassHandle);
		FRDGBarrierBatchBegin& BarriersToBegin = Pass->GetPrologueBarriersToBegin(Allocators.Transition, TransitionCreateQueue);
		Function(BarriersToBegin);
		Pass->GetPrologueBarriersToEnd(Allocators.Transition).AddDependency(&BarriersToBegin);
	}

	/** Utility function to add an immediate barrier dependency in the epilogue of the provided pass. */
	template <typename FunctionType>
	void AddToEpilogueBarriers(FRDGPassHandle PassHandle, FunctionType Function)
	{
		FRDGPass* Pass = GetEpilogueBarrierPass(PassHandle);
		FRDGBarrierBatchBegin& BarriersToBegin = Pass->GetEpilogueBarriersToBeginFor(Allocators.Transition, TransitionCreateQueue, Pass->GetPipeline());
		Function(BarriersToBegin);
		Pass->GetEpilogueBarriersToEnd(Allocators.Transition).AddDependency(&BarriersToBegin);
	}

	// Returns fences representing an allocation event, which can only happen on one pipeline at a time.
	FRHITransientAllocationFences GetAllocateFences(FRDGViewableResource* Resource) const;

	// Returns fences representing a deallocation event, which can happen on multiple pipes.
	FRHITransientAllocationFences GetDeallocateFences(FRDGViewableResource* Resource) const;

	inline ERHIPipeline GetPassPipeline(FRDGPassHandle PassHandle) const
	{
		return Passes[PassHandle]->Pipeline;
	}

	FRDGSubresourceState* AllocSubresource(const FRDGSubresourceState& Other);
	FRDGSubresourceState* AllocSubresource();

	//////////////////////////////////////////////////////////////////////////////
	// Async Setup Queue

	struct FAsyncSetupOp
	{
		enum class EType : uint8
		{
			SetupPassResources,
			CullRootBuffer,
			CullRootTexture,
			ReservedBufferCommit
		};

		static FAsyncSetupOp SetupPassResources(FRDGPass* Pass)
		{
			FAsyncSetupOp Op(EType::SetupPassResources);
			Op.Pass = Pass;
			return Op;
		}

		static FAsyncSetupOp CullRootBuffer(FRDGBuffer* Buffer)
		{
			FAsyncSetupOp Op(EType::CullRootBuffer);
			Op.Buffer = Buffer;
			return Op;
		}

		static FAsyncSetupOp CullRootTexture(FRDGTexture* Texture)
		{
			FAsyncSetupOp Op(EType::CullRootTexture);
			Op.Texture = Texture;
			return Op;
		}

		static FAsyncSetupOp ReservedBufferCommit(FRDGBuffer* Buffer, uint64 CommitSizeInBytes)
		{
			FAsyncSetupOp Op(EType::ReservedBufferCommit, CommitSizeInBytes);
			Op.Buffer = Buffer;
			return Op;
		}

		EType GetType() const
		{
			return (EType)Type;
		}

		uint64 Type : 8;
		uint64 Payload : 48;

		FAsyncSetupOp(EType InType, uint64 InPayload = 0)
			: Type((uint8)InType)
			, Payload(InPayload)
		{
			check(InPayload < (1ull << 48ull));
		}

		union
		{
			FRDGPass*    Pass;
			FRDGBuffer*  Buffer;
			FRDGTexture* Texture;
		};
	};

	struct FAsyncSetupQueue
	{
		void Push(FAsyncSetupOp Op)
		{
			UE::TScopeLock Lock(Mutex);
			Ops.Emplace(Op);
		}

		UE::FMutex Mutex;
		TArray<FAsyncSetupOp, FRDGArrayAllocator> Ops;
		UE::Tasks::FTask LastTask;
		bool bEnabled = false;

	} AsyncSetupQueue;

	void LaunchAsyncSetupQueueTask();
	void ProcessAsyncSetupQueue();

	//////////////////////////////////////////////////////////////////////////////
	// Reserved Buffer Commits

	FRDGBufferReservedCommitHandle AcquireReservedCommitHandle(FRDGBuffer* Buffer)
	{
		FRDGBufferReservedCommitHandle Handle;

		if (Buffer->PendingCommitSize > 0)
		{
			Handle = FRDGBufferReservedCommitHandle(ReservedBufferCommitSizes.Num());
			ReservedBufferCommitSizes.Emplace(Buffer->PendingCommitSize);
			Buffer->PendingCommitSize = 0;
		}

		return Handle;
	}

	uint64 GetReservedCommitSize(FRDGBufferReservedCommitHandle Handle)
	{
		return Handle.IsValid() ? ReservedBufferCommitSizes[Handle.GetIndex()] : 0;
	}

	TArray<uint64, FRDGArrayAllocator> ReservedBufferCommitSizes;

	//////////////////////////////////////////////////////////////////////////////
	// Culling

	TArray<FRDGPass*, FRDGArrayAllocator> CullPassStack;

	bool AddCullingDependency(FRDGProducerStatesByPipeline& LastProducers, const FRDGProducerState& NextState, ERHIPipeline NextPipeline);
	void AddCullRootBuffer(FRDGBuffer* Buffer);
	void AddCullRootTexture(FRDGTexture* Texture);
	void AddLastProducersToCullStack(const FRDGProducerStatesByPipeline& LastProducers);
	void FlushCullStack();

	//////////////////////////////////////////////////////////////////////////////
	// Parallel Setup

	struct
	{
		/** Array of all tasks for variants of AddSetupTask. */
		TStaticArray<TArray<UE::Tasks::FTask, FRDGArrayAllocator>, (int32)ERDGSetupTaskWaitPoint::MAX> Tasks;

		bool bEnabled = false;
		int8 TaskPriorityBias = 0;

		UE::Tasks::ETaskPriority GetTaskPriority(UE::Tasks::ETaskPriority TaskPriority) const
		{
			return (UE::Tasks::ETaskPriority)FMath::Clamp((int32)TaskPriority - TaskPriorityBias, 0, (int32)UE::Tasks::ETaskPriority::Count - 1);
		}

	} ParallelSetup;

	void WaitForParallelSetupTasks(ERDGSetupTaskWaitPoint WaitPoint);

	/////////////////////////////////////////////////////////////////////////////
	// Parallel Execution

	bool bCompiling = false;
	bool bParallelCompileEnabled = false;

	struct FParallelExecute
	{
		TArray<FParallelPassSet, FRDGArrayAllocator> ParallelPassSets;
		TOptional<UE::Tasks::FTaskEvent> TasksAwait;
		TOptional<UE::Tasks::FTaskEvent> TasksAsync;
		TOptional<UE::Tasks::FTaskEvent> DispatchTaskEventAwait;
		TOptional<UE::Tasks::FTaskEvent> DispatchTaskEventAsync;
		ERDGPassTaskMode TaskMode = ERDGPassTaskMode::Inline;

		bool IsEnabled() const { return TaskMode != ERDGPassTaskMode::Inline; }

		static UE::Tasks::FTask LastAsyncExecuteTask;

	} ParallelExecute;

	void SetupParallelExecute(TStaticArray<void*, MAX_NUM_GPUS> const& QueryBatchData);
	void SetupDispatchPassExecute();

	/////////////////////////////////////////////////////////////////////////////
	// Buffer Uploads

	struct FUploadedBuffer
	{
		FUploadedBuffer() = default;

		FUploadedBuffer(FRDGBuffer* InBuffer, const void* InData, uint64 InDataSize)
			: Buffer(InBuffer)
			, Data(InData)
			, DataSize(InDataSize)
		{}

		FUploadedBuffer(FRDGBuffer* InBuffer, FRDGBufferInitialDataFillCallback&& InDataFillCallback)
			: Buffer(InBuffer)
			, DataFillCallback(MoveTemp(InDataFillCallback))
		{}

		FUploadedBuffer(FRDGBuffer* InBuffer, const void* InData, uint64 InDataSize, FRDGBufferInitialDataFreeCallback&& InDataFreeCallback)
			: bUseFreeCallbacks(true)
			, Buffer(InBuffer)
			, Data(InData)
			, DataSize(InDataSize)
			, DataFreeCallback(MoveTemp(InDataFreeCallback))
		{}

		FUploadedBuffer(FRDGBuffer* InBuffer, FRDGBufferInitialDataCallback&& InDataCallback, FRDGBufferInitialDataSizeCallback&& InDataSizeCallback)
			: bUseDataCallbacks(true)
			, Buffer(InBuffer)
			, DataCallback(MoveTemp(InDataCallback))
			, DataSizeCallback(MoveTemp(InDataSizeCallback))
		{}

		FUploadedBuffer(FRDGBuffer* InBuffer, FRDGBufferInitialDataCallback&& InDataCallback, FRDGBufferInitialDataSizeCallback&& InDataSizeCallback, FRDGBufferInitialDataFreeCallback&& InDataFreeCallback)
			: bUseDataCallbacks(true)
			, bUseFreeCallbacks(true)
			, Buffer(InBuffer)
			, DataCallback(MoveTemp(InDataCallback))
			, DataSizeCallback(MoveTemp(InDataSizeCallback))
			, DataFreeCallback(MoveTemp(InDataFreeCallback))
		{}

		bool bUseDataCallbacks = false;
		bool bUseFreeCallbacks = false;
		FRDGBuffer* Buffer{};
		const void* Data{};
		uint64 DataSize{};

		// User provided data callbacks
		FRDGBufferInitialDataCallback DataCallback;
		FRDGBufferInitialDataSizeCallback DataSizeCallback;
		FRDGBufferInitialDataFreeCallback DataFreeCallback;

		// RDG provided buffer pointer callback.
		FRDGBufferInitialDataFillCallback DataFillCallback;
	};

	TArray<FUploadedBuffer, FRDGArrayAllocator> UploadedBuffers;

	void SubmitBufferUploads(FRHICommandList& InRHICmdList, UE::Tasks::FTaskEvent* AllocateUploadBuffersTask = nullptr);

	/////////////////////////////////////////////////////////////////////////////
	// External Access Queue

	/** Contains resources queued for either access mode change passes. */
	TArray<FRDGViewableResource*, FRDGArrayAllocator> AccessModeQueue;
	TSet<FRDGViewableResource*, DefaultKeyFuncs<FRDGViewableResource*>, FRDGSetAllocator> ExternalAccessResources;

	RENDERCORE_API void FlushAccessModeQueue();

	/////////////////////////////////////////////////////////////////////////////
	// Post-Execution Callbacks

	TArray<TUniqueFunction<void()>, FRDGArrayAllocator> PostExecuteCallbacks;

	/////////////////////////////////////////////////////////////////////////////
	// Resource Deletion Flushing

	FGraphEventArray WaitOutstandingTasks;
	bool bFlushResourcesRHI = false;
	FRHICommandListScopedExtendResourceLifetime ExtendResourceLifetimeScope;

	void BeginFlushResourcesRHI();
	void EndFlushResourcesRHI();

	/////////////////////////////////////////////////////////////////////////////
	// Clobber, Visualize, and DumpGPU tools.

	/** Tracks stack counters of auxiliary passes to avoid calling them recursively. */
	struct FAuxiliaryPass
	{
		uint8 Clobber = 0;
		uint8 Visualize = 0;
		uint8 Dump = 0;
		uint8 FlushAccessModeQueue = 0;

		bool IsDumpAllowed() const { return Dump == 0; }
		bool IsVisualizeAllowed() const { return Visualize == 0; }
		bool IsClobberAllowed() const { return Clobber == 0; }
		bool IsFlushAccessModeQueueAllowed() const { return FlushAccessModeQueue == 0; }

		bool IsActive() const { return Clobber > 0 || Visualize > 0 || Dump > 0 || FlushAccessModeQueue > 0; }

	} AuxiliaryPasses;

	void SetupAuxiliaryPasses(FRDGPass* Pass);

#if RDG_DUMP_RESOURCES
	void DumpNewGraphBuilder();
	void DumpResourcePassOutputs(const FRDGPass* Pass);
#endif

#if RDG_ENABLE_DEBUG
	RENDERCORE_API void VisualizePassOutputs(const FRDGPass* Pass);
	RENDERCORE_API void ClobberPassOutputs(const FRDGPass* Pass);
#endif

	/////////////////////////////////////////////////////////////////////////////
	// Multi-GPU

#if WITH_MGPU
	/** Copy all cross GPU external resources (not marked MultiGPUGraphIgnore) at the end of execution (bad for perf, but useful for debugging). */
	bool bForceCopyCrossGPU = false;

	void ForceCopyCrossGPU();
#endif

	/////////////////////////////////////////////////////////////////////////////
	// Validation and Tracing

	IF_RDG_ENABLE_TRACE(FRDGTrace Trace);

#if RDG_ENABLE_DEBUG
	FRDGUserValidation UserValidation;
	FRDGBarrierValidation BarrierValidation;
#endif

	/////////////////////////////////////////////////////////////////////////////

	friend FRDGTrace;
	friend FRDGAsyncComputeBudgetScopeGuard;
	friend FRDGScopedCsvStatExclusive;
	friend FRDGScopedCsvStatExclusiveConditional;
	friend FRDGDispatchPassBuilder;
};

class FRDGAsyncComputeBudgetScopeGuard final
{
public:
	FRDGAsyncComputeBudgetScopeGuard(FRDGBuilder& InGraphBuilder, EAsyncComputeBudget InAsyncComputeBudget)
		: GraphBuilder(InGraphBuilder)
	{
		// Deprecated
	}

	~FRDGAsyncComputeBudgetScopeGuard() = default;

private:
	FRDGBuilder& GraphBuilder;
};

#define RDG_ASYNC_COMPUTE_BUDGET_SCOPE(GraphBuilder, AsyncComputeBudget) \
	FRDGAsyncComputeBudgetScopeGuard UE_JOIN(FRDGAsyncComputeBudgetScope, __LINE__)(GraphBuilder, AsyncComputeBudget)

#if WITH_MGPU
	#define RDG_GPU_MASK_SCOPE(GraphBuilder, GPUMask) SCOPED_GPU_MASK(GraphBuilder.RHICmdList, GPUMask)
#else
	#define RDG_GPU_MASK_SCOPE(GraphBuilder, GPUMask)
#endif

#include "RenderGraphBuilder.inl" // IWYU pragma: export

class FRDGAsyncComputeBudgetScopeGuard;
class FRHITransientBuffer;
class FRHITransientTexture;
class FShaderParametersMetadata;
class IRHICommandContext;
class IRHITransientResourceAllocator;

