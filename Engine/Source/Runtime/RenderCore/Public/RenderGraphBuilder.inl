// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

inline FRDGTexture* FRDGBuilder::FindExternalTexture(FRHITexture* ExternalTexture) const
{
	if (FRDGTexture* const* FoundTexturePtr = ExternalTextures.Find(ExternalTexture))
	{
		return *FoundTexturePtr;
	}
	return nullptr;
}

inline FRDGTexture* FRDGBuilder::FindExternalTexture(IPooledRenderTarget* ExternalTexture) const
{
	if (ExternalTexture)
	{
		return FindExternalTexture(ExternalTexture->GetRHI());
	}
	return nullptr;
}

inline FRDGBuffer* FRDGBuilder::FindExternalBuffer(FRHIBuffer* ExternalBuffer) const
{
	if (FRDGBuffer* const* FoundBufferPtr = ExternalBuffers.Find(ExternalBuffer))
	{
		return *FoundBufferPtr;
	}
	return nullptr;
}

inline FRDGBuffer* FRDGBuilder::FindExternalBuffer(FRDGPooledBuffer* ExternalBuffer) const
{
	if (ExternalBuffer)
	{
		return FindExternalBuffer(ExternalBuffer->GetRHI());
	}
	return nullptr;
}

inline FRDGTextureRef FRDGBuilder::CreateTexture(
	const FRDGTextureDesc& Desc,
	const TCHAR* Name,
	ERDGTextureFlags Flags)
{
	FRDGTextureDesc OverrideDesc = Desc;

#if !UE_BUILD_SHIPPING
	ensureMsgf(OverrideDesc.Extent.X >= 1, TEXT("CreateTexture %s X size too small: %i, Min: %i, clamping"), Name ? Name : TEXT(""), OverrideDesc.Extent.X, 1);
	ensureMsgf(OverrideDesc.Extent.Y >= 1, TEXT("CreateTexture %s Y size too small: %i, Min: %i, clamping"), Name ? Name : TEXT(""), OverrideDesc.Extent.Y, 1);
	ensureMsgf(((uint32)OverrideDesc.Extent.X) <= GetMax2DTextureDimension(), TEXT("CreateTexture %s X size too large: %i, Max: %i, clamping"), Name ? Name : TEXT(""), OverrideDesc.Extent.X, GetMax2DTextureDimension());
	ensureMsgf(((uint32)OverrideDesc.Extent.Y) <= GetMax2DTextureDimension(), TEXT("CreateTexture %s Y size too large: %i, Max: %i, clamping"), Name ? Name : TEXT(""), OverrideDesc.Extent.Y, GetMax2DTextureDimension());
#endif
	// Clamp the texture size to that which is permissible, otherwise it's a guaranteed crash.
	OverrideDesc.Extent.X = FMath::Clamp<int32>(OverrideDesc.Extent.X, 1, GetMax2DTextureDimension());
	OverrideDesc.Extent.Y = FMath::Clamp<int32>(OverrideDesc.Extent.Y, 1, GetMax2DTextureDimension());

	IF_RDG_ENABLE_DEBUG(UserValidation.ValidateCreateTexture(OverrideDesc, Name, Flags));
	FRDGTextureRef Texture = Textures.Allocate(Allocators.Root, Name, OverrideDesc, Flags);
	IF_RDG_ENABLE_DEBUG(UserValidation.ValidateCreateTexture(Texture));
	IF_RDG_ENABLE_TRACE(Trace.AddResource(Texture));
	return Texture;
}

inline FRDGBufferRef FRDGBuilder::CreateBuffer(
	const FRDGBufferDesc& Desc,
	const TCHAR* Name,
	ERDGBufferFlags Flags)
{
	FRDGBufferDesc OverrideDesc = Desc;

	IF_RDG_ENABLE_DEBUG(UserValidation.ValidateCreateBuffer(OverrideDesc, Name, Flags));

	// Clamp the buffer size to that which is permissible, otherwise it's a guaranteed crash.
	OverrideDesc.BytesPerElement = FMath::Max<uint32>(1u, OverrideDesc.BytesPerElement);
	OverrideDesc.NumElements = FMath::Max<uint32>(1u, OverrideDesc.NumElements);

	FRDGBufferRef Buffer = Buffers.Allocate(Allocators.Root, Name, OverrideDesc, Flags);
	IF_RDG_ENABLE_DEBUG(UserValidation.ValidateCreateBuffer(Buffer));
	IF_RDG_ENABLE_TRACE(Trace.AddResource(Buffer));
	return Buffer;
}

inline FRDGBufferRef FRDGBuilder::CreateBuffer(
	const FRDGBufferDesc& Desc,
	const TCHAR* Name,
	FRDGBufferNumElementsCallback&& InNumElementsCallback,
	ERDGBufferFlags Flags)
{
	// RDG no longer supports the legacy transient resource API.
	FRDGBufferDesc OverrideDesc = Desc;

	IF_RDG_ENABLE_DEBUG(UserValidation.ValidateCreateBuffer(Desc, Name, Flags));
	FRDGBufferNumElementsCallback* NumElementsCallback = InNumElementsCallback ? Allocators.Root.AllocNoDestruct<FRDGBufferNumElementsCallback>(MoveTemp(InNumElementsCallback)) : nullptr;
	FRDGBufferRef Buffer = Buffers.Allocate(Allocators.Root, Name, OverrideDesc, Flags, NumElementsCallback);
	NumElementsCallbackBuffers.Emplace(Buffer);
	IF_RDG_ENABLE_DEBUG(UserValidation.ValidateCreateBuffer(Buffer));
	IF_RDG_ENABLE_TRACE(Trace.AddResource(Buffer));
	return Buffer;
}

inline FRDGTextureSRVRef FRDGBuilder::CreateSRV(const FRDGTextureSRVDesc& Desc)
{
	IF_RDG_ENABLE_DEBUG(UserValidation.ValidateCreateSRV(Desc));
	FRDGTextureSRVRef SRV = Views.Allocate<FRDGTextureSRV>(Allocators.Root, Desc.Texture->Name, Desc);
	IF_RDG_ENABLE_DEBUG(UserValidation.ValidateCreateSRV(SRV));
	return SRV;
}

inline FRDGBufferSRVRef FRDGBuilder::CreateSRV(const FRDGBufferSRVDesc& Desc)
{
	IF_RDG_ENABLE_DEBUG(UserValidation.ValidateCreateSRV(Desc));
	FRDGBufferSRVRef SRV = Views.Allocate<FRDGBufferSRV>(Allocators.Root, Desc.Buffer->Name, Desc);
	IF_RDG_ENABLE_DEBUG(UserValidation.ValidateCreateSRV(SRV));
	return SRV;
}

inline FRDGTextureUAVRef FRDGBuilder::CreateUAV(const FRDGTextureUAVDesc& Desc, ERDGUnorderedAccessViewFlags InFlags)
{
	IF_RDG_ENABLE_DEBUG(UserValidation.ValidateCreateUAV(Desc));
	FRDGTextureUAVRef UAV = Views.Allocate<FRDGTextureUAV>(Allocators.Root, Desc.Texture->Name, Desc, InFlags);
	IF_RDG_ENABLE_DEBUG(UserValidation.ValidateCreateUAV(UAV));
	return UAV;
}

inline FRDGBufferUAVRef FRDGBuilder::CreateUAV(const FRDGBufferUAVDesc& Desc, ERDGUnorderedAccessViewFlags InFlags)
{
	IF_RDG_ENABLE_DEBUG(UserValidation.ValidateCreateUAV(Desc));
	FRDGBufferUAVRef UAV = Views.Allocate<FRDGBufferUAV>(Allocators.Root, Desc.Buffer->Name, Desc, InFlags);
	IF_RDG_ENABLE_DEBUG(UserValidation.ValidateCreateUAV(UAV));
	return UAV;
}

inline void* FRDGBuilder::Alloc(uint64 SizeInBytes, uint32 AlignInBytes)
{
	return Allocators.Root.Alloc(SizeInBytes, AlignInBytes);
}

template <typename PODType>
inline PODType* FRDGBuilder::AllocPOD()
{
	return Allocators.Root.AllocUninitialized<PODType>();
}

template <typename PODType>
inline PODType* FRDGBuilder::AllocPODArray(uint32 Count)
{
	return Allocators.Root.AllocUninitialized<PODType>(Count);
}

template <typename PODType>
TArrayView<PODType> FRDGBuilder::AllocPODArrayView(uint32 Count)
{
	return TArrayView<PODType>(AllocPODArray<PODType>(Count), Count);
}

template <typename ObjectType, typename... TArgs>
inline ObjectType* FRDGBuilder::AllocObject(TArgs&&... Args)
{
	return Allocators.Root.Alloc<ObjectType>(Forward<TArgs&&>(Args)...);
}

template <typename ObjectType>
inline TArray<ObjectType, SceneRenderingAllocator>& FRDGBuilder::AllocArray()
{
	return *Allocators.Root.Alloc<TArray<ObjectType, SceneRenderingAllocator>>();
}

template <typename ParameterStructType>
inline ParameterStructType* FRDGBuilder::AllocParameters()
{
	return Allocators.Root.Alloc<ParameterStructType>();
}

template <typename ParameterStructType>
inline ParameterStructType* FRDGBuilder::AllocParameters(const ParameterStructType* StructToCopy)
{
	ParameterStructType* Struct = Allocators.Root.Alloc<ParameterStructType>();
	*Struct = *StructToCopy;
	return Struct;
}

template <typename BaseParameterStructType>
BaseParameterStructType* FRDGBuilder::AllocParameters(const FShaderParametersMetadata* ParametersMetadata)
{
	return &AllocParameters<BaseParameterStructType>(ParametersMetadata, 1)[0];
}

template <typename BaseParameterStructType>
TStridedView<BaseParameterStructType> FRDGBuilder::AllocParameters(const FShaderParametersMetadata* ParametersMetadata, uint32 NumStructs)
{
	// NOTE: Contents are always zero! This might differ if shader parameters have a non-zero default initializer.
	const int32 Stride = ParametersMetadata->GetSize();
	BaseParameterStructType* Contents = reinterpret_cast<BaseParameterStructType*>(Allocators.Root.Alloc(Stride * NumStructs, SHADER_PARAMETER_STRUCT_ALIGNMENT));
	FMemory::Memset(Contents, 0, Stride * NumStructs);
	TStridedView<BaseParameterStructType> ParameterArray(Stride, Contents, NumStructs);

	struct FClearUniformBuffers
	{
	public:
		FClearUniformBuffers(TStridedView<BaseParameterStructType> InParameterArray, const FRHIUniformBufferLayout& InLayout)
			: ParameterArray(InParameterArray)
			, Layout(&InLayout)
		{}

		~FClearUniformBuffers()
		{
			for (BaseParameterStructType& ParameterStruct : ParameterArray)
			{
				FRDGParameterStruct::ClearUniformBuffers(&ParameterStruct, Layout);
			}
		}

	private:
		TStridedView<BaseParameterStructType> ParameterArray;
		const FRHIUniformBufferLayout* Layout;
	};

	AllocObject<FClearUniformBuffers>(ParameterArray, ParametersMetadata->GetLayout());
	return ParameterArray;
}

inline FRDGSubresourceState* FRDGBuilder::AllocSubresource(const FRDGSubresourceState& Other)
{
	return Allocators.Transition.AllocNoDestruct<FRDGSubresourceState>(Other);
}

inline FRDGSubresourceState* FRDGBuilder::AllocSubresource()
{
	return Allocators.Transition.AllocNoDestruct<FRDGSubresourceState>();
}

template <typename ParameterStructType>
TRDGUniformBufferRef<ParameterStructType> FRDGBuilder::CreateUniformBuffer(const ParameterStructType* ParameterStruct)
{
#if !USE_NULL_RHI
	IF_RDG_ENABLE_DEBUG(UserValidation.ValidateCreateUniformBuffer(ParameterStruct, ParameterStructType::FTypeInfo::GetStructMetadata()));
	auto* UniformBuffer = UniformBuffers.Allocate<TRDGUniformBuffer<ParameterStructType>>(Allocators.Root, ParameterStruct, ParameterStructType::FTypeInfo::GetStructMetadata()->GetShaderVariableName());
	IF_RDG_ENABLE_DEBUG(UserValidation.ValidateCreateUniformBuffer(UniformBuffer));
	return UniformBuffer;
#else
	checkNoEntry();
	return nullptr;
#endif // !USE_NULL_RHI
}

#if !USE_NULL_RHI
template <typename ParameterStructType, typename ExecuteLambdaType>
FRDGPass* FRDGBuilder::AddPassInternal(
	FRDGEventName&& Name,
	const FShaderParametersMetadata* ParametersMetadata,
	const ParameterStructType* ParameterStruct,
	ERDGPassFlags Flags,
	ExecuteLambdaType&& ExecuteLambda)
{
	using LambdaPassType = TRDGLambdaPass<ParameterStructType, ExecuteLambdaType>;
	IF_RDG_ENABLE_DEBUG(UserValidation.ValidateAddPass(ParameterStruct, ParametersMetadata, Name, Flags));
	FlushAccessModeQueue();
	const TCHAR* NameString = Name.GetTCHAR();
	FRDGPass* Pass = Allocators.Root.AllocNoDestruct<LambdaPassType>(
		Forward<FRDGEventName&&>(Name),
		ParametersMetadata,
		ParameterStruct,
		OverridePassFlags(NameString, Flags),
		Forward<ExecuteLambdaType&&>(ExecuteLambda));

	IF_RDG_ENABLE_DEBUG(ClobberPassOutputs(Pass));
	Passes.Insert(Pass);
	SetupParameterPass(Pass);
	return Pass;
}
#endif // !USE_NULL_RHI

template <typename ExecuteLambdaType>
FRDGPass* FRDGBuilder::AddPass(
	FRDGEventName&& Name,
	ERDGPassFlags Flags,
	ExecuteLambdaType&& ExecuteLambda)
{
#if !USE_NULL_RHI
	using LambdaPassType = TRDGEmptyLambdaPass<ExecuteLambdaType>;
	IF_RDG_ENABLE_DEBUG(UserValidation.ValidateAddPass(Name, Flags));
	Flags |= ERDGPassFlags::NeverCull;
	FlushAccessModeQueue();
	LambdaPassType* Pass = Passes.Allocate<LambdaPassType>(Allocators.Root, Forward<FRDGEventName&&>(Name), Flags, Forward<ExecuteLambdaType&&>(ExecuteLambda));
	SetupEmptyPass(Pass);
	return Pass;
#else
	checkNoEntry();
	return nullptr;
#endif // !USE_NULL_RHI
}

template <typename ExecuteLambdaType>
FRDGPass* FRDGBuilder::AddPass(
	FRDGEventName&& Name,
	const FShaderParametersMetadata* ParametersMetadata,
	const void* ParameterStruct,
	ERDGPassFlags Flags,
	ExecuteLambdaType&& ExecuteLambda)
{
#if !USE_NULL_RHI
	return AddPassInternal(Forward<FRDGEventName&&>(Name), ParametersMetadata, ParameterStruct, Flags, Forward<ExecuteLambdaType&&>(ExecuteLambda));
#else
	checkNoEntry();
	return nullptr;
#endif // !USE_NULL_RHI
}

template <typename ParameterStructType, typename ExecuteLambdaType>
FRDGPass* FRDGBuilder::AddPass(
	FRDGEventName&& Name,
	const ParameterStructType* ParameterStruct,
	ERDGPassFlags Flags,
	ExecuteLambdaType&& ExecuteLambda)
{
#if !USE_NULL_RHI
	return AddPassInternal(Forward<FRDGEventName&&>(Name), ParameterStructType::FTypeInfo::GetStructMetadata(), ParameterStruct, Flags, Forward<ExecuteLambdaType&&>(ExecuteLambda));
#else
	checkNoEntry();
	return nullptr;
#endif // !USE_NULL_RHI
}

template <typename ParameterStructType, typename LaunchLambdaType>
FRDGPass* FRDGBuilder::AddDispatchPass(
	FRDGEventName&& Name,
	const ParameterStructType* ParameterStruct,
	ERDGPassFlags Flags,
	LaunchLambdaType&& LaunchLambda)
{
#if !USE_NULL_RHI
	using DispatchPassType = TRDGDispatchPass<ParameterStructType, LaunchLambdaType>;
	const FShaderParametersMetadata* ParametersMetadata = ParameterStructType::FTypeInfo::GetStructMetadata();

	if (EnumHasAnyFlags(Flags, ERDGPassFlags::Raster))
	{
		Flags |= ERDGPassFlags::SkipRenderPass;
	}

	IF_RDG_ENABLE_DEBUG(UserValidation.ValidateAddPass(ParameterStruct, ParametersMetadata, Name, Flags));

	FlushAccessModeQueue();

	const TCHAR* NameString = Name.GetTCHAR();

	FRDGDispatchPass* Pass = Allocators.Root.AllocNoDestruct<DispatchPassType>(
		Forward<FRDGEventName&&>(Name),
		ParametersMetadata,
		ParameterStruct,
		OverridePassFlags(NameString, Flags),
		Forward<LaunchLambdaType&&>(LaunchLambda));

	IF_RDG_ENABLE_DEBUG(ClobberPassOutputs(Pass));
	Passes.Insert(Pass);
	DispatchPasses.Emplace(Pass);
	SetupParameterPass(Pass);
	return Pass;
#else
	checkNoEntry();
	return nullptr;
#endif // !USE_NULL_RHI
}

inline void FRDGBuilder::SetPassWorkload(FRDGPass* Pass, uint32 Workload)
{
	Pass->Workload = Workload;
}

inline void FRDGBuilder::SkipInitialAsyncComputeFence()
{
	bInitialAsyncComputeFence = false;
}

inline void FRDGBuilder::QueueBufferUpload(FRDGBufferRef Buffer, const void* InitialData, uint64 InitialDataSize, ERDGInitialDataFlags InitialDataFlags)
{
	IF_RDG_ENABLE_DEBUG(UserValidation.ValidateUploadBuffer(Buffer, InitialData, InitialDataSize));

	if (InitialDataSize > 0 && !EnumHasAnyFlags(InitialDataFlags, ERDGInitialDataFlags::NoCopy))
	{
		void* InitialDataCopy = Alloc(InitialDataSize, 16);
		FMemory::Memcpy(InitialDataCopy, InitialData, InitialDataSize);
		InitialData = InitialDataCopy;
	}

	UploadedBuffers.Emplace(Buffer, InitialData, InitialDataSize);
	Buffer->bQueuedForUpload = 1;
}

inline void FRDGBuilder::QueueBufferUpload(FRDGBufferRef Buffer, const void* InitialData, uint64 InitialDataSize, FRDGBufferInitialDataFreeCallback&& InitialDataFreeCallback)
{
	IF_RDG_ENABLE_DEBUG(UserValidation.ValidateUploadBuffer(Buffer, InitialData, InitialDataSize));

	if (InitialDataSize == 0)
	{
		return;
	}

	UploadedBuffers.Emplace(Buffer, InitialData, InitialDataSize, MoveTemp(InitialDataFreeCallback));
	Buffer->bQueuedForUpload = 1;
}

inline void FRDGBuilder::QueueBufferUpload(FRDGBufferRef Buffer, FRDGBufferInitialDataFillCallback&& InitialDataFillCallback)
{
	IF_RDG_ENABLE_DEBUG(UserValidation.ValidateUploadBuffer(Buffer, InitialDataFillCallback));

	UploadedBuffers.Emplace(Buffer, MoveTemp(InitialDataFillCallback));
	Buffer->bQueuedForUpload = 1;
}

inline void FRDGBuilder::QueueBufferUpload(FRDGBufferRef Buffer, FRDGBufferInitialDataCallback&& InitialDataCallback, FRDGBufferInitialDataSizeCallback&& InitialDataSizeCallback)
{
	IF_RDG_ENABLE_DEBUG(UserValidation.ValidateUploadBuffer(Buffer, InitialDataCallback, InitialDataSizeCallback));

	UploadedBuffers.Emplace(Buffer, MoveTemp(InitialDataCallback), MoveTemp(InitialDataSizeCallback));
	Buffer->bQueuedForUpload = 1;
}

inline void FRDGBuilder::QueueBufferUpload(FRDGBufferRef Buffer, FRDGBufferInitialDataCallback&& InitialDataCallback, FRDGBufferInitialDataSizeCallback&& InitialDataSizeCallback, FRDGBufferInitialDataFreeCallback&& InitialDataFreeCallback)
{
	IF_RDG_ENABLE_DEBUG(UserValidation.ValidateUploadBuffer(Buffer, InitialDataCallback, InitialDataSizeCallback, InitialDataFreeCallback));

	UploadedBuffers.Emplace(Buffer, MoveTemp(InitialDataCallback), MoveTemp(InitialDataSizeCallback), MoveTemp(InitialDataFreeCallback));
	Buffer->bQueuedForUpload = 1;
}

inline void FRDGBuilder::QueueCommitReservedBuffer(FRDGBufferRef Buffer, uint64 CommitSizeInBytes)
{
	IF_RDG_ENABLE_DEBUG(UserValidation.ValidateCommitBuffer(Buffer, CommitSizeInBytes));
	if (!AsyncSetupQueue.bEnabled)
	{
		Buffer->PendingCommitSize = CommitSizeInBytes;
	}
	else
	{
		AsyncSetupQueue.Push(FAsyncSetupOp::ReservedBufferCommit(Buffer, CommitSizeInBytes));
	}
	Buffer->PooledBuffer->SetCommittedSize(CommitSizeInBytes);
}

inline void FRDGBuilder::QueueTextureExtraction(FRDGTextureRef Texture, TRefCountPtr<IPooledRenderTarget>* OutTexturePtr, ERHIAccess AccessFinal, ERDGResourceExtractionFlags Flags)
{
	QueueTextureExtraction(Texture, OutTexturePtr, Flags);
	SetTextureAccessFinal(Texture, AccessFinal);
}

inline void FRDGBuilder::QueueTextureExtraction(FRDGTextureRef Texture, TRefCountPtr<IPooledRenderTarget>* OutTexturePtr, ERDGResourceExtractionFlags Flags)
{
	IF_RDG_ENABLE_DEBUG(UserValidation.ValidateExtractTexture(Texture, OutTexturePtr));

	*OutTexturePtr = nullptr;

	const bool bWasExtracted = Texture->bExtracted;

	Texture->bExtracted = true;

	if (EnumHasAnyFlags(Flags, ERDGResourceExtractionFlags::AllowTransient))
	{
		if (Texture->TransientExtractionHint != FRDGTexture::ETransientExtractionHint::Disable)
		{
			Texture->TransientExtractionHint = FRDGTexture::ETransientExtractionHint::Enable;
		}
	}
	else
	{
		Texture->TransientExtractionHint = FRDGTexture::ETransientExtractionHint::Disable;
	}

	ExtractedTextures.Emplace(Texture, OutTexturePtr);

	if (!bWasExtracted)
	{
		AsyncSetupQueue.Push(FAsyncSetupOp::CullRootTexture(Texture));
	}
}

inline void FRDGBuilder::QueueBufferExtraction(FRDGBufferRef Buffer, TRefCountPtr<FRDGPooledBuffer>* OutBufferPtr)
{
	IF_RDG_ENABLE_DEBUG(UserValidation.ValidateExtractBuffer(Buffer, OutBufferPtr));

	*OutBufferPtr = nullptr;

	const bool bWasExtracted = Buffer->bExtracted;

	Buffer->bExtracted = true;
	Buffer->bForceNonTransient = true;
	ExtractedBuffers.Emplace(Buffer, OutBufferPtr);

	if (!bWasExtracted)
	{
		AsyncSetupQueue.Push(FAsyncSetupOp::CullRootBuffer(Buffer));
	}
}

inline void FRDGBuilder::QueueBufferExtraction(FRDGBufferRef Buffer, TRefCountPtr<FRDGPooledBuffer>* OutBufferPtr, ERHIAccess AccessFinal)
{
	QueueBufferExtraction(Buffer, OutBufferPtr);
	SetBufferAccessFinal(Buffer, AccessFinal);
}

inline void FRDGBuilder::AddDispatchHint()
{
	if (IsImmediateMode())
	{
		RHICmdList.ImmediateFlush(EImmediateFlushType::DispatchToRHIThread);
	}
	else if (Passes.Num() > 0)
	{
		Passes[Passes.Last()]->bDispatchAfterExecute = 1;
	}
}

template <typename TaskLambdaType>
inline UE::Tasks::FTask FRDGBuilder::AddSetupTask(TaskLambdaType&& TaskLambda, bool bCondition, ERDGSetupTaskWaitPoint WaitPoint)
{
	return AddSetupTask(MoveTemp(TaskLambda), nullptr, TArray<UE::Tasks::FTask>{}, UE::Tasks::ETaskPriority::Normal, bCondition, WaitPoint);
}

template <typename TaskLambdaType>
inline UE::Tasks::FTask FRDGBuilder::AddSetupTask(TaskLambdaType&& TaskLambda, UE::Tasks::ETaskPriority Priority, bool bCondition, ERDGSetupTaskWaitPoint WaitPoint)
{
	return AddSetupTask(MoveTemp(TaskLambda), nullptr, TArray<UE::Tasks::FTask>{}, Priority, bCondition, WaitPoint);
}

template <typename TaskLambdaType>
inline UE::Tasks::FTask FRDGBuilder::AddSetupTask(
	TaskLambdaType&& TaskLambda,
	UE::Tasks::FPipe* Pipe,
	UE::Tasks::ETaskPriority Priority,
	bool bCondition,
	ERDGSetupTaskWaitPoint WaitPoint)
{
	return AddSetupTask(MoveTemp(TaskLambda), Pipe, TArray<UE::Tasks::FTask>{}, Priority, bCondition);
}

template <typename TaskLambdaType, typename PrerequisitesCollectionType>
inline UE::Tasks::FTask FRDGBuilder::AddSetupTask(
	TaskLambdaType&& TaskLambda,
	PrerequisitesCollectionType&& Prerequisites,
	UE::Tasks::ETaskPriority Priority,
	bool bCondition,
	ERDGSetupTaskWaitPoint WaitPoint)
{
	return AddSetupTask(MoveTemp(TaskLambda), nullptr, Forward<PrerequisitesCollectionType&&>(Prerequisites), Priority, bCondition);
}

namespace UE::RDG
{
	template<typename TaskCollectionType, decltype(std::declval<TaskCollectionType>().begin())* = nullptr>
	inline bool IsCompleted(const TaskCollectionType& Tasks)
	{
		for (const UE::Tasks::FTask& Task : Tasks)
		{
			if (!Task.IsCompleted())
			{
				return false;
			}
		}
		return true;
	}

	template<typename TaskType, decltype(std::declval<TaskType>().IsCompleted())* = nullptr>
	inline bool IsCompleted(const TaskType& Task)
	{
		return Task.IsCompleted();
	}

	template<typename TaskCollectionType, decltype(std::declval<TaskCollectionType>().begin())* = nullptr>
	inline void Wait(const TaskCollectionType& Tasks)
	{
		if (!Tasks.IsEmpty())
		{
			UE::Tasks::Wait(Tasks);
		}
	}

	template<typename TaskType, decltype(std::declval<TaskType>().Wait())* = nullptr>
	inline void Wait(const TaskType& Task)
	{
		Task.Wait();
	}
}

template <typename TaskLambdaType, typename PrerequisitesCollectionType>
UE::Tasks::FTask FRDGBuilder::AddSetupTask(
	TaskLambdaType&& TaskLambda,
	UE::Tasks::FPipe* Pipe,
	PrerequisitesCollectionType&& Prerequisites,
	UE::Tasks::ETaskPriority Priority,
	bool bCondition,
	ERDGSetupTaskWaitPoint WaitPoint)
{
	UE::Tasks::FTask Task;

	if (!bCondition || IsImmediateMode())
	{
		UE::RDG::Wait(Prerequisites);
	}

	auto OuterLambda = [TaskLambda = MoveTemp(TaskLambda)]() mutable
	{
		FTaskTagScope Scope(ETaskTag::EParallelRenderingThread);
		TaskLambda();
	};

	const UE::Tasks::EExtendedTaskPriority ExtendedTaskPriority = ParallelSetup.bEnabled ? UE::Tasks::EExtendedTaskPriority::None : UE::Tasks::EExtendedTaskPriority::Inline;

	const bool bParallelEnabled = bCompiling ? bParallelCompileEnabled : ParallelSetup.bEnabled;

	if (!bCondition || (!bParallelEnabled && UE::RDG::IsCompleted(Prerequisites)))
	{
		OuterLambda();
	}
	else if (Pipe)
	{
		Task = Pipe->Launch(TEXT("FRDGBuilder::AddSetupTask"), MoveTemp(OuterLambda), Forward<PrerequisitesCollectionType&&>(Prerequisites), ParallelSetup.GetTaskPriority(Priority), ExtendedTaskPriority);
	}
	else
	{
		Task = UE::Tasks::Launch(TEXT("FRDGBuilder::AddSetupTask"), MoveTemp(OuterLambda), Forward<PrerequisitesCollectionType&&>(Prerequisites), ParallelSetup.GetTaskPriority(Priority), ExtendedTaskPriority);
	}

	if (Task.IsValid())
	{
		ParallelSetup.Tasks[(int32)WaitPoint].Emplace(Task);
	}

	return Task;
}

template <typename TaskLambdaType>
inline UE::Tasks::FTask FRDGBuilder::AddCommandListSetupTask(TaskLambdaType&& TaskLambda, bool bCondition, ERDGSetupTaskWaitPoint WaitPoint)
{
	return AddCommandListSetupTask(MoveTemp(TaskLambda), nullptr, TArray<UE::Tasks::FTask>{}, UE::Tasks::ETaskPriority::Normal, bCondition, WaitPoint);
}

template <typename TaskLambdaType>
inline UE::Tasks::FTask FRDGBuilder::AddCommandListSetupTask(TaskLambdaType&& TaskLambda, UE::Tasks::ETaskPriority TaskPriority, bool bCondition, ERDGSetupTaskWaitPoint WaitPoint)
{
	return AddCommandListSetupTask(MoveTemp(TaskLambda), nullptr, TArray<UE::Tasks::FTask>{}, TaskPriority, bCondition, WaitPoint);
}

template <typename TaskLambdaType>
inline UE::Tasks::FTask FRDGBuilder::AddCommandListSetupTask(
	TaskLambdaType&& TaskLambda,
	UE::Tasks::FPipe* Pipe,
	UE::Tasks::ETaskPriority Priority,
	bool bCondition,
	ERDGSetupTaskWaitPoint WaitPoint)
{
	return AddCommandListSetupTask(MoveTemp(TaskLambda), Pipe, TArray<UE::Tasks::FTask>{}, Priority, bCondition, WaitPoint);
}

template <typename TaskLambdaType, typename PrerequisitesCollectionType>
inline UE::Tasks::FTask FRDGBuilder::AddCommandListSetupTask(
	TaskLambdaType&& TaskLambda,
	PrerequisitesCollectionType&& Prerequisites,
	UE::Tasks::ETaskPriority Priority,
	bool bCondition,
	ERDGSetupTaskWaitPoint WaitPoint)
{
	return AddCommandListSetupTask(MoveTemp(TaskLambda), nullptr, Forward<PrerequisitesCollectionType&&>(Prerequisites), Priority, bCondition, WaitPoint);
}

template <typename TaskLambdaType, typename PrerequisitesCollectionType>
UE::Tasks::FTask FRDGBuilder::AddCommandListSetupTask(
	TaskLambdaType&& TaskLambda,
	UE::Tasks::FPipe* Pipe,
	PrerequisitesCollectionType&& Prerequisites,
	UE::Tasks::ETaskPriority Priority,
	bool bCondition,
	ERDGSetupTaskWaitPoint WaitPoint)
{
	UE::Tasks::FTask Task;

	if (!bCondition || IsImmediateMode())
	{
		UE::RDG::Wait(Prerequisites);
	}

	const bool bParallelEnabled = bCompiling ? bParallelCompileEnabled : ParallelSetup.bEnabled;

	// Need a separate command list with inline tasks when prerequisites are not complete yet.
	const bool bAllocateCommandListForTask = bCondition && (bParallelEnabled || !UE::RDG::IsCompleted(Prerequisites));

	FRHICommandList* RHICmdListTask = &RHICmdList;

	if (bAllocateCommandListForTask)
	{
		SCOPED_NAMED_EVENT(CreateCommandList, FColor::Emerald);
		RHICmdListTask = new FRHICommandList(RHICmdList.GetGPUMask());
		RHICmdList.QueueAsyncCommandListSubmit(RHICmdListTask);
	}

	auto OuterLambda = [this, TaskLambda = MoveTemp(TaskLambda), RHICmdListTask, bAllocateCommandListForTask]() mutable
	{
		FTaskTagScope Scope(ETaskTag::EParallelRenderingThread);

		if (bAllocateCommandListForTask)
		{
			RHICmdListTask->SwitchPipeline(ERHIPipeline::Graphics);
		}

		TaskLambda(*RHICmdListTask);

		if (bAllocateCommandListForTask)
		{
			RHICmdListTask->FinishRecording();
		}
	};

	const UE::Tasks::EExtendedTaskPriority ExtendedTaskPriority = bCondition ? UE::Tasks::EExtendedTaskPriority::None : UE::Tasks::EExtendedTaskPriority::Inline;

	if (!bAllocateCommandListForTask)
	{
		OuterLambda();
	}
	else if (Pipe)
	{
		Task = Pipe->Launch(TEXT("FRDGBuilder::AddCommandListSetupTask"), MoveTemp(OuterLambda), Forward<PrerequisitesCollectionType&&>(Prerequisites), ParallelSetup.GetTaskPriority(Priority), ExtendedTaskPriority);
	}
	else
	{
		Task = UE::Tasks::Launch(TEXT("FRDGBuilder::AddCommandListSetupTask"), MoveTemp(OuterLambda), Forward<PrerequisitesCollectionType&&>(Prerequisites), ParallelSetup.GetTaskPriority(Priority), ExtendedTaskPriority);
	}

	if (Task.IsValid())
	{
		ParallelSetup.Tasks[(int32)WaitPoint].Emplace(Task);
	}

	return Task;
}

inline const TRefCountPtr<IPooledRenderTarget>& FRDGBuilder::GetPooledTexture(FRDGTextureRef Texture) const
{
	IF_RDG_ENABLE_DEBUG(UserValidation.ValidateGetPooledTexture(Texture));
	return Texture->Allocation;
}

inline const TRefCountPtr<FRDGPooledBuffer>& FRDGBuilder::GetPooledBuffer(FRDGBufferRef Buffer) const
{
	IF_RDG_ENABLE_DEBUG(UserValidation.ValidateGetPooledBuffer(Buffer));
	return Buffer->Allocation;
}

inline void FRDGBuilder::SetTextureAccessFinal(FRDGTextureRef Texture, ERHIAccess AccessFinal)
{
	IF_RDG_ENABLE_DEBUG(UserValidation.ValidateSetAccessFinal(Texture, AccessFinal));
	Texture->EpilogueAccess = AccessFinal;
}

inline void FRDGBuilder::SetBufferAccessFinal(FRDGBufferRef Buffer, ERHIAccess AccessFinal)
{
	IF_RDG_ENABLE_DEBUG(UserValidation.ValidateSetAccessFinal(Buffer, AccessFinal));
	Buffer->EpilogueAccess = AccessFinal;
}