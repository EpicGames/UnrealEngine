// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ProfilingDebugging/RealtimeGPUProfiler.h"
#include "RenderGraphAllocator.h"
#include "RenderGraphFwd.h"
#include "RHIBreadcrumbs.h"
#include "RHIAccess.h"
#include "RHIResources.h"
#include "RHIDefinitions.h"
#include "RHIPipeline.h"

/** DEFINES */

/** Whether render graph debugging is enabled. */
#define RDG_ENABLE_DEBUG (!UE_BUILD_SHIPPING && !UE_BUILD_TEST)

/** Performs the operation if RDG_ENABLE_DEBUG is enabled. Useful for one-line checks without explicitly wrapping in #if. */
#if RDG_ENABLE_DEBUG
	#define IF_RDG_ENABLE_DEBUG(Op) Op
#else
	#define IF_RDG_ENABLE_DEBUG(Op)
#endif

/** Whether render graph debugging is enabled and we are compiling with the engine. */
#define RDG_ENABLE_DEBUG_WITH_ENGINE (RDG_ENABLE_DEBUG && WITH_ENGINE)

/** Whether render graph insight tracing is enabled. */
#define RDG_ENABLE_TRACE UE_TRACE_ENABLED && !IS_PROGRAM && !UE_BUILD_SHIPPING

#if RDG_ENABLE_TRACE
	#define IF_RDG_ENABLE_TRACE(Op) Op
#else
	#define IF_RDG_ENABLE_TRACE(Op)
#endif

/** Allows to dump all RDG resources of a frame. */
#define RDG_DUMP_RESOURCES (WITH_DUMPGPU)

/** The type of GPU events the render graph system supports.
 *  RDG_EVENTS == 0 means there is no string processing at all.
 *  RDG_EVENTS == 1 means the format component of the event name is stored as a const TCHAR*.
 *  RDG_EVENTS == 2 means string formatting is evaluated and stored in an FString.
 */
#define RDG_EVENTS_NONE 0
#define RDG_EVENTS_STRING_REF 1
#define RDG_EVENTS_STRING_COPY 2

/** Whether render graph GPU events are enabled. */
#if WITH_PROFILEGPU
	#if UE_BUILD_TEST || UE_BUILD_SHIPPING
		#define RDG_EVENTS RDG_EVENTS_STRING_REF
	#else
		#define RDG_EVENTS RDG_EVENTS_STRING_COPY
	#endif
#elif WITH_RHI_BREADCRUMBS
	#define RDG_EVENTS RDG_EVENTS_STRING_REF
#else
	#define RDG_EVENTS RDG_EVENTS_NONE
#endif

#define SUPPORTS_VISUALIZE_TEXTURE (WITH_ENGINE && (!UE_BUILD_SHIPPING || WITH_EDITOR))

/** An RDG pass execution lambda MAY be executed in a parallel task IF the lambda references a non-immediate command list AND the builder flags are set to execute in parallel.
 *  By default, if a pass executes in parallel, the task will be awaited at the end of FRDGBuilder::Execute(). This behavior may be overridden by tagging the lambda with FRDGAsyncTask as the
 *  first argument. A tagged lambda, when executed in parallel, is NOT awaited at the end of FRDGBuilder::Execute(). Instead, the task is recorded as an outstanding RHI command list task
 *  (which share semantics with mesh passes or other parallel command list tasks) and can be manually awaited by calling FRDGBuilder::WaitForAsyncExecuteTasks() or formed into a task
 *  graph with FRDGBuilder::GetAsyncExecuteTask() (both static methods). The lifetime of RDG allocations is tied to these tasks and RDG will not release any memory or allocated objects
 *  until the last task completes, even though the FRDGBuilder instance itself may go out of scope and destruct.
 *
 *  Consider the following examples:
 *
 *      // Builder is marked as supporting parallel execute.
 *      FRDGBuilder GraphBuilder(RDG_EVENT_NAME("MyBuilder"), ERDGBuilderFlags::Parallel)
 *
 *      GraphBuilder.AddPass(RDG_EVENT_NAME("..."), PassParameters, PassFlags, [...] (FRHICommandList& RHICmdList)
 *      {
 *         // This will execute in parallel and is awaited by RDG on the render thread at the end of FRDGBuilder::Execute().
 *      });
 *
 *      GraphBuilder.AddPass(RDG_EVENT_NAME("..."), PassParameters, PassFlags, [...] (FRHICommandListImmediate& RHICmdList)
 *      {
 *         // This will execute inline on the render thread, because the immediate command list is referenced.
 *      });
 *
 *      FMyObject* Object = GraphBuilder.AllocObject<FMyObject>();
 *
 *      GraphBuilder.AddPass(RDG_EVENT_NAME("..."), PassParameters, PassFlags, [Object] (FRDGAsyncTask, FRHICommandList& RHICmdList)
 *      {
 *         // This will execute in parallel and is NOT awaited at the end of FRDGBuilder::Execute(). Accessing 'Object' is safe.
 *      });
 *
 *      GraphBuilder.Execute();
 *
 *  Tasks can be synced in a few different ways. RDG async execute tasks are chained, so syncing the last batch will sync ALL prior batches.
 *
 *      // This will sync all RDG async execute tasks.
 *      RHICmdList.ImmediateFlush(EImmediateFlushType::WaitForOutstandingTasksOnly);
 *
 *      // This will also sync all RDG async execute tasks.
 *      FRDGBuilder::WaitForAsyncExecuteTasks();
 *
 *      // Launch a task that will do something when RDG async execute tasks complete.
 *      UE::Tasks::Launch(UE_SOURCE_LOCATION, [...] { ... }, FRDGBuilder::GetAsyncExecuteTask());
 */
struct FRDGAsyncTask {};

/** ENUMS */

enum class ERDGBuilderFlags
{
	None = 0,

	/** Allows the builder to parallelize AddSetupPass calls. Without this flag, setup passes run serially. */
	ParallelSetup = 1 << 0,
	
	/** Allows the builder to parallelize compilation of the graph. Without this flag, all passes execute on the render thread. */
	ParallelCompile = 1 << 1,

	/** Allows the builder to parallelize execution of passes. Without this flag, all passes execute on the render thread. */
	ParallelExecute = 1 << 2,

	Parallel = ParallelSetup | ParallelCompile | ParallelExecute,
};
ENUM_CLASS_FLAGS(ERDGBuilderFlags);

/** Flags to annotate a pass with when calling AddPass. */
enum class ERDGPassFlags : uint16
{
	/** Pass doesn't have any inputs or outputs tracked by the graph. This may only be used by the parameterless AddPass function. */
	None = 0,

	/** Pass uses rasterization on the graphics pipe. */
	Raster = 1 << 0,

	/** Pass uses compute on the graphics pipe. */
	Compute = 1 << 1,

	/** Pass uses compute on the async compute pipe. */
	AsyncCompute = 1 << 2,

	/** Pass uses copy commands on the graphics pipe. */
	Copy = 1 << 3,

	/** Pass (and its producers) will never be culled. Necessary if outputs cannot be tracked by the graph. */
	NeverCull = 1 << 4,

	/** Render pass begin / end is skipped and left to the user. Only valid when combined with 'Raster'. Disables render pass merging for the pass. */
	SkipRenderPass = 1 << 5,

	/** Pass will never have its render pass merged with other passes. */
	NeverMerge = 1 << 6,

	/** Pass will never run off the render thread. */
	NeverParallel = 1 << 7,

	/** Pass uses copy commands but writes to a staging resource. */
	Readback = Copy | NeverCull
};
ENUM_CLASS_FLAGS(ERDGPassFlags);

/** Flags to annotate a render graph buffer. */
enum class ERDGBufferFlags : uint8
{
	None = 0,

	/** Tag the buffer to survive through frame, that is important for multi GPU alternate frame rendering. */
	MultiFrame = 1 << 0,

	/** The buffer is ignored by RDG tracking and will never be transitioned. Use the flag when registering a buffer with no writable GPU flags.
	 *  Write access is not allowed for the duration of the graph. This flag is intended as an optimization to cull out tracking of read-only
	 *  buffers that are used frequently throughout the graph. Note that it's the user's responsibility to ensure the resource is in the correct
	 *  readable state for use with RDG passes, as RDG does not know the exact state of the resource.
	 */
	SkipTracking = 1 << 1,

	/** When set, RDG will perform its first barrier without splitting. Practically, this means the resource is left in its initial state
	 *  until the first pass it's used within the graph. Without this flag, the resource is split-transitioned at the start of the graph.
	 */
	ForceImmediateFirstBarrier = 1 << 2,
};
ENUM_CLASS_FLAGS(ERDGBufferFlags);

/** Flags to annotate a render graph texture. */
enum class ERDGTextureFlags : uint8
{
	None = 0,

	/** Tag the texture to survive through frame, that is important for multi GPU alternate frame rendering. */
	MultiFrame = 1 << 0,

	/** The buffer is ignored by RDG tracking and will never be transitioned. Use the flag when registering a buffer with no writable GPU flags.
	 *  Write access is not allowed for the duration of the graph. This flag is intended as an optimization to cull out tracking of read-only
	 *  buffers that are used frequently throughout the graph. Note that it's the user's responsibility to ensure the resource is in the correct
	 *  readable state for use with RDG passes, as RDG does not know the exact state of the resource.
	 */
	SkipTracking = 1 << 1,
	
	/** When set, RDG will perform its first barrier without splitting. Practically, this means the resource is left in its initial state
	 *  until the first pass it's used within the graph. Without this flag, the resource is split-transitioned at the start of the graph.
	 */
	ForceImmediateFirstBarrier = 1 << 2,

	/** Prevents metadata decompression on this texture. */
	MaintainCompression = 1 << 3,
};
ENUM_CLASS_FLAGS(ERDGTextureFlags);

enum class ERDGSetupTaskWaitPoint : uint8
{
	/** (Default) Setup task is synced prior to compilation. Use this mode if task mutates RDG resources (e.g. RDG buffer upload contents, buffer size callbacks, etc) */
	Compile = 0,

	/** Setup task is synced prior to execution. Use this mode if your task is stalling in RDG and doesn't affect RDG compilation in any way. */
	Execute = 1,

	MAX
};

/** Flags to annotate a view with when calling CreateUAV. */
enum class ERDGUnorderedAccessViewFlags : uint8
{
	None = 0,

	// The view will not perform UAV barriers between consecutive usage.
	SkipBarrier = 1 << 0
};
ENUM_CLASS_FLAGS(ERDGUnorderedAccessViewFlags);

/** The set of concrete parent resource types. */
enum class ERDGViewableResourceType : uint8
{
	Texture,
	Buffer,
	MAX
};

/** The set of concrete view types. */
enum class ERDGViewType : uint8
{
	TextureUAV,
	TextureSRV,
	BufferUAV,
	BufferSRV,
	MAX
};

inline ERDGViewableResourceType GetParentType(ERDGViewType ViewType)
{
	switch (ViewType)
	{
	case ERDGViewType::TextureUAV:
	case ERDGViewType::TextureSRV:
		return ERDGViewableResourceType::Texture;
	case ERDGViewType::BufferUAV:
	case ERDGViewType::BufferSRV:
		return ERDGViewableResourceType::Buffer;
	}
	checkNoEntry();
	return ERDGViewableResourceType::MAX;
}

enum class ERDGResourceExtractionFlags : uint8
{
	None = 0,

	// Allows the resource to remain transient. Only use this flag if you intend to register the resource back
	// into the graph and release the reference. This should not be used if the resource is cached for a long
	// period of time.
	AllowTransient = 1,
};
ENUM_CLASS_FLAGS(ERDGResourceExtractionFlags);

enum class ERDGInitialDataFlags : uint8
{
	/** Specifies the default behavior, which is to make a copy of the initial data for replay when
	 *  the graph is executed. The user does not need to preserve lifetime of the data pointer.
	 */
	None = 0,

	/** Specifies that the user will maintain ownership of the data until the graph is executed. The
	 *  upload pass will only use a reference to store the data. Use caution with this flag since graph
	 *  execution is deferred! Useful to avoid the copy if the initial data lifetime is guaranteed to
	 *  outlive the graph.
	 */
	 NoCopy = 1 << 0
};
ENUM_CLASS_FLAGS(ERDGInitialDataFlags)

enum class ERDGPooledBufferAlignment : uint8
{
	// The buffer size is not aligned.
	None,

	// The buffer size is aligned up to the next page size.
	Page,

	// The buffer size is aligned up to the next power of two.
	PowerOfTwo
};

/** Returns the equivalent parent resource type for a view type. */
inline ERDGViewableResourceType GetViewableResourceType(ERDGViewType ViewType)
{
	switch (ViewType)
	{
	case ERDGViewType::TextureUAV:
	case ERDGViewType::TextureSRV:
		return ERDGViewableResourceType::Texture;
	case ERDGViewType::BufferUAV:
	case ERDGViewType::BufferSRV:
		return ERDGViewableResourceType::Buffer;
	default:
		checkNoEntry();
		return ERDGViewableResourceType::MAX;
	}
}

using ERDGTextureMetaDataAccess = ERHITextureMetaDataAccess;

/** Returns the associated FRHITransitionInfo plane index. */
inline int32 GetResourceTransitionPlaneForMetadataAccess(ERDGTextureMetaDataAccess Metadata)
{
	switch (Metadata)
	{
	case ERDGTextureMetaDataAccess::CompressedSurface:
	case ERDGTextureMetaDataAccess::HTile:
	case ERDGTextureMetaDataAccess::Depth:
		return FRHITransitionInfo::kDepthPlaneSlice;
	case ERDGTextureMetaDataAccess::Stencil:
		return FRHITransitionInfo::kStencilPlaneSlice;
	default:
		return 0;
	}
}

/** HANDLE UTILITIES */

/** Handle helper class for internal tracking of RDG types. */
// Disable false positive buffer overrun warning during pgo linking step
PGO_LINK_DISABLE_WARNINGS
template <typename LocalObjectType, typename LocalIndexType>
class TRDGHandle
{
public:
	using ObjectType = LocalObjectType;
	using IndexType = LocalIndexType;

	static const TRDGHandle Null;

	TRDGHandle() = default;

	explicit inline TRDGHandle(uint32 InIndex)
	{
		check(InIndex >= 0 && InIndex <= kNullIndex);
		Index = (IndexType)InIndex;
	}

	inline IndexType GetIndex() const { check(IsValid()); return Index; }
	inline IndexType GetIndexUnchecked() const { return Index; }
	inline bool IsNull()  const { return Index == kNullIndex; }
	inline bool IsValid() const { return Index != kNullIndex; }
	inline bool operator==(TRDGHandle Other) const { return Index == Other.Index; }
	inline bool operator!=(TRDGHandle Other) const { return Index != Other.Index; }
	inline bool operator<=(TRDGHandle Other) const { check(IsValid() && Other.IsValid()); return Index <= Other.Index; }
	inline bool operator>=(TRDGHandle Other) const { check(IsValid() && Other.IsValid()); return Index >= Other.Index; }
	inline bool operator< (TRDGHandle Other) const { check(IsValid() && Other.IsValid()); return Index <  Other.Index; }
	inline bool operator> (TRDGHandle Other) const { check(IsValid() && Other.IsValid()); return Index >  Other.Index; }

	inline TRDGHandle& operator+=(int32 Increment)
	{
		check(int64(Index + Increment) <= int64(kNullIndex));
		Index += (IndexType)Increment;
		return *this;
	}

	inline TRDGHandle& operator-=(int32 Decrement)
	{
		check(int64(Index - Decrement) > 0);
		Index -= (IndexType)Decrement;
		return *this;
	}

	inline TRDGHandle operator-(int32 Subtract) const
	{
		TRDGHandle Handle = *this;
		Handle -= Subtract;
		return Handle;
	}

	inline TRDGHandle operator+(int32 Add) const
	{
		TRDGHandle Handle = *this;
		Handle += Add;
		return Handle;
	}

	inline TRDGHandle& operator++()
	{
		check(IsValid());
		++Index;
		return *this;
	}

	inline TRDGHandle& operator--()
	{
		check(IsValid());
		--Index;
		return *this;
	}

	// Returns the min of two pass handles. Returns null if both are null; returns the valid handle if one is null.
	inline static TRDGHandle Min(TRDGHandle A, TRDGHandle B)
	{
		// If either index is null is will fail the comparison.
		return A.Index < B.Index ? A : B;
	}

	// Returns the max of two pass handles. Returns null if both are null; returns the valid handle if one is null.
	inline static TRDGHandle Max(TRDGHandle A, TRDGHandle B)
	{
		// If either index is null, it will wrap around to 0 and fail the comparison.
		return (IndexType)(A.Index + 1) > (IndexType)(B.Index + 1) ? A : B;
	}

private:
	static const IndexType kNullIndex = TNumericLimits<IndexType>::Max();
	IndexType Index = kNullIndex;

	friend inline uint32 GetTypeHash(TRDGHandle Handle)
	{
		return Handle.GetIndex();
	}
};
PGO_LINK_ENABLE_WARNINGS

enum class ERDGHandleRegistryDestructPolicy
{
	Registry,
	Allocator,
	Never
};

/** Helper handle registry class for internal tracking of RDG types. */
template <typename LocalHandleType, ERDGHandleRegistryDestructPolicy DestructPolicy = ERDGHandleRegistryDestructPolicy::Registry>
class TRDGHandleRegistry
{
public:
	using HandleType = LocalHandleType;
	using ObjectType = typename HandleType::ObjectType;
	using IndexType = typename HandleType::IndexType;

	TRDGHandleRegistry() = default;
	TRDGHandleRegistry(const TRDGHandleRegistry&) = delete;
	TRDGHandleRegistry(TRDGHandleRegistry&&) = default;
	TRDGHandleRegistry& operator=(TRDGHandleRegistry&&) = default;
	TRDGHandleRegistry& operator=(const TRDGHandleRegistry&) = delete;

	~TRDGHandleRegistry()
	{
		Clear();
	}

	void Insert(ObjectType* Object)
	{
		Array.Emplace(Object);
		Object->Handle = Last();
	}

	template<typename DerivedType = ObjectType, class ...TArgs>
	DerivedType* Allocate(FRDGAllocator& Allocator, TArgs&&... Args)
	{
		static_assert(TIsDerivedFrom<DerivedType, ObjectType>::Value, "You must specify a type that derives from ObjectType");
		DerivedType* Object;
		if (DestructPolicy == ERDGHandleRegistryDestructPolicy::Allocator)
		{
			Object = Allocator.Alloc<DerivedType>(Forward<TArgs>(Args)...);
		}
		else
		{
			Object = Allocator.AllocNoDestruct<DerivedType>(Forward<TArgs>(Args)...);
		}
		Insert(Object);
		return Object;
	}

	void Clear()
	{
		if (DestructPolicy == ERDGHandleRegistryDestructPolicy::Registry)
		{
			for (int32 Index = Array.Num() - 1; Index >= 0; --Index)
			{
				Array[Index]->~ObjectType();
			}
		}
		Array.Empty();
	}

	template <typename FunctionType>
	void Enumerate(FunctionType Function)
	{
		for (ObjectType* Object : Array)
		{
			Function(Object);
		}
	}

	template <typename FunctionType>
	void Enumerate(FunctionType Function) const
	{
		for (const ObjectType* Object : Array)
		{
			Function(Object);
		}
	}

	inline const ObjectType* Get(HandleType Handle) const
	{
		return Array[Handle.GetIndex()];
	}

	inline ObjectType* Get(HandleType Handle)
	{
		return Array[Handle.GetIndex()];
	}

	inline const ObjectType* operator[] (HandleType Handle) const
	{
		return Get(Handle);
	}

	inline ObjectType* operator[] (HandleType Handle)
	{
		return Get(Handle);
	}

	inline HandleType Begin() const
	{
		return HandleType(0);
	}

	inline HandleType End() const
	{
		return HandleType(Array.Num());
	}

	inline HandleType Last() const
	{
		return HandleType(Array.Num() - 1);
	}

	inline int32 Num() const
	{
		return Array.Num();
	}

private:
	TArray<ObjectType*, FRDGArrayAllocator> Array;
};

/** Specialization of bit array with compile-time type checking for handles and a pre-configured allocator. */
template <typename HandleType>
class TRDGHandleBitArray : public TBitArray<FRDGBitArrayAllocator>
{
	using Base = TBitArray<FRDGBitArrayAllocator>;
public:
	using Base::Base;

	inline FBitReference operator[](HandleType Handle)
	{
		return Base::operator[](Handle.GetIndex());
	}

	inline const FConstBitReference operator[](HandleType Handle) const
	{
		return Base::operator[](Handle.GetIndex());
	}
};

/** Esoteric helper class which accumulates handles and will return a valid handle only if a single unique
 *  handle was added. Otherwise, it returns null until reset. This helper is chiefly used to track UAVs
 *  tagged as 'no UAV barrier'; such that a UAV barrier is issued only if a unique no-barrier UAV is used
 *  on a pass. Intended for internal use only.
 */
template <typename HandleType>
class TRDGHandleUniqueFilter
{
public:
	TRDGHandleUniqueFilter() = default;

	TRDGHandleUniqueFilter(HandleType InHandle)
	{
		AddHandle(InHandle);
	}

	void Reset()
	{
		Handle = HandleType::Null;
	}

	void AddHandle(HandleType InHandle)
	{
		checkf(InHandle != NotUniqueHandle, TEXT("Overflowed TRDGHandleUniqueFilter"));

		if (Handle != InHandle && InHandle.IsValid())
		{
			Handle = Handle.IsNull() ? InHandle : NotUniqueHandle;
		}
	}

	HandleType GetUniqueHandle() const
	{
		return Handle != NotUniqueHandle ? Handle : HandleType::Null;
	}

private:
	static const HandleType NotUniqueHandle;
	HandleType Handle;
};

template <typename ObjectType, typename IndexType>
const TRDGHandle<ObjectType, IndexType> TRDGHandle<ObjectType, IndexType>::Null;

template <typename HandleType>
const HandleType TRDGHandleUniqueFilter<HandleType>::NotUniqueHandle(TNumericLimits<typename HandleType::IndexType>::Max() - 1);

struct FRDGTextureDesc : public FRHITextureDesc
{
	static FRDGTextureDesc Create2D(
		FIntPoint           Size
		, EPixelFormat        Format
		, FClearValueBinding  ClearValue
		, ETextureCreateFlags Flags
		, uint8               NumMips = 1
		, uint8               NumSamples = 1
		, uint32              ExtData = 0
	)
	{
		const uint16 Depth = 1;
		const uint16 ArraySize = 1;
		return FRDGTextureDesc(ETextureDimension::Texture2D, Flags, Format, ClearValue, { Size.X, Size.Y }, Depth, ArraySize, NumMips, NumSamples, ExtData);
	}

	static FRDGTextureDesc Create2DArray(
		FIntPoint           Size
		, EPixelFormat        Format
		, FClearValueBinding  ClearValue
		, ETextureCreateFlags Flags
		, uint16              ArraySize
		, uint8               NumMips = 1
		, uint8               NumSamples = 1
		, uint32              ExtData = 0
	)
	{
		const uint16 Depth = 1;
		return FRDGTextureDesc(ETextureDimension::Texture2DArray, Flags, Format, ClearValue, { Size.X, Size.Y }, Depth, ArraySize, NumMips, NumSamples, ExtData);
	}

	static FRDGTextureDesc Create3D(
		FIntVector          Size
		, EPixelFormat        Format
		, FClearValueBinding  ClearValue
		, ETextureCreateFlags Flags
		, uint8               NumMips = 1
		, uint32              ExtData = 0
	)
	{
		const uint16 ArraySize = 1;
		const uint8 LocalNumSamples = 1;

		checkf(Size.Z <= TNumericLimits<decltype(FRDGTextureDesc::Depth)>::Max(), TEXT("Depth parameter (Size.Z) exceeds valid range"));

		return FRDGTextureDesc(ETextureDimension::Texture3D, Flags, Format, ClearValue, { Size.X, Size.Y }, (uint16)Size.Z, ArraySize, NumMips, LocalNumSamples, ExtData);
	}

	static FRDGTextureDesc CreateCube(
		uint32              Size
		, EPixelFormat        Format
		, FClearValueBinding  ClearValue
		, ETextureCreateFlags Flags
		, uint8               NumMips = 1
		, uint8               NumSamples = 1
		, uint32              ExtData = 0
	)
	{
		checkf(Size <= (uint32)TNumericLimits<int32>::Max(), TEXT("Size parameter exceeds valid range"));

		const uint16 Depth = 1;
		const uint16 ArraySize = 1;
		return FRDGTextureDesc(ETextureDimension::TextureCube, Flags, Format, ClearValue, { (int32)Size, (int32)Size }, Depth, ArraySize, NumMips, NumSamples, ExtData);
	}

	static FRDGTextureDesc CreateCubeArray(
		uint32              Size
		, EPixelFormat        Format
		, FClearValueBinding  ClearValue
		, ETextureCreateFlags Flags
		, uint16              ArraySize
		, uint8               NumMips = 1
		, uint8               NumSamples = 1
		, uint32              ExtData = 0
	)
	{
		checkf(Size <= (uint32)TNumericLimits<int32>::Max(), TEXT("Size parameter exceeds valid range"));

		const uint16 Depth = 1;
		return FRDGTextureDesc(ETextureDimension::TextureCubeArray, Flags, Format, ClearValue, { (int32)Size, (int32)Size }, Depth, ArraySize, NumMips, NumSamples, ExtData);
	}

	static FRDGTextureDesc CreateRenderTargetTextureDesc(
		FIntPoint Size, EPixelFormat Format
		, FClearValueBinding ClearValue
		, ETextureCreateFlags Flags
		, const bool bRequireMultiView
		, uint16 MobileMultiViewRenderTargetNumLayers = 2)
	{
		if (bRequireMultiView)
		{
			return FRDGTextureDesc::Create2DArray(Size, Format, ClearValue, Flags, MobileMultiViewRenderTargetNumLayers);
		}

		return FRDGTextureDesc::Create2D(Size, Format, ClearValue, Flags);
	}

	FRDGTextureDesc() = default;
	FRDGTextureDesc(
		ETextureDimension   InDimension
		, ETextureCreateFlags InFlags
		, EPixelFormat        InFormat
		, FClearValueBinding  InClearValue
		, FIntPoint           InExtent
		, uint16              InDepth
		, uint16              InArraySize
		, uint8               InNumMips
		, uint8               InNumSamples
		, uint32              InExtData
	)
		: FRHITextureDesc(InDimension, InFlags, InFormat, InClearValue, InExtent, InDepth, InArraySize, InNumMips, InNumSamples, InExtData)
	{
	}
};

/** FORWARD DECLARATIONS */

class FRDGBlackboard;

class FRDGAsyncComputeBudgetScopeGuard;
class FRDGScopedCsvStatExclusive;
class FRDGScopedCsvStatExclusiveConditional;

class FRDGBarrierBatch;
class FRDGBarrierBatchBegin;
class FRDGBarrierBatchEnd;
class FRDGBarrierValidation;
class FRDGEventName;
class FRDGUserValidation;

class FRDGViewableResource;

using FRDGPassHandle = TRDGHandle<FRDGPass, uint32>;
using FRDGPassRegistry = TRDGHandleRegistry<FRDGPassHandle>;
using FRDGPassHandleArray = TArray<FRDGPassHandle, TInlineAllocator<4, FRDGArrayAllocator>>;
using FRDGPassBitArray = TRDGHandleBitArray<FRDGPassHandle>;

using FRDGUniformBufferHandle = TRDGHandle<FRDGUniformBuffer, uint32>;
using FRDGUniformBufferRegistry = TRDGHandleRegistry<FRDGUniformBufferHandle>;
using FRDGUniformBufferBitArray = TRDGHandleBitArray<FRDGUniformBufferHandle>;

using FRDGViewHandle = TRDGHandle<FRDGView, uint32>;
using FRDGViewRegistry = TRDGHandleRegistry<FRDGViewHandle, ERDGHandleRegistryDestructPolicy::Never>;
using FRDGViewUniqueFilter = TRDGHandleUniqueFilter<FRDGViewHandle>;
using FRDGViewBitArray = TRDGHandleBitArray<FRDGViewHandle>;

using FRDGTextureHandle = TRDGHandle<FRDGTexture, uint32>;
using FRDGTextureRegistry = TRDGHandleRegistry<FRDGTextureHandle, ERDGHandleRegistryDestructPolicy::Never>;
using FRDGTextureBitArray = TRDGHandleBitArray<FRDGTextureHandle>;

using FRDGBufferHandle = TRDGHandle<FRDGBuffer, uint32>;
using FRDGBufferReservedCommitHandle = TRDGHandle<FRDGBuffer, uint16>;
using FRDGBufferRegistry = TRDGHandleRegistry<FRDGBufferHandle, ERDGHandleRegistryDestructPolicy::Never>;
using FRDGBufferBitArray = TRDGHandleBitArray<FRDGBufferHandle>;

class FRDGBufferPool;
class FRDGTransientRenderTarget;

using FRDGPassHandlesByPipeline = TRHIPipelineArray<FRDGPassHandle>;
using FRDGPassesByPipeline = TRHIPipelineArray<FRDGPass*>;

class FRDGTrace;
class FRDGResourceDumpContext;

using FRDGBufferNumElementsCallback = TFunction<uint32()>;
using FRDGBufferInitialDataCallback = TFunction<const void*()>;
using FRDGBufferInitialDataSizeCallback = TFunction<uint64()>;
template <typename ArrayType, 
	typename ArrayTypeNoRef = std::remove_reference_t<ArrayType>,
	typename = typename TEnableIf<TIsTArray_V<ArrayTypeNoRef>>::Type> using TRDGBufferArrayCallback = TFunction<const ArrayType&()>;
using FRDGBufferInitialDataFreeCallback = TFunction<void(const void* InData)>;
using FRDGBufferInitialDataFillCallback = TFunction<void(void* InData, uint32 InDataSize)>;
using FRDGDispatchGroupCountCallback = TFunction<FIntVector()>;
