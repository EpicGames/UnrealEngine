// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/StaticArray.h"
#include "Containers/UnrealString.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "Misc/AssertionMacros.h"
#include "MultiGPU.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "GpuProfilerTrace.h"
#include "RHIFwd.h"
#include "RHICommandList.h"
#include "RenderGraphAllocator.h"
#include "RenderGraphDefinitions.h"
#include "Templates/UnrealTemplate.h"
#include "Misc/TVariant.h"
#include "UObject/NameTypes.h"

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8
#include "DynamicRenderScaling.h"
#include "ProfilingDebugging/CsvProfilerConfig.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"
#include "ProfilingDebugging/RealtimeGPUProfiler.h"
#include "RHI.h"
#include "RendererInterface.h"
#include "Stats/Stats.h"
#endif //UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8

//////////////////////////////////////////////////////////////////////////
//
// GPU Events - Named hierarchical events emitted to external profiling tools.
//
//////////////////////////////////////////////////////////////////////////

class FRDGScopeState;

namespace DynamicRenderScaling
{
	class FBudget;
}

/** Stores a GPU event name for the render graph. Draw events can be compiled out entirely from
 *  a release build for performance.
 */
class FRDGEventName final
{
public:
	FRDGEventName() = default;

	// Constructors require a string that matches the RDG builder lifetime, as copies are not made in all configurations.
	RENDERCORE_API explicit FRDGEventName(const TCHAR* EventFormat, ...);
	RENDERCORE_API FRDGEventName(int32 NonVariadic, const TCHAR* EventName);

	FRDGEventName(const FRDGEventName& Other) = default;
	FRDGEventName& operator=(const FRDGEventName& Other) = default;

	RENDERCORE_API const TCHAR* GetTCHAR() const;

	bool HasFormattedString() const
	{
	#if RDG_EVENTS == RDG_EVENTS_STRING_COPY
		return !FormattedEventName.IsEmpty();
	#else
		return false;
	#endif
	}

private:
#if RDG_EVENTS >= RDG_EVENTS_STRING_REF
	// Event format kept around to still have a clue what error might be causing the problem in error messages.
	const TCHAR* EventFormat = TEXT("");
#endif

#if RDG_EVENTS == RDG_EVENTS_STRING_COPY
	FString FormattedEventName;
#endif
};



enum class ERDGScopeFlags : uint8
{
	None  = 0,

	// Disables any nested scopes of the same type.
	Final = 1 << 0,

	// Ensures the scope is always emitted (ignores cvars that disable scopes)
	AlwaysEnable = 1 << 1,

	// The scope includes a GPU stat, so may need to be enabled even when cvars are disabling scopes.
	Stat = 1 << 2,
};
ENUM_CLASS_FLAGS(ERDGScopeFlags);

#if CSV_PROFILER_STATS
struct FRDGScope_CSVExclusive
{
	const char* const StatName;

	FRDGScope_CSVExclusive(FRDGScopeState&, const char* StatName)
		: StatName(StatName)
	{
		FCsvProfiler::BeginExclusiveStat(StatName);
	}

	void ImmediateEnd(FRDGScopeState&)
	{
		FCsvProfiler::EndExclusiveStat(StatName);
	}

	void BeginCPU(FRHIComputeCommandList& RHICmdList, bool bPreScope)
	{
		FCsvProfiler::BeginExclusiveStat(StatName);
	}

	void EndCPU(FRHIComputeCommandList& RHICmdList, bool bPreScope)
	{
		FCsvProfiler::EndExclusiveStat(StatName);
	}

	void BeginGPU(FRHIComputeCommandList& RHICmdList)
	{
	}

	void EndGPU(FRHIComputeCommandList& RHICmdList)
	{
	}
};
#endif // CSV_PROFILER_STATS

struct FRDGScope_Budget
{
	class FRDGTimingFrame* Frame = nullptr;
	int32 ScopeId;
	bool bPop;

	RENDERCORE_API FRDGScope_Budget(FRDGScopeState& State, DynamicRenderScaling::FBudget const& Budget);
	RENDERCORE_API void ImmediateEnd(FRDGScopeState& State);

	void BeginCPU(FRHIComputeCommandList& RHICmdList, bool bPreScope) { }
	void EndCPU  (FRHIComputeCommandList& RHICmdList, bool bPreScope) { }

	RENDERCORE_API void BeginGPU(FRHIComputeCommandList& RHICmdList);
	RENDERCORE_API void EndGPU  (FRHIComputeCommandList& RHICmdList);
};

#if RDG_EVENTS

	// Scope type for inserting named events on the CPU and GPU timelines.
	class FRDGScope_RHI
	{
		FRHIBreadcrumbNode* Node = nullptr;

		inline FRDGScope_RHI(FRDGScopeState& State, FRHIBreadcrumbNode* Node);

	public:
		template <typename TDesc, typename... TValues>
		inline FRDGScope_RHI(FRDGScopeState& State, TRHIBreadcrumbInitializer<TDesc, TValues...>&& Args);

		inline void ImmediateEnd(FRDGScopeState& State);

		void BeginCPU(FRHIComputeCommandList& RHICmdList, bool bPreScope)
		{
			if (Node)
			{
				RHICmdList.BeginBreadcrumbCPU(Node, !bPreScope);
				if (!bPreScope)
				{
					RHICmdList.BeginBreadcrumbGPU(Node, RHICmdList.GetPipeline());
				}
			}
		}

		void EndCPU(FRHIComputeCommandList& RHICmdList, bool bPreScope)
		{
			if (Node)
			{
				if (!bPreScope)
				{
					RHICmdList.EndBreadcrumbGPU(Node, RHICmdList.GetPipeline());
				}
				RHICmdList.EndBreadcrumbCPU(Node, !bPreScope);
			}
		}

		// Nothing to do for Begin/EndGPU. The RHI API only requires breadcrumbs to be 
		// begun/ended once, and will automatically fixup other pipelines whenever we switch.
		void BeginGPU(FRHIComputeCommandList& RHICmdList) {}
		void EndGPU  (FRHIComputeCommandList& RHICmdList) {}

		TCHAR const* GetTCHAR(FRHIBreadcrumb::FBuffer& Buffer) const
		{
			return Node->GetTCHAR(Buffer);
		}
	};

#endif // RDG_EVENTS

//
// Main RDG scope class.
// 
// A tree of these scopes is created by the render thread as the RenderGraph is built.
// Each scope type implementation uses the following functions, which are called during different RDG phases:
// 
//    Constructor / ImmediateEnd() - Render thread timeline. Called once, either side of scoped graph building work.
// 
//    BeginCPU    / EndCPU         - Parallel threads. Called during RDG pass lambdas execution. Scopes may be 
//                                   entered / exited multiple times depending on parallel pass set bucketing.
// 
//    BeginGPU    / EndGPU         - Parallel threads. Called once for each GPU pipeline the scope covers.
//									 Used for inserting commands on the RHICmdList. The command list passed to
//                                   Begin / End may be different in each, depending on parallel pass set bucketing.
//
struct FRDGScope
{
	FRDGScope* const Parent;
	FRDGPass* CPUFirstPass = nullptr;
	FRDGPass* CPULastPass = nullptr;
	TRHIPipelineArray<FRDGPass*> GPUFirstPass { InPlace, nullptr };
	TRHIPipelineArray<FRDGPass*> GPULastPass  { InPlace, nullptr };

	template <typename... TTypes>
	class TStorage
	{
		typedef TVariant<FEmptyVariantState, TTypes...> TImpl;
		TImpl Impl;

	public:
		template <typename TCallback>
		void Dispatch(TCallback&& Callback)
		{
			size_t Index = Impl.GetIndex();
			check(Index > 0);
			((Index == Impl.template IndexOfType<TTypes>() ? Callback(Impl.template Get<TTypes>()),0 : 0), ...);
		}

		template <typename TScopeType, typename... TArgs>
		void Emplace(TArgs&&... Args)
		{
			Impl.template Emplace<TScopeType>(Forward<TArgs>(Args)...);
		}

		template <typename TScopeType>
		static constexpr SIZE_T GetTypeIndex()
		{
			return TImpl::template IndexOfType<TScopeType>();
		}

		template <typename TScopeType>
		TScopeType* Get()
		{
			return Impl.GetIndex() == Impl.template IndexOfType<TScopeType>()
				? &Impl.template Get<TScopeType>()
				: nullptr;
		}

		template <typename TScopeType>
		TScopeType const* Get() const
		{
			return const_cast<TStorage&>(*this).Get<TScopeType>();
		}
	};

	typedef TStorage<
		  FRDGScope_Budget
#if RDG_EVENTS
		, FRDGScope_RHI
#endif
#if CSV_PROFILER_STATS
		, FRDGScope_CSVExclusive
#endif
	> FStorage;
	
	FStorage Impl;

	template <typename TScopeType>
	static constexpr uint32 GetTypeMask()
	{
		return 1u << FStorage::GetTypeIndex<TScopeType>();
	}

#if RDG_ENABLE_TRACE
	bool bVisited = false;
#endif

	FRDGScope(FRDGScope* Parent)
		: Parent(Parent)
	{}

	void ImmediateEnd(FRDGScopeState& State) { Impl.Dispatch([&](auto& Scope) { Scope.ImmediateEnd(State); }); }

	void BeginCPU(FRHIComputeCommandList& RHICmdList, bool bPreScope) { Impl.Dispatch([&](auto& Scope) { Scope.BeginCPU(RHICmdList, bPreScope); }); }
	void BeginGPU(FRHIComputeCommandList& RHICmdList                ) { Impl.Dispatch([&](auto& Scope) { Scope.BeginGPU(RHICmdList           ); }); }
	void EndCPU  (FRHIComputeCommandList& RHICmdList, bool bPreScope) { Impl.Dispatch([&](auto& Scope) { Scope.EndCPU  (RHICmdList, bPreScope); }); }
	void EndGPU  (FRHIComputeCommandList& RHICmdList                ) { Impl.Dispatch([&](auto& Scope) { Scope.EndGPU  (RHICmdList           ); }); }

	template <typename TScopeType> TScopeType      * Get()       { return Impl.Get<TScopeType>(); }
	template <typename TScopeType> TScopeType const* Get() const { return Impl.Get<TScopeType>(); }

	FString GetFullPath(FRDGEventName const& PassName);
};

template <typename TScopeType>
class TRDGEventScopeGuard
{
	FRDGScopeState& State;
	FRDGScope* const Scope;

	TRDGEventScopeGuard(TRDGEventScopeGuard const&) = delete;
	TRDGEventScopeGuard(TRDGEventScopeGuard&&) = delete;

public:
	template <typename... TArgs>
	inline TRDGEventScopeGuard(FRDGScopeState& State, ERDGScopeFlags Flags, TArgs&&... Args);
	inline ~TRDGEventScopeGuard();

private:
	static constexpr uint32 TypeMask = 1u << FRDGScope::FStorage::GetTypeIndex<TScopeType>();
};


/** Macros for create render graph event names and scopes.
 *
 *  FRDGEventName Name = RDG_EVENT_NAME("MyPass %sx%s", ViewRect.Width(), ViewRect.Height());
 *
 *  RDG_EVENT_SCOPE(GraphBuilder, "MyProcessing %sx%s", ViewRect.Width(), ViewRect.Height());
 */
#if RDG_EVENTS

	namespace UE::RHI::Breadcrumbs::Private
	{
		template <>
		struct TValue<FRDGEventName>
		{
			static constexpr bool bValidType = true;

			TCHAR const* StringPointer;
			TCHAR StringStorage[FRHIBreadcrumb::MaxLength];

			TValue(FRDGEventName const& Value)
			{
				if (Value.HasFormattedString())
				{
					// We must take a string copy as RHI breadcrumb TValues must be trivially destructable.
					FCString::Strncpy(StringStorage, Value.GetTCHAR(), UE_ARRAY_COUNT(StringStorage));
					StringPointer = StringStorage;
				}
				else
				{
					StringPointer = Value.GetTCHAR();
				}
			}

			struct FConvert
			{
				TCHAR const* Inner;
				FConvert(TValue const& Value)
					: Inner(Value.StringPointer)
				{}
			};

			void Serialize(UE::RHI::GPUProfiler::FMetadataSerializer& Serializer) const
			{
				Serializer.AppendValue(StringPointer);
			}
		};

		// Type to force the selection of the below TValue specialization.
		struct FRDGFormatTag {};

		template <typename... TValues>
		struct TValue<std::tuple<FRDGFormatTag&, TValues...>>
		{
			static constexpr bool bValidType = true;

			TCHAR Value[FRHIBreadcrumb::MaxLength];

			TValue(std::tuple<FRDGFormatTag&, TValues...> const& Wrapper)
			{
				std::apply([this](FRDGFormatTag&, auto&& FormatString, auto&&... Values)
				{
					FCString::Snprintf(Value, UE_ARRAY_COUNT(Value), FormatString, std::forward<decltype(Values)>(Values)...);
				}, Wrapper);
			}

			struct FConvert
			{
				TCHAR const* Inner;
				FConvert(TValue const& Value)
					: Inner(Value.Value)
				{}
			};

			void Serialize(UE::RHI::GPUProfiler::FMetadataSerializer& Serializer) const
			{
				Serializer.AppendValue(Value);
			}
		};
	}

	// Returns an RHI breadcrumb initializer appropriate for the given varargs.
	#define RDG_BREADCRUMB_DESC_FORWARD_VALUES(StaticName, FormatString, GPUStatArgs)					\
		[&](UE::RHI::Breadcrumbs::Private::FRDGFormatTag&& Tag, auto&&... Values) -> auto				\
		{																								\
			using namespace UE::RHI::Breadcrumbs::Private;												\
			if constexpr (std::is_same_v<TRemoveCVRef<decltype(FormatString)>, FRDGEventName>)			\
			{																							\
				/* Single FRDGEventName */																\
				static_assert(sizeof...(Values) == 0, "Unexpected additional arguments");				\
				return RHI_BREADCRUMB_DESC_FORWARD_VALUES(StaticName, FForceNoSprintf(), GPUStatArgs)(	\
					std::forward<decltype(FormatString)>(FormatString)									\
				);																						\
			}																							\
			else if constexpr (TIsValidArgs<decltype(Values)...>::Value)								\
			{																							\
				/* Varargs are compatible with RHI breadcrumbs. Forward the args directly. */			\
				return RHI_BREADCRUMB_DESC_FORWARD_VALUES(StaticName, FormatString, GPUStatArgs)(		\
					std::forward<decltype(Values)>(Values)...											\
				);																						\
			}																							\
			else																						\
			{																							\
				/* Args are not compatible with RHI breadcrumbs. Do the snprintf now */					\
				return RHI_BREADCRUMB_DESC_COPY_VALUES(StaticName, FForceNoSprintf(), GPUStatArgs)(		\
					std::forward_as_tuple(																\
						  Tag																			\
						, FormatString																	\
						, std::forward<decltype(Values)>(Values)...										\
					)																					\
				);																						\
			}																							\
		}

	#define RDG_EVENT_SCOPE_CONSTRUCT(ObjectName, GraphBuilder, Condition, ScopeFlags, GPUStatArgs, StaticName, FormatString, ...)	\
		do																															\
		{																															\
			if ((Condition) && ((GraphBuilder).ShouldAllocScope((ObjectName), ScopeFlags)))											\
			{																														\
				(ObjectName).Emplace(																								\
					  (GraphBuilder)																								\
					, ScopeFlags																									\
					, RDG_BREADCRUMB_DESC_FORWARD_VALUES(																			\
						  StaticName 																								\
						, FormatString																								\
						, GPUStatArgs																								\
					)(UE::RHI::Breadcrumbs::Private::FRDGFormatTag(), ##__VA_ARGS__)												\
				);																													\
			}																														\
		} while (false)

	#define RDG_EVENT_SCOPE_IMPL(GraphBuilder, Condition, ScopeFlags, GPUStatArgs, StaticName, FormatString, ...)	\
		TOptional<TRDGEventScopeGuard<FRDGScope_RHI>> UE_JOIN(__RDG_ScopeRef_, __LINE__);							\
		RDG_EVENT_SCOPE_CONSTRUCT(																					\
			  UE_JOIN(__RDG_ScopeRef_, __LINE__)																	\
			, GraphBuilder																							\
			, Condition																								\
			, ScopeFlags																							\
			, GPUStatArgs																							\
			, StaticName																							\
			, FormatString																							\
			, ##__VA_ARGS__																							\
		)

	#if WITH_RHI_BREADCRUMBS_FULL

		#if RDG_EVENTS >= RDG_EVENTS_STRING_COPY
			#define RDG_SCOPE_ARGS(Format, ...) TEXT(Format), ##__VA_ARGS__
		#else
			#define RDG_SCOPE_ARGS(Format, ...) nullptr
		#endif

		// Skip expensive string formatting for the relatively common case of no varargs.  We detect this by stringizing the varargs and checking if the string is non-empty (more than just a null terminator).
		#define RDG_EVENT_NAME(Format, ...) (sizeof(#__VA_ARGS__ "") > 1 ? FRDGEventName(TEXT(Format), ##__VA_ARGS__) : FRDGEventName(1, TEXT(Format)))

		#define RDG_EVENT_SCOPE(                 GraphBuilder,                      Format, ...) RDG_EVENT_SCOPE_IMPL(GraphBuilder,      true, ERDGScopeFlags::None , RHI_GPU_STAT_ARGS_NONE	 , TEXT(Format)    , RDG_SCOPE_ARGS(Format, ##__VA_ARGS__))
		#define RDG_EVENT_SCOPE_STAT(            GraphBuilder,            StatName, Format, ...) RDG_EVENT_SCOPE_IMPL(GraphBuilder,      true, ERDGScopeFlags::Stat , RHI_GPU_STAT_ARGS(StatName), TEXT(Format)    , RDG_SCOPE_ARGS(Format, ##__VA_ARGS__))
		#define RDG_EVENT_SCOPE_CONDITIONAL(     GraphBuilder, Condition,           Format, ...) RDG_EVENT_SCOPE_IMPL(GraphBuilder, Condition, ERDGScopeFlags::None , RHI_GPU_STAT_ARGS_NONE	 , TEXT(Format)    , RDG_SCOPE_ARGS(Format, ##__VA_ARGS__))
		#define RDG_EVENT_SCOPE_CONDITIONAL_STAT(GraphBuilder, Condition, StatName, Format, ...) RDG_EVENT_SCOPE_IMPL(GraphBuilder, Condition, ERDGScopeFlags::Stat , RHI_GPU_STAT_ARGS(StatName), TEXT(Format)    , RDG_SCOPE_ARGS(Format, ##__VA_ARGS__))
		// The 'Final' version disables any further child scopes or pass events. It is intended to group overlapping passes as events can disable overlap on certain GPUs.
		#define RDG_EVENT_SCOPE_FINAL(           GraphBuilder,                      Format, ...) RDG_EVENT_SCOPE_IMPL(GraphBuilder,      true, ERDGScopeFlags::Final, RHI_GPU_STAT_ARGS_NONE	 , TEXT(Format)    , RDG_SCOPE_ARGS(Format, ##__VA_ARGS__))
		// Used in places which have an existing FRDGEventName, e.g. RDG pass name scopes. Prefer to use the other RDG scope macros instead.
		#define RDG_EVENT_SCOPE_CONDITIONAL_NAME(GraphBuilder, Condition,           EventName  ) RDG_EVENT_SCOPE_IMPL(GraphBuilder, Condition, ERDGScopeFlags::None , RHI_GPU_STAT_ARGS_NONE     , TEXT("RDGEvent"), EventName)

	#else // WITH_RHI_BREADCRUMBS_MINIMAL

		//
		// Keep only the STAT RDG scopes enabled in MINIMAL mode.
		// Also disable the varargs. We don't capture the format strings and varargs in MINIMAL mode
		//

		#define RDG_EVENT_NAME(Format, ...) FRDGEventName()

		#define RDG_EVENT_SCOPE_STAT(            GraphBuilder,            StatName, Format, ...) RDG_EVENT_SCOPE_IMPL(GraphBuilder,      true, ERDGScopeFlags::Stat , RHI_GPU_STAT_ARGS(StatName), TEXT(Format), nullptr)
		#define RDG_EVENT_SCOPE_CONDITIONAL_STAT(GraphBuilder, Condition, StatName, Format, ...) RDG_EVENT_SCOPE_IMPL(GraphBuilder, Condition, ERDGScopeFlags::Stat , RHI_GPU_STAT_ARGS(StatName), TEXT(Format), nullptr)

		#define RDG_EVENT_SCOPE(                 GraphBuilder,                      Format, ...) do { } while (false)
		#define RDG_EVENT_SCOPE_CONDITIONAL(     GraphBuilder, Condition,           Format, ...) do { } while (false)
		#define RDG_EVENT_SCOPE_FINAL(           GraphBuilder,                      Format, ...) do { } while (false)
		#define RDG_EVENT_SCOPE_CONDITIONAL_NAME(GraphBuilder, Condition,           EventName  ) do { } while (false)

	#endif

#else

	#define RDG_EVENT_NAME(...) FRDGEventName()

	#define RDG_EVENT_SCOPE(...)                  do { } while (false)
	#define RDG_EVENT_SCOPE_STAT(...)             do { } while (false)
	#define RDG_EVENT_SCOPE_CONDITIONAL(...)      do { } while (false)
	#define RDG_EVENT_SCOPE_CONDITIONAL_STAT(...) do { } while (false)
	#define RDG_EVENT_SCOPE_FINAL(...)            do { } while (false)
	#define RDG_EVENT_SCOPE_CONDITIONAL_NAME(...) do { } while (false)

#endif

#define RDG_GPU_STAT_SCOPE(GraphBuilder, StatName)                          UE_DEPRECATED_MACRO(5.8, "The legacy GPU profiler has been removed from the engine. RDG_GPU_STAT_SCOPE"      " has been deprecated, and now does nothing. GPU stats are handled via the _STAT versions of RDG_EVENT_SCOPE macros, e.g. RDG_EVENT_SCOPE_STAT.")
#define RDG_GPU_STAT_SCOPE_VERBOSE(GraphBuilder, StatName, Description)     UE_DEPRECATED_MACRO(5.8, "The legacy GPU profiler has been removed from the engine. RDG_GPU_STAT_SCOPE_VERBOSE has been deprecated, and now does nothing. GPU stats are handled via the _STAT versions of RDG_EVENT_SCOPE macros, e.g. RDG_EVENT_SCOPE_STAT.")

#if CSV_PROFILER_STATS
	#define RDG_CSV_STAT_EXCLUSIVE_SCOPE(GraphBuilder, StatName) TRDGEventScopeGuard<FRDGScope_CSVExclusive> UE_JOIN(__RDG_CSVStat_##StatName,__LINE__) ((GraphBuilder), ERDGScopeFlags::AlwaysEnable, #StatName);
#else
	#define RDG_CSV_STAT_EXCLUSIVE_SCOPE(GraphBuilder, StatName)
#endif

 /** Injects a scope onto both the RDG and RHI timeline. */
#define RDG_RHI_EVENT_SCOPE(     GraphBuilder, Name)       RDG_EVENT_SCOPE(GraphBuilder, #Name);            RHI_BREADCRUMB_EVENT(GraphBuilder.RHICmdList, #Name)
#define RDG_RHI_EVENT_SCOPE_STAT(GraphBuilder, Stat, Name) RDG_EVENT_SCOPE_STAT(GraphBuilder, Stat, #Name); RHI_BREADCRUMB_EVENT_STAT(GraphBuilder.RHICmdList, Stat, #Name)
#define RDG_RHI_EVENT_SCOPE_CONDITIONAL_STAT(GraphBuilder, Stat, Name, Condition) RDG_EVENT_SCOPE_CONDITIONAL_STAT(GraphBuilder, Condition, Stat, #Name); RHI_BREADCRUMB_EVENT_CONDITIONAL_STAT(GraphBuilder.RHICmdList, Stat, Condition, #Name)

#define RDG_RHI_GPU_STAT_SCOPE(  GraphBuilder, StatName)   UE_DEPRECATED_MACRO(5.8, "The legacy GPU profiler has been removed from the engine. RDG_RHI_GPU_STAT_SCOPE has been deprecated, and now does nothing. Use RDG_RHI_EVENT_SCOPE_STAT instead.")

namespace DynamicRenderScaling
{
	class FRDGScope final : public TRDGEventScopeGuard<FRDGScope_Budget>
	{
	public:
		FRDGScope(FRDGScopeState& State, FBudget const& Budget)
			: TRDGEventScopeGuard(State, ERDGScopeFlags::AlwaysEnable, Budget)
		{}
	};

} // namespace DynamicRenderScaling

enum class ERDGScopeMode : uint8
{
	Disabled              = 0,
	TopLevelOnly          = 1,
	AllEvents             = 2,
	AllEventsAndPassNames = 3
};

class FRDGScopeState
{
protected:
	struct FState
	{
		struct FRDGScope* Current = nullptr;
		DynamicRenderScaling::FBudget const* ActiveBudget = nullptr;

		uint32 Mask = 0;

		bool const bImmediate;
		bool const bParallelExecute;

#if RDG_EVENTS == RDG_EVENTS_NONE
		static constexpr ERDGScopeMode const ScopeMode = ERDGScopeMode::Disabled;
#else
		ERDGScopeMode const ScopeMode;
#endif

		FState(bool bInImmediate, bool bInParallelExecute);

	} ScopeState;
	
	struct
	{
		// Allocator for all root graph allocations on the graph builder thread.
		FRDGAllocator Root;

		// Allocator for async pass and parallel execute setup.
		FRDGAllocator Task;

		// Allocator for all allocations related to states / transitions.
		FRDGAllocator Transition;

		int32 GetByteCount() const
		{
			return Root.GetByteCount() + Task.GetByteCount() + Transition.GetByteCount();
		}

	} Allocators;

public:
	/** The RHI command list used for the render graph. */
	FRHICommandListImmediate& RHICmdList;

#if WITH_RHI_BREADCRUMBS

protected:
	FRHIBreadcrumbNode* LocalCurrentBreadcrumb = FRHIBreadcrumbNode::Sentinel;
	FRHIBreadcrumbList LocalBreadcrumbList {};
	TSharedPtr<FRHIBreadcrumbAllocator> LocalBreadcrumbAllocator;

public:
	FRHIBreadcrumbNode*& CurrentBreadcrumbRef;

	FRHIBreadcrumbAllocator& GetBreadcrumbAllocator()
	{
		if (ScopeState.bImmediate)
		{
			return RHICmdList.GetBreadcrumbAllocator();
		}
		else
		{
			if (!LocalBreadcrumbAllocator.IsValid())
			{
				LocalBreadcrumbAllocator = MakeShared<FRHIBreadcrumbAllocator>();
			}

			return *LocalBreadcrumbAllocator;
		}
	}

#endif // WITH_RHI_BREADCRUMBS

public:
	FRDGScopeState(FRHICommandListImmediate& InRHICmdList, bool bImmediate, bool bParallelExecute)
		: ScopeState(bImmediate, bParallelExecute)
		, RHICmdList(InRHICmdList)
#if WITH_RHI_BREADCRUMBS
		, CurrentBreadcrumbRef(bImmediate ? InRHICmdList.GetCurrentBreadcrumbRef() : LocalCurrentBreadcrumb)
#endif
	{}

	bool ShouldEmitEvents() const
	{
		return ScopeState.ScopeMode != ERDGScopeMode::Disabled;
	}

	template <typename TScopeType>
	inline bool ShouldAllocScope(TOptional<TRDGEventScopeGuard<TScopeType>> const&, ERDGScopeFlags Flags) const;

	template <typename TScopeType>
	friend class TRDGEventScopeGuard;
	friend FRDGScope_Budget;
#if RDG_EVENTS
	friend FRDGScope_RHI;
#endif
	friend DynamicRenderScaling::FRDGScope;
};

#include "RenderGraphEvent.inl" // IWYU pragma: export
