// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if RDG_EVENTS

	template <typename TDesc, typename... TValues>
	inline FRDGScope_RHI::FRDGScope_RHI(FRDGScopeState& State, TRHIBreadcrumbInitializer<TDesc, TValues...>&& Args)
		: FRDGScope_RHI(State, State.GetBreadcrumbAllocator().AllocBreadcrumb(MoveTemp(Args)))
	{}

	inline FRDGScope_RHI::FRDGScope_RHI(FRDGScopeState& State, FRHIBreadcrumbNode* Node)
		: Node(Node)
	{
		if (Node)
		{
			Node->SetParent(State.CurrentBreadcrumbRef);
			State.CurrentBreadcrumbRef = Node;
			Node->TraceBeginCPU();

			if (!State.ScopeState.bImmediate)
			{
				// Link breadcrumbs together, so we can iterate over them during RDG compilation.
				State.LocalBreadcrumbList.Append(Node);
			}
		}
	}

	inline void FRDGScope_RHI::ImmediateEnd(FRDGScopeState& State)
	{
	#if WITH_RHI_BREADCRUMBS
		if (Node)
		{
			Node->TraceEndCPU();
			State.CurrentBreadcrumbRef = Node->GetParent();
		}
	#endif
	}

#endif //  RDG_EVENTS

template <typename TScopeType>
template <typename... TArgs>
inline TRDGEventScopeGuard<TScopeType>::TRDGEventScopeGuard(FRDGScopeState& State, ERDGScopeFlags Flags, TArgs&&... Args)
	: State(State)
	, Scope(State.Allocators.Root.Alloc<FRDGScope>(State.ScopeState.Current))
{
	if (EnumHasAnyFlags(Flags, ERDGScopeFlags::Final))
	{
		// Mask off any nested scopes of the same type
		State.ScopeState.Mask |= FRDGScope::GetTypeMask<TScopeType>();
	}

	State.ScopeState.Current = Scope;

	Scope->Impl.Emplace<TScopeType>(State, Forward<TArgs>(Args)...);

	if (State.ScopeState.bImmediate)
	{
		Scope->BeginCPU(State.RHICmdList, false);
		Scope->BeginGPU(State.RHICmdList);
	}
}

template <typename TScopeType>
inline TRDGEventScopeGuard<TScopeType>::~TRDGEventScopeGuard()
{
	if (Scope)
	{
		if (State.ScopeState.bImmediate)
		{
			Scope->EndGPU(State.RHICmdList);
			Scope->EndCPU(State.RHICmdList, false);
		}

		Scope->ImmediateEnd(State);

		State.ScopeState.Mask &= ~(FRDGScope::GetTypeMask<TScopeType>());
		State.ScopeState.Current = State.ScopeState.Current->Parent;
	}
}

template <typename TScopeType>
inline bool FRDGScopeState::ShouldAllocScope(TOptional<TRDGEventScopeGuard<TScopeType>> const&, ERDGScopeFlags Flags) const
{
	if (ScopeState.ScopeMode == ERDGScopeMode::Disabled && !EnumHasAnyFlags(Flags, ERDGScopeFlags::AlwaysEnable))
	{
		return false;
	}

	if (ScopeState.ScopeMode == ERDGScopeMode::TopLevelOnly && (ScopeState.Mask & FRDGScope::GetTypeMask<TScopeType>()))
	{
		return false;
	}

	return true;
}
