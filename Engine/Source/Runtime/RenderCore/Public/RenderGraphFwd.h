// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

enum class ERDGBufferFlags : uint8;
enum class ERDGPassFlags : uint16;
enum class ERDGTextureFlags : uint8;
enum class ERDGUnorderedAccessViewFlags : uint8;

class FRDGBuffer;
using FRDGBufferRef = FRDGBuffer*;

struct FRDGBufferDesc;

class FRDGBufferSRV;
using FRDGBufferSRVRef = FRDGBufferSRV*;

class FRDGBufferUAV;
using FRDGBufferUAVRef = FRDGBufferUAV*;

class FRDGBuilder;

class FRDGPass;
using FRDGPassRef = FRDGPass*;

class FRDGPooledBuffer;

class FRDGPooledTexture;

class FRDGResource;
using FRDGResourceRef = FRDGResource*;

class FRDGShaderResourceView;
using FRDGShaderResourceViewRef = FRDGShaderResourceView*;

class FRDGTexture;
using FRDGTextureRef = FRDGTexture*;

struct FRDGTextureDesc;

class FRDGTextureSRV;
using FRDGTextureSRVRef = FRDGTextureSRV*;

class FRDGTextureUAV;
using FRDGTextureUAVRef = FRDGTextureUAV*;

class FRDGUniformBuffer;
using FRDGUniformBufferRef = FRDGUniformBuffer*;

class FRDGUnorderedAccessView;
using FRDGUnorderedAccessViewRef = FRDGUnorderedAccessView*;

class FRDGView;
using FRDGViewRef = FRDGView*;

template <typename TUniformStruct> class TRDGUniformBuffer;
template <typename TUniformStruct> using TRDGUniformBufferRef = TRDGUniformBuffer<TUniformStruct>*;
