// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RenderGraphDefinitions.h"
#include "ShaderParameterMacros.h"

/** A helper class for identifying and accessing a render graph pass parameter. */
class FRDGParameter final
{
public:
	FRDGParameter() = default;

	bool IsResource() const
	{
		return !IsRenderTargetBindingSlots() && !IsResourceAccessArray();
	}

	bool IsSRV() const
	{
		return MemberType == UBMT_RDG_TEXTURE_SRV || MemberType == UBMT_RDG_TEXTURE_NON_PIXEL_SRV || MemberType == UBMT_RDG_BUFFER_SRV;
	}

	bool IsUAV() const
	{
		return MemberType == UBMT_RDG_TEXTURE_UAV || MemberType == UBMT_RDG_BUFFER_UAV;
	}

	bool IsView() const
	{
		return IsSRV() || IsUAV();
	}

	bool IsTexture() const
	{
		return
			MemberType == UBMT_RDG_TEXTURE ||
			MemberType == UBMT_RDG_TEXTURE_ACCESS;
	}

	bool IsTextureAccess() const
	{
		return MemberType == UBMT_RDG_TEXTURE_ACCESS;
	}

	bool IsTextureAccessArray() const
	{
		return MemberType == UBMT_RDG_TEXTURE_ACCESS_ARRAY;
	}

	bool IsBuffer() const
	{
		return MemberType == UBMT_RDG_BUFFER_ACCESS;
	}

	bool IsBufferAccess() const
	{
		return MemberType == UBMT_RDG_BUFFER_ACCESS;
	}

	bool IsBufferAccessArray() const
	{
		return MemberType == UBMT_RDG_BUFFER_ACCESS_ARRAY;
	}

	bool IsResourceAccessArray() const
	{
		return IsBufferAccessArray() || IsTextureAccessArray();
	}

	bool IsUniformBuffer() const
	{
		return MemberType == UBMT_RDG_UNIFORM_BUFFER;
	}

	bool IsViewableResource() const
	{
		return IsTexture() || IsBuffer();
	}

	bool IsRenderTargetBindingSlots() const
	{
		return MemberType == UBMT_RENDER_TARGET_BINDING_SLOTS;
	}

	EUniformBufferBaseType GetType() const
	{
		return MemberType;
	}

	FRDGResourceRef GetAsResource() const
	{
		check(IsResource());
		return *GetAs<FRDGResourceRef>();
	}

	FRDGUniformBufferBinding GetAsUniformBuffer() const
	{
		check(IsUniformBuffer());
		return *GetAs<FRDGUniformBufferBinding>();
	}

	FRDGViewableResource* GetAsViewableResource() const
	{
		check(IsViewableResource());
		return *GetAs<FRDGViewableResource*>();
	}

	FRDGViewRef GetAsView() const
	{
		check(IsView());
		return *GetAs<FRDGViewRef>();
	}

	FRDGShaderResourceViewRef GetAsSRV() const
	{
		check(IsSRV());
		return *GetAs<FRDGShaderResourceViewRef>();
	}

	FRDGUnorderedAccessViewRef GetAsUAV() const
	{
		check(IsUAV());
		return *GetAs<FRDGUnorderedAccessViewRef>();
	}

	FRDGTextureRef GetAsTexture() const
	{
		check(IsTexture());
		return *GetAs<FRDGTextureRef>();
	}

	FRDGTextureAccess GetAsTextureAccess() const
	{
		check(MemberType == UBMT_RDG_TEXTURE_ACCESS);
		return *GetAs<FRDGTextureAccess>();
	}

	const FRDGTextureAccessArray& GetAsTextureAccessArray() const
	{
		check(MemberType == UBMT_RDG_TEXTURE_ACCESS_ARRAY);
		return *GetAs<FRDGTextureAccessArray>();
	}

	FRDGBufferRef GetAsBuffer() const
	{
		check(IsBuffer());
		return *GetAs<FRDGBufferRef>();
	}

	FRDGBufferAccess GetAsBufferAccess() const
	{
		check(MemberType == UBMT_RDG_BUFFER_ACCESS);
		return *GetAs<FRDGBufferAccess>();
	}

	const FRDGBufferAccessArray& GetAsBufferAccessArray() const
	{
		check(MemberType == UBMT_RDG_BUFFER_ACCESS_ARRAY);
		return *GetAs<FRDGBufferAccessArray>();
	}

	FRDGTextureSRVRef GetAsTextureSRV() const
	{
		check(MemberType == UBMT_RDG_TEXTURE_SRV || MemberType == UBMT_RDG_TEXTURE_NON_PIXEL_SRV);
		return *GetAs<FRDGTextureSRVRef>();
	}

	FRDGBufferSRVRef GetAsBufferSRV() const
	{
		check(MemberType == UBMT_RDG_BUFFER_SRV);
		return *GetAs<FRDGBufferSRVRef>();
	}

	FRDGTextureUAVRef GetAsTextureUAV() const
	{
		check(MemberType == UBMT_RDG_TEXTURE_UAV);
		return *GetAs<FRDGTextureUAVRef>();
	}

	FRDGBufferUAVRef GetAsBufferUAV() const
	{
		check(MemberType == UBMT_RDG_BUFFER_UAV);
		return *GetAs<FRDGBufferUAVRef>();
	}

	const FRenderTargetBindingSlots& GetAsRenderTargetBindingSlots() const
	{
		check(IsRenderTargetBindingSlots());
		return *GetAs<FRenderTargetBindingSlots>();
	}

private:
	FRDGParameter(EUniformBufferBaseType InMemberType, void* InMemberPtr)
		: MemberType(InMemberType)
		, MemberPtr(InMemberPtr)
	{}

	template <typename T>
	T* GetAs() const
	{
		return reinterpret_cast<T*>(MemberPtr);
	}

	const EUniformBufferBaseType MemberType = UBMT_INVALID;
	void* const MemberPtr = nullptr;

	friend class FRDGParameterStruct;
};

/** Wraps a pass parameter struct payload and provides helpers for traversing members. */
class FRDGParameterStruct
{
public:
	template <typename ParameterStructType>
	explicit FRDGParameterStruct(const ParameterStructType* Parameters, const FShaderParametersMetadata* InParameterMetadata)
		: Contents(reinterpret_cast<const uint8*>(Parameters))
		, Layout(InParameterMetadata->GetLayoutPtr())
		, Metadata(InParameterMetadata)
	{}

	explicit FRDGParameterStruct(const void* InContents, const FRHIUniformBufferLayout* InLayout)
		: Contents(reinterpret_cast<const uint8*>(InContents))
		, Layout(InLayout)
	{
		checkf(Contents && Layout, TEXT("Pass parameter struct created with null inputs."));
	}

	/** Returns the contents of the struct. */
	const uint8* GetContents() const { return Contents; }

	/** Returns the layout associated with this struct. */
	const FRHIUniformBufferLayout& GetLayout() const { return *Layout; }
	const FRHIUniformBufferLayout* GetLayoutPtr() const { return Layout; }

	const FShaderParametersMetadata* GetMetadata() const { return Metadata; }

	/** Helpful forwards from the layout. */
	inline bool HasRenderTargets() const   { return Layout->HasRenderTargets(); }
	inline bool HasExternalOutputs() const { return Layout->HasExternalOutputs(); }
	inline bool HasStaticSlot() const      { return Layout->HasStaticSlot(); }

	/** Returns the number of buffer parameters present on the layout. */
	uint32 GetBufferParameterCount() const  { return Layout->GraphBuffers.Num(); }

	/** Returns the number of texture parameters present on the layout. */
	uint32 GetTextureParameterCount() const { return Layout->GraphTextures.Num(); }

	/** Returns the number of RDG uniform buffers present in the layout. */
	uint32 GetUniformBufferParameterCount() const { return Layout->GraphUniformBuffers.Num(); }

	/** Returns the render target binding slots. Asserts if they don't exist. */
	const FRenderTargetBindingSlots& GetRenderTargets() const
	{
		check(HasRenderTargets());
		return *reinterpret_cast<const FRenderTargetBindingSlots*>(Contents + Layout->RenderTargetsOffset);
	}

	/** Enumerates all graph parameters on the layout. Graph uniform buffers are traversed recursively but are
	 *  also included in the enumeration.
	 *  Expected function signature: void(FRDGParameter).
	 */
	template <typename FunctionType>
	void Enumerate(FunctionType Function) const;

	/** Same as Enumerate, but only texture parameters are included. */
	template <typename FunctionType>
	void EnumerateTextures(FunctionType Function) const;

	/** Same as Enumerate, but only buffer parameters are included. */
	template <typename FunctionType>
	void EnumerateBuffers(FunctionType Function) const;

	/** Enumerates all non-null uniform buffers. Expected function signature: void(FRDGUniformBufferBinding). */
	template <typename FunctionType>
	void EnumerateUniformBuffers(FunctionType Function) const;

	/** Returns a set of static uniform buffer bindings for the parameter struct. */
	RENDERCORE_API FUniformBufferStaticBindings GetStaticUniformBuffers() const;

	/** Returns the render pass info generated from the render target binding slots. */
	RENDERCORE_API FRHIRenderPassInfo GetRenderPassInfo() const;

	/** Clears out all uniform buffer references in the parameter struct. */
	static void ClearUniformBuffers(void* Contents, const FRHIUniformBufferLayout* Layout);

private:
	FRDGParameter GetParameterInternal(TArrayView<const FRHIUniformBufferResource> Parameters, uint32 ParameterIndex) const
	{
		checkf(ParameterIndex < static_cast<uint32>(Parameters.Num()), TEXT("Attempted to access RDG pass parameter outside of index for Layout '%s'"), *Layout->GetDebugName());
		const EUniformBufferBaseType MemberType = Parameters[ParameterIndex].MemberType;
		const uint16 MemberOffset = Parameters[ParameterIndex].MemberOffset;
		return FRDGParameter(MemberType, const_cast<uint8*>(Contents + MemberOffset));
	}

	const uint8* Contents;
	FUniformBufferLayoutRHIRef Layout;
	const FShaderParametersMetadata* Metadata = nullptr;

	friend class FRDGPass;
};

template <typename ParameterStructType>
class TRDGParameterStruct
	: public FRDGParameterStruct
{
public:
	explicit TRDGParameterStruct(ParameterStructType* Parameters)
		: FRDGParameterStruct(Parameters, &ParameterStructType::FTypeInfo::GetStructMetadata()->GetLayout())
	{}

	/** Returns the contents of the struct. */
	const ParameterStructType* GetContents() const
	{
		return reinterpret_cast<const ParameterStructType*>(FRDGParameterStruct::GetContents());
	}

	const ParameterStructType* operator->() const
	{
		return GetContents();
	}
};

/** Helper function to get RHI render pass info from a pass parameter struct. Must be called
 *  within an RDG pass with the pass parameters; otherwise, the RHI access checks will assert.
 *  This helper is useful when you want to control the mechanics of render passes within an
 *  RDG raster pass by specifying 'SkipRenderPass'.
 */
template <typename TParameterStruct>
inline static FRHIRenderPassInfo GetRenderPassInfo(TParameterStruct* Parameters)
{
	return FRDGParameterStruct(Parameters, TParameterStruct::FTypeInfo::GetStructMetadata()).GetRenderPassInfo();
}

template <typename TParameterStruct>
inline static bool HasRenderPassInfo(TParameterStruct* Parameters)
{
	return FRDGParameterStruct(Parameters, TParameterStruct::FTypeInfo::GetStructMetadata()).HasRenderTargets();
}

/** Helper function to get RHI global uniform buffers out of a pass parameters struct. */
template <typename TParameterStruct>
inline static FUniformBufferStaticBindings GetStaticUniformBuffers(TParameterStruct* Parameters)
{
	return FRDGParameterStruct(Parameters, TParameterStruct::FTypeInfo::GetStructMetadata()).GetStaticUniformBuffers();
}
