// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

template <typename FunctionType>
void FRDGParameterStruct::Enumerate(FunctionType Function) const
{
	EnumerateUniformBuffers([&](FRDGUniformBufferBinding UniformBuffer)
	{
		UniformBuffer->GetParameters().Enumerate(Function);
	});

	for (uint32 Index = 0, Count = Layout->GraphResources.Num(); Index < Count; ++Index)
	{
		Function(GetParameterInternal(Layout->GraphResources, Index));
	}
}

template <typename FunctionType>
void FRDGParameterStruct::EnumerateTextures(FunctionType Function) const
{
	EnumerateUniformBuffers([&](FRDGUniformBufferBinding UniformBuffer)
	{
		UniformBuffer->GetParameters().EnumerateTextures(Function);
	});

	for (uint32 Index = 0, Count = Layout->GraphTextures.Num(); Index < Count; ++Index)
	{
		Function(GetParameterInternal(Layout->GraphTextures, Index));
	}
}

template <typename FunctionType>
void FRDGParameterStruct::EnumerateBuffers(FunctionType Function) const
{
	EnumerateUniformBuffers([&](FRDGUniformBufferBinding UniformBuffer)
	{
		UniformBuffer->GetParameters().EnumerateBuffers(Function);
	});

	for (uint32 Index = 0, Count = Layout->GraphBuffers.Num(); Index < Count; ++Index)
	{
		Function(GetParameterInternal(Layout->GraphBuffers, Index));
	}
}

template <typename FunctionType>
void FRDGParameterStruct::EnumerateUniformBuffers(FunctionType Function) const
{
	for (uint32 Index = 0, Count = Layout->GraphUniformBuffers.Num(); Index < Count; ++Index)
	{
		const FRDGParameter Parameter = GetParameterInternal(Layout->GraphUniformBuffers, Index);

		if (FRDGUniformBufferBinding UniformBuffer = Parameter.GetAsUniformBuffer())
		{
			Function(UniformBuffer);
		}
	}
}

inline void FRDGParameterStruct::ClearUniformBuffers(void* Contents, const FRHIUniformBufferLayout* Layout)
{
	FRDGParameterStruct Parameters(Contents, Layout);

	for (uint32 Index = 0, Count = Layout->UniformBuffers.Num(); Index < Count; ++Index)
	{
		const FRDGParameter Parameter = Parameters.GetParameterInternal(Layout->UniformBuffers, Index);

		if (FUniformBufferBinding* UniformBuffer = Parameter.GetAs<FUniformBufferBinding>())
		{
			*UniformBuffer = {};
		}
	}
}