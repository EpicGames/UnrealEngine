// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AutoRTFM.h"
#include "Containers/Array.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "Containers/SortedMap.h"
#include "Containers/StaticArray.h"
#include "Containers/UnrealString.h"
#include "HAL/Platform.h"
#include "Misc/AssertionMacros.h"
#include "Misc/EnumClassFlags.h"
#include "MultiGPU.h"
#include "RHI.h"
#include "RHICommandList.h"
#include "RHIDefinitions.h"
#include "RenderGraphAllocator.h"
#include "RenderGraphDefinitions.h"
#include "RenderGraphEvent.h"
#include "RenderGraphParameter.h"
#include "RenderGraphResources.h"
#include "ShaderParameterMacros.h"
#include "Stats/Stats.h"
#include "Templates/EnableIf.h"
#include "Templates/UnrealTemplate.h"
#include "Templates/UnrealTypeTraits.h"

class FRDGDispatchPassBuilder;

using FRDGTransitionQueue = TArray<const FRHITransition*, TInlineAllocator<8>>;

struct FRDGBarrierBatchBeginId
{
	FRDGBarrierBatchBeginId() = default;

	bool operator==(FRDGBarrierBatchBeginId Other) const
	{
		return Passes == Other.Passes && PipelinesAfter == Other.PipelinesAfter;
	}

	bool operator!=(FRDGBarrierBatchBeginId Other) const
	{
		return !(*this == Other);
	}

	friend uint32 GetTypeHash(FRDGBarrierBatchBeginId Id)
	{
		static_assert(sizeof(Id.Passes) <= 8);
		uint32 Hash = GetTypeHash(*(const uint64*)Id.Passes.GetData());
		return HashCombineFast(Hash, (uint32)Id.PipelinesAfter);
	}

	FRDGPassHandlesByPipeline Passes;
	ERHIPipeline PipelinesAfter = ERHIPipeline::None;
};

struct FRDGTransitionInfo
{
	static_assert((int32)ERHIAccess::Last <= (1 << 20) && (int32)ERDGViewableResourceType::MAX <= 3 && (int32)EResourceTransitionFlags::Last <= (1 << 2), "FRDGTransitionInfo packing is no longer correct.");

	uint64 AccessBefore            : 21; // 21
	uint64 AccessAfter             : 21; // 42
	uint64 ResourceHandle          : 16; // 58
	uint64 ResourceType            : 3;  // 61
	uint64 ResourceTransitionFlags : 3;  // 64

	union
	{
		struct
		{
			uint16 ArraySlice;
			uint8  MipIndex;
			uint8  PlaneSlice;

		} Texture;

		struct
		{
			uint64 CommitSize;

		} Buffer;
	};
};

struct FRDGBarrierBatchEndId
{
	FRDGBarrierBatchEndId() = default;
	FRDGBarrierBatchEndId(FRDGPassHandle InPassHandle, ERDGBarrierLocation InBarrierLocation)
		: PassHandle(InPassHandle)
		, BarrierLocation(InBarrierLocation)
	{}

	bool operator == (FRDGBarrierBatchEndId Other) const
	{
		return PassHandle == Other.PassHandle && BarrierLocation == Other.BarrierLocation;
	}

	bool operator != (FRDGBarrierBatchEndId Other) const
	{
		return *this == Other;
	}

	FRDGPassHandle PassHandle;
	ERDGBarrierLocation BarrierLocation = ERDGBarrierLocation::Epilogue;
};

class FRDGBarrierBatchBegin
{
public:
	RENDERCORE_API FRDGBarrierBatchBegin(ERHIPipeline PipelinesToBegin, ERHIPipeline PipelinesToEnd, const TCHAR* Name, FRDGPass* Pass);
	RENDERCORE_API FRDGBarrierBatchBegin(ERHIPipeline PipelinesToBegin, ERHIPipeline PipelinesToEnd, const TCHAR* Name, FRDGPassesByPipeline Passes);

	RENDERCORE_API void AddTransition(FRDGViewableResource* Resource, FRDGTransitionInfo Info);

	RENDERCORE_API void AddAlias(FRDGViewableResource* Resource, const FRHITransientAliasingInfo& Info);

	void SetUseCrossPipelineFence(bool bUseSeparateTransition)
	{
		if (bUseSeparateTransition)
		{
			bSeparateFenceTransitionNeeded = true;
		}
		else
		{
			EnumRemoveFlags(TransitionFlags, ERHITransitionCreateFlags::NoFence);
		}
		bTransitionNeeded = true;
	}

	RENDERCORE_API void CreateTransition(TConstArrayView<FRHITransitionInfo> TransitionsRHI);

	RENDERCORE_API void Submit(FRHIComputeCommandList& RHICmdList, ERHIPipeline Pipeline);
	RENDERCORE_API void Submit(FRHIComputeCommandList& RHICmdList, ERHIPipeline Pipeline, FRDGTransitionQueue& TransitionsToBegin);

	void Reserve(uint32 TransitionCount)
	{
		Transitions.Reserve(TransitionCount);
		Aliases.Reserve(TransitionCount);
	}

	bool IsTransitionNeeded() const
	{
		return bTransitionNeeded;
	}

private:
	const FRHITransition* Transition = nullptr;
	const FRHITransition* SeparateFenceTransition = nullptr;
	TRHIPipelineArray<FRDGBarrierBatchEndId> BarriersToEnd;
	TArray<FRDGTransitionInfo, FRDGArrayAllocator> Transitions;
	TArray<FRHITransientAliasingInfo, FRDGArrayAllocator> Aliases;
	ERHITransitionCreateFlags TransitionFlags = ERHITransitionCreateFlags::NoFence | ERHITransitionCreateFlags::AllowDecayPipelines;
	ERHIPipeline PipelinesToBegin;
	ERHIPipeline PipelinesToEnd;
	bool bTransitionNeeded = false;
	bool bSeparateFenceTransitionNeeded = false;

#if RDG_ENABLE_DEBUG
	FRDGPassesByPipeline DebugPasses;
	TArray<FRDGViewableResource*, FRDGArrayAllocator> DebugTransitionResources;
	TArray<FRDGViewableResource*, FRDGArrayAllocator> DebugAliasingResources;
	const TCHAR* DebugName;
#endif

	friend class FRDGBarrierBatchEnd;
	friend class FRDGBarrierValidation;
	friend class FRDGBuilder;
};

using FRDGTransitionCreateQueue = TArray<FRDGBarrierBatchBegin*, FRDGArrayAllocator>;

enum class ERDGPassTaskMode : uint8
{
	/** Execute must be called inline on the render thread. */
	Inline,

	/** Execute may be called in a task that is awaited at the end of FRDGBuilder::Execute. */
	Await,

	/** Execute may be called in a task that must be manually awaited. */
	Async
};

class FRDGBarrierBatchEnd
{
public:
	FRDGBarrierBatchEnd(FRDGPass* InPass, ERDGBarrierLocation InBarrierLocation)
		: Pass(InPass)
		, BarrierLocation(InBarrierLocation)
	{}

	/** Inserts a dependency on a begin batch. A begin batch can be inserted into more than one end batch. */
	RENDERCORE_API void AddDependency(FRDGBarrierBatchBegin* BeginBatch);

	RENDERCORE_API void Submit(FRHIComputeCommandList& RHICmdList, ERHIPipeline Pipeline);

	void Reserve(uint32 TransitionBatchCount)
	{
		Dependencies.Reserve(TransitionBatchCount);
	}

	RENDERCORE_API FRDGBarrierBatchEndId GetId() const;

	RENDERCORE_API bool IsPairedWith(const FRDGBarrierBatchBegin& BeginBatch) const;

private:
	TArray<FRDGBarrierBatchBegin*, TInlineAllocator<4, FRDGArrayAllocator>> Dependencies;
	FRDGPass* Pass;
	ERDGBarrierLocation BarrierLocation;

	friend class FRDGBarrierBatchBegin;
	friend class FRDGBarrierValidation;
};

/** Base class of a render graph pass. */
class FRDGPass
{
public:
	RENDERCORE_API FRDGPass(FRDGEventName&& InName, FRDGParameterStruct InParameterStruct, ERDGPassFlags InFlags, ERDGPassTaskMode InTaskMode);
	FRDGPass(const FRDGPass&) = delete;
	virtual ~FRDGPass() = default;

#if RDG_ENABLE_DEBUG
	RENDERCORE_API const TCHAR* GetName() const;
#else
	const TCHAR* GetName() const
	{
		return Name.GetTCHAR();
	}
#endif

	const FRDGEventName& GetEventName() const
	{
		return Name;
	}

	ERDGPassFlags GetFlags() const
	{
		return Flags;
	}

	ERHIPipeline GetPipeline() const
	{
		return Pipeline;
	}

	FRDGParameterStruct GetParameters() const
	{
		return ParameterStruct;
	}

	FRDGPassHandle GetHandle() const
	{
		return Handle;
	}

	uint32 GetWorkload() const
	{
		return Workload;
	}

	ERDGPassTaskMode GetTaskMode() const
	{
		return TaskMode;
	}

	bool IsParallelExecuteAllowed() const
	{
		return TaskMode != ERDGPassTaskMode::Inline;
	}

	bool IsMergedRenderPassBegin() const
	{
		return !bSkipRenderPassBegin && bSkipRenderPassEnd;
	}

	bool IsMergedRenderPassEnd() const
	{
		return bSkipRenderPassBegin && !bSkipRenderPassEnd;
	}

	bool SkipRenderPassBegin() const
	{
		return bSkipRenderPassBegin;
	}

	bool SkipRenderPassEnd() const
	{
		return bSkipRenderPassEnd;
	}

	bool IsAsyncCompute() const
	{
		return Pipeline == ERHIPipeline::AsyncCompute;
	}

	bool IsAsyncComputeBegin() const
	{
		return bAsyncComputeBegin;
	}

	bool IsAsyncComputeEnd() const
	{
		return bAsyncComputeEnd;
	}

	bool IsGraphicsFork() const
	{
		return bGraphicsFork;
	}

	bool IsGraphicsJoin() const
	{
		return bGraphicsJoin;
	}

	bool IsCulled() const
	{
		return bCulled;
	}

	bool IsSentinel() const
	{
		return bSentinel;
	}

	/** Returns the graphics pass responsible for forking the async interval this pass is in. */
	FRDGPassHandle GetGraphicsForkPass() const
	{
		return GraphicsForkPass;
	}

	/** Returns the graphics pass responsible for joining the async interval this pass is in. */
	FRDGPassHandle GetGraphicsJoinPass() const
	{
		return GraphicsJoinPass;
	}

	FRDGScope const* GetScope() const
	{
		return Scope;
	}

	FRHIGPUMask GetGPUMask() const
	{
#if WITH_MGPU
		return GPUMask;
#else
		return FRHIGPUMask();
#endif
	}

protected:
	RENDERCORE_API FRDGBarrierBatchBegin& GetPrologueBarriersToBegin(FRDGAllocator& Allocator, FRDGTransitionCreateQueue& CreateQueue);
	RENDERCORE_API FRDGBarrierBatchBegin& GetEpilogueBarriersToBeginForGraphics(FRDGAllocator& Allocator, FRDGTransitionCreateQueue& CreateQueue);
	RENDERCORE_API FRDGBarrierBatchBegin& GetEpilogueBarriersToBeginForAsyncCompute(FRDGAllocator& Allocator, FRDGTransitionCreateQueue& CreateQueue);
	RENDERCORE_API FRDGBarrierBatchBegin& GetEpilogueBarriersToBeginForAll(FRDGAllocator& Allocator, FRDGTransitionCreateQueue& CreateQueue);

	FRDGBarrierBatchBegin& GetEpilogueBarriersToBeginFor(FRDGAllocator& Allocator, FRDGTransitionCreateQueue& CreateQueue, ERHIPipeline PipelineForEnd)
	{
		switch (PipelineForEnd)
		{
		default: 
			checkNoEntry();
			// fall through

		case ERHIPipeline::Graphics:
			return GetEpilogueBarriersToBeginForGraphics(Allocator, CreateQueue);

		case ERHIPipeline::AsyncCompute:
			return GetEpilogueBarriersToBeginForAsyncCompute(Allocator, CreateQueue);

		case ERHIPipeline::All:
			return GetEpilogueBarriersToBeginForAll(Allocator, CreateQueue);
		}
	}

	RENDERCORE_API FRDGBarrierBatchEnd& GetPrologueBarriersToEnd(FRDGAllocator& Allocator);
	RENDERCORE_API FRDGBarrierBatchEnd& GetEpilogueBarriersToEnd(FRDGAllocator& Allocator);

	virtual void Execute(FRHIComputeCommandList& RHICmdList) {}
	virtual void LaunchDispatchPassTasks(FRDGDispatchPassBuilder& DispatchPassBuilder) {}

	// When r.RDG.Debug is enabled, this will include a full namespace path with event scopes included.
	IF_RDG_ENABLE_DEBUG(FString FullPathIfDebug);

	const FRDGEventName Name;
	const FRDGParameterStruct ParameterStruct;
	const ERDGPassFlags Flags;
	const ERDGPassTaskMode TaskMode;
	const ERHIPipeline Pipeline;
	FRDGPassHandle Handle;
	uint32 Workload = 1;

	union
	{
		struct
		{
			/** Whether the render pass begin / end should be skipped. */
			uint16 bSkipRenderPassBegin : 1;
			uint16 bSkipRenderPassEnd : 1;

			/** (AsyncCompute only) Whether this is the first / last async compute pass in an async interval. */
			uint16 bAsyncComputeBegin : 1;
			uint16 bAsyncComputeEnd : 1;

			/** (Graphics only) Whether this is a graphics fork / join pass. */
			uint16 bGraphicsFork : 1;
			uint16 bGraphicsJoin : 1;

			/** Whether this pass is a sentinel (prologue / epilogue) pass. */
			uint16 bSentinel : 1;

			/** If set, dispatches to the RHI thread after executing this pass. */
			uint16 bDispatchAfterExecute : 1;

			/** If set, this is a dispatch pass. */
			uint16 bDispatchPass : 1;
		};
		uint16 PackedBits = 0;
	};

	union
	{
		// Task-specific bits which are written in a task in parallel with reads from the other set.
		struct
		{
			/** Whether this pass does not contain parameters. */
			uint8 bEmptyParameters : 1;

			/** Whether this pass has external UAVs that are not tracked by RDG. */
			uint8 bHasExternalOutputs : 1;

			/** Whether this pass has been culled. */
			uint8 bCulled : 1;

			/** Whether this pass is used for external access transitions. */
			uint8 bExternalAccessPass : 1;

			/** Whether the pass only writes to resources in its render pass. */
			uint8 bRenderPassOnlyWrites : 1;
		};
		uint8 PacketBits_AsyncSetupQueue = 0;
	};

	union
	{
		// Task-specific bits which are written in a task in parallel with reads from the other set.
		struct
		{
			/** If set, marks the begin / end of a span of passes executed in parallel in a task. */
			uint8 bParallelExecuteBegin : 1;
			uint8 bParallelExecuteEnd : 1;

			/** If set, marks that a pass is executing in parallel. */
			uint8 bParallelExecute : 1;
		};
		uint8 PacketBits_ParallelExecute = 0;
	};

	/** Handle of the latest cross-pipeline producer. */
	FRDGPassHandle CrossPipelineProducer;

	/** (AsyncCompute only) Graphics passes which are the fork / join for async compute interval this pass is in. */
	FRDGPassHandle GraphicsForkPass;
	FRDGPassHandle GraphicsJoinPass;

	/** The passes which are handling the epilogue / prologue barriers meant for this pass. */
	FRDGPassHandle PrologueBarrierPass;
	FRDGPassHandle EpilogueBarrierPass;

	/** Number of transitions to reserve. Basically an estimate of the number of textures / buffers. */
	uint32 NumTransitionsToReserve = 0;

	/** Lists of producer passes and the full list of cross-pipeline consumer passes. */
	TArray<FRDGPassHandle, FRDGArrayAllocator> CrossPipelineConsumers;
	TArray<FRDGPass*, FRDGArrayAllocator> Producers;

	struct FTextureState
	{
		FTextureState() = default;

		FTextureState(FRDGTextureRef InTexture)
			: Texture(InTexture)
		{
			const uint32 SubresourceCount = Texture->GetSubresourceCount();
			State.SetNum(SubresourceCount);
			MergeState.SetNum(SubresourceCount);
		}

		FRDGTextureRef Texture = nullptr;
		FRDGTextureSubresourceState State;
		FRDGTextureSubresourceState MergeState;
		uint32 ReferenceCount = 0;
	};

	struct FBufferState
	{
		FBufferState() = default;

		FBufferState(FRDGBufferRef InBuffer)
			: Buffer(InBuffer)
		{}

		FRDGBufferRef Buffer = nullptr;
		FRDGSubresourceState State;
		FRDGSubresourceState* MergeState = nullptr;
		uint32 ReferenceCount = 0;
	};

	/** Maps textures / buffers to information on how they are used in the pass. */
	TArray<FTextureState, FRDGArrayAllocator> TextureStates;
	TArray<FBufferState, FRDGArrayAllocator> BufferStates;
	TArray<FRDGViewHandle, FRDGArrayAllocator> Views;
	TArray<FRDGUniformBufferHandle, FRDGArrayAllocator> UniformBuffers;

	struct FExternalAccessOp
	{
		FExternalAccessOp() = default;

		FExternalAccessOp(FRDGViewableResource* InResource, FRDGViewableResource::EAccessMode InMode)
			: Resource(InResource)
			, Mode(InMode)
		{}

		FRDGViewableResource* Resource;
		FRDGViewableResource::EAccessMode Mode;
	};

	TArray<FExternalAccessOp, FRDGArrayAllocator> ExternalAccessOps;

	/** Lists of pass parameters scheduled for begin during execution of this pass. */
	TArray<FRDGPass*, TInlineAllocator<1, FRDGArrayAllocator>> ResourcesToBegin;
	TArray<FRDGPass*, TInlineAllocator<1, FRDGArrayAllocator>> ResourcesToEnd;

	/** Split-barrier batches at various points of execution of the pass. */
	FRDGBarrierBatchBegin* PrologueBarriersToBegin = nullptr;
	FRDGBarrierBatchEnd* PrologueBarriersToEnd = nullptr;
	FRDGBarrierBatchBegin* EpilogueBarriersToBeginForGraphics = nullptr;
	FRDGBarrierBatchBegin* EpilogueBarriersToBeginForAsyncCompute = nullptr;
	FRDGBarrierBatchBegin* EpilogueBarriersToBeginForAll = nullptr;
	TArray<FRDGBarrierBatchBegin*, FRDGArrayAllocator> SharedEpilogueBarriersToBegin;
	FRDGBarrierBatchEnd* EpilogueBarriersToEnd = nullptr;

	uint32 ParallelPassSetIndex = 0;

#if WITH_MGPU
	FRHIGPUMask GPUMask;
#endif

	FRDGScope* Scope = nullptr;

#if RDG_ENABLE_TRACE
	TArray<FRDGTextureHandle, FRDGArrayAllocator> TraceTextures;
	TArray<FRDGBufferHandle, FRDGArrayAllocator> TraceBuffers;
#endif

	friend FRDGBuilder;
	friend FRDGPassRegistry;
	friend FRDGTrace;
	friend FRDGUserValidation;
	friend FRDGDispatchPassBuilder;
};

/** Render graph pass with lambda execute function. */
template <typename ParameterStructType, typename ExecuteLambdaType>
class TRDGLambdaPass
	: public FRDGPass
{
	class ExecuteLambdaTraits
	{
	private:
		// Verify that the amount of stuff captured by the pass lambda is reasonable.
		static constexpr int32 kMaximumLambdaCaptureSize = 1024;
		static_assert(sizeof(ExecuteLambdaType) <= kMaximumLambdaCaptureSize, "The amount of data of captured for the pass looks abnormally high.");

		template <typename T>
		struct TLambdaTraits
			: TLambdaTraits<decltype(&T::operator())>
		{};
		template <typename ReturnType, typename ClassType, typename ArgType>
		struct TLambdaTraits<ReturnType(ClassType::*)(ArgType&) const>
		{
			using TRHICommandList = ArgType;
			using TRDGPass = void;
			static constexpr bool bIsTaskAsync = false;
		};
		template <typename ReturnType, typename ClassType, typename ArgType>
		struct TLambdaTraits<ReturnType(ClassType::*)(ArgType&)>
		{
			using TRHICommandList = ArgType;
			using TRDGPass = void;
			using TRDGAsyncToken = void;
			static constexpr bool bIsTaskAsync = false;
		};
		template <typename ReturnType, typename ClassType, typename ArgType>
		struct TLambdaTraits<ReturnType(ClassType::*)(FRDGAsyncTask, ArgType&) const>
		{
			using TRHICommandList = ArgType;
			using TRDGPass = void;
			static constexpr bool bIsTaskAsync = true;
		};
		template <typename ReturnType, typename ClassType, typename ArgType>
		struct TLambdaTraits<ReturnType(ClassType::*)(FRDGAsyncTask, ArgType&)>
		{
			using TRHICommandList = ArgType;
			using TRDGPass = void;
			static constexpr bool bIsTaskAsync = true;
		};
		template <typename ReturnType, typename ClassType, typename ArgType1, typename ArgType2>
		struct TLambdaTraits<ReturnType(ClassType::*)(const ArgType1*, ArgType2&) const>
		{
			using TRHICommandList = ArgType2;
			using TRDGPass UE_DEPRECATED(5.5, "An FRDGPass* lambda argument is no longer supported.") = ArgType1;
			static constexpr bool bIsTaskAsync = false;
		};
		template <typename ReturnType, typename ClassType, typename ArgType1, typename ArgType2>
		struct TLambdaTraits<ReturnType(ClassType::*)(const ArgType1*, ArgType2&)>
		{
			using TRHICommandList = ArgType2;
			using TRDGPass UE_DEPRECATED(5.5, "An FRDGPass* lambda argument is no longer supported.")  = ArgType1;
			static constexpr bool bIsTaskAsync = false;
		};
		template <typename ReturnType, typename ClassType, typename ArgType1, typename ArgType2>
		struct TLambdaTraits<ReturnType(ClassType::*)(const ArgType1*, FRDGAsyncTask, ArgType2&) const>
		{
			using TRHICommandList = ArgType2;
			using TRDGPass UE_DEPRECATED(5.5, "An FRDGPass* lambda argument is no longer supported.") = ArgType1;
			static constexpr bool bIsTaskAsync = true;
		};
		template <typename ReturnType, typename ClassType, typename ArgType1, typename ArgType2>
		struct TLambdaTraits<ReturnType(ClassType::*)(const ArgType1*, FRDGAsyncTask, ArgType2&)>
		{
			using TRHICommandList = ArgType2;
			using TRDGPass UE_DEPRECATED(5.5, "An FRDGPass* lambda argument is no longer supported.") = ArgType1;
			static constexpr bool bIsTaskAsync = true;
		};

	public:
		using TRHICommandList = typename TLambdaTraits<ExecuteLambdaType>::TRHICommandList;

		static constexpr bool bIsCommandListImmediate = std::is_same_v<TRHICommandList, FRHICommandListImmediate>;
		static constexpr bool bIsPassArgValid = !std::is_same_v<typename TLambdaTraits<ExecuteLambdaType>::TRDGPass, void>;
		static constexpr ERDGPassTaskMode TaskMode = bIsCommandListImmediate
			? ERDGPassTaskMode::Inline
			: TLambdaTraits<ExecuteLambdaType>::bIsTaskAsync
				? ERDGPassTaskMode::Async
				: ERDGPassTaskMode::Await;

		static_assert((bIsCommandListImmediate && TLambdaTraits<ExecuteLambdaType>::bIsTaskAsync) == false, "RDG pass is marked with RDG_TASK_ASYNC but is using the immediate command list. This is not allowed.");
	};

public:
	TRDGLambdaPass(
		FRDGEventName&& InName,
		const FShaderParametersMetadata* InParameterMetadata,
		const ParameterStructType* InParameterStruct,
		ERDGPassFlags InPassFlags,
		ExecuteLambdaType&& InExecuteLambda)
		: FRDGPass(MoveTemp(InName), FRDGParameterStruct(InParameterStruct, InParameterMetadata), InPassFlags, ExecuteLambdaTraits::TaskMode)
		, ExecuteLambda(MoveTemp(InExecuteLambda))
#if RDG_ENABLE_DEBUG
		, DebugParameterStruct(InParameterStruct)
#endif
	{}

private:
	// TODO: Replace AUTORTFM_INFER with AUTORTFM_DISABLE on the class.
	// Will require AutoRTFM-disabling the bulk of RenderCore.
	AUTORTFM_INFER void ExecuteLambdaFunc(FRHIComputeCommandList& RHICmdList)
	{
		if constexpr (ExecuteLambdaTraits::TaskMode == ERDGPassTaskMode::Async)
		{
			if constexpr (ExecuteLambdaTraits::bIsPassArgValid)
			{
				ExecuteLambda(this, FRDGAsyncTask(), static_cast<typename ExecuteLambdaTraits::TRHICommandList&>(RHICmdList));
			}
			else
			{
				ExecuteLambda(FRDGAsyncTask(), static_cast<typename ExecuteLambdaTraits::TRHICommandList&>(RHICmdList));
			}
		}
		else
		{
			if constexpr (ExecuteLambdaTraits::bIsPassArgValid)
			{
				ExecuteLambda(this, static_cast<typename ExecuteLambdaTraits::TRHICommandList&>(RHICmdList));
			}
			else
			{
				ExecuteLambda(static_cast<typename ExecuteLambdaTraits::TRHICommandList&>(RHICmdList));
			}
		}
	}

	AUTORTFM_INFER void Execute(FRHIComputeCommandList& RHICmdList) override
	{
#if !USE_NULL_RHI
		DECLARE_CYCLE_STAT_WITH_FLAGS(TEXT("FRDGPass Execute"), STAT_FRDGPass_Execute, STATGROUP_RHI, EStatFlags::Verbose);

		SCOPE_CYCLE_COUNTER(STAT_FRDGPass_Execute);
		RHICmdList.SetStaticUniformBuffers(ParameterStruct.GetStaticUniformBuffers());
		ExecuteLambdaFunc(static_cast<typename ExecuteLambdaTraits::TRHICommandList&>(RHICmdList));
#else
		checkNoEntry();
#endif // !USE_NULL_RHI
	}

	ExecuteLambdaType ExecuteLambda;

	IF_RDG_ENABLE_DEBUG(const ParameterStructType* DebugParameterStruct);
};

class FRDGDispatchPass
	: public FRDGPass
{
public:
	FRDGDispatchPass(FRDGEventName&& InName, FRDGParameterStruct InParameterStruct, ERDGPassFlags InFlags)
		: FRDGPass(MoveTemp(InName), InParameterStruct, InFlags, ERDGPassTaskMode::Async)
	{
		bDispatchPass = 1;
	}

private:
	void Execute(FRHIComputeCommandList& RHICmdList) override
	{
		RHICmdList.GetAsImmediate().QueueAsyncCommandListSubmit(MoveTemp(CommandLists));
	}

	TArray<FRHICommandListImmediate::FQueuedCommandList, FRDGArrayAllocator> CommandLists;
	UE::Tasks::FTaskEvent CommandListsEvent{ UE_SOURCE_LOCATION };

	friend FRDGBuilder;
	friend FRDGDispatchPassBuilder;
};

class FRDGDispatchPassBuilder
{
public:
	/** Create a new command list to record into and inserts it. Call FinishRecording() on the task when done. */
	RENDERCORE_API FRHICommandList* CreateCommandList();

	/** Adds a task that the graph will wait on during execution, alongside other async execution tasks. */
	RENDERCORE_API void AddPrerequisite(const UE::Tasks::FTask& Task);

private:
	FRDGDispatchPassBuilder(FRDGDispatchPass* InPass)
		: Pass(InPass)
		, StaticUniformBuffers(Pass->ParameterStruct.GetStaticUniformBuffers())
	{
		if (Pass->ParameterStruct.HasRenderTargets())
		{
			RenderPassInfo = MakeShared<FRHIParallelRenderPassInfo, ESPMode::ThreadSafe>(Pass->ParameterStruct.GetRenderPassInfo(), TEXT("DispatchPass"));
		}
	}

	void Finish();

	FRDGDispatchPass* Pass;
	FUniformBufferStaticBindings StaticUniformBuffers;
	TSharedPtr<FRHIParallelRenderPassInfo> RenderPassInfo;

	TArray<FRHISubCommandList*> SubCommandLists;
	TArray<UE::Tasks::FTask> Prerequisites;

	friend FRDGBuilder;
};

template <typename ParameterStructType, typename LaunchLambdaType>
class TRDGDispatchPass
	: public FRDGDispatchPass
{
	// Verify that the amount of stuff captured by the pass lambda is reasonable.
	static constexpr int32 kMaximumLambdaCaptureSize = 1024;
	static_assert(sizeof(LaunchLambdaType) <= kMaximumLambdaCaptureSize, "The amount of data of captured for the pass looks abnormally high.");

public:
	TRDGDispatchPass(
		FRDGEventName&& InName,
		const FShaderParametersMetadata* InParameterMetadata,
		const ParameterStructType* InParameterStruct,
		ERDGPassFlags InPassFlags,
		LaunchLambdaType&& InLaunchLambda)
		: FRDGDispatchPass(MoveTemp(InName), FRDGParameterStruct(InParameterStruct, InParameterMetadata), InPassFlags)
		, LaunchLambda(MoveTemp(InLaunchLambda))
#if RDG_ENABLE_DEBUG
		, DebugParameterStruct(InParameterStruct)
#endif
	{}

private:
	LaunchLambdaType LaunchLambda;

	void LaunchDispatchPassTasks(FRDGDispatchPassBuilder& DispatchPassBuilder) override
	{
		LaunchLambda(DispatchPassBuilder);
	}

	IF_RDG_ENABLE_DEBUG(const ParameterStructType* DebugParameterStruct);
};

template <typename ExecuteLambdaType>
class TRDGEmptyLambdaPass
	: public TRDGLambdaPass<FEmptyShaderParameters, ExecuteLambdaType>
{
public:
	TRDGEmptyLambdaPass(FRDGEventName&& InName, ERDGPassFlags InPassFlags, ExecuteLambdaType&& InExecuteLambda)
		: TRDGLambdaPass<FEmptyShaderParameters, ExecuteLambdaType>(MoveTemp(InName), FEmptyShaderParameters::FTypeInfo::GetStructMetadata(), &EmptyShaderParameters, InPassFlags, MoveTemp(InExecuteLambda))
	{}

private:
	static FEmptyShaderParameters EmptyShaderParameters;
	friend class FRDGBuilder;
};

template <typename ExecuteLambdaType>
FEmptyShaderParameters TRDGEmptyLambdaPass<ExecuteLambdaType>::EmptyShaderParameters;

/** Render graph pass used for the prologue / epilogue passes. */
class FRDGSentinelPass final
	: public FRDGPass
{
public:
	FRDGSentinelPass(FRDGEventName&& Name, ERDGPassFlags InPassFlagsToAdd = ERDGPassFlags::None)
		: FRDGPass(MoveTemp(Name), FRDGParameterStruct(&EmptyShaderParameters, FEmptyShaderParameters::FTypeInfo::GetStructMetadata()), ERDGPassFlags::NeverCull | InPassFlagsToAdd, ERDGPassTaskMode::Async)
	{
		bSentinel = 1;
	}

private:
	static FEmptyShaderParameters EmptyShaderParameters;
};

#include "RenderGraphParameters.inl" // IWYU pragma: export

class FRDGBuilder;
class FRDGPass;
class FRDGTrace;
class FRDGUserValidation;
class FShaderParametersMetadata;
