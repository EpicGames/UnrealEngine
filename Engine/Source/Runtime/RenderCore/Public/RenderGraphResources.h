// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Math/NumericLimits.h"
#include "Math/UnrealMathSSE.h"
#include "Misc/AssertionMacros.h"
#include "Misc/EnumClassFlags.h"
#include "PixelFormat.h"
#include "RHI.h"
#include "RHIDefinitions.h"
#include "RHIResources.h"
#include "RHITransientResourceAllocator.h"
#include "RenderGraphAllocator.h"
#include "RenderGraphDefinitions.h"
#include "RenderGraphParameter.h"
#include "RenderGraphTextureSubresource.h"
#include "PooledRenderTarget.h"
#include "ShaderParameterMacros.h"
#include "Templates/Function.h"
#include "Templates/RefCounting.h"
#include "Templates/TypeHash.h"
#include "Templates/UnrealTemplate.h"

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8
#include "RendererInterface.h"
#endif //UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8

class FRDGBarrierBatchBegin;
class FRDGBarrierValidation;
class FRDGBuffer;
class FRDGBufferPool;
class FRDGBuilder;
class FRDGResourceDumpContext;
class FRDGTextureUAV;
class FRDGTrace;
class FRDGUserValidation;
class FRHITransientBuffer;
class FRHITransientTexture;
class FRenderTargetPool;
class FShaderParametersMetadata;
struct FPooledRenderTarget;

/** Used for tracking pass producer / consumer edges in the graph for culling and pipe fencing. */
struct FRDGProducerState
{
	FRDGPass* Pass = nullptr;
	FRDGPass* PassIfSkipUAVBarrier = nullptr;
	FRDGPass* PassIfReadAccess = nullptr;
	ERHIAccess Access = ERHIAccess::Unknown;
	FRDGViewHandle NoUAVBarrierHandle;
};

/** Barrier location controls where the barrier is 'Ended' relative to the pass lambda being executed.
 *  Most barrier locations are done in the prologue prior to the executing lambda. But certain cases
 *  like an aliasing discard operation need to be done *after* the pass being invoked. Therefore, when
 *  adding a transition the user can specify where to place the barrier.
 */
enum class ERDGBarrierLocation : uint8
{
	/** The barrier occurs in the prologue of the pass (before execution). */
	Prologue,

	/** The barrier occurs in the epilogue of the pass (after execution). */
	Epilogue
};

using FRDGProducerStatesByPipeline = TRHIPipelineArray<FRDGProducerState>;

/** Used for tracking the state of an individual subresource during execution. */
struct FRDGSubresourceState
{
	/** Given a before and after state, returns whether a resource barrier is required. */
	static bool IsTransitionRequired(const FRDGSubresourceState& Previous, const FRDGSubresourceState& Next);

	/** Given a before and after state, returns whether they can be merged into a single state. */
	static bool IsMergeAllowed(ERDGViewableResourceType ResourceType, const FRDGSubresourceState& Previous, const FRDGSubresourceState& Next);

	FRDGSubresourceState() = default;

	explicit FRDGSubresourceState(ERHIAccess InAccess)
		: Access(InAccess)
	{}

	explicit FRDGSubresourceState(ERHIPipeline Pipeline, FRDGPassHandle PassHandle)
	{
		SetPass(Pipeline, PassHandle);
	}

	/** Initializes the first and last pass and the pipeline. Clears any other pass state. */
	void SetPass(ERHIPipeline Pipeline, FRDGPassHandle PassHandle);

	/** Validates that the state is in a correct configuration for use. */
	void Validate();

	/** Returns whether the state is used by the pipeline. */
	bool IsUsedBy(ERHIPipeline Pipeline) const;

	/** Returns the last pass across either pipe. */
	FRDGPassHandle GetLastPass() const;

	/** Returns the first pass across either pipe. */
	FRDGPassHandle GetFirstPass() const;

	/** Returns the pipeline mask this state is used on. */
	ERHIPipeline GetPipelines() const;

	/** The last used access on the pass. */
	ERHIAccess Access = ERHIAccess::Unknown;

	/** The first pass in this state. */
	FRDGPassHandlesByPipeline FirstPass;

	/** The last pass in this state. */
	FRDGPassHandlesByPipeline LastPass;

	/** The last no-UAV barrier to be used by this subresource. */
	FRDGViewUniqueFilter NoUAVBarrierFilter;

	/** Whether this subresource state represents a commit operation for a reserved resource. */
	FRDGBufferReservedCommitHandle ReservedCommitHandle;

	/** The last used transition flags on the pass. */
	EResourceTransitionFlags Flags = EResourceTransitionFlags::None;

	/** Used to specify whether the state is applied during the prologue or epilogue of the pass. This is only used when transitioning on the same pass / pipe. */
	ERDGBarrierLocation BarrierLocation = ERDGBarrierLocation::Prologue;
};

using FRDGTextureSubresourceState = TRDGTextureSubresourceArray<FRDGSubresourceState*, FRDGArrayAllocator>;

/** Generic graph resource. */
class FRDGResource
{
public:
	FRDGResource(const FRDGResource&) = delete;
	virtual ~FRDGResource() = default;

	// Name of the resource for debugging purpose.
	const TCHAR* const Name = nullptr;

	//////////////////////////////////////////////////////////////////////////
	//! The following methods may only be called during pass execution.

	/** Marks this resource as actually used by a resource. This is to track what dependencies on pass was actually unnecessary. */
#if RDG_ENABLE_DEBUG
	RENDERCORE_API virtual void MarkResourceAsUsed();
#else
	inline  void MarkResourceAsUsed() {}
#endif

	FRHIResource* GetRHI() const
	{
		IF_RDG_ENABLE_DEBUG(ValidateRHIAccess());
		return ResourceRHI;
	}

	//////////////////////////////////////////////////////////////////////////

	FRHIResource* GetRHIUnchecked() const
	{
		return ResourceRHI;
	}

protected:
	FRDGResource(const TCHAR* InName)
		: Name(InName)
#if RDG_ENABLE_DEBUG
		, bAllowRHIAccess(0)
#endif
	{}

	bool HasRHI() const
	{
		return ResourceRHI != nullptr;
	}

	FRHIResource* ResourceRHI = nullptr;

#if RDG_ENABLE_DEBUG
	RENDERCORE_API void ValidateRHIAccess() const;
#endif

private:
#if RDG_ENABLE_DEBUG
	uint8 bAllowRHIAccess : 1;
#endif

	friend FRDGBuilder;
	friend FRDGUserValidation;
	friend FRDGBarrierValidation;
};

class FRDGUniformBuffer
	: public FRDGResource
{
public:

	RENDERCORE_API virtual ~FRDGUniformBuffer();

	inline const FRDGParameterStruct& GetParameters() const
	{
		return ParameterStruct;
	}

	inline const TCHAR* GetLayoutName() const
	{
		return *ParameterStruct.GetLayout().GetDebugName();
	}

#if RDG_ENABLE_DEBUG
	RENDERCORE_API void MarkResourceAsUsed() override;
#else
	inline void MarkResourceAsUsed() {}
#endif

	//////////////////////////////////////////////////////////////////////////
	//! The following methods may only be called during pass execution.

	FRHIUniformBuffer* GetRHI() const
	{
		return static_cast<FRHIUniformBuffer*>(FRDGResource::GetRHI());
	}

	//////////////////////////////////////////////////////////////////////////

	FRHIUniformBuffer* GetRHIUnchecked() const
	{
		return static_cast<FRHIUniformBuffer*>(FRDGResource::GetRHIUnchecked());
	}

protected:
	template <typename TParameterStruct>
	explicit FRDGUniformBuffer(const TParameterStruct* InParameters, const TCHAR* InName)
		: FRDGResource(InName)
		, ParameterStruct(InParameters, TParameterStruct::FTypeInfo::GetStructMetadata())
	{}

private:
	RENDERCORE_API void InitRHI();

	const FRDGParameterStruct ParameterStruct;
	TRefCountPtr<FRHIUniformBuffer> UniformBufferRHI;
	FRDGUniformBufferHandle Handle;
	bool bExternal = false;

	friend FRDGBuilder;
	friend FRDGUniformBufferRegistry;
	friend FRDGAllocator;
};

template <typename ParameterStructType>
class TRDGUniformBuffer : public FRDGUniformBuffer
{
public:
	virtual ~TRDGUniformBuffer() = default;

	inline const TRDGParameterStruct<ParameterStructType>& GetParameters() const
	{
		return static_cast<const TRDGParameterStruct<ParameterStructType>&>(FRDGUniformBuffer::GetParameters());
	}

	inline const ParameterStructType* GetContents() const
	{
		return Parameters;
	}

	inline TUniformBufferRef<ParameterStructType> GetRHIRef() const
	{
		return TUniformBufferRef<ParameterStructType>(GetRHI());
	}

	inline const ParameterStructType* operator->() const
	{
		return Parameters;
	}

private:
	explicit TRDGUniformBuffer(const ParameterStructType* InParameters, const TCHAR* InName)
		: FRDGUniformBuffer(InParameters, InName)
		, Parameters(InParameters)
	{}

	const ParameterStructType* Parameters;

	friend FRDGBuilder;
	friend FRDGUniformBufferRegistry;
	friend FRDGAllocator;
};

/** A render graph resource with an allocation lifetime tracked by the graph. May have child resources which reference it (e.g. views). */
class FRDGViewableResource
	: public FRDGResource
{
public:
#if RHI_USE_RESOURCE_DEBUG_NAME
	virtual ~FRDGViewableResource()
	{
		if (bHeapAllocatedDebugName)
		{
			delete[] Name;
		}
	}
#endif

	/** The type of this resource; useful for casting between types. */
	const ERDGViewableResourceType Type;

	/** Whether this resource is externally registered with the graph (i.e. the user holds a reference to the underlying resource outside the graph). */
	bool IsExternal() const
	{
		return bExternal;
	}

	/** Whether this resource is has been queued for extraction at the end of graph execution. */
	bool IsExtracted() const
	{
		return bExtracted;
	}

	/** Whether a prior pass added to the graph produced contents for this resource. External resources are not considered produced
	 *  until used for a write operation. This is a union of all subresources, so any subresource write will set this to true.
	 */
	bool HasBeenProduced() const
	{
		return bProduced;
	}

	void SetOwnerName(const FName& InOwnerName)
	{
#if RHI_ENABLE_RESOURCE_INFO
		OwnerName = InOwnerName;
#endif
	}

#if RHI_USE_RESOURCE_DEBUG_NAME
	inline void SetDebugNameIsHeapAllocated()
	{
		bHeapAllocatedDebugName = 1;
	}
#endif

	inline bool IsDebugNameHeapAllocated() const
	{
#if RHI_USE_RESOURCE_DEBUG_NAME
		return bHeapAllocatedDebugName;
#else
		return false;
#endif
	}

protected:
	RENDERCORE_API FRDGViewableResource(const TCHAR* InName, ERDGViewableResourceType InType, bool bSkipTracking, bool bImmediateFirstBarrier);

	bool IsCullRoot() const
	{
		return bExternal || bExtracted;
	}

	static const ERHIAccess DefaultEpilogueAccess = ERHIAccess::SRVMask;

	enum class ETransientExtractionHint : uint8
	{
		None,
		Disable,
		Enable
	};

	enum class EAccessMode : uint8
	{
		Internal,
		External
	};

	/** Whether this resource is allocated through the transient resource allocator. */
	uint8 bTransient : 1;

	/** Whether this resource cannot be made transient. */
	uint8 bForceNonTransient : 1;

	/** If true, the graph will skip the last graph transition. Used for aliased pooled resources. */
	uint8 bSkipLastTransition : 1;

	/** If true, this resource should skip the first split barrier and perform transition right away. */
	uint8 bSplitFirstTransition : 1;

	/** If true, the resource has been queued for an upload operation. */
	uint8 bQueuedForUpload : 1;

	/** If false, the resource needs to be collected. */
	uint8 bCollectForAllocate : 1;

#if RHI_USE_RESOURCE_DEBUG_NAME
	/** If true, the debug name passed to the constructor was heap allocated, and needs to be freed in the destructor. */
	uint8 bHeapAllocatedDebugName : 1;
#endif

#if RDG_ENABLE_DEBUG
	uint8 bClobbered : 1;
#endif

	// Use explicit booleans due to access from the render thread while the async setup queue task is running.

	/** Whether any sub-resource has been used for write by a pass. Separate from the bitfield because it is assigned by the async setup queue. */
	bool bProduced = false;

	/** Whether this is an externally registered resource. */
	bool bExternal = false;

	/** Whether this is an extracted resource. */
	bool bExtracted = false;

	/** Whether this resource is allowed to be both transient and extracted. */
	ETransientExtractionHint TransientExtractionHint;

	struct FAccessModeState
	{
		bool IsExternalAccess() const { return ActiveMode == EAccessMode::External; }

		FAccessModeState()
			: Pipelines(ERHIPipeline::None)
			, Mode(EAccessMode::Internal)
			, bLocked(0)
			, bQueued(0)
		{}

		ERHIAccess			Access = ERHIAccess::None;
		ERHIPipeline		Pipelines : 2;
		EAccessMode			Mode : 1;
		uint8				bLocked : 1;
		uint8				bQueued : 1;

		/** The actual access mode replayed on the setup pass timeline. */
		EAccessMode ActiveMode = EAccessMode::Internal;

	} AccessModeState;

	FRDGPassHandle AcquirePass;
	FRDGPassHandle DiscardPass;
	FRDGPassHandle FirstPass;
	FRDGPassHandlesByPipeline LastPasses;

	/** Number of references in passes and deferred queries. */
	uint32 ReferenceCount;

	/** Scratch index allocated for the resource in the pass being setup. */
	uint32 PassStateIndex = 0;

	/** Set of aliasing overlaps to apply to the acquire transition if transient. */
	TArrayView<const FRHITransientAliasingOverlap> AliasingOverlaps;

	/** The state of the resource at the graph epilogue. */
	ERHIAccess EpilogueAccess = DefaultEpilogueAccess;

private:
	static const uint32 DeallocatedReferenceCount = ~0;

	void SetRHI(FRHIResource* Resource)
	{
		check(!ResourceRHI);
		ResourceRHI = Resource;

	#if RHI_ENABLE_RESOURCE_INFO
		ResourceRHI->SetOwnerName(OwnerName);
	#endif
	}

	void SetExternalAccessMode(ERHIAccess InAccess, ERHIPipeline InPipelines)
	{
		check(!AccessModeState.bLocked);

		AccessModeState.Mode = EAccessMode::External;
		AccessModeState.Access = InAccess;
		AccessModeState.Pipelines = InPipelines;

		EpilogueAccess = InAccess;
	}

#if RHI_ENABLE_RESOURCE_INFO
	FName OwnerName;	// For RHI resource tracking
#endif

#if RDG_ENABLE_TRACE
	TArray<FRDGPassHandle, FRDGArrayAllocator> TracePasses;
	uint32 TraceOrder = 0;
#endif

	friend bool IsExtendedLifetimeResource(FRDGViewableResource*);

	friend FRDGBuilder;
	friend FRDGUserValidation;
	friend FRDGBarrierBatchBegin;
	friend FRDGResourceDumpContext;
	friend FRDGTrace;
	friend FRDGPass;
};

/** A render graph resource (e.g. a view) which references a single viewable resource (e.g. a texture / buffer). Provides an abstract way to access the viewable resource. */
class FRDGView
	: public FRDGResource
{
public:
	/** The type of this child resource; useful for casting between types. */
	const ERDGViewType Type;

	/** Returns the referenced parent render graph resource. */
	virtual FRDGViewableResource* GetParent() const = 0;

	ERDGViewableResourceType GetParentType() const
	{
		return ::GetParentType(Type);
	}

	FRDGViewHandle GetHandle() const
	{
		return Handle;
	}

protected:
	FRDGView(const TCHAR* Name, ERDGViewType InType)
		: FRDGResource(Name)
		, Type(InType)
	{}

private:
	FRDGViewHandle Handle;
	FRDGPassHandle LastPass;

	friend FRDGBuilder;
	friend FRDGViewRegistry;
	friend FRDGAllocator;
};

/** Translates from a pooled render target descriptor to an RDG texture descriptor. */
inline FRDGTextureDesc Translate(const FPooledRenderTargetDesc& InDesc);

/** Translates from an RHI/RDG texture descriptor to a pooled render target descriptor. */
inline FPooledRenderTargetDesc Translate(const FRHITextureDesc& InDesc);

class FRDGPooledTexture final
	: public FRefCountedObject
{
public:
	FRDGPooledTexture(FRHITexture* InTexture)
		: Texture(InTexture)
	{
		Fences.Emplace();
	}

	/** Finds a UAV matching the descriptor in the cache or creates a new one and updates the cache. */
	inline FRHIUnorderedAccessView* GetOrCreateUAV(FRHICommandListBase& RHICmdList, const FRHITextureUAVCreateInfo& UAVDesc) { return ViewCache.GetOrCreateUAV(RHICmdList, Texture, UAVDesc); }

	/** Finds a SRV matching the descriptor in the cache or creates a new one and updates the cache. */
	inline FRHIShaderResourceView* GetOrCreateSRV(FRHICommandListBase& RHICmdList, const FRHITextureSRVCreateInfo& SRVDesc) { return ViewCache.GetOrCreateSRV(RHICmdList, Texture, SRVDesc); }

	inline FRHITexture* GetRHI() const { return Texture; }

private:
	TRefCountPtr<FRHITexture> Texture;
	FRHITextureViewCache ViewCache;
	TOptional<FRHITransientAllocationFences> Fences;

	friend FRDGBuilder;
	friend FRenderTargetPool;
};

/** Render graph tracked Texture. */
class FRDGTexture final
	: public FRDGViewableResource
{
public:
	static const ERDGViewableResourceType StaticType = ERDGViewableResourceType::Texture;

	const FRDGTextureDesc Desc;
	const ERDGTextureFlags Flags;

	//////////////////////////////////////////////////////////////////////////
	//! The following methods may only be called during pass execution.

	/** Returns the allocated RHI texture. */
	inline FRHITexture* GetRHI() const
	{
		return static_cast<FRHITexture*>(FRDGResource::GetRHI());
	}

	//////////////////////////////////////////////////////////////////////////

	/** Returns RHI texture without access checks. */
	FRHITexture* GetRHIUnchecked() const
	{
		return static_cast<FRHITexture*>(FRDGResource::GetRHIUnchecked());
	}

	inline FRDGTextureHandle GetHandle() const
	{
		return Handle;
	}

	inline FRDGTextureSubresourceLayout GetSubresourceLayout() const
	{
		return Layout;
	}

	inline FRDGTextureSubresourceRange GetSubresourceRange() const
	{
		return WholeRange;
	}

	inline uint32 GetSubresourceCount() const
	{
		return SubresourceCount;
	}

	inline FRDGTextureSubresource GetSubresource(uint32 SubresourceIndex) const
	{
		return Layout.GetSubresource(SubresourceIndex);
	}

	RENDERCORE_API FRDGTextureSubresourceRange GetSubresourceRangeSRV() const;

	bool IsCulled() const
	{
		return ReferenceCount == 0;
	}

private:
	RENDERCORE_API FRDGTexture(const TCHAR* InName, const FRDGTextureDesc& InDesc, ERDGTextureFlags InFlags);

	/** The handle registered with the builder. */
	FRDGTextureHandle Handle;

	/** The previous / next texture to own the PooledTexture allocation during execution. */
	FRDGTextureHandle PreviousOwner;
	FRDGTextureHandle NextOwner;

	/** The layout used to facilitate subresource transitions. */
	FRDGTextureSubresourceLayout Layout;
	FRDGTextureSubresourceRange  WholeRange;
	const uint16 SubresourceCount;

	/** Tracks subresource states as the graph is built. */
	FRDGTextureSubresourceState State;

	/** Tracks the first state in the graph for each subresource. */
	FRDGTextureSubresourceState FirstState;

	/** Tracks merged subresource states as the graph is built. */
	FRDGTextureSubresourceState MergeState;

	/** Tracks pass producers for each subresource as the graph is built. */
	TRDGTextureSubresourceArray<FRDGProducerStatesByPipeline, FRDGArrayAllocator> LastProducers;

	/** The assigned render target to use during execution. Never reset. */
	IPooledRenderTarget* RenderTarget = nullptr;

	/** The assigned transient texture to use during execution. Never reset. */
	FRHITransientTexture* TransientTexture = nullptr;

	/** The assigned view cache for this texture (sourced from transient / pooled texture). Never reset. */
	FRHITextureViewCache* ViewCache = nullptr;

	/** Valid strictly when holding a strong reference; use PooledRenderTarget instead. */
	TRefCountPtr<IPooledRenderTarget> Allocation;

#if SUPPORTS_VISUALIZE_TEXTURE
	FIntPoint VisualizeTextureExtent = { 0, 0 };
public:
	inline void EncloseVisualizeExtent(const FIntPoint& Point)
	{
		VisualizeTextureExtent.X = FMath::Max(VisualizeTextureExtent.X, Point.X);
		VisualizeTextureExtent.Y = FMath::Max(VisualizeTextureExtent.Y, Point.Y);
	}
	inline FIntPoint GetVisualizeExtent() const
	{
		return VisualizeTextureExtent;
	}
private:
#else
public:
	inline void EncloseVisualizeExtent(const FIntPoint& Point) {}
private:
#endif

	friend FRDGBuilder;
	friend FRDGUserValidation;
	friend FRDGBarrierValidation;
	friend FRDGTextureRegistry;
	friend FRDGAllocator;
	friend FPooledRenderTarget;
	friend FRDGTrace;
	friend FRDGTextureUAV;
};

/** Render graph tracked SRV. */
class FRDGShaderResourceView
	: public FRDGView
{
public:
	/** Returns the allocated RHI SRV. */
	FRHIShaderResourceView* GetRHI() const
	{
		return static_cast<FRHIShaderResourceView*>(FRDGResource::GetRHI());
	}

protected:
	FRDGShaderResourceView(const TCHAR* InName, ERDGViewType InType)
		: FRDGView(InName, InType)
	{}

	/** Returns the allocated RHI SRV without access checks. */
	FRHIShaderResourceView* GetRHIUnchecked() const
	{
		return static_cast<FRHIShaderResourceView*>(FRDGResource::GetRHIUnchecked());
	}
};

/** Render graph tracked UAV. */
class FRDGUnorderedAccessView
	: public FRDGView
{
public:
	const ERDGUnorderedAccessViewFlags Flags;

	/** Returns the allocated RHI UAV. */
	FRHIUnorderedAccessView* GetRHI() const
	{
		return static_cast<FRHIUnorderedAccessView*>(FRDGResource::GetRHI());
	}

protected:
	FRDGUnorderedAccessView(const TCHAR* InName, ERDGViewType InType, ERDGUnorderedAccessViewFlags InFlags)
		: FRDGView(InName, InType)
		, Flags(InFlags)
	{}

	/** Returns the allocated RHI UAV without access checks. */
	FRHIUnorderedAccessView* GetRHIUnchecked() const
	{
		return static_cast<FRHIUnorderedAccessView*>(FRDGResource::GetRHIUnchecked());
	}
};

/** Descriptor for render graph tracked SRV. */
class FRDGTextureSRVDesc final
	: public FRHITextureSRVCreateInfo
{
public:
	FRDGTextureSRVDesc() = default;

	FRDGTextureSRVDesc(FRDGTexture* InTexture)
	{
		Texture = InTexture;
		NumMipLevels = InTexture->Desc.NumMips;
		if (InTexture->Desc.IsTextureArray())
		{
			NumArraySlices = InTexture->Desc.ArraySize;
		}
	}

	/** Create SRV that access all sub resources of texture. */
	static FRDGTextureSRVDesc Create(FRDGTextureRef Texture)
	{
		return FRDGTextureSRVDesc(Texture);
	}

	/** Create SRV that access one specific mip level. */
	static FRDGTextureSRVDesc CreateForMipLevel(FRDGTextureRef Texture, int32 MipLevel)
	{
		check(MipLevel >= -1 && MipLevel <= TNumericLimits<int8>::Max());
		FRDGTextureSRVDesc Desc = FRDGTextureSRVDesc::Create(Texture);
		Desc.MipLevel = (int8)MipLevel;
		Desc.NumMipLevels = 1;
		return Desc;
	}

	/** Create SRV that access one specific slice. */
	static FRDGTextureSRVDesc CreateForSlice(FRDGTextureRef Texture, int32 SliceIndex)
	{
		check(Texture);
        check(Texture->Desc.Dimension == ETextureDimension::Texture2DArray);
		check(SliceIndex >= 0 && SliceIndex < Texture->Desc.ArraySize);
        
		FRDGTextureSRVDesc Desc = FRDGTextureSRVDesc::Create(Texture);
		Desc.FirstArraySlice = (uint16)SliceIndex;
		Desc.NumArraySlices = 1;
        Desc.DimensionOverride = ETextureDimension::Texture2D;
        
		return Desc;
	}

	/** Create SRV that access all sub resources of texture with a specific pixel format. */
	static FRDGTextureSRVDesc CreateWithPixelFormat(FRDGTextureRef Texture, EPixelFormat PixelFormat)
	{
		FRDGTextureSRVDesc Desc = FRDGTextureSRVDesc::Create(Texture);
		Desc.Format = PixelFormat;
		return Desc;
	}

	/** Create SRV with access to a specific meta data plane */
	static FRDGTextureSRVDesc CreateForMetaData(FRDGTextureRef Texture, ERDGTextureMetaDataAccess MetaData)
	{
		FRDGTextureSRVDesc Desc = FRDGTextureSRVDesc::Create(Texture);
		Desc.MetaData = MetaData;
		return Desc;
	}

	bool operator == (const FRDGTextureSRVDesc& Other) const
	{
		return Texture == Other.Texture && FRHITextureSRVCreateInfo::operator==(Other);
	}

	bool operator != (const FRDGTextureSRVDesc& Other) const
	{
		return !(*this == Other);
	}

	friend uint32 GetTypeHash(const FRDGTextureSRVDesc& Desc)
	{
		return HashCombine(GetTypeHash(static_cast<const FRHITextureSRVCreateInfo&>(Desc)), GetTypeHash(Desc.Texture));
	}

	/** Returns whether this descriptor conforms to requirements. */
	bool IsValid() const
	{
		if (!Texture)
		{
			return false;
		}
		return FRHITextureSRVCreateInfo::Validate(Texture->Desc, *this, Texture->Name, /* bFatal = */ false);
	}

	FRDGTextureRef Texture = nullptr;
};

/** Render graph tracked SRV. */
class FRDGTextureSRV final
	: public FRDGShaderResourceView
{
public:
	static const ERDGViewType StaticType = ERDGViewType::TextureSRV;

	/** Descriptor of the graph tracked SRV. */
	const FRDGTextureSRVDesc Desc;

	FRDGTextureRef GetParent() const override
	{
		return Desc.Texture;
	}

	FRDGTextureSubresourceRange GetSubresourceRange() const;

private:
	FRDGTextureSRV(const TCHAR* InName, const FRDGTextureSRVDesc& InDesc)
		: FRDGShaderResourceView(InName, ERDGViewType::TextureSRV)
		, Desc(InDesc)
	{}

	friend FRDGBuilder;
	friend FRDGViewRegistry;
	friend FRDGAllocator;
};

/** Descriptor for render graph tracked UAV. */
class FRDGTextureUAVDesc final
	: public FRHITextureUAVCreateInfo
{
public:
	FRDGTextureUAVDesc() = default;

	FRDGTextureUAVDesc(FRDGTextureRef InTexture, uint8 InMipLevel = 0, EPixelFormat InFormat = PF_Unknown, uint16 InFirstArraySlice = 0, uint16 InNumArraySlices = 0)
		: FRHITextureUAVCreateInfo(InMipLevel, InFormat != PF_Unknown ? InFormat : InTexture->Desc.UAVFormat, InFirstArraySlice, InNumArraySlices)
		, Texture(InTexture)
	{}

	/** Create UAV with access to a specific meta data plane */
	static FRDGTextureUAVDesc CreateForMetaData(FRDGTextureRef Texture, ERDGTextureMetaDataAccess MetaData)
	{
		FRDGTextureUAVDesc Desc = FRDGTextureUAVDesc(Texture, 0);
		Desc.MetaData = MetaData;
		return Desc;
	}

	bool operator == (const FRDGTextureUAVDesc& Other) const
	{
		return Texture == Other.Texture && FRHITextureUAVCreateInfo::operator==(Other);
	}

	bool operator != (const FRDGTextureUAVDesc& Other) const
	{
		return !(*this == Other);
	}

	friend uint32 GetTypeHash(const FRDGTextureUAVDesc& Desc)
	{
		return HashCombine(GetTypeHash(static_cast<const FRHITextureUAVCreateInfo&>(Desc)), GetTypeHash(Desc.Texture));
	}

	FRDGTextureRef Texture = nullptr;
};

/** Render graph tracked texture UAV. */
class FRDGTextureUAV final
	: public FRDGUnorderedAccessView
{
public:
	static const ERDGViewType StaticType = ERDGViewType::TextureUAV;

	/** Descriptor of the graph tracked UAV. */
	const FRDGTextureUAVDesc Desc;

	FRDGTextureRef GetParent() const override
	{
		return Desc.Texture;
	}

	// Can be used instead of GetParent()->GetRHI() to access the underlying texture for a UAV resource in a Pass, without triggering
	// validation errors.  The RDG validation logic only flags the UAV as accessible, not the parent texture.
	FRHITexture* GetParentRHI() const
	{
		IF_RDG_ENABLE_DEBUG(ValidateRHIAccess());
		return Desc.Texture->GetRHIUnchecked();
	}

	FRDGTextureSubresourceRange GetSubresourceRange() const;

private:
	FRDGTextureUAV(const TCHAR* InName, const FRDGTextureUAVDesc& InDesc, ERDGUnorderedAccessViewFlags InFlags)
		: FRDGUnorderedAccessView(InName, ERDGViewType::TextureUAV, InFlags)
		, Desc(InDesc)
	{}

	friend FRDGBuilder;
	friend FRDGViewRegistry;
	friend FRDGAllocator;
};

/** Descriptor for render graph tracked Buffer. */
struct FRDGBufferDesc
{
	static FRDGBufferDesc CreateByteAddressDesc(uint32 NumBytes)
	{
		check(NumBytes % 4 == 0);
		FRDGBufferDesc Desc;
		Desc.Usage = EBufferUsageFlags::Static | EBufferUsageFlags::UnorderedAccess | EBufferUsageFlags::ShaderResource | EBufferUsageFlags::StructuredBuffer | EBufferUsageFlags::ByteAddressBuffer;
		Desc.BytesPerElement = 4;
		Desc.NumElements = NumBytes / 4;
		return Desc;
	}

	template<typename ParameterStruct>
	static FRDGBufferDesc CreateByteAddressDesc(uint32 NumElements)
	{
		FRDGBufferDesc Desc = CreateByteAddressDesc(sizeof(ParameterStruct) * NumElements);
		Desc.Metadata = ParameterStruct::FTypeInfo::GetStructMetadata();
		return Desc;
	}

	static FRDGBufferDesc CreateIndirectDesc(uint32 BytesPerElement, uint32 NumElements)
	{
		FRDGBufferDesc Desc;
		Desc.Usage = EBufferUsageFlags::Static | EBufferUsageFlags::DrawIndirect | EBufferUsageFlags::UnorderedAccess | EBufferUsageFlags::ShaderResource | EBufferUsageFlags::VertexBuffer;
		Desc.BytesPerElement = BytesPerElement;
		Desc.NumElements = NumElements;
		return Desc;
	}

	static FRDGBufferDesc CreateRawIndirectDesc(uint32 NumBytes)
	{
		FRDGBufferDesc Desc = CreateByteAddressDesc(NumBytes);
		Desc.Usage |=  EBufferUsageFlags::DrawIndirect;
		return Desc;
	}

	/** Create the descriptor for an indirect RHI call.
	 *
	 * Note, IndirectParameterStruct should be one of the:
	 *		struct FRHIDispatchIndirectParameters
	 *		struct FRHIDrawIndirectParameters
	 *		struct FRHIDrawIndexedIndirectParameters
	 */
	template<typename IndirectParameterStruct>
	static FRDGBufferDesc CreateIndirectDesc(uint32 NumElements = 1)
	{
		return CreateIndirectDesc(sizeof(IndirectParameterStruct), NumElements);
	}

	static FRDGBufferDesc CreateIndirectDesc(uint32 NumElements = 1)
	{
		return CreateIndirectDesc(4u, NumElements);
	}

	static FRDGBufferDesc CreateStructuredDesc(uint32 BytesPerElement, uint32 NumElements)
	{
		FRDGBufferDesc Desc;
		Desc.Usage = EBufferUsageFlags::Static | EBufferUsageFlags::UnorderedAccess | EBufferUsageFlags::ShaderResource | EBufferUsageFlags::StructuredBuffer;
		Desc.BytesPerElement = BytesPerElement;
		Desc.NumElements = NumElements;
		return Desc;
	}

	template<typename ParameterStruct>
	static FRDGBufferDesc CreateStructuredDesc(uint32 NumElements)
	{
		FRDGBufferDesc Desc = CreateStructuredDesc(sizeof(ParameterStruct), NumElements);
		Desc.Metadata = ParameterStruct::FTypeInfo::GetStructMetadata();
		return Desc;
	}

	static FRDGBufferDesc CreateBufferDesc(uint32 BytesPerElement, uint32 NumElements)
	{
		FRDGBufferDesc Desc;
		Desc.Usage = EBufferUsageFlags::Static | EBufferUsageFlags::UnorderedAccess | EBufferUsageFlags::ShaderResource | EBufferUsageFlags::VertexBuffer;
		Desc.BytesPerElement = BytesPerElement;
		Desc.NumElements = NumElements;
		return Desc;
	}

	template<typename ParameterStruct>
	static FRDGBufferDesc CreateBufferDesc(uint32 NumElements)
	{
		FRDGBufferDesc Desc = CreateBufferDesc(sizeof(ParameterStruct), NumElements);
		Desc.Metadata = ParameterStruct::FTypeInfo::GetStructMetadata();
		return Desc;
	}

	static FRDGBufferDesc CreateUploadDesc(uint32 BytesPerElement, uint32 NumElements)
	{
		FRDGBufferDesc Desc;
		Desc.Usage = EBufferUsageFlags::ShaderResource | EBufferUsageFlags::VertexBuffer;
		Desc.BytesPerElement = BytesPerElement;
		Desc.NumElements = NumElements;
		return Desc;
	}

	template<typename ParameterStruct>
	static FRDGBufferDesc CreateUploadDesc(uint32 NumElements)
	{
		FRDGBufferDesc Desc = CreateUploadDesc(sizeof(ParameterStruct), NumElements);
		Desc.Metadata = ParameterStruct::FTypeInfo::GetStructMetadata();
		return Desc;
	}

	static FRDGBufferDesc CreateStructuredUploadDesc(uint32 BytesPerElement, uint32 NumElements)
	{
		FRDGBufferDesc Desc;
		Desc.Usage = EBufferUsageFlags::ShaderResource | EBufferUsageFlags::StructuredBuffer;
		Desc.BytesPerElement = BytesPerElement;
		Desc.NumElements = NumElements;
		return Desc;
	}

	template<typename ParameterStruct>
	static FRDGBufferDesc CreateStructuredUploadDesc(uint32 NumElements)
	{
		FRDGBufferDesc Desc = CreateStructuredUploadDesc(sizeof(ParameterStruct), NumElements);
		Desc.Metadata = ParameterStruct::FTypeInfo::GetStructMetadata();
		return Desc;
	}

	static FRDGBufferDesc CreateByteAddressUploadDesc(uint32 NumBytes)
	{
		check(NumBytes % 4 == 0);
		FRDGBufferDesc Desc;
		Desc.Usage = EBufferUsageFlags::ShaderResource | EBufferUsageFlags::ByteAddressBuffer | EBufferUsageFlags::StructuredBuffer;
		Desc.BytesPerElement = 4;
		Desc.NumElements = NumBytes / 4;
		return Desc;
	}

	template<typename ParameterStruct>
	static FRDGBufferDesc CreateByteAddressUploadDesc(uint32 NumElements)
	{
		FRDGBufferDesc Desc = CreateByteAddressUploadDesc(sizeof(ParameterStruct) * NumElements);
		Desc.Metadata = ParameterStruct::FTypeInfo::GetStructMetadata();
		return Desc;
	}

	/** Returns the total number of bytes allocated for a such buffer. */
	uint32 GetSize() const
	{
		return BytesPerElement * NumElements;
	}

	friend uint32 GetTypeHash(const FRDGBufferDesc& Desc)
	{
		uint32 Hash = GetTypeHash(Desc.BytesPerElement);
		Hash = HashCombine(Hash, GetTypeHash(Desc.NumElements));
		Hash = HashCombine(Hash, GetTypeHash(Desc.Usage));
		Hash = HashCombine(Hash, GetTypeHash(Desc.Metadata));
		return Hash;
	}

	bool operator == (const FRDGBufferDesc& Other) const
	{
		return
			BytesPerElement == Other.BytesPerElement &&
			NumElements == Other.NumElements &&
			Usage == Other.Usage;
	}

	bool operator != (const FRDGBufferDesc& Other) const
	{
		return !(*this == Other);
	}

	/** Stride in bytes for index and structured buffers. */
	uint32 BytesPerElement = 1;

	/** Number of elements. */
	uint32 NumElements = 1;

	/** Bitfields describing the uses of that buffer. */
	EBufferUsageFlags Usage = EBufferUsageFlags::None;

	/** Meta data of the layout of the buffer for debugging purposes. */
	const FShaderParametersMetadata* Metadata = nullptr;
};

struct FRDGBufferSRVDesc final
	: public FRHIBufferSRVCreateInfo
{
	FRDGBufferSRVDesc() = default;

	FRDGBufferSRVDesc(FRDGBufferRef InBuffer);

	FRDGBufferSRVDesc(FRDGBufferRef InBuffer, EPixelFormat InFormat)
		: FRHIBufferSRVCreateInfo(InFormat)
		, Buffer(InBuffer)
	{
	}

	FRDGBufferSRVDesc(FRDGBufferRef InBuffer, uint32 InStartOffsetBytes, uint32 InNumElements)
		: FRHIBufferSRVCreateInfo(InStartOffsetBytes, InNumElements)
		, Buffer(InBuffer)
	{}

	FRDGBufferSRVDesc(FRDGBufferRef InBuffer, FRHIRayTracingScene* InRayTracingScene, uint32 InStartOffsetBytes)
		: FRHIBufferSRVCreateInfo(InRayTracingScene, InStartOffsetBytes)
		, Buffer(InBuffer)
	{}

	bool operator == (const FRDGBufferSRVDesc& Other) const
	{
		return Buffer == Other.Buffer && FRHIBufferSRVCreateInfo::operator==(Other);
	}

	bool operator != (const FRDGBufferSRVDesc& Other) const
	{
		return !(*this == Other);
	}

	friend uint32 GetTypeHash(const FRDGBufferSRVDesc& Desc)
	{
		return HashCombine(GetTypeHash(static_cast<const FRHIBufferSRVCreateInfo&>(Desc)), GetTypeHash(Desc.Buffer));
	}

	FRDGBufferRef Buffer = nullptr;
};

struct FRDGBufferUAVDesc final
	: public FRHIBufferUAVCreateInfo
{
	FRDGBufferUAVDesc() = default;

	FRDGBufferUAVDesc(FRDGBufferRef InBuffer);

	FRDGBufferUAVDesc(FRDGBufferRef InBuffer, EPixelFormat InFormat)
		: FRHIBufferUAVCreateInfo(InFormat)
		, Buffer(InBuffer)
	{}

	bool operator == (const FRDGBufferUAVDesc& Other) const
	{
		return Buffer == Other.Buffer && FRHIBufferUAVCreateInfo::operator==(Other);
	}

	bool operator != (const FRDGBufferUAVDesc& Other) const
	{
		return !(*this == Other);
	}

	friend uint32 GetTypeHash(const FRDGBufferUAVDesc& Desc)
	{
		return HashCombine(GetTypeHash(static_cast<const FRHIBufferUAVCreateInfo&>(Desc)), GetTypeHash(Desc.Buffer));
	}

	FRDGBufferRef Buffer = nullptr;
};

/** Translates from a RDG buffer descriptor to a RHI buffer creation info */
inline FRHIBufferCreateInfo Translate(const FRDGBufferDesc& InDesc);

class FRDGPooledBuffer final
	: public UE::Private::FQueryableRefCountedObject
{
public:
	FRDGPooledBuffer(FRHICommandListBase& RHICmdList, TRefCountPtr<FRHIBuffer> InBuffer, const FRDGBufferDesc& InDesc, uint32 InNumAllocatedElements, const TCHAR* InName)
		: Desc(InDesc)
		, Buffer(MoveTemp(InBuffer))
		, Name(InName)
		, NumAllocatedElements(InNumAllocatedElements)
	{
		if (EnumHasAnyFlags(InDesc.Usage, EBufferUsageFlags::StructuredBuffer | EBufferUsageFlags::ByteAddressBuffer))
		{
			CachedSRV = GetOrCreateSRV(RHICmdList, FRHIBufferSRVCreateInfo());
		}

		Fences.Emplace();
	}

	RENDERCORE_API FRDGPooledBuffer(TRefCountPtr<FRHIBuffer> InBuffer, const FRDGBufferDesc& InDesc, uint32 InNumAllocatedElements, const TCHAR* InName);

	const FRDGBufferDesc Desc;

	/** Finds a UAV matching the descriptor in the cache or creates a new one and updates the cache. */
	inline FRHIUnorderedAccessView* GetOrCreateUAV(FRHICommandListBase& RHICmdList, const FRHIBufferUAVCreateInfo& UAVDesc) { return ViewCache.GetOrCreateUAV(RHICmdList, Buffer, UAVDesc); }

	/** Finds a SRV matching the descriptor in the cache or creates a new one and updates the cache. */
	inline FRHIShaderResourceView* GetOrCreateSRV(FRHICommandListBase& RHICmdList, const FRHIBufferSRVCreateInfo& SRVDesc) { return ViewCache.GetOrCreateSRV(RHICmdList, Buffer, SRVDesc); }

	/** Returns the RHI buffer. */
	inline FRHIBuffer* GetRHI() const { return Buffer; }

	/** Returns the default SRV. */
	inline FRHIShaderResourceView* GetSRV()
	{
		checkf(CachedSRV, TEXT("Only byte address and structured buffers can use the default GetSRV call"));
		return CachedSRV;
	}

	inline FRHIShaderResourceView* GetSRV(FRHICommandListBase& RHICmdList, const FRHIBufferSRVCreateInfo& SRVDesc)
	{
		return GetOrCreateSRV(RHICmdList, SRVDesc);
	}

	inline uint32 GetSize() const
	{
		return Desc.GetSize();
	}

	inline uint32 GetAlignedSize() const
	{
		return Desc.BytesPerElement * NumAllocatedElements;
	}

	inline uint64 GetCommittedSize() const
	{
		return FMath::Min<uint64>(CommittedSizeInBytes, GetSize());
	}

	const TCHAR* GetName() const
	{
		return Name;
	}

private:
	TRefCountPtr<FRHIBuffer> Buffer;
	FRHIShaderResourceView* CachedSRV = nullptr;
	FRHIBufferViewCache ViewCache;

	FRDGBufferDesc GetAlignedDesc() const
	{
		FRDGBufferDesc AlignedDesc = Desc;
		AlignedDesc.NumElements = NumAllocatedElements;
		return AlignedDesc;
	}

	void SetDebugLabelName(FRHICommandListBase& RHICmdList, const TCHAR* InName);

	// Used internally by FRDGBuilder::QueueCommitReservedBuffer(),
	// which is expected to be the only way to resize physical memory for FRDGPooledBuffer
	void SetCommittedSize(uint64 InCommittedSizeInBytes)
	{
		if (InCommittedSizeInBytes == UINT64_MAX)
		{
			InCommittedSizeInBytes = GetSize();
		}

		checkf(EnumHasAllFlags(Desc.Usage, EBufferUsageFlags::ReservedResource), TEXT("CommitReservedResource() may only be used on reserved buffers"));
		checkf(InCommittedSizeInBytes <= GetSize(), TEXT("Attempting to commit more memory than was reserved for this buffer during creation"));

		CommittedSizeInBytes = InCommittedSizeInBytes;
	}

	const TCHAR* Name = nullptr;

	// Size of the GPU physical memory committed to a reserved buffer.
	// May be UINT64_MAX for regular (non-reserved) buffers or when the entire resource is committed.
	uint64 CommittedSizeInBytes = UINT64_MAX;

#if RHI_USE_RESOURCE_DEBUG_NAME
	// Tracks the number of items in ViewCache when the debug Name was updated, so we know if items were added and we need to propagate the debug name to the new items
	int32 NameUpdatedViewCacheNum = 0;
#endif

	const uint32 NumAllocatedElements;
	uint32 LastUsedFrame = 0;

	TOptional<FRHITransientAllocationFences> Fences;

	friend FRDGBuilder;
	friend FRDGBufferPool;
};

/** A render graph tracked buffer. */
class FRDGBuffer final
	: public FRDGViewableResource
{
public:
	static const ERDGViewableResourceType StaticType = ERDGViewableResourceType::Buffer;

	FRDGBufferDesc Desc;
	const ERDGBufferFlags Flags;

	//////////////////////////////////////////////////////////////////////////
	//! The following methods may only be called during pass execution.

	/** Returns the underlying RHI buffer resource */
	FRHIBuffer* GetRHI() const
	{
		return static_cast<FRHIBuffer*>(FRDGViewableResource::GetRHI());
	}

	/** Returns the buffer to use for indirect RHI calls. */
	inline FRHIBuffer* GetIndirectRHICallBuffer() const
	{
		checkf(Desc.Usage & BUF_DrawIndirect, TEXT("Buffer %s was not flagged for indirect draw usage."), Name);
		return GetRHI();
	}

	//////////////////////////////////////////////////////////////////////////

	FRHIBuffer* GetRHIUnchecked() const
	{
		return static_cast<FRHIBuffer*>(FRDGResource::GetRHIUnchecked());
	}

	FRDGBufferHandle GetHandle() const
	{
		return Handle;
	}

	inline uint32 GetSize() const
	{
		return Desc.GetSize();
	}

	inline uint32 GetStride() const
	{
		return Desc.BytesPerElement;
	}

	inline uint64 GetCommittedSize() const
	{
		return PooledBuffer ? PooledBuffer->GetCommittedSize() : 0;
	}

	bool IsCulled() const
	{
		return ReferenceCount == 0 && PendingCommitSize == 0;
	}

private:
	RENDERCORE_API FRDGBuffer(const TCHAR* InName, const FRDGBufferDesc& InDesc, ERDGBufferFlags InFlags);
	RENDERCORE_API FRDGBuffer(const TCHAR* InName, const FRDGBufferDesc& InDesc, ERDGBufferFlags InFlags, FRDGBufferNumElementsCallback* InNumElementsCallback);

	/** Finalizes any pending field of the buffer descriptor. */
	RENDERCORE_API void FinalizeDesc();

	/** Registered handle set by the builder. */
	FRDGBufferHandle Handle;

	/** The previous / next buffer to own the PooledBuffer allocation during execution. */
	FRDGBufferHandle PreviousOwner;
	FRDGBufferHandle NextOwner;

	/** Assigned pooled buffer pointer. Never reset once assigned. */
	FRDGPooledBuffer* PooledBuffer = nullptr;

	/** Assigned transient buffer pointer. Never reset once assigned. */
	FRHITransientBuffer* TransientBuffer = nullptr;

	/** The assigned buffer view cache (sourced from the pooled / transient buffer. Never reset. */
	FRHIBufferViewCache* ViewCache = nullptr;

	/** Valid strictly when holding a strong reference; use PooledBuffer instead. */
	TRefCountPtr<FRDGPooledBuffer> Allocation;

	/** Tracks the last pass that produced this resource as the graph is built. */
	FRDGProducerStatesByPipeline LastProducer;

	/** Optional callback to supply NumElements after the creation of this FRDGBuffer. */
	FRDGBufferNumElementsCallback* NumElementsCallback = nullptr;

	/** Optional reserved resource commit size to apply on the first resource transition. */
	uint64 PendingCommitSize = 0;

	/** Cached state pointer from the pooled / transient buffer. */
	FRDGSubresourceState* State = nullptr;

	/** Tracks the first state in the graph for this buffer. */
	FRDGSubresourceState* FirstState = nullptr;

	/** Tracks the merged subresource state as the graph is built. */
	FRDGSubresourceState* MergeState = nullptr;

	friend FRDGBuilder;
	friend FRDGBarrierValidation;
	friend FRDGUserValidation;
	friend FRDGBufferRegistry;
	friend FRDGAllocator;
	friend FRDGTrace;
};

/** Render graph tracked buffer SRV. */
class FRDGBufferSRV final
	: public FRDGShaderResourceView
{
public:
	static const ERDGViewType StaticType = ERDGViewType::BufferSRV;

	/** Descriptor of the graph tracked SRV. */
	const FRDGBufferSRVDesc Desc;

	FRDGBufferRef GetParent() const override
	{
		return Desc.Buffer;
	}

private:
	FRDGBufferSRV(const TCHAR* InName, const FRDGBufferSRVDesc& InDesc)
		: FRDGShaderResourceView(InName, ERDGViewType::BufferSRV)
		, Desc(InDesc)
	{}

	friend FRDGBuilder;
	friend FRDGViewRegistry;
	friend FRDGAllocator;
};

/** Render graph tracked buffer UAV. */
class FRDGBufferUAV final
	: public FRDGUnorderedAccessView
{
public:
	static const ERDGViewType StaticType = ERDGViewType::BufferUAV;

	/** Descriptor of the graph tracked UAV. */
	const FRDGBufferUAVDesc Desc;

	FRDGBufferRef GetParent() const override
	{
		return Desc.Buffer;
	}

private:
	FRDGBufferUAV(const TCHAR* InName, const FRDGBufferUAVDesc& InDesc, ERDGUnorderedAccessViewFlags InFlags)
		: FRDGUnorderedAccessView(InName, ERDGViewType::BufferUAV, InFlags)
		, Desc(InDesc)
	{}

	friend FRDGBuilder;
	friend FRDGViewRegistry;
	friend FRDGAllocator;
};

template <typename ViewableResourceType>
inline ViewableResourceType* GetAs(FRDGViewableResource* Resource)
{
	check(ViewableResourceType::StaticType == Resource->Type);
	return static_cast<ViewableResourceType*>(Resource);
}

template <typename ViewType>
inline ViewType* GetAs(FRDGView* View)
{
	check(ViewType::StaticType == View->Type);
	return static_cast<ViewType*>(View);
}

inline FRDGBuffer* GetAsBuffer(FRDGViewableResource* Resource)
{
	return GetAs<FRDGBuffer>(Resource);
}

inline FRDGTexture* GetAsTexture(FRDGViewableResource* Resource)
{
	return GetAs<FRDGTexture>(Resource);
}

inline FRDGBufferUAV* GetAsBufferUAV(FRDGView* View)
{
	return GetAs<FRDGBufferUAV>(View);
}

inline FRDGBufferSRV* GetAsBufferSRV(FRDGView* View)
{
	return GetAs<FRDGBufferSRV>(View);
}

inline FRDGTextureUAV* GetAsTextureUAV(FRDGView* View)
{
	return GetAs<FRDGTextureUAV>(View);
}

inline FRDGTextureSRV* GetAsTextureSRV(FRDGView* View)
{
	return GetAs<FRDGTextureSRV>(View);
}

inline FGraphicsPipelineRenderTargetsInfo ExtractRenderTargetsInfo(const FRDGParameterStruct& ParameterStruct);
inline FGraphicsPipelineRenderTargetsInfo ExtractRenderTargetsInfo(const FRenderTargetBindingSlots& RenderTargets);

#include "RenderGraphResources.inl" // IWYU pragma: export
