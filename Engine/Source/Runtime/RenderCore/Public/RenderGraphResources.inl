// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

PGO_LINK_DISABLE_WARNINGS

inline void FRDGSubresourceState::SetPass(ERHIPipeline Pipeline, FRDGPassHandle PassHandle)
{
	FirstPass = {};
	LastPass = {};
	FirstPass[Pipeline] = PassHandle;
	LastPass[Pipeline] = PassHandle;
}

inline void FRDGSubresourceState::Validate()
{
#if RDG_ENABLE_DEBUG
	for (ERHIPipeline Pipeline : MakeFlagsRange(ERHIPipeline::All))
	{
		checkf(FirstPass[Pipeline].IsValid() == LastPass[Pipeline].IsValid(), TEXT("Subresource state has unset first or last pass on '%s."), *GetRHIPipelineName(Pipeline));
	}
#endif
}

inline bool FRDGSubresourceState::IsUsedBy(ERHIPipeline Pipeline) const
{
	check(FirstPass[Pipeline].IsValid() == LastPass[Pipeline].IsValid());
	return FirstPass[Pipeline].IsValid();
}

inline FRDGPassHandle FRDGSubresourceState::GetLastPass() const
{
	return FRDGPassHandle::Max(LastPass[ERHIPipeline::Graphics], LastPass[ERHIPipeline::AsyncCompute]);
}

inline FRDGPassHandle FRDGSubresourceState::GetFirstPass() const
{
	return FRDGPassHandle::Min(FirstPass[ERHIPipeline::Graphics], FirstPass[ERHIPipeline::AsyncCompute]);
}

inline ERHIPipeline FRDGSubresourceState::GetPipelines() const
{
	ERHIPipeline Pipelines = ERHIPipeline::None;
	Pipelines |= FirstPass[ERHIPipeline::Graphics].IsValid()     ? ERHIPipeline::Graphics     : ERHIPipeline::None;
	Pipelines |= FirstPass[ERHIPipeline::AsyncCompute].IsValid() ? ERHIPipeline::AsyncCompute : ERHIPipeline::None;
	return Pipelines;
}

inline FPooledRenderTargetDesc Translate(const FRHITextureDesc& InDesc)
{
	check(InDesc.IsValid());

	FPooledRenderTargetDesc OutDesc;
	OutDesc.ClearValue = InDesc.ClearValue;
	OutDesc.Flags = InDesc.Flags;
	OutDesc.Format = InDesc.Format;
	OutDesc.UAVFormat = InDesc.UAVFormat;
	OutDesc.Extent.X = InDesc.Extent.X;
	OutDesc.Extent.Y = InDesc.Extent.Y;
	OutDesc.Depth = InDesc.Dimension == ETextureDimension::Texture3D ? InDesc.Depth : 0;
	OutDesc.ArraySize = InDesc.ArraySize;
	OutDesc.NumMips = InDesc.NumMips;
	OutDesc.NumSamples = InDesc.NumSamples;
	OutDesc.bIsArray = InDesc.IsTextureArray();
	OutDesc.bIsCubemap = InDesc.IsTextureCube();
	OutDesc.FastVRAMPercentage = InDesc.FastVRAMPercentage;
	OutDesc.AliasableFormats = InDesc.AliasableFormats;

	check(OutDesc.IsValid());
	return OutDesc;
}

inline FRHIBufferCreateInfo Translate(const FRDGBufferDesc& InDesc)
{
	FRHIBufferCreateInfo CreateInfo;
	CreateInfo.Size = InDesc.GetSize();

	if (EnumHasAnyFlags(InDesc.Usage, EBufferUsageFlags::VertexBuffer))
	{
		CreateInfo.Stride = 0;
		CreateInfo.Usage = InDesc.Usage | BUF_VertexBuffer;
	}
	else if (EnumHasAnyFlags(InDesc.Usage, EBufferUsageFlags::StructuredBuffer))
	{
		CreateInfo.Stride = InDesc.BytesPerElement;
		CreateInfo.Usage = InDesc.Usage | BUF_StructuredBuffer;
	}
	else
	{
		check(0);
	}

	return CreateInfo;
}

inline FRDGTextureDesc Translate(const FPooledRenderTargetDesc& InDesc)
{
	check(InDesc.IsValid());

	FRDGTextureDesc OutDesc;
	OutDesc.ClearValue = InDesc.ClearValue;
	OutDesc.Format = InDesc.Format;
	OutDesc.UAVFormat = InDesc.UAVFormat;
	OutDesc.Extent = InDesc.Extent;
	OutDesc.ArraySize = InDesc.ArraySize;
	OutDesc.NumMips = InDesc.NumMips;
	OutDesc.NumSamples = InDesc.NumSamples;

	if (InDesc.Depth > 0)
	{
		OutDesc.Depth = InDesc.Depth;
		OutDesc.Dimension = ETextureDimension::Texture3D;
	}
	else if (InDesc.bIsCubemap)
	{
		OutDesc.Dimension = InDesc.bIsArray ? ETextureDimension::TextureCubeArray : ETextureDimension::TextureCube;
	}
	else if (InDesc.bIsArray)
	{
		OutDesc.Dimension = ETextureDimension::Texture2DArray;
	}

	OutDesc.Flags = InDesc.Flags;
	// UE-TODO: UE-188415 fix flags that we force in Render Target Pool
	//OutDesc.Flags |= ETextureCreateFlags::ForceIntoNonStreamingMemoryTracking;
	OutDesc.FastVRAMPercentage = InDesc.FastVRAMPercentage;
	OutDesc.AliasableFormats = InDesc.AliasableFormats;
	check(OutDesc.IsValid());

	return OutDesc;
}

inline FRDGTextureSubresourceRange FRDGTextureSRV::GetSubresourceRange() const
{
	FRDGTextureSubresourceRange Range = GetParent()->GetSubresourceRange();
	Range.MipIndex = Desc.MipLevel;
	Range.ArraySlice = Desc.FirstArraySlice;
	Range.PlaneSlice = GetResourceTransitionPlaneForMetadataAccess(Desc.MetaData);

	if (Desc.MetaData == ERDGTextureMetaDataAccess::None && Desc.Texture && Desc.Texture->Desc.Format == PF_DepthStencil)
	{
		// PF_X24_G8 is used to indicate that this is a view on the stencil plane. Otherwise, it is a view on the depth plane
		Range.PlaneSlice = Desc.Format == PF_X24_G8 ? FRHITransitionInfo::kStencilPlaneSlice : FRHITransitionInfo::kDepthPlaneSlice;
		Range.NumPlaneSlices = 1;
	}

	if (Desc.NumMipLevels != 0)
	{
		Range.NumMips = Desc.NumMipLevels;
	}

	if (Desc.NumArraySlices != 0)
	{
		Range.NumArraySlices = Desc.NumArraySlices;
	}

	if (Desc.MetaData != ERDGTextureMetaDataAccess::None)
	{
		Range.NumPlaneSlices = 1;
	}

	return Range;
}

inline FRDGTextureSubresourceRange FRDGTextureUAV::GetSubresourceRange() const
{
	FRDGTextureSubresourceRange Range = GetParent()->GetSubresourceRange();
	Range.MipIndex = Desc.MipLevel;
	Range.ArraySlice = Desc.FirstArraySlice;
	Range.NumMips = 1;
	Range.PlaneSlice = GetResourceTransitionPlaneForMetadataAccess(Desc.MetaData);

	if (Desc.NumArraySlices != 0)
	{
		Range.NumArraySlices = Desc.NumArraySlices;
	}

	if (Desc.MetaData != ERDGTextureMetaDataAccess::None)
	{
		Range.NumPlaneSlices = 1;
	}

	return Range;
}

inline FRDGBufferSRVDesc::FRDGBufferSRVDesc(FRDGBufferRef InBuffer)
	: Buffer(InBuffer)
{
	if (EnumHasAnyFlags(Buffer->Desc.Usage, EBufferUsageFlags::DrawIndirect))
	{
		Format = PF_R32_UINT;
	}
	else if (EnumHasAnyFlags(Buffer->Desc.Usage, EBufferUsageFlags::AccelerationStructure))
	{
		// nothing special here
	}
}

inline FRDGBufferUAVDesc::FRDGBufferUAVDesc(FRDGBufferRef InBuffer)
	: Buffer(InBuffer)
{
	if (EnumHasAnyFlags(Buffer->Desc.Usage, EBufferUsageFlags::DrawIndirect))
	{
		Format = PF_R32_UINT;
	}
}

inline FGraphicsPipelineRenderTargetsInfo ExtractRenderTargetsInfo(const FRDGParameterStruct& ParameterStruct)
{
	return ExtractRenderTargetsInfo(ParameterStruct.GetRenderTargets());
}

inline FGraphicsPipelineRenderTargetsInfo ExtractRenderTargetsInfo(const FRenderTargetBindingSlots& RenderTargets)
{
	FGraphicsPipelineRenderTargetsInfo RenderTargetsInfo;
	uint32 RenderTargetIndex = 0;

	RenderTargetsInfo.NumSamples = 1;

	RenderTargets.Enumerate([&](FRenderTargetBinding RenderTarget)
	{
		FRDGTextureRef Texture = RenderTarget.GetTexture();

		RenderTargetsInfo.RenderTargetFormats[RenderTargetIndex] = (uint8)Texture->Desc.Format;
		RenderTargetsInfo.RenderTargetFlags[RenderTargetIndex] = Texture->Desc.Flags;
		RenderTargetsInfo.NumSamples |= Texture->Desc.NumSamples;
		++RenderTargetIndex;
	});

	RenderTargetsInfo.RenderTargetsEnabled = RenderTargetIndex;
	for (; RenderTargetIndex < MaxSimultaneousRenderTargets; ++RenderTargetIndex)
	{
		RenderTargetsInfo.RenderTargetFormats[RenderTargetIndex] = PF_Unknown;
	}

	const FDepthStencilBinding& DepthStencil = RenderTargets.DepthStencil;
	if (FRDGTextureRef DepthTexture = DepthStencil.GetTexture())
	{
		RenderTargetsInfo.DepthStencilTargetFormat = DepthTexture->Desc.Format;
		RenderTargetsInfo.DepthStencilTargetFlag = DepthTexture->Desc.Flags;
		RenderTargetsInfo.NumSamples |= DepthTexture->Desc.NumSamples;

		RenderTargetsInfo.DepthTargetLoadAction = DepthStencil.GetDepthLoadAction();
		RenderTargetsInfo.StencilTargetLoadAction = DepthStencil.GetStencilLoadAction();

		RenderTargetsInfo.DepthStencilAccess = DepthStencil.GetDepthStencilAccess();
		ERenderTargetStoreAction StoreAction = EnumHasAnyFlags(DepthTexture->Desc.Flags, TexCreate_Memoryless) ? ERenderTargetStoreAction::ENoAction : ERenderTargetStoreAction::EStore;
		if (DepthStencil.GetResolveTexture())
		{
			StoreAction = ERenderTargetStoreAction::EMultisampleResolve;
		}
		RenderTargetsInfo.DepthTargetStoreAction = RenderTargetsInfo.DepthStencilAccess.IsUsingDepth() ? StoreAction : ERenderTargetStoreAction::ENoAction;
		RenderTargetsInfo.StencilTargetStoreAction = RenderTargetsInfo.DepthStencilAccess.IsUsingStencil() ? StoreAction : ERenderTargetStoreAction::ENoAction;
	}
	else
	{
		RenderTargetsInfo.DepthStencilTargetFormat = PF_Unknown;
	}

	RenderTargetsInfo.MultiViewCount = RenderTargets.MultiViewCount;
	RenderTargetsInfo.bHasFragmentDensityAttachment = RenderTargets.ShadingRateTexture != nullptr;

	return RenderTargetsInfo;
}

PGO_LINK_ENABLE_WARNINGS
