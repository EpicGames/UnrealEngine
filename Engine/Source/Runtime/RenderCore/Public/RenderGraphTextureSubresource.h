// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RHI.h"

template <typename InElementType, typename InAllocatorType = FDefaultAllocator>
using TRDGTextureSubresourceArray = TArray<InElementType, InAllocatorType>;

struct FRDGTextureSubresource
{
	FRDGTextureSubresource()
		: MipIndex(0)
		, PlaneSlice(0)
		, ArraySlice(0)
	{}

	FRDGTextureSubresource(uint32 InMipIndex, uint32 InArraySlice, uint32 InPlaneSlice)
		: MipIndex(InMipIndex)
		, PlaneSlice(InPlaneSlice)
		, ArraySlice(InArraySlice)
	{}

	inline bool operator == (const FRDGTextureSubresource& RHS) const
	{
		return MipIndex == RHS.MipIndex
			&& PlaneSlice == RHS.PlaneSlice
			&& ArraySlice == RHS.ArraySlice;
	}

	inline bool operator != (const FRDGTextureSubresource& RHS) const
	{
		return !(*this == RHS);
	}

	inline bool operator < (const FRDGTextureSubresource& RHS) const
	{
		return MipIndex < RHS.MipIndex
			&& PlaneSlice < RHS.PlaneSlice
			&& ArraySlice < RHS.ArraySlice;
	}

	inline bool operator <= (const FRDGTextureSubresource& RHS) const
	{
		return MipIndex <= RHS.MipIndex
			&& PlaneSlice <= RHS.PlaneSlice
			&& ArraySlice <= RHS.ArraySlice;
	}

	inline bool operator > (const FRDGTextureSubresource& RHS) const
	{
		return MipIndex > RHS.MipIndex
			&& PlaneSlice > RHS.PlaneSlice
			&& ArraySlice > RHS.ArraySlice;
	}

	inline bool operator >= (const FRDGTextureSubresource& RHS) const
	{
		return MipIndex >= RHS.MipIndex
			&& PlaneSlice >= RHS.PlaneSlice
			&& ArraySlice >= RHS.ArraySlice;
	}

	uint32 MipIndex   : 8;
	uint32 PlaneSlice : 8;
	uint32 ArraySlice : 16;
};

struct FRDGTextureSubresourceLayout
{
	FRDGTextureSubresourceLayout()
		: NumMips(0)
		, NumPlaneSlices(0)
		, NumArraySlices(0)
	{}

	FRDGTextureSubresourceLayout(uint32 InNumMips, uint32 InNumArraySlices, uint32 InNumPlaneSlices)
		: NumMips(InNumMips)
		, NumPlaneSlices(InNumPlaneSlices)
		, NumArraySlices(InNumArraySlices)
	{}

	FRDGTextureSubresourceLayout(const FRHITextureDesc& Desc)
		: FRDGTextureSubresourceLayout(Desc.NumMips, Desc.ArraySize * (Desc.IsTextureCube() ? 6 : 1), IsStencilFormat(Desc.Format) ? 2 : 1)
	{}

	inline uint32 GetSubresourceCount() const
	{
		return NumMips * NumArraySlices * NumPlaneSlices;
	}

	inline uint32 GetSubresourceIndex(FRDGTextureSubresource Subresource) const
	{
		check(Subresource < GetMaxSubresource());
		return Subresource.MipIndex + (Subresource.ArraySlice * NumMips) + (Subresource.PlaneSlice * NumMips * NumArraySlices);
	}

	inline FRDGTextureSubresource GetSubresource(uint32 Index) const
	{
		FRDGTextureSubresource Subresource;
		Subresource.MipIndex = Index % NumMips;
		Subresource.ArraySlice = (Index / NumMips) % NumArraySlices;
		Subresource.PlaneSlice = Index / (NumMips * NumArraySlices);
		return Subresource;
	}

	inline FRDGTextureSubresource GetMaxSubresource() const
	{
		return FRDGTextureSubresource(NumMips, NumArraySlices, NumPlaneSlices);
	}

	inline bool operator == (FRDGTextureSubresourceLayout const& RHS) const
	{
		return NumMips == RHS.NumMips
			&& NumPlaneSlices == RHS.NumPlaneSlices
			&& NumArraySlices == RHS.NumArraySlices;
	}

	inline bool operator != (FRDGTextureSubresourceLayout const& RHS) const
	{
		return !(*this == RHS);
	}

	uint32 NumMips        : 8;
	uint32 NumPlaneSlices : 8;
	uint32 NumArraySlices : 16;
};

struct FRDGTextureSubresourceRange
{
	FRDGTextureSubresourceRange()
		: MipIndex(0)
		, PlaneSlice(0)
		, ArraySlice(0)
		, NumMips(0)
		, NumPlaneSlices(0)
		, NumArraySlices(0)
	{}

	explicit FRDGTextureSubresourceRange(FRDGTextureSubresourceLayout Layout)
		: MipIndex(0)
		, PlaneSlice(0)
		, ArraySlice(0)
		, NumMips(Layout.NumMips)
		, NumPlaneSlices(Layout.NumPlaneSlices)
		, NumArraySlices(Layout.NumArraySlices)
	{}

	bool operator == (FRDGTextureSubresourceRange const& RHS) const
	{
		return MipIndex == RHS.MipIndex
			&& PlaneSlice == RHS.PlaneSlice
			&& ArraySlice == RHS.ArraySlice
			&& NumMips == RHS.NumMips
			&& NumPlaneSlices == RHS.NumPlaneSlices
			&& NumArraySlices == RHS.NumArraySlices;
	}

	bool operator != (FRDGTextureSubresourceRange const& RHS) const
	{
		return !(*this == RHS);
	}

	inline uint32 GetSubresourceCount() const
	{
		return NumMips * NumArraySlices * NumPlaneSlices;
	}

	FRDGTextureSubresource GetMinSubresource() const
	{
		return FRDGTextureSubresource(MipIndex, ArraySlice, PlaneSlice);
	}

	FRDGTextureSubresource GetMaxSubresource() const
	{
		return FRDGTextureSubresource(MipIndex + NumMips, ArraySlice + NumArraySlices, PlaneSlice + NumPlaneSlices);
	}

	template <typename TFunction>
	void EnumerateSubresources(TFunction Function) const
	{
		const FRDGTextureSubresource MaxSubresource = GetMaxSubresource();
		const FRDGTextureSubresource MinSubresource = GetMinSubresource();

		for (uint32 LocalPlaneSlice = MinSubresource.PlaneSlice; LocalPlaneSlice < MaxSubresource.PlaneSlice; ++LocalPlaneSlice)
		{
			for (uint32 LocalArraySlice = MinSubresource.ArraySlice; LocalArraySlice < MaxSubresource.ArraySlice; ++LocalArraySlice)
			{
				for (uint32 LocalMipIndex = MinSubresource.MipIndex; LocalMipIndex < MaxSubresource.MipIndex; ++LocalMipIndex)
				{
					Function(FRDGTextureSubresource(LocalMipIndex, LocalArraySlice, LocalPlaneSlice));
				}
			}
		}
	}

	bool IsWholeResource(const FRDGTextureSubresourceLayout& Layout) const
	{
		return MipIndex == 0
			&& PlaneSlice == 0
			&& ArraySlice == 0
			&& NumMips == Layout.NumMips
			&& NumPlaneSlices == Layout.NumPlaneSlices
			&& NumArraySlices == Layout.NumArraySlices;
	}

	bool IsValid(const FRDGTextureSubresourceLayout& Layout) const
	{
		return MipIndex + NumMips <= Layout.NumMips
			&& PlaneSlice + NumPlaneSlices <= Layout.NumPlaneSlices
			&& ArraySlice + NumArraySlices <= Layout.NumArraySlices;
	}

	uint32 MipIndex       : 8;
	uint32 PlaneSlice     : 8;
	uint32 ArraySlice     : 16;
	uint32 NumMips        : 8;
	uint32 NumPlaneSlices : 8;
	uint32 NumArraySlices : 16;
};

template <typename ElementType, typename AllocatorType>
inline void VerifyLayout(const TRDGTextureSubresourceArray<ElementType, AllocatorType>& SubresourceArray, const FRDGTextureSubresourceLayout& Layout)
{
	checkf(Layout.GetSubresourceCount() > 0, TEXT("Subresource layout has no subresources."));
	checkf(SubresourceArray.Num() == Layout.GetSubresourceCount(), TEXT("Subresource array does not match the subresource layout."));
}

template <typename ElementType, typename AllocatorType>
inline void InitTextureSubresources(TRDGTextureSubresourceArray<ElementType, AllocatorType>& SubresourceArray, const FRDGTextureSubresourceLayout& Layout, const ElementType& Element = {})
{
	const uint32 SubresourceCount = Layout.GetSubresourceCount();
	SubresourceArray.SetNumUninitialized(SubresourceCount, EAllowShrinking::No);
	for (uint32 SubresourceIndex = 0; SubresourceIndex < SubresourceCount; ++SubresourceIndex)
	{
		SubresourceArray[SubresourceIndex] = Element;
	}
}

template <typename ElementType, typename AllocatorType>
inline const ElementType& GetSubresource(const TRDGTextureSubresourceArray<ElementType, AllocatorType>& SubresourceArray, const FRDGTextureSubresourceLayout& Layout, FRDGTextureSubresource Subresource)
{
	VerifyLayout(SubresourceArray, Layout);
	return SubresourceArray[Layout.GetSubresourceIndex(Subresource)];
}

template <typename ElementType, typename AllocatorType>
inline ElementType& GetSubresource(TRDGTextureSubresourceArray<ElementType, AllocatorType>& SubresourceArray, const FRDGTextureSubresourceLayout& Layout, FRDGTextureSubresource Subresource)
{
	VerifyLayout(SubresourceArray, Layout);
	return SubresourceArray[Layout.GetSubresourceIndex(Subresource)];
}

template <typename ElementType, typename AllocatorType, typename FunctionType>
inline void EnumerateSubresourceRange(TRDGTextureSubresourceArray<ElementType, AllocatorType>& SubresourceArray, const FRDGTextureSubresourceLayout& Layout, const FRDGTextureSubresourceRange& Range, FunctionType Function)
{
	VerifyLayout(SubresourceArray, Layout);
	Range.EnumerateSubresources([&](FRDGTextureSubresource Subresource)
	{
		Function(GetSubresource(SubresourceArray, Layout, Subresource));
	});
}

template <typename ElementType, typename AllocatorType, typename FunctionType>
inline void EnumerateSubresourceRange(const TRDGTextureSubresourceArray<ElementType, AllocatorType>& SubresourceArray, const FRDGTextureSubresourceLayout& Layout, const FRDGTextureSubresourceRange& Range, FunctionType Function)
{
	VerifyLayout(SubresourceArray, Layout);
	Range.EnumerateSubresources([&](FRDGTextureSubresource Subresource)
	{
		Function(GetSubresource(SubresourceArray, Layout, Subresource));
	});
}