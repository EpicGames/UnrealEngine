// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "RHITransientResourceAllocator.h"
#include "RenderGraphDefinitions.h"
#include "RenderGraphPass.h"
#include "Trace/Trace.h"
#include "RenderTracing.h"

class FRDGBuffer;
class FRDGBuilder;
class FRDGPass;
class FRDGDispatchPass;
class FRDGTexture;
class FRDGViewableResource;
class FRHICommandListBase;
namespace UE { namespace Trace { class FChannel; } }

#if RDG_ENABLE_TRACE

UE_TRACE_CHANNEL_EXTERN(RDGChannel, RENDERCORE_API);

class FRDGTrace
{
public:
	RENDERCORE_API FRDGTrace();

	RENDERCORE_API void OutputGraphBegin();
	RENDERCORE_API void OutputGraphEnd(const FRDGBuilder& GraphBuilder);

	RENDERCORE_API void AddResource(FRDGViewableResource* Resource);
	RENDERCORE_API void AddTexturePassDependency(FRDGTexture* Texture, FRDGPass* Pass);
	RENDERCORE_API void AddBufferPassDependency(FRDGBuffer* Buffer, FRDGPass* Pass);

	FRHITransientAllocationStats TransientAllocationStats;

	RENDERCORE_API bool IsEnabled() const;

private:
	uint64 GraphStartCycles{};
	uint32 ResourceOrder{};
	bool bEnabled;
};

#endif // RDG_ENABLE_TRACE

#if UE_RENDER_TRACING_ENABLED

namespace RenderTracing
{

void BeginExecuteRDGPass(const FRDGPass& Pass, const FRHICommandListBase& CmdList);
void EndExecuteRDGPass(const FRDGPass& Pass);
void BeginExecuteDispatchPass(const FRDGDispatchPass& Pass);
void AddDispatchPassCommandList(const FRDGDispatchPass& Pass, const FRHICommandListBase& CmdList);
void EndExecuteDispatchPass(const FRDGDispatchPass& Pass, const FRHICommandListBase* EpilogueCmdList);

}

#endif // UE_RENDER_TRACING_ENABLED

