// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/ContainersFwd.h"
#include "Containers/UnrealString.h"
#include "HAL/UnrealMemory.h"
#include "Math/Color.h"
#include "Math/IntPoint.h"
#include "Math/IntRect.h"
#include "Math/IntVector.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector2D.h"
#include "Math/Vector4.h"
#include "Misc/AssertionMacros.h"
#include "Misc/EnumClassFlags.h"
#include "Misc/Optional.h"
#include "RHI.h"
#include "RHICommandList.h"
#include "RHIDefinitions.h"
#include "RenderGraphAllocator.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphDefinitions.h"
#include "RenderGraphEvent.h"
#include "RenderGraphResources.h"
#include "RendererInterface.h"
#include "Shader.h"
#include "ShaderParameterMacros.h"
#include "ShaderParameterStruct.h"
#include "Stats/Stats.h"
#include "Templates/Function.h"
#include "Templates/Invoke.h"
#include "Templates/RefCounting.h"
#include "Templates/UnrealTemplate.h"
#include "Templates/UnrealTypeTraits.h"
#include "PipelineStateCache.h"

#include <initializer_list>

class FGlobalShaderMap;
class FRHIGPUBufferReadback;
class FRHIGPUTextureReadback;
class FShaderParametersMetadata;

// Callback to modify pass parameters just prior to dispatch, used for indirect dispatches where the group count callback cannot be used for this purpose.
using FRDGDispatchLateParamCallback = TFunction<void()>;

/** Returns whether the resource was produced by a prior pass. */
inline bool HasBeenProduced(FRDGViewableResource* Resource)
{
	return Resource && Resource->HasBeenProduced();
}

/** Returns the texture if it was produced by a prior pass, or null otherwise. */
inline FRDGTextureRef GetIfProduced(FRDGTextureRef Texture, FRDGTextureRef FallbackTexture = nullptr)
{
	return HasBeenProduced(Texture) ? Texture : FallbackTexture;
}

/** Returns the buffer if has been produced by a prior pass, or null otherwise. */
inline FRDGBufferRef GetIfProduced(FRDGBufferRef Buffer, FRDGBufferRef FallbackBuffer = nullptr)
{
	return HasBeenProduced(Buffer) ? Buffer : FallbackBuffer;
}

/** Returns 'Load' if the texture has already been produced by a prior pass, or the requested initial action. */
inline ERenderTargetLoadAction GetLoadActionIfProduced(FRDGTextureRef Texture, ERenderTargetLoadAction ActionIfNotProduced)
{
	return HasBeenProduced(Texture) ? ERenderTargetLoadAction::ELoad : ActionIfNotProduced;
}

/** Returns a binding with the requested initial action, or a load action if the resource has been produced by a prior pass. */
inline FRenderTargetBinding GetLoadBindingIfProduced(FRDGTextureRef Texture, ERenderTargetLoadAction ActionIfNotProduced)
{
	return FRenderTargetBinding(Texture, GetLoadActionIfProduced(Texture, ActionIfNotProduced));
}

/** Returns the RHI texture from an RDG texture if it exists, or null otherwise. */
inline FRHITexture* TryGetRHI(FRDGTextureRef Texture)
{
	return Texture ? Texture->GetRHI() : nullptr;
}

inline FRHIBuffer* TryGetRHI(FRDGBuffer* Buffer)
{
	return Buffer ? Buffer->GetRHI() : nullptr;
}

inline FRHIBuffer* TryGetRHI(FRDGPooledBuffer* Buffer)
{
	return Buffer ? Buffer->GetRHI() : nullptr;
}

inline FRHIShaderResourceView* TryGetSRV(FRDGPooledBuffer* Buffer)
{
	return Buffer ? Buffer->GetSRV() : 0;
}

inline uint64 TryGetSize(const FRDGBuffer* Buffer)
{
	return Buffer ? Buffer->GetSize() : 0;
}

inline uint64 TryGetSize(const FRDGPooledBuffer* Buffer)
{
	return Buffer ? Buffer->GetSize() : 0;
}

inline bool IsRegistered(FRDGBuilder& GraphBuilder, const TRefCountPtr<IPooledRenderTarget>& RenderTarget)
{
	return GraphBuilder.FindExternalTexture(RenderTarget) != nullptr;
}

inline bool IsRegistered(FRDGBuilder& GraphBuilder, const TRefCountPtr<FRDGPooledBuffer>& Buffer)
{
	return GraphBuilder.FindExternalBuffer(Buffer) != nullptr;
}

inline FRenderTargetBindingSlots GetRenderTargetBindings(ERenderTargetLoadAction ColorLoadAction, TArrayView<FRDGTextureRef> ColorTextures)
{
	check(ColorTextures.Num() <= MaxSimultaneousRenderTargets);

	FRenderTargetBindingSlots BindingSlots;
	for (int32 Index = 0, Count = ColorTextures.Num(); Index < Count; ++Index)
	{
		check(ColorTextures[Index]);
		BindingSlots[Index] = FRenderTargetBinding(ColorTextures[Index], ColorLoadAction);
	}
	return BindingSlots;
}

struct FTextureRenderTargetBinding
{
	FRDGTextureRef Texture;
	int16 ArraySlice;
	bool bNeverClear;

	FTextureRenderTargetBinding()
		: Texture(nullptr)
		, ArraySlice(-1)
		, bNeverClear(false)
	{}

	FTextureRenderTargetBinding(FRDGTextureRef InTexture, bool bInNeverClear)
		: Texture(InTexture)
		, ArraySlice(-1)
		, bNeverClear(bInNeverClear)
	{}

	FTextureRenderTargetBinding(FRDGTextureRef InTexture, int16 InArraySlice = -1, bool bInNeverClear = false)
		: Texture(InTexture)
		, ArraySlice(InArraySlice)
		, bNeverClear(bInNeverClear)
	{}
};
inline FRenderTargetBindingSlots GetRenderTargetBindings(ERenderTargetLoadAction ColorLoadAction, TArrayView<FTextureRenderTargetBinding> ColorTextures)
{
	check(ColorTextures.Num() <= MaxSimultaneousRenderTargets);

	FRenderTargetBindingSlots BindingSlots;
	for (int32 Index = 0, Count = ColorTextures.Num(); Index < Count; ++Index)
	{
		check(ColorTextures[Index].Texture);
		BindingSlots[Index] = FRenderTargetBinding(ColorTextures[Index].Texture, ColorLoadAction, 0, ColorTextures[Index].ArraySlice);
		if (ColorLoadAction == ERenderTargetLoadAction::EClear && ColorTextures[Index].bNeverClear)
		{
			BindingSlots[Index].SetLoadAction(ERenderTargetLoadAction::ELoad);
		}
	}
	return BindingSlots;
}

/**
 * Clears all render graph tracked resources that are not bound by a shader.
 * Excludes any resources on the ExcludeList from being cleared regardless of whether the 
 * shader binds them or not. This is needed for resources that are used outside of shader
 * bindings such as indirect arguments buffers.
 */
extern RENDERCORE_API void ClearUnusedGraphResourcesImpl(
	const FShaderParameterBindings& ShaderBindings,
	const FShaderParametersMetadata* ParametersMetadata,
	void* InoutParameters,
	std::initializer_list<FRDGResourceRef> ExcludeList);

/** Similar to the function above, but takes a list of shader bindings and only clears if none of the shaders contain the resource. */
extern RENDERCORE_API void ClearUnusedGraphResourcesImpl(
	TArrayView<const FShaderParameterBindings*> ShaderBindingsList,
	const FShaderParametersMetadata* ParametersMetadata,
	void* InoutParameters,
	std::initializer_list<FRDGResourceRef> ExcludeList);

template <typename TShaderClass>
void ClearUnusedGraphResources(
	const TShaderRef<TShaderClass>& Shader,
	const FShaderParametersMetadata* ParametersMetadata,
	typename TShaderClass::FParameters* InoutParameters,
	std::initializer_list<FRDGResourceRef> ExcludeList = {})
{
	// Verify the shader have all the parameters it needs. This is done before the
	// ClearUnusedGraphResourcesImpl() to not mislead user on why some resource are missing
	// when debugging a validation failure.
	ValidateShaderParameters(Shader, ParametersMetadata, InoutParameters);

	// Clear the resources the shader won't need.
	return ClearUnusedGraphResourcesImpl(Shader->Bindings, ParametersMetadata, InoutParameters, ExcludeList);
}

template <typename TShaderClass>
void ClearUnusedGraphResources(
	const TShaderRef<TShaderClass>& Shader,
	typename TShaderClass::FParameters* InoutParameters,
	std::initializer_list<FRDGResourceRef> ExcludeList = {})
{
	const FShaderParametersMetadata* ParametersMetadata = TShaderClass::FParameters::FTypeInfo::GetStructMetadata();
	return ClearUnusedGraphResources(Shader, ParametersMetadata, InoutParameters, MoveTemp(ExcludeList));
}

template <typename TShaderClassA, typename TShaderClassB, typename TPassParameterStruct>
void ClearUnusedGraphResources(
	const TShaderRef<TShaderClassA>& ShaderA,
	const TShaderRef<TShaderClassB>& ShaderB,
	TPassParameterStruct* InoutParameters,
	std::initializer_list<FRDGResourceRef> ExcludeList = {})
{
	static_assert(std::is_same_v<typename TShaderClassA::FParameters, TPassParameterStruct>, "First shader FParameter type must match pass parameters.");
	static_assert(std::is_same_v<typename TShaderClassB::FParameters, TPassParameterStruct>, "Second shader FParameter type must match pass parameters.");
	const FShaderParametersMetadata* ParametersMetadata = TPassParameterStruct::FTypeInfo::GetStructMetadata();

	// Verify the shader have all the parameters it needs. This is done before the
	// ClearUnusedGraphResourcesImpl() to not mislead user on why some resource are missing
	// when debugging a validation failure.
	ValidateShaderParameters(ShaderA, ParametersMetadata, InoutParameters);
	ValidateShaderParameters(ShaderB, ParametersMetadata, InoutParameters);

	// Clear the resources the shader won't need.
	const FShaderParameterBindings* ShaderBindings[] = { &ShaderA->Bindings, &ShaderB->Bindings };
	return ClearUnusedGraphResourcesImpl(ShaderBindings, ParametersMetadata, InoutParameters, ExcludeList);
}

/**
 * Register external texture with fallback if the resource is invalid.
 *
 * CAUTION: use this function very wisely. It may actually remove shader parameter validation
 * failure when a pass is actually trying to access a resource not yet or no longer available.
 */
RENDERCORE_API FRDGTextureRef RegisterExternalTextureWithFallback(
	FRDGBuilder& GraphBuilder,
	const TRefCountPtr<IPooledRenderTarget>& ExternalPooledTexture,
	const TRefCountPtr<IPooledRenderTarget>& FallbackPooledTexture);

/** Variants of RegisterExternalTexture which will returns null (rather than assert) if the external texture is null. */
inline FRDGTextureRef TryRegisterExternalTexture(
	FRDGBuilder& GraphBuilder,
	const TRefCountPtr<IPooledRenderTarget>& ExternalPooledTexture,
	ERDGTextureFlags Flags = ERDGTextureFlags::None)
{
	return ExternalPooledTexture ? GraphBuilder.RegisterExternalTexture(ExternalPooledTexture, Flags) : nullptr;
}

/** Variants of RegisterExternalBuffer which will return null (rather than assert) if the external buffer is null. */
inline FRDGBufferRef TryRegisterExternalBuffer(
	FRDGBuilder& GraphBuilder,
	const TRefCountPtr<FRDGPooledBuffer>& ExternalPooledBuffer,
	ERDGBufferFlags Flags = ERDGBufferFlags::None)
{
	return ExternalPooledBuffer ? GraphBuilder.RegisterExternalBuffer(ExternalPooledBuffer, Flags) : nullptr;
}

inline FRDGTextureRef RegisterExternalTexture(FRDGBuilder& GraphBuilder, FRHITexture* Texture, const TCHAR* NameIfUnregistered, ERDGTextureFlags Flags)
{
	if (FRDGTextureRef FoundTexture = GraphBuilder.FindExternalTexture(Texture))
	{
		return FoundTexture;
	}

	return GraphBuilder.RegisterExternalTexture(CreateRenderTarget(Texture, NameIfUnregistered), Flags);
}

inline FRDGTextureRef RegisterExternalTexture(FRDGBuilder& GraphBuilder, FRHITexture* Texture, const TCHAR* NameIfUnregistered)
{
	return RegisterExternalTexture(GraphBuilder, Texture, NameIfUnregistered, ERDGTextureFlags::None);
}

/** Simple pair of RDG textures used for MSAA. */
struct FRDGTextureMSAA
{
	FRDGTextureMSAA() = default;

	FRDGTextureMSAA(FRDGTextureRef InTarget, FRDGTextureRef InResolve)
		: Target(InTarget)
		, Resolve(InResolve)
	{}

	FRDGTextureMSAA(FRDGTextureRef InTexture)
		: Target(InTexture)
		, Resolve(InTexture)
	{}

	bool IsValid() const
	{
		return Target != nullptr && Resolve != nullptr;
	}

	bool IsSeparate() const
	{
		return Target != Resolve;
	}

	bool operator==(FRDGTextureMSAA Other) const
	{
		return Target == Other.Target && Resolve == Other.Resolve;
	}

	bool operator!=(FRDGTextureMSAA Other) const
	{
		return !(*this == Other);
	}

	FRDGTextureRef Target = nullptr;
	FRDGTextureRef Resolve = nullptr;
};

RENDERCORE_API FRDGTextureMSAA CreateTextureMSAA(
	FRDGBuilder& GraphBuilder,
	FRDGTextureDesc Desc,
	const TCHAR* NameMultisampled, const TCHAR* NameResolved,
	ETextureCreateFlags ResolveFlagsToAdd = TexCreate_None);

/** All utils for compute shaders.
 */
namespace FComputeShaderUtils
{
	/** Ideal size of group size 8x8 to occupy at least an entire wave on GCN, two warp on Nvidia. */
	static constexpr int32 kGolden2DGroupSize = 8;

	/** Compute the number of groups to dispatch. */
	inline FIntVector GetGroupCount(const int32 ThreadCount, const int32 GroupSize)
	{
		return FIntVector(
			FMath::DivideAndRoundUp(ThreadCount, GroupSize),
			1,
			1);
	}
	inline FIntVector GetGroupCount(const FIntPoint& ThreadCount, const FIntPoint& GroupSize)
	{
		return FIntVector(
			FMath::DivideAndRoundUp(ThreadCount.X, GroupSize.X),
			FMath::DivideAndRoundUp(ThreadCount.Y, GroupSize.Y),
			1);
	}
	inline FIntVector GetGroupCount(const FIntPoint& ThreadCount, const int32 GroupSize)
	{
		return FIntVector(
			FMath::DivideAndRoundUp(ThreadCount.X, GroupSize),
			FMath::DivideAndRoundUp(ThreadCount.Y, GroupSize),
			1);
	}
	inline FIntVector GetGroupCount(const FIntVector& ThreadCount, const FIntVector& GroupSize)
	{
		return FIntVector(
			FMath::DivideAndRoundUp(ThreadCount.X, GroupSize.X),
			FMath::DivideAndRoundUp(ThreadCount.Y, GroupSize.Y),
			FMath::DivideAndRoundUp(ThreadCount.Z, GroupSize.Z));
	}
	inline FIntVector GetGroupCount(const FIntVector& ThreadCount, const int32 GroupSize)
	{
		return FIntVector(
			FMath::DivideAndRoundUp(ThreadCount.X, GroupSize),
			FMath::DivideAndRoundUp(ThreadCount.Y, GroupSize),
			FMath::DivideAndRoundUp(ThreadCount.Z, GroupSize));
	}

	/**
	 * Constant stride used when wrapping too large 1D dispatches using GetGroupCountWrapped, selected as 128 appears to be the lowest common denominator 
	 * for mobile (GLES 3.1). For PC (with ~64k groups / dimension) this yields ~8M groups (500M threads @ group size 64) before even wrapping into Z.
	 * NOTE: this value must match WRAPPED_GROUP_STRIDE in ComputeShaderUtils.ush
	 */
	static constexpr int32 WrappedGroupStride = 128;

	/**
	 * Wrapping number of groups to Y and Z dimension if X group count overflows GRHIMaxDispatchThreadGroupsPerDimension.
	 * Calculate the linear group index as (or use GetUnWrappedDispatchGroupId(GroupId) in ComputeShaderUtils.ush):
	 *  uint LinearGroupId = GroupId.X + (GroupId.Z * WrappedGroupStride + GroupId.Y) * WrappedGroupStride;
	 * Note that you must use an early out because LinearGroupId may be larger than the ideal due to wrapping.
	 */
	inline FIntVector GetGroupCountWrapped(const int32 TargetGroupCount)
	{
		check(GRHIMaxDispatchThreadGroupsPerDimension.X >= WrappedGroupStride && GRHIMaxDispatchThreadGroupsPerDimension.Y >= WrappedGroupStride);

		FIntVector GroupCount(TargetGroupCount, 1, 1);

		if (GroupCount.X > GRHIMaxDispatchThreadGroupsPerDimension.X)
		{
			GroupCount.Y = FMath::DivideAndRoundUp(GroupCount.X, WrappedGroupStride);
			GroupCount.X = WrappedGroupStride;
		}
		if (GroupCount.Y > GRHIMaxDispatchThreadGroupsPerDimension.Y)
		{
			GroupCount.Z = FMath::DivideAndRoundUp(GroupCount.Y, WrappedGroupStride);
			GroupCount.Y = WrappedGroupStride;
		}

		check(TargetGroupCount <= GroupCount.X * GroupCount.Y * GroupCount.Z);

		return GroupCount;
	}

	/**
	 * Compute the number of groups to dispatch and allow wrapping to Y and Z dimension if X group count overflows. 
	 * Calculate the linear group index as (or use GetUnWrappedDispatchGroupId(GroupId) in ComputeShaderUtils.ush):
	 *  uint LinearGroupId = GroupId.X + (GroupId.Z * WrappedGroupStride + GroupId.Y) * WrappedGroupStride;
	 * Note that you must use an early out because LinearGroupId may be larger than the ideal due to wrapping.
	 */
	inline FIntVector GetGroupCountWrapped(const int32 ThreadCount, const int32 GroupSize)
	{
		return GetGroupCountWrapped(FMath::DivideAndRoundUp(ThreadCount, GroupSize));
	}

	inline void ValidateGroupCount(const FIntVector& GroupCount)
	{
		ensure(GroupCount.X <= GRHIMaxDispatchThreadGroupsPerDimension.X);
		ensure(GroupCount.Y <= GRHIMaxDispatchThreadGroupsPerDimension.Y);
		ensure(GroupCount.Z <= GRHIMaxDispatchThreadGroupsPerDimension.Z);
	}

	inline void ValidateIndirectArgsBuffer(uint32 IndirectArgsBufferSize, uint32 IndirectArgOffset)
	{
		constexpr uint32 IndirectArgsSize = sizeof(FRHIDispatchIndirectParametersNoPadding);
		checkf((IndirectArgOffset % 4) == 0, TEXT("IndirectArgOffset for compute shader indirect dispatch needs to be a multiple of 4."));
		checkf(
			(IndirectArgOffset + IndirectArgsSize) <= IndirectArgsBufferSize,
			TEXT("Indirect parameters buffer for compute shader indirect dispatch at byte offset %d doesn't have enough room for one element."),
			IndirectArgOffset);
#if PLATFORM_DISPATCH_INDIRECT_ARGUMENT_BOUNDARY_SIZE != 0
		checkf(IndirectArgOffset / PLATFORM_DISPATCH_INDIRECT_ARGUMENT_BOUNDARY_SIZE == (IndirectArgOffset + IndirectArgsSize - 1) / PLATFORM_DISPATCH_INDIRECT_ARGUMENT_BOUNDARY_SIZE, TEXT("Compute indirect dispatch arguments cannot cross %d byte boundary."), PLATFORM_DISPATCH_INDIRECT_ARGUMENT_BOUNDARY_SIZE);
#endif // #if PLATFORM_DISPATCH_INDIRECT_ARGUMENT_BOUNDARY_SIZE != 0
	}

	inline void ValidateIndirectArgsBuffer(const FRDGBufferRef IndirectArgsBuffer, uint32 IndirectArgOffset)
	{
		checkf(EnumHasAnyFlags(IndirectArgsBuffer->Desc.Usage, EBufferUsageFlags::VertexBuffer | EBufferUsageFlags::ByteAddressBuffer), TEXT("The buffer %s needs to be a vertex or byte address buffer to be used as an indirect dispatch parameters"), IndirectArgsBuffer->Name);
		checkf(EnumHasAnyFlags(IndirectArgsBuffer->Desc.Usage, EBufferUsageFlags::DrawIndirect), TEXT("The buffer %s for indirect dispatch parameters was not flagged with BUF_DrawIndirect"), IndirectArgsBuffer->Name);
		ValidateIndirectArgsBuffer(IndirectArgsBuffer->GetSize(), IndirectArgOffset);
	}

	namespace Private
	{
		template<typename TShaderClass>
		inline void PrepareDispatch(
			FRHIComputeCommandList& RHICmdList,
			const TShaderRef<TShaderClass>& ComputeShader,
			const FShaderParametersMetadata* ParametersMetadata,
			const typename TShaderClass::FParameters& Parameters)
		{
			FRHIComputeShader* ShaderRHI = ComputeShader.GetComputeShader();
			SetComputePipelineState(RHICmdList, { ShaderRHI, ComputeShader.GetSpecializationConstants() });
			SetShaderParameters(RHICmdList, ComputeShader, ShaderRHI, ParametersMetadata, Parameters);
		}

		template<typename TShaderClass>
		inline void PrepareDispatch(
			FRHIComputeCommandList& RHICmdList,
			const TShaderRef<TShaderClass>& ComputeShader,
			const typename TShaderClass::FParameters& Parameters)
		{
			const FShaderParametersMetadata* ParametersMetadata = TShaderClass::FParameters::FTypeInfo::GetStructMetadata();
			PrepareDispatch(RHICmdList, ComputeShader, ParametersMetadata, Parameters);
		}

		template<typename TShaderClass>
		inline void AfterDispatch(FRHIComputeCommandList& RHICmdList, const TShaderRef<TShaderClass>& ComputeShader)
		{
			UnsetShaderUAVs(RHICmdList, ComputeShader, ComputeShader.GetComputeShader());
		}
	}

	/** Dispatch a compute shader to rhi command list with its parameters. */
	template<typename TShaderClass>
	inline void Dispatch(
		FRHIComputeCommandList& RHICmdList, 
		const TShaderRef<TShaderClass>& ComputeShader, 
		const FShaderParametersMetadata* ParametersMetadata,
		const typename TShaderClass::FParameters& Parameters,
		FIntVector GroupCount)
	{
		ValidateGroupCount(GroupCount);

		Private::PrepareDispatch(RHICmdList, ComputeShader, ParametersMetadata, Parameters);
		RHICmdList.DispatchComputeShader(GroupCount.X, GroupCount.Y, GroupCount.Z);
		Private::AfterDispatch(RHICmdList, ComputeShader);
	}

	template<typename TShaderClass>
	inline void Dispatch(
		FRHIComputeCommandList& RHICmdList,
		const TShaderRef<TShaderClass>& ComputeShader,
		const typename TShaderClass::FParameters& Parameters,
		FIntVector GroupCount)
	{
		const FShaderParametersMetadata* ParametersMetadata = TShaderClass::FParameters::FTypeInfo::GetStructMetadata();
		Dispatch(RHICmdList, ComputeShader, ParametersMetadata, Parameters, GroupCount);
	}
	
	/** Indirect dispatch a compute shader to rhi command list with its parameters. */
	template<typename TShaderClass>
	inline void DispatchIndirect(
		FRHIComputeCommandList& RHICmdList,
		const TShaderRef<TShaderClass>& ComputeShader,
		const typename TShaderClass::FParameters& Parameters,
		FRHIBuffer* IndirectArgsBuffer,
		uint32 IndirectArgOffset)
	{
		ValidateIndirectArgsBuffer(IndirectArgsBuffer->GetSize(), IndirectArgOffset);

		Private::PrepareDispatch(RHICmdList, ComputeShader, Parameters);
		RHICmdList.DispatchIndirectComputeShader(IndirectArgsBuffer, IndirectArgOffset);
		Private::AfterDispatch(RHICmdList, ComputeShader);
	}

	/** Dispatch a compute shader to rhi command list with its parameters and indirect args. */
	template<typename TShaderClass>
	inline void DispatchIndirect(
		FRHIComputeCommandList& RHICmdList,
		const TShaderRef<TShaderClass>& ComputeShader,
		const typename TShaderClass::FParameters& Parameters,
		FRDGBufferRef IndirectArgsBuffer,
		uint32 IndirectArgOffset)
	{
		ValidateIndirectArgsBuffer(IndirectArgsBuffer, IndirectArgOffset);

		Private::PrepareDispatch(RHICmdList, ComputeShader, Parameters);
		RHICmdList.DispatchIndirectComputeShader(IndirectArgsBuffer->GetIndirectRHICallBuffer(), IndirectArgOffset);
		Private::AfterDispatch(RHICmdList, ComputeShader);
	}

	/** Dispatch a compute shader to render graph builder with its parameters. */
	template<typename TShaderClass>
	inline FRDGPassRef AddPass(
		FRDGBuilder& GraphBuilder,
		FRDGEventName&& PassName,
		ERDGPassFlags PassFlags,
		const TShaderRef<TShaderClass>& ComputeShader,
		const FShaderParametersMetadata* ParametersMetadata,
		typename TShaderClass::FParameters* Parameters,
		FIntVector GroupCount)
	{
		checkf(
			 EnumHasAnyFlags(PassFlags, ERDGPassFlags::Compute | ERDGPassFlags::AsyncCompute) &&
			!EnumHasAnyFlags(PassFlags, ERDGPassFlags::Copy | ERDGPassFlags::Raster), TEXT("AddPass only supports 'Compute' or 'AsyncCompute'."));

		ValidateGroupCount(GroupCount);
		ClearUnusedGraphResources(ComputeShader, ParametersMetadata, Parameters);

		return GraphBuilder.AddPass(
			Forward<FRDGEventName>(PassName),
			ParametersMetadata,
			Parameters,
			PassFlags,
			[ParametersMetadata, Parameters, ComputeShader, GroupCount](FRDGAsyncTask, FRHIComputeCommandList& RHICmdList)
		{
			FComputeShaderUtils::Dispatch(RHICmdList, ComputeShader, ParametersMetadata, *Parameters, GroupCount);
		});
	}

	/** Dispatch a compute shader to render graph builder with its parameters. GroupCount is supplied through a callback.
	 *  This allows adding a dispatch with unknown GroupCount but the value must be ready before the pass is executed.
	 */
	template<typename TShaderClass>
	inline FRDGPassRef AddPass(
		FRDGBuilder& GraphBuilder,
		FRDGEventName&& PassName,
		ERDGPassFlags PassFlags,
		const TShaderRef<TShaderClass>& ComputeShader,
		const FShaderParametersMetadata* ParametersMetadata,
		typename TShaderClass::FParameters* Parameters,
		FRDGDispatchGroupCountCallback&& GroupCountCallback)
	{
		checkf(
			EnumHasAnyFlags(PassFlags, ERDGPassFlags::Compute | ERDGPassFlags::AsyncCompute) &&
			!EnumHasAnyFlags(PassFlags, ERDGPassFlags::Copy | ERDGPassFlags::Raster), TEXT("AddPass only supports 'Compute' or 'AsyncCompute'."));

		ClearUnusedGraphResources(ComputeShader, ParametersMetadata, Parameters);

		return GraphBuilder.AddPass(
			Forward<FRDGEventName>(PassName),
			ParametersMetadata,
			Parameters,
			PassFlags,
			[ParametersMetadata, Parameters, ComputeShader, GroupCountCallback = MoveTemp(GroupCountCallback)](FRDGAsyncTask, FRHIComputeCommandList& RHICmdList)
			{
				const FIntVector GroupCount = GroupCountCallback();
				if (GroupCount.X > 0 && GroupCount.Y > 0 && GroupCount.Z > 0)
				{
					ValidateGroupCount(GroupCount);
					FComputeShaderUtils::Dispatch(RHICmdList, ComputeShader, ParametersMetadata, *Parameters, GroupCount);
				}
			});
	}

	template<typename TShaderClass>
	inline FRDGPassRef AddPass(
		FRDGBuilder& GraphBuilder,
		FRDGEventName&& PassName,
		ERDGPassFlags PassFlags,
		const TShaderRef<TShaderClass>& ComputeShader,
		typename TShaderClass::FParameters* Parameters,
		FIntVector GroupCount)
	{
		const FShaderParametersMetadata* ParametersMetadata = TShaderClass::FParameters::FTypeInfo::GetStructMetadata();
		return AddPass(GraphBuilder, Forward<FRDGEventName>(PassName), PassFlags, ComputeShader, ParametersMetadata, Parameters, GroupCount);
	}

	template <typename TShaderClass>
	inline FRDGPassRef AddPass(
		FRDGBuilder& GraphBuilder,
		FRDGEventName&& PassName,
		const TShaderRef<TShaderClass>& ComputeShader,
		typename TShaderClass::FParameters* Parameters,
		FIntVector GroupCount)
	{
		const FShaderParametersMetadata* ParametersMetadata = TShaderClass::FParameters::FTypeInfo::GetStructMetadata();
		return AddPass(GraphBuilder, Forward<FRDGEventName>(PassName), ERDGPassFlags::Compute, ComputeShader, ParametersMetadata, Parameters, GroupCount);
	}

	template <typename TShaderClass>
	inline FRDGPassRef AddPass(
		FRDGBuilder& GraphBuilder,
		FRDGEventName&& PassName,
		const TShaderRef<TShaderClass>& ComputeShader,
		typename TShaderClass::FParameters* Parameters,
		FRDGDispatchGroupCountCallback&& GroupCountCallback)
	{
		const FShaderParametersMetadata* ParametersMetadata = TShaderClass::FParameters::FTypeInfo::GetStructMetadata();
		return AddPass(GraphBuilder, Forward<FRDGEventName>(PassName), ERDGPassFlags::Compute, ComputeShader, ParametersMetadata, Parameters, MoveTemp(GroupCountCallback));
	}

	/** Dispatch a compute shader to render graph builder with its parameters. */
	template<typename TShaderClass>
	inline FRDGPassRef AddPass(
		FRDGBuilder& GraphBuilder,
		FRDGEventName&& PassName,
		ERDGPassFlags PassFlags,
		const TShaderRef<TShaderClass>& ComputeShader,
		typename TShaderClass::FParameters* Parameters,
		FRDGBufferRef IndirectArgsBuffer,
		uint32 IndirectArgsOffset,
		FRDGDispatchLateParamCallback&& DispatchLateParamCallback = FRDGDispatchLateParamCallback())
	{
		checkf(PassFlags == ERDGPassFlags::Compute || PassFlags == ERDGPassFlags::AsyncCompute, TEXT("AddPass only supports 'Compute' or 'AsyncCompute'."));
		checkf(IndirectArgsBuffer->Desc.Usage & BUF_DrawIndirect, TEXT("The buffer %s was not flagged for indirect draw parameters"), IndirectArgsBuffer->Name);

		ValidateIndirectArgsBuffer(IndirectArgsBuffer, IndirectArgsOffset);
		ClearUnusedGraphResources(ComputeShader, Parameters, { IndirectArgsBuffer });

		return GraphBuilder.AddPass(
			Forward<FRDGEventName>(PassName),
			Parameters,
			PassFlags,
			[Parameters, ComputeShader, IndirectArgsBuffer, IndirectArgsOffset, DispatchLateParamCallback = MoveTemp(DispatchLateParamCallback)](FRDGAsyncTask, FRHIComputeCommandList& RHICmdList)
		{
			// Marks the indirect draw parameter as used by the pass manually, given it can't be bound directly by any of the shader,
			// meaning SetShaderParameters() won't be able to do it.
			IndirectArgsBuffer->MarkResourceAsUsed();

			if (DispatchLateParamCallback)
			{
				DispatchLateParamCallback();
			}
			FComputeShaderUtils::DispatchIndirect(RHICmdList, ComputeShader, *Parameters, IndirectArgsBuffer->GetIndirectRHICallBuffer(), IndirectArgsOffset);
		});
	}

	template<typename TShaderClass>
	inline FRDGPassRef AddPass(
		FRDGBuilder& GraphBuilder,
		FRDGEventName&& PassName,
		const TShaderRef<TShaderClass>& ComputeShader,
		typename TShaderClass::FParameters* Parameters,
		FRDGBufferRef IndirectArgsBuffer,
		uint32 IndirectArgsOffset,
		FRDGDispatchLateParamCallback&& DispatchLateParamCallback = FRDGDispatchLateParamCallback())
	{
		return AddPass(GraphBuilder, Forward<FRDGEventName>(PassName), ERDGPassFlags::Compute, ComputeShader, Parameters, IndirectArgsBuffer, IndirectArgsOffset, MoveTemp(DispatchLateParamCallback));
	}

	/**
	 * Create and set up an 1D indirect dispatch argument from some GPU-side integer in a buffer (InputCountBuffer).
	 * 	Sets up a group count as (InputCountBuffer[InputCountOffset] * Multiplier + Divisor - 1U) / Divisor;
	 *  Commonly use Divisor <=> number of threads per group.
	 */
	RENDERCORE_API FRDGBufferRef AddIndirectArgsSetupCsPass1D(FRDGBuilder& GraphBuilder, ERHIFeatureLevel::Type FeatureLevel, FRDGBuffer* InputCountBuffer, const TCHAR* OutputBufferName, uint32 Divisor, uint32 InputCountOffset = 0U, uint32 Multiplier = 1U);
};

/** Adds a render graph pass to copy a region from one texture to another. Uses RHICopyTexture under the hood.
 *  Formats of the two textures must match. The output and output texture regions be within the respective extents.
 */
RENDERCORE_API void AddCopyTexturePass(
	FRDGBuilder& GraphBuilder,
	FRDGTextureRef InputTexture,
	FRDGTextureRef OutputTexture,
	const FRHICopyTextureInfo& CopyInfo);

/** Simpler variant of the above function for 2D textures.
 *  @param InputPosition The pixel position within the input texture of the top-left corner of the box.
 *  @param OutputPosition The pixel position within the output texture of the top-left corner of the box.
 *  @param Size The size in pixels of the region to copy from input to output. If zero, the full extent of
 *         the input texture is copied.
 */
inline void AddCopyTexturePass(
	FRDGBuilder& GraphBuilder,
	FRDGTextureRef InputTexture,
	FRDGTextureRef OutputTexture,
	FIntPoint InputPosition = FIntPoint::ZeroValue,
	FIntPoint OutputPosition = FIntPoint::ZeroValue,
	FIntPoint Size = FIntPoint::ZeroValue)
{
	FRHICopyTextureInfo CopyInfo;
	CopyInfo.SourcePosition.X = InputPosition.X;
	CopyInfo.SourcePosition.Y = InputPosition.Y;
	CopyInfo.DestPosition.X = OutputPosition.X;
	CopyInfo.DestPosition.Y = OutputPosition.Y;
	if (Size != FIntPoint::ZeroValue)
	{
		CopyInfo.Size = FIntVector(Size.X, Size.Y, 1);
	}
	AddCopyTexturePass(GraphBuilder, InputTexture, OutputTexture, CopyInfo);
}

struct FRDGDrawTextureInfo
{
	// Number of texels to copy. By default it will copy the whole resource if no size is specified.
	FIntPoint Size = FIntPoint::ZeroValue;

	// Position of the copy from the source texture/to destination texture
	FIntPoint SourcePosition = FIntPoint::ZeroValue;
	FIntPoint DestPosition = FIntPoint::ZeroValue;

	uint32 SourceSliceIndex = 0;
	uint32 DestSliceIndex = 0;
	uint32 NumSlices = 1;

	// Mips to copy and destination mips
	uint32 SourceMipIndex = 0;
	uint32 DestMipIndex = 0;
	uint32 NumMips = 1;
};

RENDERCORE_API void AddDrawTexturePass(
	FRDGBuilder& GraphBuilder,
	const FGlobalShaderMap* ShaderMap,
	FRDGTextureRef InputTexture,
	FRDGTextureRef OutputTexture,
	const FRDGDrawTextureInfo& DrawInfo);

/** Adds a render graph pass to clear a texture or buffer UAV with a single typed value. */
RENDERCORE_API void AddClearUAVPass(FRDGBuilder& GraphBuilder, FRDGBufferUAVRef BufferUAV, uint32 Value, ERDGPassFlags ComputePassFlags = ERDGPassFlags::Compute);

RENDERCORE_API void AddClearUAVFloatPass(FRDGBuilder& GraphBuilder, FRDGBufferUAVRef BufferUAV, float Value, ERDGPassFlags ComputePassFlags = ERDGPassFlags::Compute);

RENDERCORE_API void AddClearUAVPass(FRDGBuilder& GraphBuilder, FRDGTextureUAVRef TextureUAV, float ClearValue, ERDGPassFlags ComputePassFlags = ERDGPassFlags::Compute);

RENDERCORE_API void AddClearUAVPass(FRDGBuilder& GraphBuilder, FRDGTextureUAVRef TextureUAV, uint32 ClearValue, ERDGPassFlags ComputePassFlags = ERDGPassFlags::Compute);

RENDERCORE_API void AddClearUAVPass(FRDGBuilder& GraphBuilder, FRDGTextureUAVRef TextureUAV, const FIntPoint& ClearValue, ERDGPassFlags ComputePassFlags = ERDGPassFlags::Compute);

RENDERCORE_API void AddClearUAVPass(FRDGBuilder& GraphBuilder, FRDGTextureUAVRef TextureUAV, const FVector2D& ClearValue, ERDGPassFlags ComputePassFlags = ERDGPassFlags::Compute);

RENDERCORE_API void AddClearUAVPass(FRDGBuilder& GraphBuilder, FRDGTextureUAVRef TextureUAV, const FVector& ClearValue, ERDGPassFlags ComputePassFlags = ERDGPassFlags::Compute);

RENDERCORE_API void AddClearUAVPass(FRDGBuilder& GraphBuilder, FRDGTextureUAVRef TextureUAV, const FUintVector4& ClearValues, ERDGPassFlags ComputePassFlags = ERDGPassFlags::Compute);

RENDERCORE_API void AddClearUAVPass(FRDGBuilder& GraphBuilder, FRDGTextureUAVRef TextureUAV, const FVector4& ClearValues, ERDGPassFlags ComputePassFlags = ERDGPassFlags::Compute);

RENDERCORE_API void AddClearUAVPass(FRDGBuilder& GraphBuilder, FRDGTextureUAVRef TextureUAV, const float(&ClearValues)[4], ERDGPassFlags ComputePassFlags = ERDGPassFlags::Compute);

RENDERCORE_API void AddClearUAVPass(FRDGBuilder& GraphBuilder, FRDGTextureUAVRef TextureUAV, const uint32(&ClearValues)[4], ERDGPassFlags ComputePassFlags = ERDGPassFlags::Compute);

RENDERCORE_API void AddClearUAVPass(FRDGBuilder& GraphBuilder, FRDGTextureUAVRef TextureUAV, const FLinearColor& ClearColor, ERDGPassFlags ComputePassFlags = ERDGPassFlags::Compute);

/** Clears parts of UAV specified by an array of screen rects. If no rects are specific, then it falls back to a standard UAV clear. */
RENDERCORE_API void AddClearUAVPass(FRDGBuilder& GraphBuilder, ERHIFeatureLevel::Type FeatureLevel, FRDGTextureUAVRef TextureUAV, const uint32(&ClearValues)[4], FRDGBufferSRVRef RectMinMaxBufferSRV, uint32 NumRects);

/** Adds a render graph pass to clear a render target to its clear value (single mip, single slice) */
RENDERCORE_API void AddClearRenderTargetPass(FRDGBuilder& GraphBuilder, FRDGTextureRef Texture);

/** Adds a render graph pass to clear a render target (single mip, single slice). Uses render pass clear actions if the clear color matches the fast clear color. */
RENDERCORE_API void AddClearRenderTargetPass(FRDGBuilder& GraphBuilder, FRDGTextureRef Texture, const FLinearColor& ClearColor);

/** Adds a render graph pass to clear a part of a render target (single mip, single slice). Draws a quad to the requested viewport. */
RENDERCORE_API void AddClearRenderTargetPass(FRDGBuilder& GraphBuilder, FRDGTextureRef Texture, const FLinearColor& ClearColor, FIntRect Viewport);

struct FRDGTextureClearInfo
{
	FRDGTextureClearInfo() = default;

	// Optional : specifies the area of the texture to clear (draws a quad to the requested viewport). If zero (and clear color is not passed or matches the fast clear color), the clear will be done with render pass clear actions :
	FIntRect Viewport;

	// Optional : specifies the clear color. If not specified or if the clear color matches the fast clear color (and Viewport is zero), the clear will be done with render pass clear actions :
	TOptional<FLinearColor> ClearColor;

	// For texture arrays, specifies which slice(s) to clear
	uint32 FirstSliceIndex = 0;
	uint32 NumSlices = 1;

	// Specifies which mip(s) to clear
	uint32 FirstMipIndex = 0;
	uint32 NumMips = 1;
};

/** Adds a render graph pass to clear a render target to its clear value. */
RENDERCORE_API void AddClearRenderTargetPass(FRDGBuilder& GraphBuilder, FRDGTextureRef Texture, const FRDGTextureClearInfo& TextureClearInfo);

/** Adds a render graph pass to clear a depth stencil target. Prefer to use clear actions if possible. */
RENDERCORE_API void AddClearDepthStencilPass(
	FRDGBuilder& GraphBuilder,
	FRDGTextureRef Texture,
	bool bClearDepth,
	float Depth,
	bool bClearStencil,
	uint8 Stencil);

/** Adds a render graph pass to clear a depth stencil target to its optimized clear value using a raster pass. */
RENDERCORE_API void AddClearDepthStencilPass(
	FRDGBuilder& GraphBuilder,
	FRDGTextureRef Texture,
	ERenderTargetLoadAction DepthLoadAction = ERenderTargetLoadAction::EClear,
	ERenderTargetLoadAction StencilLoadAction = ERenderTargetLoadAction::EClear);

/** Adds a render graph pass to clear the stencil portion of a depth / stencil target to its fast clear value. */
RENDERCORE_API void AddClearStencilPass(FRDGBuilder& GraphBuilder, FRDGTextureRef Texture);

/** Adds a render graph pass to resummarize the htile plane. */
RENDERCORE_API void AddResummarizeHTilePass(FRDGBuilder& GraphBuilder, FRDGTextureRef DepthTexture);

/** Adds a render graph pass to copy SrcBuffer content into DstBuffer. */
RENDERCORE_API void AddCopyBufferPass(FRDGBuilder& GraphBuilder, FRDGBufferRef DstBuffer, uint64 DstOffset, FRDGBufferRef SrcBuffer, uint64 SrcOffset, uint64 NumBytes);

RENDERCORE_API void AddCopyBufferPass(FRDGBuilder& GraphBuilder, FRDGBufferRef DstBuffer, FRDGBufferRef SrcBuffer);

/** Adds a pass to readback contents of an RDG texture. */
RENDERCORE_API void AddEnqueueCopyPass(FRDGBuilder& GraphBuilder, FRHIGPUTextureReadback* Readback, FRDGTextureRef SourceTexture, FResolveRect Rect = FResolveRect(), uint32 SourceSlice = 0);

/** Adds a pass to readback contents of an RDG buffer. */
RENDERCORE_API void AddEnqueueCopyPass(FRDGBuilder& GraphBuilder, FRHIGPUBufferReadback* Readback, FRDGBufferRef SourceBuffer, uint32 NumBytes);

/** Helper class to allocate data from a GraphBuilder in order to upload said data to an RDG resource.
*   Allocating from the GraphBuilder makes it so we don't have to copy the data before deferring the upload.
*/
template<typename InElementType>
struct FRDGUploadData : public TArrayView<InElementType, int32>
{
	FRDGUploadData() = delete;
	FRDGUploadData(FRDGBuilder& GraphBuilder, uint32 InCount)
		: TArrayView<InElementType, int32>(GraphBuilder.AllocPODArray<InElementType>(InCount), InCount)
	{
	}

	inline int32 GetTotalSize() const { return this->Num() * this->GetTypeSize(); }
};

/** Creates a structured buffer with initial data by creating an upload pass. */
RENDERCORE_API FRDGBufferRef CreateStructuredBuffer(
	FRDGBuilder& GraphBuilder,
	const TCHAR* Name,
	uint32 BytesPerElement,
	uint32 NumElements,
	const void* InitialData,
	uint64 InitialDataSize,
	ERDGInitialDataFlags InitialDataFlags = ERDGInitialDataFlags::None);

/** A variant where NumElements, InitialData, and InitialDataSize are supplied through callbacks. This allows creating a buffer with
 *  information unknown at creation time. Though, data must be ready before the most recent RDG pass that references the buffer
 *  is executed.
 */
RENDERCORE_API FRDGBufferRef CreateStructuredBuffer(
	FRDGBuilder& GraphBuilder,
	const TCHAR* Name,
	uint32 BytesPerElement,
	FRDGBufferNumElementsCallback&& NumElementsCallback,
	FRDGBufferInitialDataCallback&& InitialDataCallback,
	FRDGBufferInitialDataSizeCallback&& InitialDataSizeCallback);

/**
 * Helper to create a structured buffer with initial data from a TArray.
 */
template <typename ElementType, typename AllocatorType>
inline FRDGBufferRef CreateStructuredBuffer(
	FRDGBuilder& GraphBuilder,
	const TCHAR* Name,
	const TArray<ElementType, AllocatorType>& InitialData,
	ERDGInitialDataFlags InitialDataFlags = ERDGInitialDataFlags::None)
{
	static const ElementType DummyElement = ElementType();
	if (InitialData.Num() == 0)
	{
		return CreateStructuredBuffer(GraphBuilder, Name, InitialData.GetTypeSize(), 1, &DummyElement, InitialData.GetTypeSize(), ERDGInitialDataFlags::NoCopy);
	}
	return CreateStructuredBuffer(GraphBuilder, Name, InitialData.GetTypeSize(), InitialData.Num(), InitialData.GetData(), InitialData.Num() * InitialData.GetTypeSize(), InitialDataFlags);
}

/**
 * Helper to create a structured buffer with initial data from a TArray with move semantics, this can be cheaper as it guarantees the lifetimes of the data & permits copy-free upload.
 */
template <typename ElementType, typename AllocatorType>
inline FRDGBufferRef CreateStructuredBuffer(
	FRDGBuilder& GraphBuilder,
	const TCHAR* Name,
	TArray<ElementType, AllocatorType>&& InitialData)
{
	static const ElementType DummyElement = ElementType();
	if (InitialData.Num() == 0)
	{
		return CreateStructuredBuffer(GraphBuilder, Name, InitialData.GetTypeSize(), 1, &DummyElement, InitialData.GetTypeSize(), ERDGInitialDataFlags::NoCopy);
	}

	// Create a move-initialized copy of the TArray with RDG lifetime & move the data there.
	TArray<ElementType, AllocatorType>& UploadData = *GraphBuilder.AllocObject<TArray<ElementType, AllocatorType> >(MoveTemp(InitialData));
	return CreateStructuredBuffer(GraphBuilder, Name, UploadData.GetTypeSize(), UploadData.Num(), UploadData.GetData(), UploadData.Num() * UploadData.GetTypeSize(), ERDGInitialDataFlags::NoCopy);
}

/**
 * Helper to create a structured buffer with initial data from a TConstArrayView.
 */
template <typename ElementType>
inline FRDGBufferRef CreateStructuredBuffer(
	FRDGBuilder& GraphBuilder,
	const TCHAR* Name,
	TConstArrayView<ElementType> InitialData,
	ERDGInitialDataFlags InitialDataFlags = ERDGInitialDataFlags::None)
{
	static const ElementType DummyElement = ElementType();
	if (InitialData.Num() == 0)
	{
		return CreateStructuredBuffer(GraphBuilder, Name, InitialData.GetTypeSize(), 1, &DummyElement, InitialData.GetTypeSize(), ERDGInitialDataFlags::NoCopy);
	}
	return CreateStructuredBuffer(GraphBuilder, Name, InitialData.GetTypeSize(), InitialData.Num(), InitialData.GetData(), InitialData.Num() * InitialData.GetTypeSize(), InitialDataFlags);
}

/** A variant where the TArray is supplied through callbacks. This allows creating a buffer with
 *  information unknown at creation time. Though, data must be ready before the most recent RDG pass that references the buffer
 *  is executed.
 */
template <typename ArrayType>
inline FRDGBufferRef CreateStructuredBuffer_Impl(
	FRDGBuilder& GraphBuilder,
	const TCHAR* Name,
	TRDGBufferArrayCallback<ArrayType>&& ArrayCallback)
{
	return CreateStructuredBuffer(GraphBuilder, Name, sizeof(typename std::remove_reference_t<ArrayType>::ElementType),
		/*NumElementsCallback = */[ArrayCallback]() { return ArrayCallback().Num(); },
		/*InitialDataCallback = */[ArrayCallback]() { return ArrayCallback().GetData(); },
		/*InitialDataSizeCallback = */[ArrayCallback]() { return ArrayCallback().Num() * ArrayCallback().GetTypeSize(); });
}

/** Same as the previous function but where the type of the array is automatically inferred, so we can do : 
 *  TArray<FSomeType>& Array = GraphBuilder.AllocArray<...>();
 *  CreateStructuredBuffer(..., [&]() -> auto&{ return Array; });
 */
template <typename GetArrayRefCallback, typename Type = TInvokeResult_T<GetArrayRefCallback>>
inline FRDGBufferRef CreateStructuredBuffer(
	FRDGBuilder& GraphBuilder,
	const TCHAR* Name,
	GetArrayRefCallback&& ArrayCallback)
{
	return CreateStructuredBuffer_Impl<Type>(GraphBuilder, Name, MoveTemp(ArrayCallback));
}

template <typename ElementType>
inline FRDGBufferRef CreateStructuredBuffer(
	FRDGBuilder& GraphBuilder,
	const TCHAR* Name,
	const FRDGUploadData<ElementType>& InitialData)
{
	static const ElementType DummyElement = ElementType();
	if (InitialData.Num() == 0)
	{
		return CreateStructuredBuffer(GraphBuilder, Name, InitialData.GetTypeSize(), 1, &DummyElement, InitialData.GetTypeSize(), ERDGInitialDataFlags::NoCopy);
	}
	return CreateStructuredBuffer(GraphBuilder, Name, InitialData.GetTypeSize(), InitialData.Num(), InitialData.GetData(), InitialData.GetTotalSize(), ERDGInitialDataFlags::NoCopy);
}

/** Creates a byte address buffer with initial data by creating an upload pass. */
RENDERCORE_API FRDGBufferRef CreateByteAddressBuffer(
	FRDGBuilder& GraphBuilder,
	const TCHAR* Name,
	uint32 NumBytes,
	const void* InitialData,
	uint64 InitialDataSize,
	ERDGInitialDataFlags InitialDataFlags = ERDGInitialDataFlags::None);

/** A variant where NumElements, InitialData, and InitialDataSize are supplied through callbacks. This allows creating a buffer with
 *  information unknown at creation time. Though, data must be ready before the most recent RDG pass that references the buffer
 *  is executed. For byte address buffers, NumElements must be Size / 4.
 */
RENDERCORE_API FRDGBufferRef CreateByteAddressBuffer(
	FRDGBuilder& GraphBuilder,
	const TCHAR* Name,
	FRDGBufferNumElementsCallback&& NumElementsCallback,
	FRDGBufferInitialDataCallback&& InitialDataCallback,
	FRDGBufferInitialDataSizeCallback&& InitialDataSizeCallback);

/**
 * Helper to create a byte address buffer with initial data from a TArray with move semantics, this can be cheaper as it guarantees the lifetimes of the data & permits copy-free upload.
 */
template <typename ElementType, typename AllocatorType>
inline FRDGBufferRef CreateByteAddressBuffer(
	FRDGBuilder& GraphBuilder,
	const TCHAR* Name,
	TArray<ElementType, AllocatorType>&& InitialData)
{
	static const uint32 DummyElement = 0;
	if (InitialData.Num() == 0)
	{
		return CreateByteAddressBuffer(GraphBuilder, Name, 4, &DummyElement, 4, ERDGInitialDataFlags::NoCopy);
	}

	// Create a move-initialized copy of the TArray with RDG lifetime & move the data there.
	TArray<ElementType, AllocatorType>& UploadData = *GraphBuilder.AllocObject<TArray<ElementType, AllocatorType> >(MoveTemp(InitialData));
	return CreateByteAddressBuffer(GraphBuilder, Name, UploadData.Num() * UploadData.GetTypeSize(), UploadData.GetData(), UploadData.Num() * UploadData.GetTypeSize(), ERDGInitialDataFlags::NoCopy);
}

/**
 * Helper to create a byte address buffer with initial data from a TConstArrayView.
 */
template <typename ElementType>
inline FRDGBufferRef CreateByteAddressBuffer(
	FRDGBuilder& GraphBuilder,
	const TCHAR* Name,
	TConstArrayView<ElementType> InitialData,
	ERDGInitialDataFlags InitialDataFlags = ERDGInitialDataFlags::None)
{
	static const ElementType DummyElement = ElementType();
	if (InitialData.Num() == 0)
	{
		return CreateByteAddressBuffer(GraphBuilder, Name, InitialData.GetTypeSize(), &DummyElement, InitialData.GetTypeSize(), ERDGInitialDataFlags::NoCopy);
	}
	return CreateByteAddressBuffer(GraphBuilder, Name, InitialData.Num() * InitialData.GetTypeSize(), InitialData.GetData(), InitialData.Num() * InitialData.GetTypeSize(), InitialDataFlags);
}

/** A variant where the TArray is supplied through callbacks. This allows creating a buffer with
 *  information unknown at creation time. Though, data must be ready before the most recent RDG pass that references the buffer
 *  is executed.
 */
template <typename ArrayType>
inline FRDGBufferRef CreateByteAddressBuffer_Impl(
	FRDGBuilder& GraphBuilder,
	const TCHAR* Name,
	TRDGBufferArrayCallback<ArrayType>&& ArrayCallback)
{
	return CreateByteAddressBuffer(GraphBuilder, Name,
		/*NumElementsCallback = */[ArrayCallback]() { const auto& Array = ArrayCallback(); return Array.Num() * Array.GetTypeSize() / 4; },
		/*InitialDataCallback = */[ArrayCallback]() { return ArrayCallback().GetData(); },
		/*InitialDataSizeCallback = */[ArrayCallback]() { const auto& Array = ArrayCallback(); return Array.Num() * Array.GetTypeSize(); });
}

/** Same as the previous function but where the type of the array is automatically inferred, so we can do : 
 *  TArray<FSomeType> Array = GraphBuilder.AllocArray<...>();
 *  CreateByteAddressBuffer(..., [&]() -> auto&{ return Array; });
 */
template <typename GetArrayRefCallback, typename Type = TInvokeResult_T<GetArrayRefCallback>>
inline FRDGBufferRef CreateByteAddressBuffer(
	FRDGBuilder& GraphBuilder,
	const TCHAR* Name,
	GetArrayRefCallback&& ArrayCallback)
{
	return CreateByteAddressBuffer_Impl<Type>(GraphBuilder, Name, MoveTemp(ArrayCallback));
}

template <typename ElementType>
inline FRDGBufferRef CreateByteAddressBuffer(
	FRDGBuilder& GraphBuilder,
	const TCHAR* Name,
	const FRDGUploadData<ElementType>& InitialData)
{
	static const uint32 DummyElement = 0;
	if (InitialData.Num() == 0)
	{
		return CreateByteAddressBuffer(GraphBuilder, Name, 4, &DummyElement, 4, ERDGInitialDataFlags::NoCopy);
	}
	return CreateByteAddressBuffer(GraphBuilder, Name, InitialData.Num() * InitialData.GetTypeSize(), InitialData.GetData(), InitialData.GetTotalSize(), ERDGInitialDataFlags::NoCopy);
}

RENDERCORE_API FRDGBufferRef CreateUploadBuffer(
	FRDGBuilder& GraphBuilder,
	const TCHAR* Name,
	uint32 BytesPerElement,
	uint32 NumElements,
	const void* InitialData,
	uint64 InitialDataSize,
	ERDGInitialDataFlags InitialDataFlags = ERDGInitialDataFlags::None);

template <typename ElementType>
inline FRDGBufferRef CreateUploadBuffer(
	FRDGBuilder& GraphBuilder,
	const TCHAR* Name,
	uint32 BytesPerElement,
	uint32 NumElements,
	const FRDGUploadData<ElementType>& InitialData)
{
	return CreateUploadBuffer(GraphBuilder, Name, BytesPerElement, NumElements, InitialData.GetData(), InitialData.GetTotalSize(), ERDGInitialDataFlags::NoCopy);
}

template <typename ElementType>
inline FRDGBufferRef CreateUploadBuffer(
	FRDGBuilder& GraphBuilder,
	const TCHAR* Name,
	const FRDGUploadData<ElementType>& InitialData)
{
	return CreateUploadBuffer(GraphBuilder, Name, sizeof(ElementType), InitialData.Num(), InitialData);
}

template <typename ElementType>
inline FRDGBufferRef CreateUploadBuffer(
	FRDGBuilder& GraphBuilder,
	const TCHAR* Name,
	TConstArrayView<ElementType> InitialData,
	ERDGInitialDataFlags InitialDataFlags = ERDGInitialDataFlags::None)
{
	static const uint32 DummyElement = 0;
	if (InitialData.Num() == 0)
	{
		return CreateUploadBuffer(GraphBuilder, Name, 4, 1, &DummyElement, 4, ERDGInitialDataFlags::NoCopy);
	}
	return CreateUploadBuffer(GraphBuilder, Name, sizeof(ElementType), InitialData.Num(), InitialData.GetData(), sizeof(ElementType) * InitialData.Num(), InitialDataFlags);
}

/**
 * Helper to create a structured upload buffer with initial data from a TArray.
 * NOTE: does not provide a 1-size fallback for empty initial data.
 */
template <typename ElementType, typename AllocatorType>
inline FRDGBufferRef CreateStructuredUploadBuffer(
	FRDGBuilder& GraphBuilder,
	const TCHAR* Name,
	const TArray<ElementType, AllocatorType> &InitialData,
	ERDGInitialDataFlags InitialDataFlags = ERDGInitialDataFlags::None)
{
	FRDGBufferRef Buffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateStructuredUploadDesc(sizeof(ElementType), InitialData.Num()), Name);
	GraphBuilder.QueueBufferUpload(Buffer, TConstArrayView<ElementType>(InitialData), InitialDataFlags);
	return Buffer;
}

/**
 * Helper to create a byte address upload buffer with initial data from a TArray.
 * NOTE: does not provide a 1-size fallback for empty initial data.
 */
template <typename ElementType, typename AllocatorType>
inline FRDGBufferRef CreateByteAddressUploadBuffer(
	FRDGBuilder& GraphBuilder,
	const TCHAR* Name,
	const TArray<ElementType, AllocatorType> &InitialData,
	ERDGInitialDataFlags InitialDataFlags = ERDGInitialDataFlags::None)
{
	FRDGBufferRef Buffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateByteAddressUploadDesc(sizeof(ElementType) * InitialData.Num()), Name);
	GraphBuilder.QueueBufferUpload(Buffer, TConstArrayView<ElementType>(InitialData), InitialDataFlags);
	return Buffer;
}

/** A variant where NumElements, InitialData, and InitialDataSize are supplied through callbacks. This allows creating a buffer with
 *  information unknown at creation time. Though, data must be ready before the most recent RDG pass that references the buffer
 *  is executed. For byte address buffers, NumElements must be Size / 4.
 */
inline FRDGBufferRef CreateByteAddressUploadBuffer(
	FRDGBuilder& GraphBuilder,
	const TCHAR* Name,
	FRDGBufferNumElementsCallback&& NumElementsCallback,
	FRDGBufferInitialDataCallback&& InitialDataCallback,
	FRDGBufferInitialDataSizeCallback&& InitialDataSizeCallback)
{
	FRDGBufferRef Buffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateByteAddressUploadDesc(4), Name, MoveTemp(NumElementsCallback));
	GraphBuilder.QueueBufferUpload(Buffer, MoveTemp(InitialDataCallback), MoveTemp(InitialDataSizeCallback));
	return Buffer;
}

/** A variant where the TArray is supplied through callbacks. This allows creating a buffer with
 *  information unknown at creation time. Though, data must be ready before the most recent RDG pass that references the buffer
 *  is executed.
 */
template <typename ArrayType>
inline FRDGBufferRef CreateByteAddressUploadBuffer_Impl(
	FRDGBuilder& GraphBuilder,
	const TCHAR* Name,
	TRDGBufferArrayCallback<ArrayType>&& ArrayCallback)
{
	return CreateByteAddressUploadBuffer(GraphBuilder, Name,
		/*NumElementsCallback = */[ArrayCallback]() { const auto& Array = ArrayCallback(); return Array.Num() * Array.GetTypeSize() / 4; },
		/*InitialDataCallback = */[ArrayCallback]() { return ArrayCallback().GetData(); },
		/*InitialDataSizeCallback = */[ArrayCallback]() { const auto& Array = ArrayCallback(); return Array.Num() * Array.GetTypeSize(); });
}

/** Same as the previous function but where the type of the array is automatically inferred, so we can do :
 *  TArray<FSomeType> Array = GraphBuilder.AllocArray<...>();
 *  CreateByteAddressBuffer(..., [&]() -> auto&{ return Array; });
 */
template <typename GetArrayRefCallback, typename Type = TInvokeResult_T<GetArrayRefCallback>>
inline FRDGBufferRef CreateByteAddressUploadBuffer(
	FRDGBuilder& GraphBuilder,
	const TCHAR* Name,
	GetArrayRefCallback&& ArrayCallback)
{
	return CreateByteAddressUploadBuffer_Impl<Type>(GraphBuilder, Name, MoveTemp(ArrayCallback));
}

/** Creates a vertex buffer with initial data by creating an upload pass. */
RENDERCORE_API FRDGBufferRef CreateVertexBuffer(
	FRDGBuilder& GraphBuilder,
	const TCHAR* Name,
	const FRDGBufferDesc& Desc,
	const void* InitialData,
	uint64 InitialDataSize,
	ERDGInitialDataFlags InitialDataFlags = ERDGInitialDataFlags::None);

/** Helper functions to add parameterless passes to the graph. */
template <typename ExecuteLambdaType>
inline void AddPass(FRDGBuilder& GraphBuilder, FRDGEventName&& Name, ExecuteLambdaType&& ExecuteLambda)
{
	GraphBuilder.AddPass(MoveTemp(Name), ERDGPassFlags::None, MoveTemp(ExecuteLambda));
}

template <typename ExecuteLambdaType>
inline void AddPassIfDebug(FRDGBuilder& GraphBuilder, FRDGEventName&& Name, ExecuteLambdaType&& ExecuteLambda)
{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	AddPass(GraphBuilder, MoveTemp(Name), MoveTemp(ExecuteLambda));
#endif
}

inline void AddDispatchToRHIThreadPass(FRDGBuilder& GraphBuilder)
{
	GraphBuilder.AddDispatchHint();
}

BEGIN_SHADER_PARAMETER_STRUCT(FReadbackTextureParameters, )
	RDG_TEXTURE_ACCESS(Texture, ERHIAccess::CopySrc)
END_SHADER_PARAMETER_STRUCT()

template <typename ExecuteLambdaType>
void AddReadbackTexturePass(FRDGBuilder& GraphBuilder, FRDGEventName&& Name, FRDGTextureRef Texture, ExecuteLambdaType&& ExecuteLambda)
{
	auto* PassParameters = GraphBuilder.AllocParameters<FReadbackTextureParameters>();
	PassParameters->Texture = Texture;
	GraphBuilder.AddPass(MoveTemp(Name), PassParameters, ERDGPassFlags::Readback, MoveTemp(ExecuteLambda));
}

BEGIN_SHADER_PARAMETER_STRUCT(FReadbackBufferParameters, )
	RDG_BUFFER_ACCESS(Buffer, ERHIAccess::CopySrc)
END_SHADER_PARAMETER_STRUCT()

template <typename ExecuteLambdaType>
void AddReadbackBufferPass(FRDGBuilder& GraphBuilder, FRDGEventName&& Name, FRDGBufferRef Buffer, ExecuteLambdaType&& ExecuteLambda)
{
	auto* PassParameters = GraphBuilder.AllocParameters<FReadbackBufferParameters>();
	PassParameters->Buffer = Buffer;
	GraphBuilder.AddPass(MoveTemp(Name), PassParameters, ERDGPassFlags::Readback, MoveTemp(ExecuteLambda));
}

/** Batches up RDG external resource access mode requests and submits them all at once to RDG. */
class FRDGExternalAccessQueue
{
public:
	FRDGExternalAccessQueue() = default;

	~FRDGExternalAccessQueue()
	{
		checkf(IsEmpty(), TEXT("Submit must be called before destruction."));
	}

	void Reserve(uint32 ResourceCount)
	{
		Resources.Reserve(ResourceCount);
	}

	void Add(FRDGViewableResource* Resource, ERHIAccess Access = ERHIAccess::SRVMask, ERHIPipeline Pipelines = ERHIPipeline::Graphics)
	{
		if (!Resource)
		{
			return;
		}

		Validate(Resource, Access, Pipelines);
		Resources.Emplace(Resource, Access, Pipelines);
	}

	void AddUnique(FRDGViewableResource* Resource, ERHIAccess Access = ERHIAccess::SRVMask, ERHIPipeline Pipelines = ERHIPipeline::Graphics)
	{
		if (!Resource)
		{
			return;
		}

		Validate(Resource, Access, Pipelines);

		if (Contains(Resource))
		{
			return;
		}

		Resources.Emplace(Resource, Access, Pipelines);
	}

	RENDERCORE_API void Submit(FRDGBuilder& GraphBuilder);

	bool Contains(FRDGViewableResource* Resource)
	{
		return Resources.ContainsByPredicate([&](const FResource& InResource) { return InResource.Resource == Resource; });
	}

	bool IsEmpty() const
	{
		return Resources.IsEmpty();
	}

private:
	void Validate(FRDGViewableResource* Resource, ERHIAccess Access, ERHIPipeline Pipelines)
	{
		checkf(IsValidAccess(Access) && Access != ERHIAccess::Unknown, TEXT("Attempted to finalize texture %s with an invalid access %s."), Resource->Name, *GetRHIAccessName(Access));
		check(Pipelines != ERHIPipeline::None);
	}

	struct FResource
	{
		FResource() = default;

		FResource(FRDGViewableResource* InResource, ERHIAccess InAccess, ERHIPipeline InPipelines)
			: Resource(InResource)
			, Access(InAccess)
			, Pipelines(InPipelines)
		{}

		FRDGViewableResource* Resource;
		ERHIAccess Access;
		ERHIPipeline Pipelines = ERHIPipeline::None;
	};

	TArray<FResource, FRDGArrayAllocator> Resources;
};

inline const TRefCountPtr<IPooledRenderTarget>& ConvertToExternalAccessTexture(
	FRDGBuilder& GraphBuilder,
	FRDGTexture* Texture,
	ERHIAccess Access = ERHIAccess::SRVMask,
	ERHIPipeline Pipelines = ERHIPipeline::Graphics)
{
	const TRefCountPtr<IPooledRenderTarget>& PooledTexture = GraphBuilder.ConvertToExternalTexture(Texture);
	GraphBuilder.UseExternalAccessMode(Texture, Access, Pipelines);
	return PooledTexture;
}

inline const TRefCountPtr<FRDGPooledBuffer>& ConvertToExternalAccessBuffer(
	FRDGBuilder& GraphBuilder,
	FRDGBuffer* Buffer,
	ERHIAccess Access = ERHIAccess::SRVMask,
	ERHIPipeline Pipelines = ERHIPipeline::Graphics)
{
	const TRefCountPtr<FRDGPooledBuffer>& PooledBuffer = GraphBuilder.ConvertToExternalBuffer(Buffer);
	GraphBuilder.UseExternalAccessMode(Buffer, Access, Pipelines);
	return PooledBuffer;
}

inline const TRefCountPtr<IPooledRenderTarget>& ConvertToExternalAccessTexture(
	FRDGBuilder& GraphBuilder,
	FRDGExternalAccessQueue& ExternalAccessQueue,
	FRDGTexture* Texture,
	ERHIAccess Access = ERHIAccess::SRVMask,
	ERHIPipeline Pipelines = ERHIPipeline::Graphics)
{
	ExternalAccessQueue.Add(Texture, Access, Pipelines);
	return GraphBuilder.ConvertToExternalTexture(Texture);
}

inline const TRefCountPtr<FRDGPooledBuffer>& ConvertToExternalAccessBuffer(
	FRDGBuilder& GraphBuilder,
	FRDGExternalAccessQueue& ExternalAccessQueue,
	FRDGBuffer* Buffer,
	ERHIAccess Access = ERHIAccess::SRVMask,
	ERHIPipeline Pipelines = ERHIPipeline::Graphics)
{
	ExternalAccessQueue.Add(Buffer, Access, Pipelines);
	return GraphBuilder.ConvertToExternalBuffer(Buffer);
}

// Allocates an RDG pooled buffer instance. Attempts to reuse allocation if Out has a value. Returns true a new instance was allocated, or false if the existing allocation was reused.
RENDERCORE_API bool AllocatePooledBuffer(
	const FRDGBufferDesc& Desc,
	TRefCountPtr<FRDGPooledBuffer>& Out,
	const TCHAR* Name,
	ERDGPooledBufferAlignment Alignment = ERDGPooledBufferAlignment::Page);

RENDERCORE_API TRefCountPtr<FRDGPooledBuffer> AllocatePooledBuffer(
	const FRDGBufferDesc& Desc,
	const TCHAR* Name,
	ERDGPooledBufferAlignment Alignment = ERDGPooledBufferAlignment::Page);

// Same as AllocatePooledBuffer but will rebase the allocated buffer under the current LLM tag.
#if ENABLE_LOW_LEVEL_MEM_TRACKER || UE_MEMORY_TRACE_ENABLED
RENDERCORE_API TRefCountPtr<FRDGPooledBuffer> AllocatePooledBufferCurrentLLMTag(
	FRHICommandListBase& RHICmdList,
	const FRDGBufferDesc& Desc,
	const TCHAR* Name,
	ERDGPooledBufferAlignment Alignment = ERDGPooledBufferAlignment::Page);
#else // #if ENABLE_LOW_LEVEL_MEM_TRACKER
inline TRefCountPtr<FRDGPooledBuffer> AllocatePooledBufferCurrentLLMTag(
	FRHICommandListBase& RHICmdList,
	const FRDGBufferDesc& Desc,
	const TCHAR* Name,
	ERDGPooledBufferAlignment Alignment = ERDGPooledBufferAlignment::Page)
{
	return AllocatePooledBuffer(Desc, Name, Alignment);
}
#endif // #if ENABLE_LOW_LEVEL_MEM_TRACKER

RENDERCORE_API bool AllocatePooledTexture(
	const FRDGTextureDesc& Desc,
	TRefCountPtr<IPooledRenderTarget>& Out,
	const TCHAR* Name);

RENDERCORE_API TRefCountPtr<IPooledRenderTarget> AllocatePooledTexture(const FRDGTextureDesc& Desc, const TCHAR* Name);

/*
	Inserts a pass that transitions the resources into the requested state early. This can be used to move barriers to occur earlier in the graph.
	This works if the state matches the state in a later pass. For example:

	Prepare(A)
	Prepare(B)
	AddBarrierPass(A, B) <- Barriers are flushed here.
	Dispatch(A)
	Dispatch(B)

	On platforms without split barriers, the barriers would otherwise be interleaved:

	Prepare(A)
	Prepare(B)
	Dispatch(A) <- Wait for A
	Dispatch(B) <- Wait for B
*/
RENDERCORE_API void AddBarrierPass(FRDGBuilder& GraphBuilder, ERDGPassFlags PassFlags, FRDGTextureAccessArray Textures, FRDGBufferAccessArray Buffers);

inline void AddBarrierPass(FRDGBuilder& GraphBuilder, ERDGPassFlags PassFlags, FRDGBufferAccessArray Buffers)
{
	AddBarrierPass(GraphBuilder, PassFlags, {}, MoveTemp(Buffers));
}

inline void AddBarrierPass(FRDGBuilder& GraphBuilder, ERDGPassFlags PassFlags, FRDGTextureAccessArray Textures)
{
	AddBarrierPass(GraphBuilder, PassFlags, MoveTemp(Textures), {});
}