// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Containers/SparseArray.h"
#include "HAL/Platform.h"
#include "RHI.h"
#include "RenderGraphAllocator.h"
#include "RenderGraphDefinitions.h"
#include "RenderGraphPass.h"
#include "RenderGraphResources.h"

class FRDGBarrierBatchBegin;
class FRDGBarrierBatchEnd;
class FRDGEventName;
class FRDGPass;
class FShaderParametersMetadata;
struct IPooledRenderTarget;
template <typename ReferencedType> class TRefCountPtr;

#if RDG_ENABLE_DEBUG

/** Used by the render graph builder to validate correct usage of the graph API from setup to execution.
 *  Validation is compiled out in shipping builds. This class tracks resources and passes as they are
 *  added to the graph. It will then validate execution of the graph, including whether resources are
 *  used during execution, and that they are properly produced before being consumed. All found issues
 *  must be clear enough to help the user identify the problem in client code. Validation should occur
 *  as soon as possible in the graph lifecycle. It's much easier to catch an issue at the setup location
 *  rather than during deferred execution.
 *
 *  Finally, this class is designed for user validation, not for internal graph validation. In other words,
 *  if the user can break the graph externally via the client-facing API, this validation layer should catch it.
 *  Any internal validation of the graph state should be kept out of this class in order to provide a clear
 *  and modular location to extend the validation layer as well as clearly separate the graph implementation
 *  details from events in the graph.
 */
class FRDGUserValidation final
{
public:
	FRDGUserValidation(FRDGAllocator& Allocator);
	FRDGUserValidation(const FRDGUserValidation&) = delete;
	RENDERCORE_API ~FRDGUserValidation();

	/** Tracks and validates inputs into resource creation functions. */
	RENDERCORE_API void ValidateCreateTexture(const FRDGTextureDesc& Desc, const TCHAR* Name, ERDGTextureFlags Flags);
	RENDERCORE_API void ValidateCreateBuffer(const FRDGBufferDesc& Desc, const TCHAR* Name, ERDGBufferFlags Flags);
	RENDERCORE_API void ValidateCreateSRV(const FRDGTextureSRVDesc& Desc);
	RENDERCORE_API void ValidateCreateSRV(const FRDGBufferSRVDesc& Desc);
	RENDERCORE_API void ValidateCreateUAV(const FRDGTextureUAVDesc& Desc);
	RENDERCORE_API void ValidateCreateUAV(const FRDGBufferUAVDesc& Desc);
	RENDERCORE_API void ValidateCreateUniformBuffer(const void* ParameterStruct, const FShaderParametersMetadata* Metadata);

	/** Tracks and validates the creation of a new externally created / registered resource instances. */
	RENDERCORE_API void ValidateCreateTexture(FRDGTextureRef Texture);
	RENDERCORE_API void ValidateCreateBuffer(FRDGBufferRef Buffer);
	RENDERCORE_API void ValidateCreateSRV(FRDGTextureSRVRef SRV);
	RENDERCORE_API void ValidateCreateSRV(FRDGBufferSRVRef SRV);
	RENDERCORE_API void ValidateCreateUAV(FRDGTextureUAVRef UAV);
	RENDERCORE_API void ValidateCreateUAV(FRDGBufferUAVRef UAV);
	RENDERCORE_API void ValidateCreateUniformBuffer(FRDGUniformBufferRef UniformBuffer);

	RENDERCORE_API void ValidateRegisterExternalTexture(
		const TRefCountPtr<IPooledRenderTarget>& ExternalPooledTexture,
		const TCHAR* Name,
		ERDGTextureFlags Flags);

	RENDERCORE_API void ValidateRegisterExternalBuffer(
		const TRefCountPtr<FRDGPooledBuffer>& ExternalPooledBuffer,
		const TCHAR* Name,
		ERDGBufferFlags Flags);

	RENDERCORE_API void ValidateRegisterExternalTexture(FRDGTextureRef Texture);
	RENDERCORE_API void ValidateRegisterExternalBuffer(FRDGBufferRef Buffer);

	RENDERCORE_API void ValidateUploadBuffer(FRDGBufferRef Buffer, const void* InitialData, uint64 InitialDataSize);
	RENDERCORE_API void ValidateUploadBuffer(FRDGBufferRef Buffer, const void* InitialData, uint64 InitialDataSize, const FRDGBufferInitialDataFreeCallback& InitialDataFreeCallback);
	RENDERCORE_API void ValidateUploadBuffer(FRDGBufferRef Buffer, const FRDGBufferInitialDataFillCallback& InitialDataFillCallback);
	RENDERCORE_API void ValidateUploadBuffer(FRDGBufferRef Buffer, const FRDGBufferInitialDataCallback& InitialDataCallback, const FRDGBufferInitialDataSizeCallback& InitialDataSizeCallback);
	RENDERCORE_API void ValidateUploadBuffer(FRDGBufferRef Buffer, const FRDGBufferInitialDataCallback& InitialDataCallback, const FRDGBufferInitialDataSizeCallback& InitialDataSizeCallback, const FRDGBufferInitialDataFreeCallback& InitialDataFreeCallback);

	RENDERCORE_API void ValidateCommitBuffer(FRDGBufferRef Buffer, uint64 CommitSizeInBytes);

	/** Validates a resource extraction operation. */
	RENDERCORE_API void ValidateExtractTexture(FRDGTextureRef Texture, TRefCountPtr<IPooledRenderTarget>* OutTexturePtr);
	RENDERCORE_API void ValidateExtractBuffer(FRDGBufferRef Buffer, TRefCountPtr<FRDGPooledBuffer>* OutBufferPtr);

	RENDERCORE_API void ValidateConvertToExternalResource(FRDGViewableResource* Resource);
	RENDERCORE_API void ValidateConvertToExternalUniformBuffer(FRDGUniformBuffer* UniformBuffer);

	/** Tracks and validates the addition of a new pass to the graph.
	 *  @param bSkipPassAccessMarking Skips marking the pass as a producer or incrementing the pass access. Useful when
	 *      the builder needs to inject a pass for debugging while preserving error messages and warnings for the original
	 *      graph structure.
	 */
	RENDERCORE_API void ValidateAddPass(const void* ParameterStruct, const FShaderParametersMetadata* Metadata, const FRDGEventName& Name, ERDGPassFlags Flags);
	RENDERCORE_API void ValidateAddPass(const FRDGEventName& Name, ERDGPassFlags Flags);
	RENDERCORE_API void ValidateAddPass(const FRDGPass* Pass);

	/** Validate pass state before and after execution. */
	RENDERCORE_API void ValidateExecutePassBegin(const FRDGPass* Pass);
	RENDERCORE_API void ValidateExecutePassEnd(const FRDGPass* Pass);

	/** Validate graph state before and after execution. */
	RENDERCORE_API void ValidateExecuteBegin();
	RENDERCORE_API void ValidateExecuteEnd();

	/** Attempts to mark a resource for clobbering. If already marked, returns false.  */
	RENDERCORE_API bool TryMarkForClobber(FRDGViewableResource* Resource) const;

	RENDERCORE_API void ValidateGetPooledTexture(FRDGTextureRef Texture) const;
	RENDERCORE_API void ValidateGetPooledBuffer(FRDGBufferRef Buffer) const;

	RENDERCORE_API void ValidateSetAccessFinal(FRDGViewableResource* Resource, ERHIAccess AccessFinal);

	RENDERCORE_API void ValidateAddSubresourceAccess(FRDGViewableResource* Resource, const FRDGSubresourceState& Subresource, ERHIAccess Access);

	RENDERCORE_API void ValidateUseExternalAccessMode(FRDGViewableResource* Resource, ERHIAccess ReadOnlyAccess, ERHIPipeline Pipelines);
	RENDERCORE_API void ValidateUseInternalAccessMode(FRDGViewableResource* Resaource);

	RENDERCORE_API void ValidateExternalAccess(FRDGViewableResource* Resource, ERHIAccess Access, const FRDGPass* Pass);

	/** Traverses all resources in the pass and marks whether they are externally accessible by user pass implementations. */
	static RENDERCORE_API void SetAllowRHIAccess(const FRDGPass* Pass, bool bAllowAccess);

	void SetParallelExecuteEnabled(bool bInParallelExecuteEnabled)
	{
		bParallelExecuteEnabled = bInParallelExecuteEnabled;
	}

private:
	void ValidateCreateViewableResource(FRDGViewableResource* Resource);
	void ValidateCreateResource(FRDGResourceRef Resource);
	void ValidateExtractResource(FRDGViewableResource* Resource);

	FRDGAllocator& Allocator;

	/** Map tracking all active resources in the graph. */
	TSet<FRDGResourceRef, DefaultKeyFuncs<FRDGResourceRef>, FRDGSetAllocator> ResourceMap;

	/** Whether the Execute() has already been called. */
	bool bHasExecuted = false;
	bool bHasExecuteBegun = false;

	bool bParallelExecuteEnabled;

	void ExecuteGuard(const TCHAR* Operation, const TCHAR* ResourceName);
};

/** This class validates and logs barriers submitted by the graph. */
class FRDGBarrierValidation
{
public:
	FRDGBarrierValidation(const FRDGPassRegistry* InPasses, const FRDGEventName& InGraphName);
	FRDGBarrierValidation(const FRDGBarrierValidation&) = delete;

	/** Validates a begin barrier batch just prior to submission to the command list. */
	void ValidateBarrierBatchBegin(const FRDGPass* Pass, const FRDGBarrierBatchBegin& Batch);

	/** Validates an end barrier batch just prior to submission to the command list. */
	void ValidateBarrierBatchEnd(const FRDGPass* Pass, const FRDGBarrierBatchEnd& Batch);

private:
	struct FResourceMap
	{
		TMap<FRDGTextureRef, TArray<FRDGTransitionInfo>> Textures;
		TMap<FRDGBufferRef, FRDGTransitionInfo> Buffers;
		TMap<FRDGViewableResource*, FRHITransientAliasingInfo> Aliases;
	};

	using FBarrierBatchMap = TMap<const FRDGBarrierBatchBegin*, FResourceMap>;

	FBarrierBatchMap BatchMap;

	const FRDGPassRegistry* Passes = nullptr;
	const TCHAR* GraphName = nullptr;
};

#endif
