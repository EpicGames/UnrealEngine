// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "PackedNormal.h"

/**
* Constructs a basis matrix for the axis vectors and returns the sign of the determinant
*
* @param XAxis - x axis (tangent)
* @param YAxis - y axis (binormal)
* @param ZAxis - z axis (normal)
* @return sign of determinant either -1 or +1
*/
inline float GetBasisDeterminantSign(const FVector& XAxis, const FVector& YAxis, const FVector& ZAxis)
{
	FMatrix Basis(
		FPlane(XAxis, 0),
		FPlane(YAxis, 0),
		FPlane(ZAxis, 0),
		FPlane(0, 0, 0, 1)
	);
	// RotDeterminant is the determinant of the upper left 3x3 matrix and equivalent to the full 4x4 determinant for this matrix
	// at one fourth the cost. 
	return (Basis.RotDeterminant() < 0) ? -1.0f : +1.0f;
}

/**
* Constructs a basis matrix for the axis vectors and returns the sign of the determinant
*
* @param XAxis - x axis (tangent)
* @param YAxis - y axis (binormal)
* @param ZAxis - z axis (normal)
* @return sign of determinant either -127 (-1) or +1 (127)
*/
inline int8 GetBasisDeterminantSignByte(const FPackedNormal& XAxis, const FPackedNormal& YAxis, const FPackedNormal& ZAxis)
{
	return GetBasisDeterminantSign(XAxis.ToFVector(), YAxis.ToFVector(), ZAxis.ToFVector()) < 0 ? -127 : 127;
}

/**
 * Given 2 axes of a basis stored as a packed type, regenerates the y-axis tangent vector and scales by z.W
 * @param XAxis - x axis (tangent)
 * @param ZAxis - z axis (normal), the sign of the determinant is stored in ZAxis.W
 * @return y axis (binormal)
 */
template<typename VectorType>
inline FVector GenerateYAxis(const VectorType& XAxis, const VectorType& ZAxis)
{
	static_assert(std::is_same_v<VectorType, FPackedNormal> ||
		std::is_same_v<VectorType, FPackedRGBA16N>, "ERROR: Must be FPackedNormal or FPackedRGBA16N");
	FVector  x = XAxis.ToFVector();
	FVector4 z = ZAxis.ToFVector4();
	return (FVector(z) ^ x) * z.W;
}
