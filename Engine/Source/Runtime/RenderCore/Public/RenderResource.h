// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	RenderResource.h: Render resource definitions.
=============================================================================*/

#pragma once

#include "RHIFwd.h"
#include "RHIShaderPlatform.h"
#include "RHIFeatureLevel.h"
#include "RenderTimer.h"
#include "CoreGlobals.h"

class FRenderCommandPipe;
class FRDGPooledBuffer;
class FResourceArrayInterface;

enum class ERenderResourceState : uint8
{
	Default,
	BatchReleased,
	Deleted,
};

enum class ERayTracingMode : uint8
{
	Disabled,	
	Enabled,
	Dynamic
};

/**
 * A rendering resource which is owned by the rendering thread.
 */
class FRenderResource
{
public:
	////////////////////////////////////////////////////////////////////////////////////

	/** Controls initialization order of render resources. Early engine resources utilize the 'Pre' phase to avoid static init ordering issues. */
	enum class EInitPhase : uint8
	{
		Pre,
		Default,
		MAX
	};

	/** Release all render resources that are currently initialized. */
	static RENDERCORE_API void ReleaseRHIForAllResources();

	/** Initialize all resources initialized before the RHI was initialized. */
	static RENDERCORE_API void InitPreRHIResources();

	/** Reinitializes render resources at a new feature level. */
	static RENDERCORE_API void ChangeFeatureLevel(ERHIFeatureLevel::Type NewFeatureLevel);

	/** Set the scoped (thread local) resource name, used for error reporting */
	static RENDERCORE_API FName SetScopeName(FName Name);

	////////////////////////////////////////////////////////////////////////////////////

	/** Default constructor. */
	RENDERCORE_API FRenderResource();

	/** Constructor when we know what feature level this resource should support */
	RENDERCORE_API FRenderResource(ERHIFeatureLevel::Type InFeatureLevel);

	/** Misc copy/assignment */
	RENDERCORE_API FRenderResource(const FRenderResource&);
	RENDERCORE_API FRenderResource(FRenderResource&&);
	RENDERCORE_API FRenderResource& operator=(const FRenderResource& Other);
	RENDERCORE_API FRenderResource& operator=(FRenderResource&& Other);

	/** Destructor used to catch unreleased resources. */
	RENDERCORE_API virtual ~FRenderResource();

	/**
	 * Initializes the RHI resources used by this resource.
	 * Called when entering the state where both the resource and the RHI have been initialized.
	 * This is only called by the rendering thread.
	 */
	virtual void InitRHI(FRHICommandListBase& RHICmdList) {}

	/**
	 * Releases the RHI resources used by this resource.
	 * Called when leaving the state where both the resource and the RHI have been initialized.
	 * This is only called by the rendering thread.
	 */
	virtual void ReleaseRHI() {}

	/**
	 * Initializes the resource.
	 * This is only called by the rendering thread.
	 */
	RENDERCORE_API virtual void InitResource(FRHICommandListBase& RHICmdList);

	/**
	 * Prepares the resource for deletion.
	 * This is only called by the rendering thread.
	 */
	RENDERCORE_API virtual void ReleaseResource();

	/**
	 * If the resource's RHI resources have been initialized, then release and reinitialize it.  Otherwise, do nothing.
	 * This is only called by the rendering thread.
	 */
	RENDERCORE_API void UpdateRHI(FRHICommandListBase& RHICmdList);

	/** @return The resource's friendly name.  Typically a UObject name. */
	virtual FString GetFriendlyName() const { return TEXT("undefined"); }

	// Accessors.
	inline bool IsInitialized() const { return ListIndex != INDEX_NONE; }

	int32 GetListIndex() const { return ListIndex; }
	EInitPhase GetInitPhase() const { return InitPhase; }

	/** SetOwnerName should be called before BeginInitResource for the owner name to be successfully tracked. */
	void SetOwnerName(FName InOwnerName)
	{
#if RHI_ENABLE_RESOURCE_INFO
		OwnerName = InOwnerName;
#endif
	}

	FName GetOwnerName() const
	{
#if RHI_ENABLE_RESOURCE_INFO
		return OwnerName;
#else
		return NAME_None;
#endif
	}

	void SetResourceName(FName InResourceName)
	{
#if RHI_ENABLE_RESOURCE_INFO
		ResourceName = InResourceName;
#endif
	}

	FName GetResourceName() const
	{
#if RHI_ENABLE_RESOURCE_INFO
		return ResourceName;
#else
		return NAME_None;
#endif
	}

protected:
	// This is used during mobile editor preview refactor, this will eventually be replaced with a parameter to InitRHI() etc..
	void SetFeatureLevel(const FStaticFeatureLevel InFeatureLevel) { FeatureLevel = (ERHIFeatureLevel::Type)InFeatureLevel; }
	const FStaticFeatureLevel GetFeatureLevel() const { return FeatureLevel == ERHIFeatureLevel::Num ? FStaticFeatureLevel(GMaxRHIFeatureLevel) : FeatureLevel; }
	inline bool HasValidFeatureLevel() const { return FeatureLevel < ERHIFeatureLevel::Num; }

	// Helper for submitting a resource array to RHI and freeing eligible CPU memory
	template<typename T>
	FBufferRHIRef CreateRHIBuffer(FRHICommandListBase& RHICmdList, T& InOutResourceObject, uint32 ResourceCount, EBufferUsageFlags InBufferUsageFlags, const TCHAR* InDebugName)
	{
		FBufferRHIRef Buffer;

		FResourceArrayInterface* RESTRICT ResourceArray = InOutResourceObject ? InOutResourceObject->GetResourceArray() : nullptr;
		if (ResourceCount != 0)
		{
			Buffer = CreateRHIBufferInternal(RHICmdList, InDebugName, GetOwnerName(), ResourceCount, InBufferUsageFlags, ResourceArray, InOutResourceObject == nullptr);
		}

		// If the buffer creation emptied the resource array, delete the containing structure as well
		if (ShouldFreeResourceObject(InOutResourceObject, ResourceArray))
		{
			delete InOutResourceObject;
			InOutResourceObject = nullptr;
		}

		return Buffer;
	}

	static RENDERCORE_API FRHICommandListBase& GetImmediateCommandList();

	void SetInitPhase(EInitPhase InInitPhase)
	{
		check(InInitPhase != EInitPhase::MAX);
		check(!IsInitialized());
		InitPhase = InInitPhase;
	}

private:
	static RENDERCORE_API bool ShouldFreeResourceObject(void* ResourceObject, FResourceArrayInterface* ResourceArray);
	static RENDERCORE_API FBufferRHIRef CreateRHIBufferInternal(
		FRHICommandListBase& RHICmdList,
		const TCHAR* InDebugName,
		const FName& InOwnerName,
		uint32 ResourceCount,
		EBufferUsageFlags InBufferUsageFlags,
		FResourceArrayInterface* ResourceArray,
		bool bWithoutNativeResource
	);

#if RHI_ENABLE_RESOURCE_INFO
	FName OwnerName;
	FName ResourceName;
#endif

	int32 ListIndex;
	TEnumAsByte<ERHIFeatureLevel::Type> FeatureLevel;
	EInitPhase InitPhase = EInitPhase::Default;

public:
	ERenderResourceState ResourceState = ERenderResourceState::Default;
};

/**
 * Sends a message to the rendering thread to initialize a resource.
 * This is called in the game thread.
 */
extern RENDERCORE_API void BeginInitResource(FRenderResource* Resource, FRenderCommandPipe* RenderCommandPipe = nullptr);

inline void BeginInitResource(FName OwnerName, FRenderResource* Resource, FRenderCommandPipe* RenderCommandPipe = nullptr)
{
#if RHI_ENABLE_RESOURCE_INFO
	Resource->SetOwnerName(OwnerName);
#endif

	BeginInitResource(Resource, RenderCommandPipe);
}

/**
 * Sends a message to the rendering thread to update a resource.
 * This is called in the game thread.
 */
extern RENDERCORE_API void BeginUpdateResourceRHI(FRenderResource* Resource, FRenderCommandPipe* RenderCommandPipe = nullptr);

/**
 * Sends a message to the rendering thread to release a resource.
 * This is called in the game thread.
 */
extern RENDERCORE_API void BeginReleaseResource(FRenderResource* Resource, FRenderCommandPipe* RenderCommandPipe = nullptr);

/**
* Enables the batching of calls to BeginReleaseResource
* This is called in the game thread.
*/
extern RENDERCORE_API void StartBatchedRelease();

/**
* Disables the batching of calls to BeginReleaseResource
* This is called in the game thread.
*/
extern RENDERCORE_API void EndBatchedRelease();

/**
 * Sends a message to the rendering thread to release a resource, and spins until the rendering thread has processed the message.
 * This is called in the game thread.
 */
extern RENDERCORE_API void ReleaseResourceAndFlush(FRenderResource* Resource);

/** A textures resource. */
class FTexture : public FRenderResource
{
public:

	/** The texture's RHI resource. */
	FTextureRHIRef		TextureRHI;

	/** The sampler state to use for the texture. */
	FSamplerStateRHIRef SamplerStateRHI;

	/** Sampler state to be used in deferred passes when discontinuities in ddx / ddy would cause too blurry of a mip to be used. */
	FSamplerStateRHIRef DeferredPassSamplerStateRHI;

	/** The last time the texture has been bound */
	mutable double		LastRenderTime = -FLT_MAX;

	/** bGreyScaleFormat indicates the texture is actually in R channel but should be read as Grey (replicate R to RGBA)
	 *  this is set from CompressionSettings, not PixelFormat
	 *  this is only used by Editor/Debug shaders, not real game materials, which use SamplerType from MaterialExpressions
	 */
	bool				bGreyScaleFormat = false;

	/**
	 * true if the texture is in the same gamma space as the intended rendertarget (e.g. screenshots).
	 * The texture will have sRGB==false and bIgnoreGammaConversions==true, causing a non-sRGB texture lookup
	 * and no gamma-correction in the shader.
	 * 
	 * This was only ever checked in the Canvas renderer, not the standard Material shader path.
	 * It is no longer set or checked.
	 */
	//UE_DEPRECATED(5.5,"bIgnoreGammaConversions should not be used")
	bool				bIgnoreGammaConversions = false;

	/** 
	 * Is the pixel data in this texture sRGB?
	 **/
	bool				bSRGB = false;

	RENDERCORE_API FTexture();
	RENDERCORE_API virtual ~FTexture();
	RENDERCORE_API FTexture(const FTexture&);
	RENDERCORE_API FTexture(FTexture&&);
	RENDERCORE_API FTexture& operator=(const FTexture& Other);
	RENDERCORE_API FTexture& operator=(FTexture&& Other);

	const FTextureRHIRef& GetTextureRHI() { return TextureRHI; }

	/** Returns the width of the texture in pixels. */
	RENDERCORE_API virtual uint32 GetSizeX() const;

	/** Returns the height of the texture in pixels. */
	RENDERCORE_API virtual uint32 GetSizeY() const;

	/** Returns the depth of the texture in pixels. */
	RENDERCORE_API virtual uint32 GetSizeZ() const;

	// FRenderResource interface.
	RENDERCORE_API virtual void ReleaseRHI() override;
	RENDERCORE_API virtual FString GetFriendlyName() const override;

protected:
	static RENDERCORE_API FRHISamplerState* GetOrCreateSamplerState(const FSamplerStateInitializerRHI& Initializer);
};

/** A textures resource that includes an SRV. */
class FTextureWithSRV : public FTexture
{
public:
	RENDERCORE_API FTextureWithSRV();
	RENDERCORE_API virtual ~FTextureWithSRV();

	RENDERCORE_API virtual void ReleaseRHI() override;

	/** SRV that views the entire texture */
	FShaderResourceViewRHIRef ShaderResourceViewRHI;
};

/** A texture reference resource. */
class FTextureReference : public FRenderResource
{
public:
	/** The texture reference's RHI resource. */
	FTextureReferenceRHIRef	TextureReferenceRHI;

private:
	/** True if the texture reference has been initialized from the game thread. */
	bool bInitialized_GameThread;

public:
	/** Default constructor. */
	RENDERCORE_API FTextureReference();

	// Destructor
	RENDERCORE_API virtual ~FTextureReference();

	/** Returns the last time the texture has been rendered via this reference. */
	RENDERCORE_API double GetLastRenderTime() const;

	/** Invalidates the last render time. */
	RENDERCORE_API void InvalidateLastRenderTime();

	/** Returns true if the texture reference has been initialized from the game thread. */
	bool IsInitialized_GameThread() const { return bInitialized_GameThread; }

	/** Kicks off the initialization process on the game thread. */
	RENDERCORE_API void BeginInit_GameThread();

	/** Kicks off the release process on the game thread. */
	RENDERCORE_API void BeginRelease_GameThread();

	// FRenderResource interface.
	RENDERCORE_API virtual void InitRHI(FRHICommandListBase& RHICmdList);
	RENDERCORE_API virtual void ReleaseRHI();
	RENDERCORE_API virtual FString GetFriendlyName() const;
};

/** A vertex buffer resource */
class FVertexBuffer : public FRenderResource
{
public:
	RENDERCORE_API FVertexBuffer();
	RENDERCORE_API FVertexBuffer(const FVertexBuffer&);
	RENDERCORE_API FVertexBuffer& operator=(const FVertexBuffer& Other);
	RENDERCORE_API virtual ~FVertexBuffer();

	// FRenderResource interface.
	RENDERCORE_API virtual void ReleaseRHI() override;
	RENDERCORE_API virtual FString GetFriendlyName() const override;

	const FBufferRHIRef& GetRHI() const { return VertexBufferRHI; }

	RENDERCORE_API void SetRHI(const FBufferRHIRef& BufferRHI);

	FBufferRHIRef VertexBufferRHI;
};

class FVertexBufferWithSRV : public FVertexBuffer
{
public:
	RENDERCORE_API FVertexBufferWithSRV();
	RENDERCORE_API ~FVertexBufferWithSRV();

	RENDERCORE_API virtual void ReleaseRHI() override;

	/** SRV that views the entire texture */
	FShaderResourceViewRHIRef ShaderResourceViewRHI;

	/** *optional* UAV that views the entire texture */
	FUnorderedAccessViewRHIRef UnorderedAccessViewRHI;
};

/** An index buffer resource. */
class FIndexBuffer : public FRenderResource
{
public:
	RENDERCORE_API FIndexBuffer();
	RENDERCORE_API FIndexBuffer(const FIndexBuffer&);
	RENDERCORE_API FIndexBuffer& operator=(const FIndexBuffer& Other);
	RENDERCORE_API virtual ~FIndexBuffer();

	// FRenderResource interface.
	RENDERCORE_API virtual void ReleaseRHI() override;
	RENDERCORE_API virtual FString GetFriendlyName() const override;

	const FBufferRHIRef& GetRHI() const { return IndexBufferRHI; }

	RENDERCORE_API void SetRHI(const FBufferRHIRef& BufferRHI);

	FBufferRHIRef IndexBufferRHI;
};

class FBufferWithRDG : public FRenderResource
{
public:
	RENDERCORE_API FBufferWithRDG();
	RENDERCORE_API FBufferWithRDG(const FBufferWithRDG& Other);
	RENDERCORE_API FBufferWithRDG& operator=(const FBufferWithRDG& Other);
	RENDERCORE_API ~FBufferWithRDG() override;

	RENDERCORE_API void ReleaseRHI() override;

	TRefCountPtr<FRDGPooledBuffer> Buffer;
};

/** Used to declare a render resource that is initialized/released by static initialization/destruction. */
template<class ResourceType, FRenderResource::EInitPhase InInitPhase = FRenderResource::EInitPhase::Default>
class TGlobalResource final : public ResourceType
{
public:
	/** Default constructor. */
	TGlobalResource()
	{
		InitGlobalResource();
	}

	/** Initialization constructor: 1 parameter. */
	template<typename... Args>
	explicit TGlobalResource(Args... InArgs)
		: ResourceType(InArgs...)
	{
		InitGlobalResource();
	}

	/** Destructor. */
	virtual ~TGlobalResource()
	{
		ReleaseGlobalResource();
	}

private:

	/**
	 * Initialize the global resource.
	 */
	void InitGlobalResource()
	{
		ResourceType::SetInitPhase(InInitPhase);

		if (IsInRenderingThread())
		{
			// If the resource is constructed in the rendering thread, directly initialize it.
			((ResourceType*)this)->InitResource(FRenderResource::GetImmediateCommandList());
		}
		else
		{
			// If the resource is constructed outside of the rendering thread, enqueue a command to initialize it.
			BeginInitResource((ResourceType*)this);
		}
	}

	/**
	 * Release the global resource.
	 */
	void ReleaseGlobalResource()
	{
		// This should be called in the rendering thread, or at shutdown when the rendering thread has exited.
		// However, it may also be called at shutdown after an error, when the rendering thread is still running.
		// To avoid a second error in that case we don't assert.
#if 0
		check(IsInRenderingThread());
#endif

		// Cleanup the resource.
		((ResourceType*)this)->ReleaseResource();
	}
};

/** Helper for scoped render resource names */
struct FRenderResourceNameScope
{
	FRenderResourceNameScope(FName Name)
	{
		PreviousName = FRenderResource::SetScopeName(Name);
	}

	~FRenderResourceNameScope()
	{
		FRenderResource::SetScopeName(PreviousName);
	}

private:
	FName PreviousName;
};
