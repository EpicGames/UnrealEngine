// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	RenderTargetPool.h: Scene render target pool manager.
=============================================================================*/

#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "HAL/PlatformAtomics.h"
#include "RHI.h"
#include "RenderGraphDefinitions.h"
#include "RenderGraphResources.h"
#include "RenderResource.h"
#include "RendererInterface.h"
#include "Templates/RefCounting.h"
#include "Async/RecursiveMutex.h"

class FOutputDevice;
class FRHICommandList;
class FRenderTargetPool;

/** The reference to a pooled render target, use like this: TRefCountPtr<IPooledRenderTarget> */
struct FPooledRenderTarget final : public IPooledRenderTarget
{
	FPooledRenderTarget(FRHITexture* Texture, const FPooledRenderTargetDesc& InDesc, FRenderTargetPool* InRenderTargetPool) 
		: RenderTargetPool(InRenderTargetPool)
		, Desc(InDesc)
		, PooledTexture(Texture)
	{
		RenderTargetItem.TargetableTexture = RenderTargetItem.ShaderResourceTexture = Texture;
	}

	uint32 GetUnusedForNFrames() const 
	{ 
		return UnusedForNFrames; 
	}

	const FPooledRenderTargetDesc& GetDesc() const override { return Desc; }

	uint32 AddRef() const override
	{
		return NumRefs.fetch_add(1, std::memory_order_relaxed) + 1;
	}

	uint32 Release() override
	{
		const uint32 OldNumRefs = NumRefs.fetch_sub(1, std::memory_order_release);
		check(OldNumRefs > 0);
		if (OldNumRefs == 1)
		{
			std::atomic_thread_fence(std::memory_order_acquire);
			delete this;
		}
		return OldNumRefs - 1;
	}

	uint32 GetRefCount() const override
	{
		return NumRefs.load(std::memory_order_relaxed);
	}

	RENDERCORE_API bool IsFree() const override;
	bool IsTracked() const override { return RenderTargetPool != nullptr; }
	RENDERCORE_API uint32 ComputeMemorySize() const override;

private:
	RENDERCORE_API void SetDebugLabelName(FRHICommandListBase& RHICmdList, const TCHAR* Name);

	/** Pointer back to the pool for render targets which are actually pooled, otherwise NULL. */
	FRenderTargetPool* RenderTargetPool;
	
	/** All necessary data to create the render target */
	FPooledRenderTargetDesc Desc;

	/** For pool management (only if NumRef == 0 the element can be reused) */
	mutable std::atomic_uint32_t NumRefs{0};

	/** Allows to defer the release to save performance on some hardware (DirectX) */
	uint32 UnusedForNFrames = 0;

	/** Pooled textures for use with RDG. */
	FRDGPooledTexture PooledTexture;

	/** @return true:release this one, false otherwise */
	RENDERCORE_API bool OnFrameStart();

	friend class FRDGTexture;
	friend class FRDGBuilder;
	friend class FRenderTargetPool;
};

/**
 * Encapsulates the render targets pools that allows easy sharing (mostly used on the render thread side)
 */
class FRenderTargetPool : public FRenderResource
{
public:
	FRenderTargetPool() = default;

	RENDERCORE_API TRefCountPtr<IPooledRenderTarget> FindFreeElement(FRHICommandListBase& RHICmdList, FRHITextureCreateInfo Desc, const TCHAR* Name);

	RENDERCORE_API bool FindFreeElement(FRHICommandListBase& RHICmdList, const FRHITextureCreateInfo& Desc, TRefCountPtr<IPooledRenderTarget>& Out, const TCHAR* Name);

	TRefCountPtr<IPooledRenderTarget> FindFreeElement(FRHITextureCreateInfo Desc, const TCHAR* Name)
	{
		return FindFreeElement(FRHICommandListImmediate::Get(), Desc, Name);
	}

	bool FindFreeElement(const FRHITextureCreateInfo& Desc, TRefCountPtr<IPooledRenderTarget>& Out, const TCHAR* Name)
	{
		return FindFreeElement(FRHICommandListImmediate::Get(), Desc, Out, Name);
	}

	/**
	 * @param DebugName must not be 0, we only store the pointer
	 * @param Out is not the return argument to avoid double allocation because of wrong reference counting
	 * call from RenderThread only
	 * @return true if the old element was still valid, false if a new one was assigned
	 */
	bool FindFreeElement(
		FRHICommandListBase& RHICmdList,
		const FPooledRenderTargetDesc& Desc,
		TRefCountPtr<IPooledRenderTarget>& Out,
		const TCHAR* InDebugName)
	{
		return FindFreeElement(RHICmdList, Translate(Desc), Out, InDebugName);
	}

	RENDERCORE_API void CreateUntrackedElement(const FPooledRenderTargetDesc& Desc, TRefCountPtr<IPooledRenderTarget>& Out, const FSceneRenderTargetItem& Item);

	/** Only to get statistics on usage and free elements. Normally only called in renderthread or if FlushRenderingCommands was called() */
	RENDERCORE_API void GetStats(uint32& OutWholeCount, uint32& OutWholePoolInKB, uint32& OutUsedInKB) const;
	/**
	 * Can release RT, should be called once per frame.
	 * call from RenderThread only
	 */
	RENDERCORE_API void TickPoolElements();
	/** Free renderer resources */
	RENDERCORE_API void ReleaseRHI();

	/** Allows to remove a resource so it cannot be shared and gets released immediately instead a/some frame[s] later. */
	RENDERCORE_API void FreeUnusedResource(TRefCountPtr<IPooledRenderTarget>& In);

	/** Good to call between levels or before memory intense operations. */
	RENDERCORE_API void FreeUnusedResources();

	// for debugging purpose, assumes you call FlushRenderingCommands() be
	// @return can be 0, that doesn't mean iteration is done
	RENDERCORE_API FPooledRenderTarget* GetElementById(uint32 Id) const;

	uint32 GetElementCount() const { return PooledRenderTargets.Num(); }

	// @return -1 if not found
	RENDERCORE_API int32 FindIndex(IPooledRenderTarget* In) const;

	// Logs out usage information.
	RENDERCORE_API void DumpMemoryUsage(FOutputDevice& OutputDevice);

private:
	void FreeElementAtIndex(int32 Index);

	FPooledRenderTarget* CreateRenderTarget(FRHICommandListBase& RHICmdList, const FRHITextureCreateInfo& Desc, uint32 DescHash, const TCHAR* Name);

	template <typename T>
	FPooledRenderTarget* TryFindRenderTarget(const FRHITextureCreateInfo& Desc, uint32 DescHash, T&& Predicate) const;

	FPooledRenderTarget* TryFindRenderTarget(const FRHITextureCreateInfo& Desc, uint32 DescHash) const
	{
		return TryFindRenderTarget(Desc, DescHash, [](FPooledRenderTarget*) { return true; });
	}

	//////////////////////////////////////////////////////////////////////////
	// Methods for scheduling allocations for RDG

	FPooledRenderTarget* ScheduleAllocation(
		FRHICommandListBase& RHICmdList,
		FRHITextureCreateInfo Desc,
		const TCHAR* Name,
		const FRHITransientAllocationFences& Fences);

	void ScheduleDeallocation(FPooledRenderTarget* RenderTarget, const FRHITransientAllocationFences& Fences);

	void FinishSchedule(FRHICommandListBase& RHICmdList, FPooledRenderTarget* RenderTarget, const TCHAR* Name);

	//////////////////////////////////////////////////////////////////////////

	mutable UE::FRecursiveMutex Mutex;

	/** Elements can be 0, we compact the buffer later. */
	TArray<uint32> PooledRenderTargetHashes;
	TArray< TRefCountPtr<FPooledRenderTarget> > PooledRenderTargets;
	TArray< TRefCountPtr<FPooledRenderTarget> > DeferredDeleteArray;

	// redundant, can always be computed with GetStats(), to debug "out of memory" situations and used for r.RenderTargetPoolMin
	uint32 AllocationLevelInKB = 0;

	// to avoid log spam
	bool bCurrentlyOverBudget = false;

	// could be done on the fly but that makes the RenderTargetPoolEvents harder to read
	void CompactPool();

	friend struct FPooledRenderTarget;
	friend class FVisualizeTexture;
	friend class FVisualizeTexturePresent;
	friend class FRDGBuilder;
};

/** The global render targets for easy shading. */
extern RENDERCORE_API TGlobalResource<FRenderTargetPool> GRenderTargetPool;
