// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/PlatformTime.h"

/** How many cycles the renderthread used (excluding idle time). It's set once per frame in FViewport::Draw. */
extern RENDERCORE_API uint32 GRenderThreadTime;

/** How many cycles the renderthread was waiting. It's set once per frame in FViewport::Draw. */
extern RENDERCORE_API uint32 GRenderThreadWaitTime;

/** How many cycles the rhithread used (excluding idle time). */
extern RENDERCORE_API uint32 GRHIThreadTime;

/** How many cycles the gamethread used (excluding idle time). It's set once per frame in FViewport::Draw. */
extern RENDERCORE_API uint32 GGameThreadTime;

/** How much idle time in the game thread. It's set once per frame in FViewport::Draw. */
extern RENDERCORE_API uint32 GGameThreadWaitTime;

/** How many cycles it took to swap buffers to present the frame. */
extern RENDERCORE_API uint32 GSwapBufferTime;

/** How many cycles the gamethread used, including dependent wait time. */
extern RENDERCORE_API uint32 GGameThreadTimeCriticalPath;

/** How many cycles the renderthread used, including dependent wait time. */
extern RENDERCORE_API uint32 GRenderThreadTimeCriticalPath;
