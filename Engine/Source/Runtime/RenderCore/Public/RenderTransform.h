// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Math/Box.h"
#include "Math/BoxSphereBounds.h"
#include "Math/Float32.h"
#include "Math/Matrix.h"
#include "Math/NumericLimits.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector.h"
#include "Math/Vector2D.h"
#include "Math/VectorRegister.h"
#include "Serialization/Archive.h"
#include "Serialization/MemoryLayout.h"
#include "Serialization/StructuredArchiveAdapters.h"

// TODO: Further compress data size with tighter encoding
// LWC_TODO: Rebasing support (no 64bit types in here)
// TODO: Optimization (avoid full 4x4 math)
struct FRenderTransform
{
	FVector3f TransformRows[3];
	FVector3f Origin;

public:
	FRenderTransform() = default;
	FRenderTransform(FRenderTransform&&) = default;
	FRenderTransform(const FRenderTransform&) = default;
	FRenderTransform& operator=(FRenderTransform&&) = default;
	FRenderTransform& operator=(const FRenderTransform&) = default;

	inline FRenderTransform(const FVector3f& InXAxis, const FVector3f& InYAxis, const FVector3f& InZAxis, const FVector3f& InOrigin)
	{
		TransformRows[0] = InXAxis;
		TransformRows[1] = InYAxis;
		TransformRows[2] = InZAxis;
		Origin = InOrigin;
	}

	inline FRenderTransform(const FMatrix44f& M)
	{
		TransformRows[0]	= FVector3f(M.M[0][0], M.M[0][1], M.M[0][2]);
		TransformRows[1]	= FVector3f(M.M[1][0], M.M[1][1], M.M[1][2]);
		TransformRows[2]	= FVector3f(M.M[2][0], M.M[2][1], M.M[2][2]);
		Origin				= FVector3f(M.M[3][0], M.M[3][1], M.M[3][2]);
	}

	inline FRenderTransform(const FMatrix44d& M)
	{
		// LWC_TODO: Precision loss
		TransformRows[0] = FVector3f((float)M.M[0][0], (float)M.M[0][1], (float)M.M[0][2]);
		TransformRows[1] = FVector3f((float)M.M[1][0], (float)M.M[1][1], (float)M.M[1][2]);
		TransformRows[2] = FVector3f((float)M.M[2][0], (float)M.M[2][1], (float)M.M[2][2]);
		Origin = FVector3f((float)M.M[3][0], (float)M.M[3][1], (float)M.M[3][2]);
	}

	inline FRenderTransform& operator=(const FMatrix44f& From)
	{
		TransformRows[0]	= FVector3f(From.M[0][0], From.M[0][1], From.M[0][2]);
		TransformRows[1]	= FVector3f(From.M[1][0], From.M[1][1], From.M[1][2]);
		TransformRows[2]	= FVector3f(From.M[2][0], From.M[2][1], From.M[2][2]);
		Origin				= FVector3f(From.M[3][0], From.M[3][1], From.M[3][2]);
		return *this;
	}

	inline FRenderTransform& operator=(const FMatrix44d& From)
	{
		// LWC_TODO: Precision loss
		TransformRows[0] = FVector3f((float)From.M[0][0], (float)From.M[0][1], (float)From.M[0][2]);
		TransformRows[1] = FVector3f((float)From.M[1][0], (float)From.M[1][1], (float)From.M[1][2]);
		TransformRows[2] = FVector3f((float)From.M[2][0], (float)From.M[2][1], (float)From.M[2][2]);
		Origin = FVector3f((float)From.M[3][0], (float)From.M[3][1], (float)From.M[3][2]);
		return *this;
	}

	inline bool Equals(const FRenderTransform& Other, float Tolerance = UE_KINDA_SMALL_NUMBER) const
	{
		return
			TransformRows[0].Equals(Other.TransformRows[0], Tolerance) &&
			TransformRows[1].Equals(Other.TransformRows[1], Tolerance) &&
			TransformRows[2].Equals(Other.TransformRows[2], Tolerance) &&
			Origin.Equals(Other.Origin, Tolerance);
	}

	inline FMatrix44f ToMatrix44f() const
	{
		FMatrix44f Matrix;
	#if PLATFORM_ENABLE_VECTORINTRINSICS
		VectorRegister4Float Row0 = VectorSet_W0(VectorLoad(&TransformRows[0].X)); // Intentionally loads W from the next row (then set to 0)
		VectorRegister4Float Row1 = VectorSet_W0(VectorLoad(&TransformRows[1].X)); // Intentionally loads W from the next row (then set to 0)
		VectorRegister4Float Row2 = VectorSet_W0(VectorLoad(&TransformRows[2].X)); // Intentionally loads W from the next row (then set to 0)
		VectorRegister4Float Row3 = MakeVectorRegisterFloat(Origin.X, Origin.Y, Origin.Z, 1.0f); // Do not read past Origin

		float* RESTRICT Dest = &Matrix.M[0][0];
		VectorStore(Row0, &Dest[0]);
		VectorStore(Row1, &Dest[4]);
		VectorStore(Row2, &Dest[8]);
		VectorStore(Row3, &Dest[12]);
	#else
		Matrix.M[0][0] = TransformRows[0].X;
		Matrix.M[0][1] = TransformRows[0].Y;
		Matrix.M[0][2] = TransformRows[0].Z;
		Matrix.M[0][3] = 0.0f;
		Matrix.M[1][0] = TransformRows[1].X;
		Matrix.M[1][1] = TransformRows[1].Y;
		Matrix.M[1][2] = TransformRows[1].Z;
		Matrix.M[1][3] = 0.0f;
		Matrix.M[2][0] = TransformRows[2].X;
		Matrix.M[2][1] = TransformRows[2].Y;
		Matrix.M[2][2] = TransformRows[2].Z;
		Matrix.M[2][3] = 0.0f;
		Matrix.M[3][0] = Origin.X;
		Matrix.M[3][1] = Origin.Y;
		Matrix.M[3][2] = Origin.Z;
		Matrix.M[3][3] = 1.0f;
	#endif
		return Matrix;
	}

	inline FMatrix ToMatrix() const
	{
		return (FMatrix)ToMatrix44f();
	}

	inline void To3x4MatrixTranspose(float* Result) const
	{
		float* RESTRICT Dest = Result;

	#if PLATFORM_ENABLE_VECTORINTRINSICS
		VectorRegister4Float InRow0 = VectorLoad(&TransformRows[0].X); // Intentionally loads W from the next row (discarded)
		VectorRegister4Float InRow1 = VectorLoad(&TransformRows[1].X); // Intentionally loads W from the next row (discarded)
		VectorRegister4Float InRow2 = VectorLoad(&TransformRows[2].X); // Intentionally loads W from the next row (discarded)
		VectorRegister4Float InRow3 = MakeVectorRegisterFloat(Origin.X, Origin.Y, Origin.Z, 0.0f); // Do not read past Origin

		VectorRegister4Float Temp0 = VectorCombineLow( InRow0, InRow1);
		VectorRegister4Float Temp1 = VectorCombineLow( InRow2, InRow3);
		VectorRegister4Float Temp2 = VectorCombineHigh(InRow0, InRow1);
		VectorRegister4Float Temp3 = VectorCombineHigh(InRow2, InRow3);

		VectorRegister4Float Row0, Row1, Row2, Row3;
		VectorDeinterleave(Row0, Row1, Temp0, Temp1);
		VectorDeinterleave(Row2, Row3, Temp2, Temp3);

		VectorStore(Row0, &Dest[0]);
		VectorStore(Row1, &Dest[4]);
		VectorStore(Row2, &Dest[8]);
	#else
		Dest[ 0] = TransformRows[0].X;	// [0][0]
		Dest[ 1] = TransformRows[1].X;	// [1][0]
		Dest[ 2] = TransformRows[2].X;	// [2][0]
		Dest[ 3] = Origin.X;			// [3][0]

		Dest[ 4] = TransformRows[0].Y;	// [0][1]
		Dest[ 5] = TransformRows[1].Y;	// [1][1]
		Dest[ 6] = TransformRows[2].Y;	// [2][1]
		Dest[ 7] = Origin.Y;			// [3][1]

		Dest[ 8] = TransformRows[0].Z;	// [0][2]
		Dest[ 9] = TransformRows[1].Z;	// [1][2]
		Dest[10] = TransformRows[2].Z;	// [2][2]
		Dest[11] = Origin.Z;			// [3][2]
	#endif
	}

	inline FRenderTransform operator* (const FRenderTransform& Other) const
	{
		// Use vectorized 4x4 implementation
		const FMatrix44f LHS = ToMatrix44f();
		const FMatrix44f RHS = Other.ToMatrix44f();
		return (LHS * RHS);
	}

	inline FRenderTransform operator* (const FMatrix44f& Other) const
	{
		// Use vectorized 4x4 implementation
		const FMatrix44f LHS = ToMatrix44f();
		return (LHS * Other);
	}

	inline float RotDeterminant() const
	{
		return
			TransformRows[0].X * (TransformRows[1].Y * TransformRows[2].Z - TransformRows[1].Z * TransformRows[2].Y) -
			TransformRows[1].X * (TransformRows[0].Y * TransformRows[2].Z - TransformRows[0].Z * TransformRows[2].Y) +
			TransformRows[2].X * (TransformRows[0].Y * TransformRows[1].Z - TransformRows[0].Z * TransformRows[1].Y);
	}

	inline FRenderTransform Inverse() const
	{
		// Use vectorized 4x4 implementation
		return ToMatrix44f().Inverse();
	}

	inline FRenderTransform InverseFast() const
	{
		// Use vectorized 4x4 implementation
		return ToMatrix44f().InverseFast();
	}

	inline FVector3f GetScale() const
	{
		// Extract per axis scales
		FVector3f Scale;
		Scale.X = TransformRows[0].Size();
		Scale.Y = TransformRows[1].Size();
		Scale.Z = TransformRows[2].Size();
		return Scale;
	}

	inline bool IsScaleNonUniform() const
	{
		const FVector3f Scale = GetScale();
		return
		(
			!FMath::IsNearlyEqual(Scale.X, Scale.Y) ||
			!FMath::IsNearlyEqual(Scale.X, Scale.Z) ||
			!FMath::IsNearlyEqual(Scale.Y, Scale.Z)
		);
	}

	inline void Orthogonalize()
	{
		FVector3f X = TransformRows[0];
		FVector3f Y = TransformRows[1];
		FVector3f Z = TransformRows[2];

		if (X.IsZero() || Y.IsZero())
		{
			return;
		}

		// Modified Gram-Schmidt orthogonalization
		Y -= (Y | X) / (X | X) * X;
		Z -= (Z | X) / (X | X) * X;
		Z -= (Z | Y) / (Y | Y) * Y;

		TransformRows[0] = X;
		TransformRows[1] = Y;
		TransformRows[2] = Z;
	}

	inline void SetIdentity()
	{
		TransformRows[0] = FVector3f(1.0f, 0.0f, 0.0f);
		TransformRows[1] = FVector3f(0.0f, 1.0f, 0.0f);
		TransformRows[2] = FVector3f(0.0f, 0.0f, 1.0f);
		Origin = FVector3f::ZeroVector;
	}

	/**
	 * Serializes the render transform.
	 *
	 * @param Ar Reference to the serialization archive.
	 * @param T Reference to the render transform being serialized.
	 * @return Reference to the Archive after serialization.
	 */
	inline friend FArchive& operator<< (FArchive& Ar, FRenderTransform& T)
	{
		Ar << T.TransformRows[0].X << T.TransformRows[0].Y << T.TransformRows[0].Z;
		Ar << T.TransformRows[1].X << T.TransformRows[1].Y << T.TransformRows[1].Z;
		Ar << T.TransformRows[2].X << T.TransformRows[2].Y << T.TransformRows[2].Z;
		Ar << T.Origin.X << T.Origin.Y << T.Origin.Z;
		return Ar;
	}

	RENDERCORE_API static FRenderTransform Identity;
};

struct FRenderBounds
{
	FVector3f Min;
	FVector3f Max;

public:
	inline FRenderBounds()
	: Min( MAX_flt,  MAX_flt,  MAX_flt)
	, Max(-MAX_flt, -MAX_flt, -MAX_flt)
	{
	}

	inline FRenderBounds(const FVector3f& InMin, const FVector3f& InMax)
	: Min(InMin)
	, Max(InMax)
	{
	}

	inline FRenderBounds(const FBox& Box)
	{
		Min = FVector3f(Box.Min);	//LWC_TODO: Precision loss
		Max = FVector3f(Box.Max);	//LWC_TODO: Precision loss
	}

	inline FRenderBounds(const FBoxSphereBounds& Bounds)
	{
		Min = FVector3f(Bounds.Origin - Bounds.BoxExtent);	//LWC_TODO: Precision loss
		Max = FVector3f(Bounds.Origin + Bounds.BoxExtent);	//LWC_TODO: Precision loss
	}

	inline FBox ToBox() const
	{
		return FBox(FVector(Min), FVector(Max));
	}

	inline FBoxSphereBounds ToBoxSphereBounds() const
	{
		return FBoxSphereBounds(ToBox());
	}

	bool operator==(const FRenderBounds& Other) const = default;

	inline FRenderBounds& operator = (const FVector3f& Other)
	{
		Min = Other;
		Max = Other;
		return *this;
	}

	inline FRenderBounds& operator += (const FVector3f& Other)
	{
		const VectorRegister VecOther = VectorLoadFloat3(&Other.X);
		VectorStoreFloat3(VectorMin(VectorLoadFloat3(&Min.X), VecOther), &Min);
		VectorStoreFloat3(VectorMax(VectorLoadFloat3(&Max.X), VecOther), &Max);
		return *this;
	}

	inline FRenderBounds& operator += (const FRenderBounds& Other)
	{
		VectorStoreFloat3(VectorMin(VectorLoadFloat3(&Min), VectorLoadFloat3(&Other.Min)), &Min);
		VectorStoreFloat3(VectorMax(VectorLoadFloat3(&Max), VectorLoadFloat3(&Other.Max)), &Max);
		return *this;
	}

	inline FRenderBounds operator+ (const FRenderBounds& Other) const
	{
		return FRenderBounds(*this) += Other;
	}

	inline bool Equals(const FRenderBounds& Other, float Tolerance = UE_KINDA_SMALL_NUMBER) const
	{
		return Min.Equals(Other.Min, Tolerance) && Max.Equals(Other.Max, Tolerance);
	}

	inline const FVector3f& GetMin() const
	{
		return Min;
	}

	inline const FVector3f& GetMax() const
	{
		return Max;
	}

	inline FVector3f GetCenter() const
	{
		return (Max + Min) * 0.5f;
	}

	inline FVector3f GetExtent() const
	{
		return (Max - Min) * 0.5f;
	}

	inline float GetSurfaceArea() const
	{
		FVector3f Size = Max - Min;
		return 0.5f * (Size.X * Size.Y + Size.X * Size.Z + Size.Y * Size.Z);
	}

	/**
	 * Gets a bounding volume transformed by a matrix.
	 *
	 * @param M The matrix.
	 * @return The transformed volume.
	 */
	RENDERCORE_API FRenderBounds TransformBy(const FMatrix44f& M) const;
	RENDERCORE_API FRenderBounds TransformBy(const FMatrix44d& M) const;

	/**
	 * Gets a bounding volume transformed by a render transform.
	 *
	 * @param T The render transform.
	 * @return The transformed volume.
	 */
	RENDERCORE_API FRenderBounds TransformBy(const FRenderTransform& T) const;

	/**
	 * Computes a squared distance to point
	 *
	 * @param Point Point
	 * @return Squared distance to specified point
	 */
	inline float ComputeSquaredDistanceToPoint(const FVector3f& Point) const
	{
		FVector3f AxisDistances = (Point - GetCenter()).GetAbs() - GetExtent();
		AxisDistances = FVector3f::Max(AxisDistances, FVector3f(0.0f, 0.0f, 0.0f));
		return AxisDistances | AxisDistances;
	}

	/**
	 * Serializes the render bounds.
	 *
	 * @param Ar Reference to the serialization archive.
	 * @param B Reference to the render bounds being serialized.
	 * @return Reference to the Archive after serialization.
	 */
	inline friend FArchive& operator<< (FArchive& Ar, FRenderBounds& B)
	{
		Ar << B.Min;
		Ar << B.Max;
		return Ar;
	}
};


// [Frisvad 2012, "Building an Orthonormal Basis from a 3D Unit Vector Without Normalization"]
inline void GetHemiOrthoBasis( FVector3f& BasisX, FVector3f& BasisY, const FVector3f& BasisZ )
{
	float A = 1.0f / ( 1.0f + BasisZ.Z );
	float B = -BasisZ.X * BasisZ.Y * A;
	BasisX = FVector3f( 1.0f - BasisZ.X * BasisZ.X * A, B, -BasisZ.X );
	BasisY = FVector3f( B, 1.0f - BasisZ.Y * BasisZ.Y * A, -BasisZ.Y );
}

inline FVector2f UnitVectorToHemiOctahedron( const FVector3f& N )
{
	float Sum = FMath::Abs( N.X ) + FMath::Abs( N.Y ) + FMath::Abs( N.Z );
	return FVector2f( N.X + N.Y, N.X - N.Y ) / Sum;
}

inline FVector3f HemiOctahedronToUnitVector( const FVector2f& Oct )
{
	FVector3f N;
	N.X = Oct.X + Oct.Y;
	N.Y = Oct.X - Oct.Y;
	N.Z = 2.0f - FMath::Abs( N.X ) - FMath::Abs( N.Y );
	return N.GetUnsafeNormal();
}

struct FCompressedTransform
{
	static constexpr uint32 ExpBits				= 8;
	static constexpr uint32 ExpBias				= ( 1 << (ExpBits - 1) ) - 1;
	static constexpr uint32 SignMantissaBits	= 16;
	static constexpr uint32 SignMantissaMask	= (1 << SignMantissaBits) - 1;
	static constexpr uint32 MantissaBits		= SignMantissaBits - 1;

	FVector3f	Translation;		// 4B padding
	uint16		Rotation[4];		// 2B padding
	uint16		Scale_SharedExp[4];	// 1B padding if SignMantissaBits == 16

	inline FCompressedTransform() {}
	inline FCompressedTransform( const FRenderTransform& In )
	{
		Translation = In.Origin;

		FVector3f Scale;
		FVector3f Axis[3];
		for( int i = 0; i < 3; i++ )
		{
			Axis[i] = In.TransformRows[i];
			Scale[i] = Axis[i].Size();
			Axis[i] /= (Scale[i] != 0.f) ? Scale[i] : 1.f;
		}

		// Rotation
		{
			if( Axis[2].Z < 0.0f )
			{
				Axis[2]  *= -1.0f;
				Scale[2] *= -1.0f;
			}

			FVector2f OctZ = UnitVectorToHemiOctahedron( Axis[2] );

			FVector3f BasisX, BasisY;
			GetHemiOrthoBasis( BasisX, BasisY, Axis[2] );

			float X = Axis[0] | BasisX;
			float Y = Axis[0] | BasisY;

			float aX = FMath::Abs( X );
			float aY = FMath::Abs( Y );

			bool bSpinIsX = aX < aY;
			float Spin0 = bSpinIsX ? X : Y;
			float Spin1 = bSpinIsX ? Y : X;
			float Sign1 = Spin1 < 0.0f ? -1.0f : 1.0f;
		
			//Axis[0]	*= Sign1;
			Scale[0]*= Sign1;
			Spin0	*= Sign1;

			FVector3f GeneratedY = Axis[2] ^ Axis[0];
			Scale[1] *= ( Axis[1] | GeneratedY ) < 0.0f ? -Sign1 : Sign1;

#if 1
			// Avoid sign extension in shader by biasing
			Rotation[0] = static_cast<uint16>(FMath::RoundToInt( OctZ.X * 32767.0f ) + 32768);
			Rotation[1] = static_cast<uint16>(FMath::RoundToInt( OctZ.Y * 32767.0f ) + 32768);
			Rotation[2] = static_cast<uint16>(FMath::RoundToInt( Spin0  * 16383.0f * 1.41421356f ));	// sqrt(2)
			
			Rotation[2] = ( Rotation[2] + 16384 ) & 0x7fff;
			Rotation[2] |= bSpinIsX ? (1 << 15) : 0;
#else
			Rotation[0]  = ( FMath::RoundToInt( OctZ.X * 32767.0f ) + 32768 ) << 0;
			Rotation[0] |= ( FMath::RoundToInt( OctZ.Y * 32767.0f ) + 32768 ) << 16;
			Rotation[1] = FMath::RoundToInt( Spin0  * 16383.0f * 1.41421356f );	// sqrt(2)
			
			Rotation[1] = ( Rotation[1] + 16384 ) & 0x7fff;
			Rotation[1] |= bSpinIsX ? (1 << 15) : 0;
#endif
		}

		// Scale
		{
			FFloat32 MaxComponent = Scale.GetAbsMax();

			// Need +1 because of losing the implicit leading bit of mantissa
			// TODO assumes ExpBits == 8
			// TODO clamp to expressable range
			uint16 SharedExp = (uint16)MaxComponent.Components.Exponent + 1;
	
			FFloat32 ExpScale( 1.0f );
			ExpScale.Components.Exponent = 127 + ExpBias + MantissaBits - SharedExp;

			if( FMath::RoundToInt( MaxComponent.FloatValue * ExpScale.FloatValue ) == (1 << MantissaBits) )
			{
				// Mantissa rounded up
				SharedExp++;
				ExpScale.FloatValue *= 0.5f;
			}

#if 0
			Scale_SharedExp = (uint64)SharedExp << ( SignMantissaBits * 3 );
			for( int i = 0; i < 3; i++ )
			{
				uint32 Mantissa = FMath::RoundToInt( Scale[i] * ExpScale.FloatValue ) + (1 << MantissaBits);
				Scale_SharedExp |= (uint64)Mantissa << ( SignMantissaBits * i );
			}
#else
			Scale_SharedExp[3] = SharedExp;
			for( int i = 0; i < 3; i++ )
			{
				Scale_SharedExp[i] = static_cast<uint16>(FMath::RoundToInt( Scale[i] * ExpScale.FloatValue ) + (1 << MantissaBits));
			}
#endif
		}
	}

	inline FRenderTransform ToRenderTransform() const
	{
		FRenderTransform Out;

		Out.Origin = Translation;

		// Rotation
		{
			FVector2f OctZ;
			float Spin0;
			OctZ.X = float( (int32)Rotation[0] - 32768 ) * (1.0f / 32767.0f);
			OctZ.Y = float( (int32)Rotation[1] - 32768 ) * (1.0f / 32767.0f);
			Spin0  = float( (int32)( Rotation[2] & 0x7fff ) - 16384 ) * (0.70710678f / 16383.0f);	// rsqrt(2)
			bool bSpinIsX = Rotation[2] > 0x7fff;

			Out.TransformRows[2] = HemiOctahedronToUnitVector( OctZ );

			FVector3f BasisX, BasisY;
			GetHemiOrthoBasis( BasisX, BasisY, Out.TransformRows[2] );

			float Spin1 = FMath::Sqrt( 1.0f - Spin0 * Spin0 );
			float X = bSpinIsX ? Spin0 : Spin1;
			float Y = bSpinIsX ? Spin1 : Spin0;

			Out.TransformRows[0] = BasisX * X + BasisY * Y;
			Out.TransformRows[1] = Out.TransformRows[2] ^ Out.TransformRows[0];
		}

		// Scale
		{
			//uint32 SharedExp = Scale_SharedExp >> ( SignMantissaBits * 3 );
			uint32 SharedExp = Scale_SharedExp[3];

			FFloat32 ExpScale( 1.0f );
			ExpScale.Components.Exponent = 127 - ExpBias - MantissaBits + SharedExp;

#if 0
			for( int i = 0; i < 3; i++ )
			{
				int32 Mantissa = ( Scale_SharedExp >> ( SignMantissaBits * i ) ) & SignMantissaMask;
				float Scale = Mantissa - (1 << MantissaBits);
				Scale *= ExpScale.FloatValue;
				Out.TransformRows[i] *= Scale;
			}
#else
			for( int i = 0; i < 3; i++ )
			{
				float Scale = static_cast<float>(int32(Scale_SharedExp[i]) - (1 << MantissaBits));
				Scale *= ExpScale.FloatValue;
				Out.TransformRows[i] *= Scale;
			}
#endif
		}

		return Out;
	}
};