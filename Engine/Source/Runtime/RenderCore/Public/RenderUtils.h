// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RHIFwd.h"
#include "RHIShaderPlatform.h"
#include "RHIFeatureLevel.h"
#include "ReadOnlyCVARCache.h"

class ITargetPlatform;
class ITargetPlatformSettings;
struct IPooledRenderTarget;
enum class ERayTracingMode : uint8;

extern RENDERCORE_API void RenderUtilsInit();

#define NUM_DEBUG_UTIL_COLORS (32)
inline constexpr FColor DebugUtilColor[NUM_DEBUG_UTIL_COLORS] = 
{
	FColor(20,226,64),
	FColor(210,21,0),
	FColor(72,100,224),
	FColor(14,153,0),
	FColor(186,0,186),
	FColor(54,0,175),
	FColor(25,204,0),
	FColor(15,189,147),
	FColor(23,165,0),
	FColor(26,206,120),
	FColor(28,163,176),
	FColor(29,0,188),
	FColor(130,0,50),
	FColor(31,0,163),
	FColor(147,0,190),
	FColor(1,0,109),
	FColor(2,126,203),
	FColor(3,0,58),
	FColor(4,92,218),
	FColor(5,151,0),
	FColor(18,221,0),
	FColor(6,0,131),
	FColor(7,163,176),
	FColor(8,0,151),
	FColor(102,0,216),
	FColor(10,0,171),
	FColor(11,112,0),
	FColor(12,167,172),
	FColor(13,189,0),
	FColor(16,155,0),
	FColor(178,161,0),
	FColor(19,25,126)
};

#define NUM_CUBE_VERTICES 36

/** The indices for drawing a cube. */
extern RENDERCORE_API const uint16 GCubeIndices[36];

/**
 * Maps from an X,Y,Z cube vertex coordinate to the corresponding vertex index.
 */
inline uint16 GetCubeVertexIndex(uint32 X,uint32 Y,uint32 Z) { return (uint16)(X * 4 + Y * 2 + Z); }

/**
* A 3x1 of xyz(11:11:10) format.
*/
struct FPackedPosition
{
	union
	{
		struct
		{
#if PLATFORM_LITTLE_ENDIAN
			int32	X :	11;
			int32	Y : 11;
			int32	Z : 10;
#else
			int32	Z : 10;
			int32	Y : 11;
			int32	X : 11;
#endif
		} Vector;

		uint32		Packed;
	};

	// Constructors.
	FPackedPosition() : Packed(0) {}
	FPackedPosition(const FVector3f& Other) : Packed(0) 
	{
		Set(Other);
	}
	FPackedPosition(const FVector3d& Other) : Packed(0) 
	{
		Set(Other);
	}
	
	// Conversion operators.
	FPackedPosition& operator=( FVector3f Other )
	{
		Set( Other );
		return *this;
	}
	FPackedPosition& operator=( FVector3d Other )
	{
		Set( Other );
		return *this;
	}

	operator FVector3f() const;
	VectorRegister GetVectorRegister() const;

	// Set functions.
	void Set(const FVector3f& InVector);
	void Set(const FVector3d& InVector);

	// Serializer.
	friend FArchive& operator<<(FArchive& Ar,FPackedPosition& N);
};


/** Flags that control ConstructTexture(2D/2DArray/Volume/etc.) */
enum EConstructTextureFlags : uint32
{
	/** Compress RGBA8 to DXT */
	CTF_Compress =				0x01,
	/** Don't actually compress until the package is saved */
	CTF_DeferCompression =		0x02,
	/** Enable SRGB on the texture */
	CTF_SRGB =					0x04,
	/** Generate mipmaps for the texture */
	CTF_AllowMips =				0x08,
	/** Use DXT1a to get 1 bit alpha but only 4 bits per pixel (note: color of alpha'd out part will be black) */
	CTF_ForceOneBitAlpha =		0x10,
	/** When rendering a masked material, the depth is in the alpha, and anywhere not rendered will be full depth, which should actually be alpha of 0, and anything else is alpha of 255 */
	CTF_RemapAlphaAsMasked =	0x20,
	/** Ensure the alpha channel of the texture is opaque white (255). */
	CTF_ForceOpaque =			0x40,
	/** Don't call a post edit change on the texture. */
	CTF_SkipPostEdit =			0x80,

	/** Default flags (maps to previous defaults to ConstructTexture2D) */
	CTF_Default = CTF_Compress | CTF_SRGB,
};

/**
 * Calculates the amount of memory used for a single mip-map of a texture 3D.
 *
 * Use GPixelFormats[Format].Get3DTextureMipSizeInBytes() instead.
 * 
 * @param TextureSizeX		Number of horizontal texels (for the base mip-level)
 * @param TextureSizeY		Number of vertical texels (for the base mip-level)
 * @param TextureSizeZ		Number of slices (for the base mip-level)
 * @param Format	Texture format
 * @param MipIndex	The index of the mip-map to compute the size of.
 */
RENDERCORE_API SIZE_T CalcTextureMipMapSize3D( uint32 TextureSizeX, uint32 TextureSizeY, uint32 TextureSizeZ, EPixelFormat Format, uint32 MipIndex);

/**
 * Calculates the extent of a mip.
 *
 * Incorrectly forces min mip size to be block dimensions: UE-159189
 * 
 * @param TextureSizeX		Number of horizontal texels (for the base mip-level)
 * @param TextureSizeY		Number of vertical texels (for the base mip-level)
 * @param TextureSizeZ		Number of depth texels (for the base mip-level)
 * @param Format			Texture format
 * @param MipIndex			The index of the mip-map to compute the size of.
 * @param OutXExtent		The extent X of the mip
 * @param OutYExtent		The extent Y of the mip
 * @param OutZExtent		The extent Z of the mip
 */
RENDERCORE_API void CalcMipMapExtent3D( uint32 TextureSizeX, uint32 TextureSizeY, uint32 TextureSizeZ, EPixelFormat Format, uint32 MipIndex, uint32& OutXExtent, uint32& OutYExtent, uint32& OutZExtent );

/**
 * Calculates the extent of a mip.
 *
 * Incorrectly forces min mip size to be block dimensions: UE-159189
 * 
 * @param TextureSizeX		Number of horizontal texels (for the base mip-level)
 * @param TextureSizeY		Number of vertical texels (for the base mip-level)
 * @param Format	Texture format
 * @param MipIndex	The index of the mip-map to compute the size of.
 */
RENDERCORE_API FIntPoint CalcMipMapExtent( uint32 TextureSizeX, uint32 TextureSizeY, EPixelFormat Format, uint32 MipIndex );

/**
 * Calculates the amount of memory used for a single mip-map of a texture.
 * 
 * Use GPixelFormats[Format].Get2DTextureMipSizeInBytes() instead.
 *
 * @param TextureSizeX		Number of horizontal texels (for the base mip-level)
 * @param TextureSizeY		Number of vertical texels (for the base mip-level)
 * @param Format	Texture format
 * @param MipIndex	The index of the mip-map to compute the size of.
 */
RENDERCORE_API SIZE_T CalcTextureMipMapSize( uint32 TextureSizeX, uint32 TextureSizeY, EPixelFormat Format, uint32 MipIndex );

/**
 * Calculates the amount of memory used for a texture.
 *
 * Use GPixelFormats[Format].Get2DTextureSizeInBytes() instead.
 * 
 * @param SizeX		Number of horizontal texels (for the base mip-level)
 * @param SizeY		Number of vertical texels (for the base mip-level)
 * @param Format	Texture format
 * @param MipCount	Number of mip-levels (including the base mip-level)
 */
RENDERCORE_API SIZE_T CalcTextureSize( uint32 SizeX, uint32 SizeY, EPixelFormat Format, uint32 MipCount );

/**
 * Calculates the amount of memory used for a texture.
 *
 * Use GPixelFormats[Format].Get3DTextureSizeInBytes() instead.
 * 
 * @param SizeX		Number of horizontal texels (for the base mip-level)
 * @param SizeY		Number of vertical texels (for the base mip-level)
 * @param SizeY		Number of depth texels (for the base mip-level)
 * @param Format	Texture format
 * @param MipCount	Number of mip-levels (including the base mip-level)
 */
RENDERCORE_API SIZE_T CalcTextureSize3D( uint32 SizeX, uint32 SizeY, uint32 SizeZ, EPixelFormat Format, uint32 MipCount );

/**
 * Copies the data for a 2D texture between two buffers with potentially different strides.
 * @param Source       - The source buffer
 * @param Dest         - The destination buffer.
 * @param SizeY        - The height of the texture data to copy in pixels.
 * @param Format       - The format of the texture being copied.
 * @param SourceStride - The stride of the source buffer.
 * @param DestStride   - The stride of the destination buffer.
 */
RENDERCORE_API void CopyTextureData2D(const void* Source,void* Dest,uint32 SizeY,EPixelFormat Format,uint32 SourceStride,uint32 DestStride);

/**
 *  Returns the valid channels for this pixel format
 * 
 * @return e.g. EPixelFormatChannelFlags::R for PF_R8
 */
RENDERCORE_API EPixelFormatChannelFlags GetPixelFormatValidChannels(EPixelFormat InPixelFormat);


/**
 * Convert from ECubeFace to text string
 * @param Face - ECubeFace type to convert
 * @return text string for cube face enum value
 */
RENDERCORE_API const TCHAR* GetCubeFaceName(ECubeFace Face);

/**
 * Convert from text string to ECubeFace 
 * @param Name e.g. RandomNamePosX
 * @return CubeFace_MAX if not recognized
 */
RENDERCORE_API ECubeFace GetCubeFaceFromName(const FString& Name);

RENDERCORE_API FVertexDeclarationRHIRef& GetVertexDeclarationFVector4();

RENDERCORE_API FVertexDeclarationRHIRef& GetVertexDeclarationFVector3();

RENDERCORE_API FVertexDeclarationRHIRef& GetVertexDeclarationFVector2();

/** True if HDR is enabled for the mobile renderer. */
inline bool IsMobileHDR()
{
	return FReadOnlyCVARCache::MobileHDR();
}

inline bool MobileSupportsGPUScene()
{
	return FReadOnlyCVARCache::MobileSupportsGPUScene();
}

inline bool IsMobileDeferredShadingEnabled(const FStaticShaderPlatform Platform)
{
	return FReadOnlyCVARCache::MobileDeferredShading(Platform) && IsMobileHDR();
}

inline bool MobileForwardEnableLocalLights(const FStaticShaderPlatform Platform)
{
	return FReadOnlyCVARCache::MobileForwardLocalLights(Platform) > 0;
}

inline bool MobileForwardEnableParticleLights(const FStaticShaderPlatform Platform)
{
	return MobileForwardEnableLocalLights(Platform) && FReadOnlyCVARCache::MobileForwardParticleLights(Platform);
}

RENDERCORE_API bool PlatformGPUSceneUsesUniformBufferView(const FStaticShaderPlatform Platform);

RENDERCORE_API bool MobileRequiresSceneDepthAux(const FStaticShaderPlatform Platform);

RENDERCORE_API bool MobileAllowFramebufferFetch(const FStaticShaderPlatform Platform);

RENDERCORE_API bool SupportsTextureCubeArray(ERHIFeatureLevel::Type FeatureLevel);

RENDERCORE_API bool MaskedInEarlyPass(const FStaticShaderPlatform Platform);

RENDERCORE_API bool AllowPixelDepthOffset(const FStaticShaderPlatform Platform);

RENDERCORE_API bool AllowPerPixelShadingModels(const FStaticShaderPlatform Platform);

RENDERCORE_API uint32 GetPlatformShadingModelsMask(const FStaticShaderPlatform Platform);

/** Returns whether AO is enabled. Only valid for a mobile forward shading. */
RENDERCORE_API bool IsMobileAmbientOcclusionEnabled(const FStaticShaderPlatform Platform);

RENDERCORE_API bool AreMobileScreenSpaceReflectionsEnabled(const FStaticShaderPlatform Platform);

RENDERCORE_API bool IsMobileDistanceFieldEnabled(const FStaticShaderPlatform Platform);

RENDERCORE_API bool IsMobileMovableSpotlightShadowsEnabled(const FStaticShaderPlatform Platform);

RENDERCORE_API bool IsMobileCapsuleShadowsEnabled(const FStaticShaderPlatform Platform);

RENDERCORE_API bool IsMobileCapsuleDirectShadowsEnabled(const FStaticShaderPlatform Platform);

RENDERCORE_API bool MobileForwardEnableClusteredReflections(const FStaticShaderPlatform Platform);

RENDERCORE_API bool MobileUsesShadowMaskTexture(const FStaticShaderPlatform Platform);

RENDERCORE_API bool MobileUsesExtendedGBuffer(const FStaticShaderPlatform Platform);

RENDERCORE_API bool MobileUsesGBufferCustomData(const FStaticShaderPlatform Platform);

RENDERCORE_API bool MobileBasePassAlwaysUsesCSM(const FStaticShaderPlatform Platform);

RENDERCORE_API bool MobileUsesFullDepthPrepass(const FStaticShaderPlatform Platform);

RENDERCORE_API bool ShouldForceFullDepthPass(const FStaticShaderPlatform Platform);

RENDERCORE_API bool SupportsGen4TAA(const FStaticShaderPlatform Platform);

RENDERCORE_API bool SupportsTSR(const FStaticShaderPlatform Platform);

RENDERCORE_API bool PlatformSupportsVelocityRendering(const FStaticShaderPlatform Platform);

RENDERCORE_API bool IsUsingDBuffers(const FStaticShaderPlatform Platform);

/** Returns if ForwardShading is enabled. Only valid for the current platform (otherwise call ITargetPlatform::UsesForwardShading()). */
RENDERCORE_API bool IsForwardShadingEnabled(const FStaticShaderPlatform Platform);

/** Return true if all forward shaded material should blend the interpolated sky boxes for higher quality. */
RENDERCORE_API bool ForwardShadingForcesSkyLightCubemapBlending(const FStaticShaderPlatform Platform);

/** Return true the sky light capture can affect the height fog. */
RENDERCORE_API bool SupportsSkyLightCaptureAffectsHeightFogInScattering(const FStaticShaderPlatform Platform);

/** Returns if the GBuffer is used. Only valid for the current platform. */
RENDERCORE_API bool IsUsingGBuffers(const FStaticShaderPlatform Platform);

/** Returns whether the base pass should output to the velocity buffer is enabled for a given shader platform */
RENDERCORE_API bool IsUsingBasePassVelocity(const FStaticShaderPlatform Platform);

/** Returns whether the base pass should use selective outputs for a given shader platform */
RENDERCORE_API bool IsUsingSelectiveBasePassOutputs(const FStaticShaderPlatform Platform);

/** Returns whether distance fields are enabled for a given shader platform */
RENDERCORE_API bool IsUsingDistanceFields(const FStaticShaderPlatform Platform);

/** Returns if water should render distance field shadow a second time for the water surface. This is for a platofrm so can be used at cook time. */
RENDERCORE_API bool IsWaterDistanceFieldShadowEnabled(const FStaticShaderPlatform Platform);

/** Returns if water needs a separate main directional light texture. This is for a platofrm so can be used at cook time or at runtime. */
RENDERCORE_API bool IsWaterSeparateMainDirLightEnabled(const FStaticShaderPlatform Platform);

/** Returns if Mobile Distance Field AO Enabled. */
RENDERCORE_API bool IsMobileDistanceFieldAOEnabled(const FStaticShaderPlatform Platform);

/** Returns if Stencil For LOD Dither is Enabled for this ShaderPlatform. */
RENDERCORE_API bool IsStencilForLODDitherEnabled(const FStaticShaderPlatform Platform);

/** Returns shading path node material override. Returns -1 if no override. */
RENDERCORE_API int32 GetMaterialShadingPathNodeOverride(const FStaticShaderPlatform Platform);

/** Returns whether to simulate mobile for low material quality. */
RENDERCORE_API bool GetMaterialSimulateMobileForLowQuality(const FStaticShaderPlatform Platform);

UE_DEPRECATED(5.8, "This function is deprecated, please use UseGPUScene now by only passing the EShaderPlatform as an argument")
RENDERCORE_API bool UseGPUScene(const FStaticShaderPlatform Platform, const FStaticFeatureLevel FeatureLevel);

RENDERCORE_API bool UseGPUScene(const FStaticShaderPlatform Platform);

RENDERCORE_API bool ForceSimpleSkyDiffuse(const FStaticShaderPlatform Platform);

RENDERCORE_API bool VelocityEncodeDepth(const FStaticShaderPlatform Platform);

RENDERCORE_API bool VelocityEncodeHasPixelAnimation(const FStaticShaderPlatform Platform);

RENDERCORE_API bool VelocitySupportsTemporalResponsiveness(const FStaticShaderPlatform Platform);

RENDERCORE_API bool VelocitySupportsPixelShaderMotionVectorWorldOffset(const FStaticShaderPlatform Platform);

RENDERCORE_API bool PlatformSupportsOpenXRMotionVectors(const FStaticShaderPlatform Platform);

RENDERCORE_API bool VelocityIncludeStationaryPrimitives(const FStaticShaderPlatform Platform);

RENDERCORE_API bool PlatformSupportsSeparatedSubsurfaceDiffuse(const FStaticShaderPlatform Platform);

/** Unit cube vertex buffer (VertexDeclarationFVector4) */
RENDERCORE_API FBufferRHIRef& GetUnitCubeVertexBuffer();

/** Unit cube index buffer */
RENDERCORE_API FBufferRHIRef& GetUnitCubeIndexBuffer();

/** Unit cube AABB vertex buffer (useful to create procedural raytracing geometry) */
RENDERCORE_API FBufferRHIRef& GetUnitCubeAABBVertexBuffer();

/**
* Takes the requested buffer size and quantizes it to an appropriate size for the rest of the
* rendering pipeline. Currently ensures that sizes are multiples of 4 so that they can safely
* be halved in size several times.
*/
RENDERCORE_API void QuantizeSceneBufferSize(const FIntPoint& InBufferSize, FIntPoint& OutBufferSize, const uint32 SuggestedDivisor = 0);

/**
* Checks if virtual texturing enabled and supported
*/
RENDERCORE_API bool UseVirtualTexturing(const FStaticShaderPlatform InShaderPlatform);
RENDERCORE_API bool UseVirtualTexturing(const ITargetPlatformSettings* InTargetPlatform);
RENDERCORE_API bool UseVirtualTexturing(const FStaticShaderPlatform InShaderPlatform, const ITargetPlatformSettings* InTargetPlatform);

RENDERCORE_API bool NaniteAtomicsSupported();
RENDERCORE_API bool NaniteWorkGraphMaterialsSupported();
RENDERCORE_API bool NaniteMeshShadersSupported(EShaderPlatform Platform);
RENDERCORE_API bool NaniteSplineMeshesSupported();
RENDERCORE_API bool NaniteSkinnedMeshesSupported();
RENDERCORE_API bool NaniteTranslucencySupported();
RENDERCORE_API bool NaniteAssembliesSupported();
RENDERCORE_API bool NaniteVoxelsSupported();
RENDERCORE_API bool NaniteCurvesSupported();

RENDERCORE_API bool UseNaniteBindlessShading(EShaderPlatform Platform);
RENDERCORE_API bool UseNaniteBindlessRasterization(EShaderPlatform Platform);

RENDERCORE_API bool UseNaniteFastTileClear();
RENDERCORE_API bool UseNaniteTessellation();

RENDERCORE_API bool DoesPlatformSupportNanite(EShaderPlatform Platform, bool bCheckForProjectSetting = true);
RENDERCORE_API bool DoesRuntimeSupportNanite(EShaderPlatform ShaderPlatform, bool bCheckForAtomicSupport, bool bCheckForProjectSetting);
RENDERCORE_API bool DoesTargetPlatformSupportNanite(const ITargetPlatform* TargetPlatform);
RENDERCORE_API bool DoesMaterialQualitySupportNanite(EShaderPlatform ShaderPlatform, int32 MaterialQualityLevel);

/**
 * Returns true if Nanite rendering should be used for the given shader platform.
 */
RENDERCORE_API bool UseNanite(EShaderPlatform ShaderPlatform, bool bCheckForAtomicSupport = true, bool bCheckForProjectSetting = true);

/**
* Returns true if Nanite Fallback Meshes are enabled for the given shader platform.
*/
RENDERCORE_API bool AreNaniteFallbackMeshesEnabledForPlatform(const FStaticShaderPlatform Platform);

/**
* Returns true if classic (non-virtual) shadow map techniques are supported on the given shader platform.
*/
RENDERCORE_API bool DoesPlatformSupportClassicShadowMaps(EShaderPlatform Platform);

/**
 * Returns true if Virtual Shadow Maps should be used for the given shader platform.
 * Note: Virtual Shadow Maps require Nanite support.
 */
RENDERCORE_API bool UseVirtualShadowMaps(EShaderPlatform ShaderPlatform);

/**
 * Returns true if Virtual Shadow Maps should be used for the given shader platform.
 * Note: Virtual Shadow Maps require Nanite support.
 */
RENDERCORE_API bool UseVirtualShadowMaps(EShaderPlatform ShaderPlatform, const FStaticFeatureLevel FeatureLevel);

/**
* Returns true if Virtual Shadow Mapsare supported for the given shader platform.
* Note: Virtual Shadow Maps require Nanite platform support.
*/
RENDERCORE_API bool DoesPlatformSupportVirtualShadowMaps(EShaderPlatform Platform);

/**
* Returns true if non-Nanite virtual shadow maps are enabled by CVar r.Shadow.Virtual.NonNanite.ProjectEnabled
* and the runtime supports Nanite/virtual shadow maps.
*/
RENDERCORE_API bool DoesPlatformSupportNonNaniteVirtualShadowMaps(EShaderPlatform ShaderPlatform);

/**
* Similar to DoesPlatformSupportNonNaniteVirtualShadowMaps, but checks if nanite and virtual shadow maps are enabled (at runtime).
*/
RENDERCORE_API bool UseNonNaniteVirtualShadowMaps(EShaderPlatform ShaderPlatform, FStaticFeatureLevel FeatureLevel);

/** Returns if water should evaluate virtual shadow maps a second time for the water surface. This is for a platform so can be used at cook time. */
RENDERCORE_API bool IsWaterVirtualShadowMapFilteringEnabled(const FStaticShaderPlatform Platform);

/*
 * Detect (at runtime) if the runtime supports heterogeneous volumes.
 */
RENDERCORE_API bool DoesRuntimeSupportHeterogeneousVolumes(EShaderPlatform Platform);

/**
 * Returns true if Heterogeneous Volumes should be used for the given shader platform.
 */
RENDERCORE_API bool DoesPlatformSupportHeterogeneousVolumes(EShaderPlatform Platform);

/**
 * Returns true if SparseVolumeTextures are supported on the given shader platform.
 */
RENDERCORE_API bool DoesPlatformSupportSparseVolumeTextures(EShaderPlatform Platform);

/**
 * Returns true if SparseVolumeTextures are supported on the given shader platform and are enabled.
 */
RENDERCORE_API bool UseSparseVolumeTextures(EShaderPlatform Platform);

/**
*	(Non-runtime) Checks if the depth prepass for single layer water is enabled. This also depends on virtual shadow maps to be supported on the platform.
*/
RENDERCORE_API bool IsSingleLayerWaterDepthPrepassEnabled(const FStaticShaderPlatform Platform, FStaticFeatureLevel FeatureLevel);

/**
*	(Runtime) Whether platform uses translucent fallback for SLW rendering.
*/
RENDERCORE_API bool IsUsingSingleLayerWaterFallback(FStaticShaderPlatform Platform);

/**
*	Checks if virtual texturing lightmap enabled and supported
*/
RENDERCORE_API bool UseVirtualTextureLightmap(const FStaticShaderPlatform Platform, const ITargetPlatformSettings* TargetPlatform = nullptr);

/**
*	Checks if platform uses a Nanite landscape mesh
*/
RENDERCORE_API bool UseNaniteLandscapeMesh(EShaderPlatform ShaderPlatform);

/**
 *  Checks if the non-pipeline shaders will not be compild and ones from FShaderPipeline used instead.
 */
RENDERCORE_API bool ExcludeNonPipelinedShaderTypes(EShaderPlatform ShaderPlatform);

/**
 *  Checks if shader pipelines is enable for a specific platform.
 */
RENDERCORE_API bool UseShaderPipelines(EShaderPlatform ShaderPlatform);

/**
 *  Checks if we can strip unused interpolators for a specific platform.
 */
RENDERCORE_API bool UseRemoveUnsedInterpolators(EShaderPlatform ShaderPlatform);

/**
 *   Checks if inline skinning shaders are enabled for the platform (via r.SkinCache.CompileShaders)
 */
RENDERCORE_API bool AreInlineSkinningShadersEnabled(EShaderPlatform Platform);

/**
 *   Checks if skin cache shaders are enabled for the platform (via r.SkinCache.CompileShaders)
 */
RENDERCORE_API bool AreSkinCacheShadersEnabled(EShaderPlatform Platform);

/**
 * Checks if skin cache shaders are allowed for the platform (via r.SkinCache.Allow)
 */
RENDERCORE_API bool IsGPUSkinCacheAllowed(EShaderPlatform Platform);

/**
 * Can the skin cache be used (ie shaders added, etc)
 */
RENDERCORE_API bool IsGPUSkinCacheAvailable(EShaderPlatform Platform);

/**
 * Checks if mesh deformers can be used for a specific platform.
 */
RENDERCORE_API bool AreMeshDeformersAvailable(EShaderPlatform Platform);

/**
 * Does the platform support GPUSkinPassthrough permutations.
 * This knowledge can be used to indicate if we need to create SRV for index/vertex buffers.
 */
RENDERCORE_API bool IsGPUSkinPassThroughSupported(EShaderPlatform Platform);

/**
 * Returns true if we need to create SRVs for all index/vertex buffers.
 * This has a memory overhead for the RHI objects so we only return true if the platform or project settings require this.
 */
RENDERCORE_API bool AreBufferSRVsAlwaysCreatedByDefault(EShaderPlatform ShaderPlatform);

/*
 * Detect (at runtime) if the runtime supports rendering one-pass point light shadows (i.e., cube maps)
 */
RENDERCORE_API bool DoesRuntimeSupportOnePassPointLightShadows(EShaderPlatform Platform);

/**
 * Read-only switch to check if translucency per object shadows are enabled.
 */
RENDERCORE_API bool AllowTranslucencyPerObjectShadows(const FStaticShaderPlatform Platform);

/**
* Read-only switch to check if compiling explicit 128BitRT BasePass Pixel Shaders is allowed
* They are infrequently necessary, and can save 50k shaders / 15 MiB / platform on large projects
* Check bRequiresExplicit128bitRT in a platform's DataDrivenPlatformInfo.ini to see if that platform requires explicit 128Bit shaders
*/
RENDERCORE_API bool Allow128BitBasePassPSCompilation(const FStaticShaderPlatform Platform);

/**Note, this should only be used when a platform requires special shader compilation for 32 bit pixel format render targets.
Does not replace pixel format associations across the board**/

RENDERCORE_API bool PlatformRequires128bitRT(EPixelFormat PixelFormat);

RENDERCORE_API bool IsRayTracingEnabledForProject(EShaderPlatform ShaderPlatform);
RENDERCORE_API bool ShouldCompileRayTracingShadersForProject(EShaderPlatform ShaderPlatform);
RENDERCORE_API bool ShouldCompileRayTracingCallableShadersForProject(EShaderPlatform ShaderPlatform);

// Returns `true` when running on RT-capable machine, RT support is enabled for the project and by game graphics options and RT is enabled with r.Raytracing.Enable
// This function may only be called at runtime, never during cooking.
extern RENDERCORE_API bool IsRayTracingEnabled();

// Returns 'true' when running on RT-capable machine, RT support is enabled for the project and by game graphics options and ShaderPlatform supports RT and RT is enabled with r.Raytracing.Enable
// This function may only be called at runtime, never during cooking.
RENDERCORE_API bool IsRayTracingEnabled(EShaderPlatform ShaderPlatform);

// Returns 'true' when running on RT-capable machine, RT support is enabled for the project and by game graphics options
// This function may only be called at runtime, never during cooking.
extern RENDERCORE_API bool IsRayTracingAllowed();

// Returns the ray tracing mode if ray tracing is allowed.
// This function may only be called at runtime, never during cooking.
extern RENDERCORE_API ERayTracingMode GetRayTracingMode();

// Returns 'true' when using TLAS references to determine ray tracing geometry residency
// In which case the runtime should track which ray tracing geometries are referenced in the TLAS
RENDERCORE_API bool IsRayTracingUsingReferenceBasedResidency();

// Returns 'true' when ray tracing can be toggled on/off at runtime.
RENDERCORE_API bool IsRayTracingEnableOnDemandSupported();

RENDERCORE_API bool ShouldGenerateRayTracingProxiesByDefault();

// Returns 'true' when static lighting is enabled for the project
inline bool IsStaticLightingAllowed()
{
	return FReadOnlyCVARCache::AllowStaticLighting();
}

// Returns 'true' when first person primitives are tagged in the GBuffer.
RENDERCORE_API bool HasFirstPersonGBufferBit(EShaderPlatform ShaderPlatform);

RENDERCORE_API bool DoesProjectSupportLumenGI(EShaderPlatform Platform);

extern RENDERCORE_API bool DoesPlatformSupportLumenGI(EShaderPlatform Platform, bool bSkipProjectCheck = false);
extern RENDERCORE_API bool DoesProjectSupportLumenRayTracedTranslucentRefraction();

RENDERCORE_API bool ShouldCompileMegaLightsShaders(EShaderPlatform ShaderPlatform);

RENDERCORE_API bool DoesProjectSupportExpFogMatchesVolumetricFog();

/** Whether or not the platform supports the scene spline texture for spline meshes */
RENDERCORE_API bool UseSplineMeshSceneResources(const FStaticShaderPlatform Platform);

RENDERCORE_API bool RenderRectLightsAsSpotLights(const FStaticFeatureLevel FeatureLevel);

namespace Substrate
{
	RENDERCORE_API bool IsSubstrateEnabled();
	RENDERCORE_API bool IsSubstrateBlendableGBufferEnabled(EShaderPlatform InPlatform);
	RENDERCORE_API bool IsStochasticLightingEnabled(EShaderPlatform InPlatform);
	RENDERCORE_API bool IsRoughDiffuseEnabled();
	RENDERCORE_API bool IsRoughDiffuseEnabled(EShaderPlatform InPlatform);
	UE_DEPRECATED(5.7, "Please use IsGlintEnabled(EShaderPlatform InPlatform)")
	RENDERCORE_API bool IsGlintEnabled();
	RENDERCORE_API bool IsGlintEnabled(EShaderPlatform InPlatform);
	RENDERCORE_API uint32 GlintLUTIndex();
	RENDERCORE_API float GlintLevelBias();
	RENDERCORE_API float GlintLevelMin();
	RENDERCORE_API bool IsToonProfileEnabled();
	RENDERCORE_API bool IsSpecularProfileEnabled();
	RENDERCORE_API bool IsSpecularProfileEnabled(EShaderPlatform InPlatform);
	RENDERCORE_API bool IsDBufferPassEnabled(EShaderPlatform InPlatform);
	RENDERCORE_API bool IsOpaqueRoughRefractionEnabled(EShaderPlatform InPlatform);
	UE_DEPRECATED(5.7, "Please use IsAdvancedVisualizationEnabled(EShaderPlatform InPlatform)")
	RENDERCORE_API bool IsAdvancedVisualizationEnabled();
	RENDERCORE_API bool IsAdvancedVisualizationEnabled(EShaderPlatform InPlatform);
	RENDERCORE_API bool IsMaterialLayeringSupportEnabled();

	RENDERCORE_API bool IsHiddenMaterialAssetConversionEnabled();

	RENDERCORE_API bool AreLightingPassesTiled();
	RENDERCORE_API bool UseTileSize16();

	RENDERCORE_API uint32 GetRayTracingMaterialPayloadSizeInBytes(bool bFullySimplifiedMaterial);

	RENDERCORE_API uint32 GetBytePerPixel(EShaderPlatform InPlatform);

	UE_DEPRECATED(5.8, "GetClosurePerPixel(EShaderPlatform InPlatform, uint8 InMaterialQualityLevel)")
	RENDERCORE_API uint32 GetClosurePerPixel(EShaderPlatform InPlatform);

	const uint8 IgnoreMaterialQualityLevelsForClosurePerPixel = 255;
	RENDERCORE_API uint32 GetClosurePerPixel(EShaderPlatform InPlatform, uint8 InMaterialQualityLevel);

	RENDERCORE_API uint32 GetNormalQuality();

	UE_DEPRECATED(5.8, "This function is no longer used. It will always return 1.")
	RENDERCORE_API uint32 GetSheenQuality();
	UE_DEPRECATED(5.8, "This function is no longer used. It will always return 1.")
	RENDERCORE_API uint32 GetSheenQuality(EShaderPlatform InPlatform);

	RENDERCORE_API uint32 GetShadingQuality();
	RENDERCORE_API uint32 GetShadingQuality(EShaderPlatform InPlatform);
}

// Light function atlas settings
RENDERCORE_API int32 GetLightFunctionAtlasFormat();
// Light function atlas project settings triggering shader compilation
RENDERCORE_API bool GetSingleLayerWaterUsesLightFunctionAtlas();
RENDERCORE_API bool GetTranslucentUsesLightFunctionAtlas();
RENDERCORE_API bool GetTranslucentUsesLightRectLights();
RENDERCORE_API bool GetTranslucentUsesLightIESProfiles();
RENDERCORE_API bool GetTranslucentUsesShadowedLocalLights();

RENDERCORE_API bool GetHairStrandsUsesTriangleStrips();
RENDERCORE_API uint32 GetHairStrandsLODMode();

// Helper function to calculate Z params for froxel grids
// Used in shaders with ComputeDepthFromZSlice(...)
inline FVector CalculateGridZParams(float NearPlane, float FarPlane, float DepthDistributionScale, int32 GridSizeZ)
{
	// S = distribution scale
	// B, O are solved for given the z distances of the first+last slice, and the # of slices.
	//
	// slice = log2(z*B + O) * S

	// Don't spend lots of resolution right in front of the near plane
	double NearOffset = .095 * 100;
	// Space out the slices so they aren't all clustered at the near plane
	double S = DepthDistributionScale;

	double N = NearPlane + NearOffset;
	double F = FarPlane;

	double O = (F - N * FMath::Exp2(GridSizeZ / S)) / (F - N);
	double B = (1 - O) / N;

	return FVector(B, O, S);
}

// LuminanceMax is the amount of light that will cause the sensor to saturate at EV100.
//  See also https://en.wikipedia.org/wiki/Film_speed and https://en.wikipedia.org/wiki/Exposure_value for more info.
inline float EV100ToLuminance(float LuminanceMax, float EV100)
{
	return LuminanceMax * FMath::Pow(2.0f, EV100);
}

inline float EV100ToLuminance(float EV100)
{
	// LuminanceMax set to 1 for lighting to be unitless (1.0cd/m^2 becomes 1.0 at EV100)
	return EV100ToLuminance(1.0f, EV100);
}

inline float EV100ToLog2(float LuminanceMax, float EV100)
{
	return EV100 + FMath::Log2(LuminanceMax);
}

inline float LuminanceToEV100(float LuminanceMax, float Luminance)
{
	return FMath::Log2(Luminance / LuminanceMax);
}

inline float LuminanceToEV100(float Luminance)
{
	// LuminanceMax set to 1 for lighting to be unitless (1.0cd/m^2 becomes 1.0 at EV100)
	return FMath::Log2(Luminance);
}

inline float Log2ToEV100(float LuminanceMax, float Log2)
{
	return Log2 - FMath::Log2(LuminanceMax);
}

// Whether or not VRS is supported via r.VRS.Support and the current platform's DDPI
RENDERCORE_API bool HardwareVariableRateShadingSupportedByPlatform(EShaderPlatform ShaderPlatform);
