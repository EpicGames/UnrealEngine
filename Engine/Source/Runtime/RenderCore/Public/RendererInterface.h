// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	RendererInterface.h: Renderer interface definition.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "Templates/RefCounting.h"
#include "Misc/MemStack.h"
#include "Modules/ModuleInterface.h"
#include "Misc/EnumClassFlags.h"
#include "RenderGraphFwd.h"
#include "RHIResources.h"
#include "RHIFeatureLevel.h"
#include "VirtualTexturingFwd.h"
#include "PathTracingOutputInvalidateReason.h"

#include "SceneRenderingAllocator.h"
#include "PooledRenderTarget.h"

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8
#include "RHI.h"
#include "RenderResource.h"
#include "RenderUtils.h"
#include "UniformBuffer.h"
#include "VirtualTexturing.h"
#include "RenderGraphDefinitions.h"
#endif //UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8

class FCanvas;
class FCanvasRenderContext;
class FMaterial;
class FSceneInterface;
class FSceneView;
class FSceneViewFamily;
class FSceneTextureUniformParameters;
class FMobileSceneTextureUniformParameters;
class FGlobalDistanceFieldParameterData;
struct FMeshBatch;
struct FSynthBenchmarkResults;
struct FSceneTextures;
struct FViewMatrices;
class FShader;
class FShaderMapPointerTable;
class FRDGBuilder;
class FMaterialRenderProxy;
class FGPUScenePrimitiveCollector;
class FViewInfo;
template<typename ShaderType, typename PointerTableType> class TShaderRefBase;
class FSceneUniformBuffer;
class FBatchedPrimitiveParameters;
class ISceneRenderer;
class IStateStreamManager;
class ISceneRenderBuilder;
class FPrimitiveSceneProxy;

namespace Nanite
{
	struct FResources;
};

class FRHITransientTexture;

// use r.DrawDenormalizedQuadMode to override the function call setting (quick way to see if an artifact is caused by this optimization)
enum EDrawRectangleFlags
{
	// Rectangle is created by 2 triangles (diagonal can cause some slightly less efficient shader execution), this is the default as it has no artifacts
	EDRF_Default,
	//
	EDRF_UseTriangleOptimization,
	//
	EDRF_UseTesselatedIndexBuffer
};

enum class EBeginScenePrimitiveRenderingFlags
{
	None = 0,
	UpdateScenePrimitives = 1 << 0,
	All = UpdateScenePrimitives
};
ENUM_CLASS_FLAGS(EBeginScenePrimitiveRenderingFlags)

class FPostOpaqueRenderParameters
{
public:
	FIntRect ViewportRect;
	FMatrix ViewMatrix;
	FMatrix ProjMatrix;
	FRDGTexture* ColorTexture = nullptr;
	FRDGTexture* DepthTexture = nullptr;
	FRDGTexture* NormalTexture = nullptr;
	FRDGTexture* VelocityTexture = nullptr;
	FRDGTexture* SmallDepthTexture = nullptr;
	FRDGBuilder* GraphBuilder = nullptr;
	FRHIUniformBuffer* ViewUniformBuffer = nullptr;
	TRDGUniformBufferRef<FSceneTextureUniformParameters> SceneTexturesUniformParams = nullptr;
	TRDGUniformBufferRef<FMobileSceneTextureUniformParameters> MobileSceneTexturesUniformParams = nullptr;
	const FGlobalDistanceFieldParameterData* GlobalDistanceFieldParams = nullptr;
	void* Uid = nullptr; // A unique identifier for the view.
	const FViewInfo* View = nullptr; 
};
DECLARE_MULTICAST_DELEGATE_OneParam(FOnPostOpaqueRender, class FPostOpaqueRenderParameters&);
typedef FOnPostOpaqueRender::FDelegate FPostOpaqueRenderDelegate;

class ICustomVisibilityQuery: public IRefCountedObject
{
public:
	/** prepares the query for visibility tests */
	virtual bool Prepare() = 0;

	/** test primitive visiblity */
	virtual bool IsVisible(int32 VisibilityId, const FBoxSphereBounds& Bounds) = 0;

	/** return true if we can call IsVisible from a ParallelFor */
	virtual bool IsThreadsafe()
	{
		return false;
	}
};

class ICustomCulling
{
public:
	virtual ICustomVisibilityQuery* CreateQuery (const FSceneView& View) = 0;
};

/**
 * Class use to add FScene pixel inspect request
 */
class FPixelInspectorRequest
{
public:
	FPixelInspectorRequest()
	{
		SourceViewportUV = FVector2f(-1, -1);
		BufferIndex = -1;
		RenderingCommandSend = false;
		RequestComplete = true;
		ViewId = -1;
		GBufferPrecision = 1;
		AllowStaticLighting = true;
		FrameCountAfterRenderingCommandSend = 0;
		RequestTickSinceCreation = 0;
		PreExposure = 1;
	}

	void SetRequestData(FVector2f SrcViewportUV, int32 TargetBufferIndex, int32 ViewUniqueId, int32 GBufferFormat, bool StaticLightingEnable, float InPreExposure)
	{
		SourceViewportUV = SrcViewportUV;
		BufferIndex = TargetBufferIndex;
		RenderingCommandSend = false;
		RequestComplete = false;
		ViewId = ViewUniqueId;
		GBufferPrecision = GBufferFormat;
		AllowStaticLighting = StaticLightingEnable;
		PreExposure = InPreExposure;
		FrameCountAfterRenderingCommandSend = 0;
		RequestTickSinceCreation = 0;
	}

	void MarkSendToRendering() { RenderingCommandSend = true; }

	~FPixelInspectorRequest() = default;

	bool RenderingCommandSend;
	int32 FrameCountAfterRenderingCommandSend;
	int32 RequestTickSinceCreation;
	bool RequestComplete;
	FVector2f SourceViewportUV;
	int32 BufferIndex;
	int32 ViewId;

	//GPU state at capture time
	int32 GBufferPrecision;
	bool AllowStaticLighting;
	float PreExposure;
};

class IPersistentViewUniformBufferExtension
{
public:
	virtual void BeginFrame() {}
	virtual void PrepareView(const FSceneView* View) {}
	virtual void BeginRenderView(const FSceneView* View, bool bShouldWaitForJobs = true) {}
	virtual void EndFrame() {}
};

/**
 */
class IScenePrimitiveRenderingContext
{
public:
	virtual ~IScenePrimitiveRenderingContext() = default;
	virtual ISceneRenderer* GetSceneRenderer() = 0;
};

struct FScenePrimitiveRenderingContextScopeHelper
{
	FScenePrimitiveRenderingContextScopeHelper(IScenePrimitiveRenderingContext* InScenePrimitiveRenderingContext)
	: ScenePrimitiveRenderingContext(InScenePrimitiveRenderingContext)
	{
	}

	~FScenePrimitiveRenderingContextScopeHelper()
	{
		// GPUCULL_TODO: Is new/delete reasonable here?
		delete ScenePrimitiveRenderingContext;
	}

	IScenePrimitiveRenderingContext* ScenePrimitiveRenderingContext;
};

DECLARE_MULTICAST_DELEGATE_OneParam(FSceneRemovedEvent, FSceneInterface* /*Scene*/);

/**
 * The public interface of the renderer module.
 */
class IRendererModule : public IModuleInterface
{
public:

	/** Call from the game thread to send a message to the rendering thread to being rendering this view family. */
	virtual void BeginRenderingViewFamily(FCanvas* Canvas, FSceneViewFamily* ViewFamily) = 0;
	
	/** Call from the render thread to create and initalize a new FViewInfo with the specified options, and add it to the specified view family. */
	virtual void CreateAndInitSingleView(FRHICommandListImmediate& RHICmdList, class FSceneViewFamily* ViewFamily, const struct FSceneViewInitOptions* ViewInitOptions) = 0;

	/**
	 * Allocates a new instance of the private FScene implementation for the given world.
	 * @param World - An optional world to associate with the scene.
	 * @param bInRequiresHitProxies - Indicates that hit proxies should be rendered in the scene.
	 */
	virtual FSceneInterface* AllocateScene(UWorld* World, bool bInRequiresHitProxies, bool bCreateFXSystem, ERHIFeatureLevel::Type InFeatureLevel) = 0;
	
	virtual void RemoveScene(FSceneInterface* Scene) = 0;

	/**
	 * Invoked when the scene is scheduled to be destroyed.  Last chance to safely dereference it from the game thread.  Any code that caches raw
	 * pointers to FSceneInterface should subscribe to this and null out their references to the removed scenes.
	 */
	virtual FSceneRemovedEvent::RegistrationType& GetSceneRemovedEvent() = 0;

#if WITH_STATE_STREAM
	virtual IStateStreamManager* GetStateStreamManager() = 0;
	virtual void DebugLogRenderProxies(FName Type, uint32 Num) = 0;
#endif

	UE_DEPRECATED(5.8, "No longer implemented. For similar use-cases refer to UpdateStaticDrawListsForMaterials.")
	virtual void UpdateStaticDrawLists() = 0;

	/**
	 * Updates static draw lists for the given set of materials for each allocated scene.
	 */
	virtual void UpdateStaticDrawListsForMaterials(const TArray<const FMaterial*>& Materials) = 0;

	/** Allocates a new instance of the private scene manager implementation of FSceneViewStateInterface */
	virtual class FSceneViewStateInterface* AllocateViewState(ERHIFeatureLevel::Type FeatureLevel) = 0;
	virtual class FSceneViewStateInterface* AllocateViewState(ERHIFeatureLevel::Type FeatureLevel, FSceneViewStateInterface* ShareOriginTarget) = 0;

	/** @return The number of lights that affect a primitive. */
	UE_DEPRECATED(5.8, "No longer supported.")
	virtual uint32 GetNumDynamicLightsAffectingPrimitive(const class FPrimitiveSceneInfo* PrimitiveSceneInfo,const class FLightCacheInterface* LCI) = 0;

	virtual void OnWorldCleanup(UWorld* World, bool bSessionEnded, bool bCleanupResources, bool bWorldChanged) = 0;

	virtual void InitializeSystemTextures(FRHICommandListImmediate& RHICmdList) = 0;

	/** Create a Scene Uniform Buffer containing only the scene representation for a single primitive */
	UE_EXPERIMENTAL(5.8, "Potentially replaced with scene descriptors using IPrimitiveComponent")
	virtual FSceneUniformBuffer* CreateSinglePrimitiveSceneUniformBuffer(FRDGBuilder& GraphBuilder, const FPrimitiveSceneProxy* PrimitiveSceneProxy, ERHIFeatureLevel::Type FeatureLevel, FMeshBatch& Mesh) = 0;
	virtual FSceneUniformBuffer* CreateSinglePrimitiveSceneUniformBuffer(FRDGBuilder& GraphBuilder, ERHIFeatureLevel::Type FeatureLevel, FMeshBatch& Mesh) = 0;
	virtual FSceneUniformBuffer* CreateSinglePrimitiveSceneUniformBuffer(FRDGBuilder& GraphBuilder, const FViewInfo& SceneView, FMeshBatch& Mesh) = 0;
	/** Create a Uniform Buffer containing representation for a single primitive. (For platforms that use "UniformView" path) */
	virtual TRDGUniformBufferRef<FBatchedPrimitiveParameters> CreateSinglePrimitiveUniformView(FRDGBuilder& GraphBuilder, ERHIFeatureLevel::Type FeatureLevel, EShaderPlatform ShaderPlatform, FMeshBatch& Mesh) = 0;
	virtual TRDGUniformBufferRef<FBatchedPrimitiveParameters> CreateSinglePrimitiveUniformView(FRDGBuilder& GraphBuilder, const FViewInfo& SceneView, FMeshBatch& Mesh) = 0;

	/** Draws a tile mesh element with the specified view. */
	virtual void DrawTileMesh(FCanvasRenderContext& RenderContext, struct FMeshPassProcessorRenderState& DrawRenderState, const FSceneView& View, FMeshBatch& Mesh, bool bIsHitTesting, const class FHitProxyId& HitProxyId, bool bUse128bitRT = false) = 0;

	virtual const TSet<FSceneInterface*>& GetAllocatedScenes() = 0;

	/** Renderer gets a chance to log some useful crash data */
	virtual void DebugLogOnCrash() = 0;

	// @param WorkScale >0, 10 for normal precision and runtime of less than a second
	virtual void GPUBenchmark(FSynthBenchmarkResults& InOut, float WorkScale = 10.0f) = 0;

	virtual void ExecVisualizeTextureCmd(const FString& Cmd) = 0;

	virtual void UpdateMapNeedsLightingFullyRebuiltState(UWorld* World) = 0;

	/**
	 * Draws a quad with the given vertex positions and UVs in denormalized pixel/texel coordinates.
	 * The platform-dependent mapping from pixels to texels is done automatically.
	 * Note that the positions are affected by the current viewport.
	 * NOTE: DrawRectangle should be used in the vertex shader to calculate the correct position and uv for vertices.
	 *
	 * X, Y							Position in screen pixels of the top left corner of the quad
	 * SizeX, SizeY					Size in screen pixels of the quad
	 * U, V							Position in texels of the top left corner of the quad's UV's
	 * SizeU, SizeV					Size in texels of the quad's UV's
	 * TargetSizeX, TargetSizeY		Size in screen pixels of the target surface
	 * TextureSize                  Size in texels of the source texture
	 * VertexShader					The vertex shader used for rendering
	 * Flags						see EDrawRectangleFlags
	 */
	virtual void DrawRectangle(
		FRHICommandList& RHICmdList,
		float X,
		float Y,
		float SizeX,
		float SizeY,
		float U,
		float V,
		float SizeU,
		float SizeV,
		FIntPoint TargetSize,
		FIntPoint TextureSize,
		const TShaderRefBase<FShader, FShaderMapPointerTable>& VertexShader,
		EDrawRectangleFlags Flags = EDRF_Default
		) = 0;

	/** Register/unregister a custom occlusion culling implementation */
	virtual void RegisterCustomCullingImpl(ICustomCulling* impl) = 0;
	virtual void UnregisterCustomCullingImpl(ICustomCulling* impl) = 0;

	virtual FDelegateHandle RegisterPostOpaqueRenderDelegate(const FPostOpaqueRenderDelegate& PostOpaqueRenderDelegate) = 0;
	virtual void RemovePostOpaqueRenderDelegate(FDelegateHandle PostOpaqueRenderDelegate) = 0;
	virtual FDelegateHandle RegisterOverlayRenderDelegate(const FPostOpaqueRenderDelegate& OverlayRenderDelegate) = 0;
	virtual void RemoveOverlayRenderDelegate(FDelegateHandle OverlayRenderDelegate) = 0;

	/** Delegate that is called upon resolving scene color. */
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnResolvedSceneColor, FRDGBuilder& /*GraphBuilder*/, const FSceneTextures& /*SceneTextures*/);

	/** Accessor for post scene color resolve delegates */
	virtual FOnResolvedSceneColor& GetResolvedSceneColorCallbacks() = 0;

	virtual void PostRenderAllViewports() = 0;

	/** Performs necessary per-frame cleanup. Only use when rendering through scene renderer (i.e. BeginRenderingViewFamily) is skipped */
	virtual void PerFrameCleanupIfSkipRenderer() = 0;

	/** Get the shared material cache tag provider, always valid */
	virtual class IMaterialCacheTagProvider* GetMaterialCacheTagProvider() = 0;
	
	/** Get the shared material cache virtual texture allocator */
	virtual class IMaterialCacheVirtualTextureAllocator* GetMaterialCacheVirtualTextureAllocator() = 0;

	virtual IAllocatedVirtualTexture* AllocateVirtualTexture(FRHICommandListBase& RHICmdList, const FAllocatedVTDescription& Desc) = 0;
	RENDERCORE_API IAllocatedVirtualTexture* AllocateVirtualTexture(const FAllocatedVTDescription& Desc);

	virtual void DestroyVirtualTexture(IAllocatedVirtualTexture* AllocatedVT) = 0;

	virtual IAdaptiveVirtualTexture* AllocateAdaptiveVirtualTexture(FRHICommandListBase& RHICmdList, const FAdaptiveVTDescription& AdaptiveVTDesc, const FAllocatedVTDescription& AllocatedVTDesc) = 0;
	RENDERCORE_API IAdaptiveVirtualTexture* AllocateAdaptiveVirtualTexture(const FAdaptiveVTDescription& AdaptiveVTDesc, const FAllocatedVTDescription& AllocatedVTDesc);
	virtual void DestroyAdaptiveVirtualTexture(IAdaptiveVirtualTexture* AdaptiveVT) = 0;

	virtual FVirtualTextureProducerHandle RegisterVirtualTextureProducer(FRHICommandListBase& RHICmdList, const FVTProducerDescription& Desc, IVirtualTexture* Producer) = 0;
	RENDERCORE_API FVirtualTextureProducerHandle RegisterVirtualTextureProducer(const FVTProducerDescription& Desc, IVirtualTexture* Producer);

	virtual void ReleaseVirtualTextureProducer(const FVirtualTextureProducerHandle& Handle) = 0;
	virtual void AddVirtualTextureProducerDestroyedCallback(const FVirtualTextureProducerHandle& Handle, FVTProducerDestroyedFunction* Function, void* Baton) = 0;
	virtual uint32 RemoveAllVirtualTextureProducerDestroyedCallbacks(const void* Baton) = 0;
	virtual void ReleaseVirtualTexturePendingResources() = 0;

	virtual IVirtualTexture* FindProducer(const FVirtualTextureProducerHandle& Handle) = 0;

	virtual void RequestVirtualTextureTiles(const FVector2D& InScreenSpaceSize, int32 InMipLevel) = 0;
	virtual void RequestVirtualTextureTiles(const FMaterialRenderProxy* InMaterialRenderProxy, const FVector2D& InScreenSpaceSize, ERHIFeatureLevel::Type InFeatureLevel) = 0;

	/**
	 * Helper function to request loading of tiles for a virtual texture that will be displayed in the UI. 
	 * It will request only the tiles that will be visible after clipping to the provided viewport.
	 * @param AllocatedVT			The virtual texture.
	 * @param InScreenSpaceSize		Size on screen at which the texture is to be displayed.
	 * @param InViewportPosition	Position in the viewport where the texture will be displayed.
	 * @param InViewportSize		Size of the viewport.
	 * @param InUV0					UV coordinate to use for the top left corner of the texture.
	 * @param InUV1					UV coordinate to use for the bottom right corner of the texture.
	 * @param InMipLevel [optional] Specific mip level to fetch tiles for.
	 */
	virtual void RequestVirtualTextureTiles(IAllocatedVirtualTexture* AllocatedVT, const FVector2D& InScreenSpaceSize, const FVector2D& InViewportPosition, const FVector2D& InViewportSize, const FVector2D& InUV0, const FVector2D& InUV1, int32 InMipLevel) = 0;

	/** Ensure that any tiles requested by 'RequestVirtualTextureTiles' are loaded, must be called from render thread */
	virtual void LoadPendingVirtualTextureTiles(FRHICommandListImmediate& RHICmdList, ERHIFeatureLevel::Type FeatureLevel) = 0;
	
	/** 
	 * Lock all tiles associated with a producer up to and including a given mip level.
	 * The mip level is stored on the producer, so that if we call twice then the second call will either lock or unlock tiles to reach the new level.
	 * Locked tiles should remain resident once loaded, though there is no guarantee over how quickly they will be loaded.
	 * Also if the virtual texture pools are already oversubscribed then we will not be able to satisfy the loading.
	 */
	virtual void LockVirtualTextureTiles(FVirtualTextureProducerHandle ProducerHandle, int32 InMipLevel) = 0;

	/** Allocate a buffer and record all virtual texture page requests until the next call to either SetVirtualTextureRequestRecordBuffer or GetVirtualTextureRequestRecordBuffer. */
	virtual void SetVirtualTextureRequestRecordBuffer(uint64 Handle) = 0;
	/** Fetch the virtual texture page requests recorded since the last call to SetVirtualTextureRequestRecordBuffer. The requests are in a TMap of opaque PageRequestId to Priority. Returns the handle that was passed in. */
	virtual uint64 GetVirtualTextureRequestRecordBuffer(TMap<uint64, uint32>& OutRequestData) = 0;
	/** Pack page requests returned by GetVirtualTextureRequestRecordBuffer() into an opaque compressed stream that we can pass to RequestVirtualTextureTiles(). */
	virtual void PackVirtualTextureRequestsToStream(TMap<uint64, uint32> const& InPageRequests, TArray<uint32>& OutRequestStream) = 0;
	/**	Request Virtual Texture pages from an opaque request stream. Provide an optional InCaptureResolution so that a bias can be applied to account for the current playback resolution if necessary. */
	virtual void RequestVirtualTextureTiles(TArray<uint32>&& InRequestStream, FIntPoint const& InCaptureResolution = FIntPoint::ZeroValue) = 0;

	/** Evict all data from virtual texture caches. */
	virtual void FlushVirtualTextureCache() = 0;
	/** Evict all data from virtual texture cache for a given allocated virtual texture. */
	virtual void FlushVirtualTextureCache(IAllocatedVirtualTexture* AllocatedVT, const FVector2f& InUV0, const FVector2f& InUV1) = 0;

	/** Do a full virtual texture system update to read and process feedback. This will sync until all outstanding page rendering or streaming is complete. */
	virtual void SyncVirtualTextureUpdates(FRHICommandListImmediate& RHICmdList, ERHIFeatureLevel::Type FeatureLevel) = 0;

	/** Allocate a buffer and record all nanite page requests until the next call to either SetNaniteRequestRecordBuffer or GetNaniteRequestRecordBuffer. */
	virtual void SetNaniteRequestRecordBuffer(uint64 Handle) = 0;
	/** Fetch the page requests recorded since the last call to SetNaniteRequestRecordBuffer. The requests are in a TMap of opaque PageRequestId to Priority. Returns the handle that was passed in. */
	virtual uint64 GetNaniteRequestRecordBuffer(TMap<uint64, uint32>& OutRequestData) = 0;
	/** Pack page requests returned by GetNaniteRequestRecordBuffer() into an opaque compressed stream that we can pass to RequestNanitePages(). */
	virtual void PackNaniteRequestsToStream(TMap<uint64, uint32> const& InPageRequests, TArray<uint32>& OutRequestStream) = 0;
	/**	Request Nanite pages from a request stream. */
	virtual void RequestNanitePages(TArray<uint32>&& InRequestStream) = 0;

	/**	Start prefetching streaming data for Nanite resource that will soon be used for rendering. TODO: Implement callback mechanism */
	virtual void PrefetchNaniteResource(const Nanite::FResources* Resource, uint32 NumFramesUntilRender) = 0;

	virtual void RegisterPersistentViewUniformBufferExtension(IPersistentViewUniformBufferExtension* Extension) = 0;

	/**
	 * Prepare Scene primitive rendering and return context. 
	 * Ensures all primitives that are created are commited and GPU-Scene is updated (if EBeginScenePrimitiveRenderingFlags::UpdateScenePrimitives is passed in BeginScenePrimitiveRenderingFlags).
	 * Allocates a dynamic primitive context.
	 * The intended use is for stand-alone rendering that involves Scene proxies (that then may need the machinery to render GPU-Scene aware primitives.
	 */
	virtual IScenePrimitiveRenderingContext* BeginScenePrimitiveRendering(FRDGBuilder& GraphBuilder, FSceneViewFamily* ViewFamily, EBeginScenePrimitiveRenderingFlags BeginScenePrimitiveRenderingFlags = EBeginScenePrimitiveRenderingFlags::All) = 0;
	virtual IScenePrimitiveRenderingContext* BeginScenePrimitiveRendering(FRDGBuilder& GraphBuilder, FSceneInterface& Scene, EBeginScenePrimitiveRenderingFlags BeginScenePrimitiveRenderingFlags = EBeginScenePrimitiveRenderingFlags::All) = 0;

	/** Mark all the current scenes as needing to restart path tracer accumulation */
	virtual void InvalidatePathTracedOutput(PathTracing::EInvalidateReason InvalidateReason = PathTracing::EInvalidateReason::Uncategorized) = 0;

	/** Experimental:  Render multiple view families in a single scene render call.  All families must reference the same FScene.  Scene Capture not yet supported. */
	virtual void BeginRenderingViewFamilies(FCanvas* Canvas, TConstArrayView<FSceneViewFamily*> ViewFamilies) = 0;

	/** Resets the scene texture extent history. Call this method after rendering with very large render
	 *  targets. The next scene render will create them at the requested size.
	 */
	virtual void ResetSceneTextureExtentHistory() = 0;

	virtual const FViewMatrices& GetPreviousViewMatrices(const FSceneView& View) = 0;
	virtual const FGlobalDistanceFieldParameterData* GetGlobalDistanceFieldParameterData(const FSceneView& View) = 0;
	virtual void RequestStaticMeshUpdate(FPrimitiveSceneInfo* Info) = 0;
	virtual void AddMeshBatchToGPUScene(FGPUScenePrimitiveCollector* Collector, FMeshBatch& MeshBatch) = 0;

	virtual TUniquePtr<ISceneRenderBuilder> CreateSceneRenderBuilder(FSceneInterface* Scene) = 0;
};

