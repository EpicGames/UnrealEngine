// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	RenderingThread.h: Rendering thread definitions.
=============================================================================*/

#pragma once

#include "AutoRTFM.h"
#include "Async/TaskGraphInterfaces.h"
#include "Containers/Array.h"
#include "Containers/List.h"
#include "CoreGlobals.h"
#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "Delegates/Delegate.h"
#include "HAL/PlatformMemory.h"
#include "Misc/AssertionMacros.h"
#include "Misc/TVariant.h"
#include "MultiGPU.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"
#include "RenderCommandTag.h"
#include "RHI.h"
#include "RHICommandList.h"
#include "Serialization/MemoryLayout.h"
#include "Templates/Atomic.h"
#include "Templates/Function.h"
#include "Templates/UnrealTemplate.h"
#include "Templates/UnrealTypeTraits.h"
#include "Trace/Trace.h"
#include "Async/Mutex.h"
#include "Tasks/Pipe.h"

#define UE_API RENDERCORE_API

namespace UE { namespace Trace { class FChannel; } }

enum class EParallelForFlags;

////////////////////////////////////
// Render thread API
////////////////////////////////////

namespace FFrameEndSync
{
	enum class EFlushMode
	{
		// Blocks the caller until the N - m frame has completed, where m is driven by various config.
		EndFrame,

		// Blocks the caller until all rendering work is completed on the CPU. Does not sync with the GPU.
		Threads
	};

	// Syncs the game thread based on progress throughout the rendering pipeline.
	RENDERCORE_API void Sync(EFlushMode FlushMode);
};

/**
 * Whether the renderer is currently running in a separate thread.
 * If this is false, then all rendering commands will be executed immediately instead of being enqueued in the rendering command buffer.
 */
extern RENDERCORE_API bool GIsThreadedRendering;

/**
 * Whether the rendering thread should be created or not.
 * Currently set by command line parameter and by the ToggleRenderingThread console command.
 */
extern RENDERCORE_API bool GUseThreadedRendering;

// Global for handling the "togglerenderthread" command.
extern RENDERCORE_API TOptional<bool> GPendingUseThreadedRendering;

#if (UE_BUILD_SHIPPING || UE_BUILD_TEST)
	inline void CheckNotBlockedOnRenderThread() {}
#else // #if (UE_BUILD_SHIPPING || UE_BUILD_TEST)
	/** Whether the main thread is currently blocked on the rendering thread, e.g. a call to FlushRenderingCommands. */
	extern RENDERCORE_API TAtomic<bool> GMainThreadBlockedOnRenderThread;

	/** Asserts if called from the main thread when the main thread is blocked on the rendering thread. */
	inline void CheckNotBlockedOnRenderThread() { ensure(!GMainThreadBlockedOnRenderThread.Load(EMemoryOrder::Relaxed) || !IsInGameThread()); }
#endif // #if (UE_BUILD_SHIPPING || UE_BUILD_TEST)


// Called during engine init to setup the rendering thread.
extern RENDERCORE_API void InitRenderingThread();

// Called during engine shutdown to stop the rendering thread.
extern RENDERCORE_API void ShutdownRenderingThread();

// Called once per frame by the game thread to latch the latest render thread config.
extern RENDERCORE_API void LatchRenderThreadConfiguration();

/**
 * Checks if the rendering thread is healthy and running.
 * If it has crashed, UE_LOG is called with the exception information.
 */
extern RENDERCORE_API void CheckRenderingThreadHealth();

/** Checks if the rendering thread is healthy and running, without crashing */
extern RENDERCORE_API bool IsRenderingThreadHealthy();

/**
 * Advances stats for the rendering thread. Called from the game thread.
 */
extern RENDERCORE_API void AdvanceRenderingThreadStatsGT( bool bDiscardCallstack, int64 StatsFrame, int32 DisableChangeTagStartFrame );

/**
 * Waits for the rendering thread to finish executing all pending rendering commands.  Should only be used from the game thread.
 */
extern RENDERCORE_API void FlushRenderingCommands();

extern RENDERCORE_API void StartRenderCommandFenceBundler();
extern RENDERCORE_API void StopRenderCommandFenceBundler();

class FCoreRenderDelegates
{
public:
	DECLARE_MULTICAST_DELEGATE(FOnFlushRenderingCommandsStart);
	static RENDERCORE_API FOnFlushRenderingCommandsStart OnFlushRenderingCommandsStart;

	DECLARE_MULTICAST_DELEGATE(FOnFlushRenderingCommandsEnd);
	static RENDERCORE_API FOnFlushRenderingCommandsEnd OnFlushRenderingCommandsEnd;
};

////////////////////////////////////
// Render commands
////////////////////////////////////

UE_TRACE_CHANNEL_EXTERN(RenderCommandsChannel, RENDERCORE_API);

//
// Macros for using render commands.
//
// ideally this would be inline, however that changes the module dependency situation
extern RENDERCORE_API class FRHICommandListImmediate& GetImmediateCommandList_ForRenderCommand();

// Log render commands on server for debugging
#if 0 // UE_SERVER && UE_BUILD_DEBUG
	#define LogRenderCommand(TypeName)				UE_LOGF(LogRHI, Warning, "Render command '%ls' is being executed on a dedicated server.", TEXT(#TypeName))
#else
	#define LogRenderCommand(TypeName)
#endif 

// conditions when rendering commands are executed in the thread
#if UE_SERVER
	#define	ShouldExecuteOnRenderThread()			false
#else
	#define	ShouldExecuteOnRenderThread()			(LIKELY(GIsThreadedRendering || !IsInGameThread()))
#endif // UE_SERVER

/** Describes which pipes are configured to use the render command pipe system. */
enum class ERenderCommandPipeMode : uint8
{
	/** Bypasses the render command pipe system altogether. Render commands are issued using tasks. */
	None,

	/** The render command pipe on the render thread pipe is active, and all other pipes forward to the render thread pipe. */
	RenderThread,

	/** All render command pipes are active. */
	All
};

enum class ERenderCommandPipeFlags : uint8
{
	None = 0,

	/** Initializes the render command pipe in a disabled state. */
	Disabled = 1 << 0
};

ENUM_CLASS_FLAGS(ERenderCommandPipeFlags);

class FRenderCommandPipe;
using FRenderCommandPipeBitArrayAllocator = TInlineAllocator<1, FConcurrentLinearArrayAllocator>;
using FRenderCommandPipeBitArray = TBitArray<FRenderCommandPipeBitArrayAllocator>;
using FRenderCommandPipeSetBitIterator = TConstSetBitIterator<FRenderCommandPipeBitArrayAllocator>;

using FRenderCommandFunctionVariant = TVariant<
	  TUniqueFunction<void()>
	, TUniqueFunction<void(FRHICommandList&)>
	, TUniqueFunction<void(FRHICommandListImmediate&)>
>;

namespace UE::RenderCommandPipe
{
	// [Game Thread] Initializes all statically initialized render command pipes.
	extern RENDERCORE_API void Initialize();

	// [Game Thread (Parallel)] Returns whether any render command pipes are currently recording on the game thread timeline.
	extern RENDERCORE_API bool IsRecording();

	// [Render Thread (Parallel)] Returns whether any render command pipes are currently replaying commands on the render thread timeline.
	extern RENDERCORE_API bool IsReplaying();

	// [Render Thread (Parallel)] Returns whether the specific render command pipe is replaying.
	extern RENDERCORE_API bool IsReplaying(const FRenderCommandPipe& Pipe);

	// [Game Thread] Starts recording render commands into pipes. Returns whether the operation succeeded.
	extern RENDERCORE_API void StartRecording();
	extern RENDERCORE_API void StartRecording(const FRenderCommandPipeBitArray& PipeBits);

	// [Game Thread] Stops recording commands into pipes and syncs all remaining pipe work to the render thread. Returns whether the operation succeeded.
	extern RENDERCORE_API FRenderCommandPipeBitArray StopRecording();
	extern RENDERCORE_API FRenderCommandPipeBitArray StopRecording(TConstArrayView<FRenderCommandPipe*> Pipes);

	// Returns the list of all registered pipes.
	extern RENDERCORE_API TConstArrayView<FRenderCommandPipe*> GetPipes();

	// A delegate to receive events at sync points when recording is stopped. Input is a bit array of pipes that were stopped.
	DECLARE_MULTICAST_DELEGATE_OneParam(FStopRecordingDelegate, const FRenderCommandPipeBitArray&);
	extern RENDERCORE_API FStopRecordingDelegate& GetStopRecordingDelegate();

	// [Game Thread] Stops render command pipe recording during the duration of the scope and restarts recording once the scope is complete.
	class FSyncScope
	{
	public:
		UE_API FSyncScope();
		UE_API FSyncScope(TConstArrayView<FRenderCommandPipe*> Pipes);
		UE_API ~FSyncScope();

	private:
		FRenderCommandPipeBitArray PipeBits;
	};

	// Utility class containing a simple linked list of render commands.
	class FCommandList
	{
		enum class ECommandType : uint8
		{
			ExecuteFunction,
			ExecuteCommandList
		};

		struct FCommand
		{
			virtual ~FCommand() = default;

			FCommand(ECommandType InType)
				: Type(InType)
			{}

			FCommand* Next = nullptr;
			ECommandType Type;
		};

		struct FExecuteFunctionCommand
			: public FCommand
			, public UE::FInheritedContextBase
		{
			FExecuteFunctionCommand(FRenderCommandFunctionVariant&& InFunction, const FRenderCommandTag& InTag)
				: FCommand(ECommandType::ExecuteFunction)
				, Tag(InTag)
				, Function(MoveTemp(InFunction))
			{
				CaptureInheritedContext();
			}

			const FRenderCommandTag& Tag;
			FRenderCommandFunctionVariant Function;
		};

		struct FExecuteCommandListCommand : public FCommand
		{
			FExecuteCommandListCommand(FCommandList* InCommandList)
				: FCommand(ECommandType::ExecuteCommandList)
				, CommandList(InCommandList)
			{}

			FCommandList* CommandList;
		};

	public:
		FCommandList(FMemStackBase& InAllocator)
			: Allocator(InAllocator)
		{}

		// Assigns the allocator reference and then moves the command list and its allocator contents into this one.
		FCommandList(FMemStackBase& InAllocator, FCommandList& CommandListToConsume)
			: Allocator(InAllocator)
		{
			Allocator = MoveTemp(CommandListToConsume.Allocator);
			Commands = CommandListToConsume.Commands;
			CommandListToConsume.Commands = {};

#if DO_CHECK
			bClosed = CommandListToConsume.bClosed;
			CommandListToConsume.bClosed = false;
#endif
		}

		~FCommandList()
		{
			Release();
		}

		template <typename RenderCommandTag, typename FunctionType>
		inline bool Enqueue(FunctionType&& Function)
		{
			return Enqueue<RenderCommandTag>(FRenderCommandFunctionVariant(TInPlaceType<FunctionType>(), MoveTemp(Function)));
		}

		template <typename RenderCommandTag>
		inline bool Enqueue(FRenderCommandFunctionVariant&& Function)
		{
			return Enqueue(MoveTemp(Function), RenderCommandTag::Get());
		}

		RENDERCORE_API bool Enqueue(FRenderCommandFunctionVariant&& Function, const FRenderCommandTag& Tag);

		RENDERCORE_API bool Enqueue(FCommandList* CommandList);

		void Close()
		{
#if DO_CHECK
			bClosed = true;
#endif
		}

		template <typename LambdaType>
		void ConsumeCommands(const LambdaType& Lambda)
		{
#if DO_CHECK
			check(bClosed);
#endif

			for (FCommand* Command = Commands.Head; Command; Command = Command->Next)
			{
				if (Command->Type == ECommandType::ExecuteFunction)
				{
					FExecuteFunctionCommand* FunctionCommand = static_cast<FExecuteFunctionCommand*>(Command);
					{
						UE::FInheritedContextScope InheritedContextScope = FunctionCommand->RestoreInheritedContext();
						Lambda(MoveTemp(FunctionCommand->Function), FunctionCommand->Tag);
					}
					FunctionCommand->~FExecuteFunctionCommand();
				}
				else
				{
					check(Command->Type == ECommandType::ExecuteCommandList);
					FExecuteCommandListCommand* CommandListCommand = static_cast<FExecuteCommandListCommand*>(Command);
					CommandListCommand->CommandList->ConsumeCommands(Lambda);
					CommandListCommand->~FExecuteCommandListCommand();
				}
			}
			Commands = {};
#if DO_CHECK
			bClosed = false;
#endif
		}

		inline int32 NumCommands() const
		{
			return Commands.Num;
		}

		inline bool IsEmpty() const
		{
			return !Commands.Head;
		}

	private:
		RENDERCORE_API bool Enqueue(FCommand* Command);

		template <typename T, typename... TArgs>
		inline T* AllocNoDestruct(TArgs&&... Args)
		{
			return new (Allocator) T(Forward<TArgs&&>(Args)...);
		}

		RENDERCORE_API void Release();

		FMemStackBase& Allocator;

		struct
		{
			FCommand* Head = nullptr;
			FCommand* Tail = nullptr;
			int32     Num  = 0;

		} Commands;

#if DO_CHECK
		bool bClosed = false;
#endif
	};
}

extern RENDERCORE_API ERenderCommandPipeMode GRenderCommandPipeMode;

class FRenderCommandList;

class FRenderCommandPipeBase
{
public:
	~FRenderCommandPipeBase()
	{
		delete Context;
	}

protected:
	void ResetContext()
	{
		// If the context's command list is not empty then a task must have been launched that will consume its contents.
		// Replace the context with a new one and mark the old one for deletion. Any new commands issued into the new context
		// will issue a new task scheduled after this command list executes.

		if (!Context->CommandList.IsEmpty())
		{
			Context->bDeleteAfterExecute = true;
			Context = new FContext();
		}
	}

	struct FContext
	{
		FContext()
			: Allocator(FMemStackBase::EPageSize::Large)
			, CommandList(Allocator)
		{}

		FContext(FContext&& Other)
			: CommandList(Allocator, Other.CommandList)
			, bDeleteAfterExecute(Other.bDeleteAfterExecute)
		{}

		FMemStackBase Allocator;
		UE::RenderCommandPipe::FCommandList CommandList;
		bool bDeleteAfterExecute = false;
	};

	FContext* Context = new FContext;
	UE::FMutex Mutex;
};

class FRenderThreadCommandPipe : public FRenderCommandPipeBase
{
	RENDERCORE_API static void ExecuteCommands(FRenderCommandList* CommandList);
	RENDERCORE_API static void ExecuteCommands(UE::RenderCommandPipe::FCommandList& CommandList);

	class FRenderCommandTaskBase
	{
	public:
		static ENamedThreads::Type GetDesiredThread()
		{
			check(!GIsThreadedRendering || ENamedThreads::GetRenderThread() != ENamedThreads::GameThread);
			return ENamedThreads::GetRenderThread();
		}

		static ESubsequentsMode::Type GetSubsequentsMode()
		{
			return ESubsequentsMode::FireAndForget;
		}
	};

	template <typename LambdaType>
	class TRenderCommandTask : public FRenderCommandTaskBase
	{
	public:
		TRenderCommandTask(LambdaType&& InLambda, const FRenderCommandTag& InTag)
			: Tag(InTag)
			, Lambda(Forward<LambdaType>(InLambda))
		{}

		void DoTask(ENamedThreads::Type, const FGraphEventRef&)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE_ON_CHANNEL_STR(Tag.GetName(), RenderCommandsChannel);
			Lambda(GetImmediateCommandList_ForRenderCommand());
		}

		inline TStatId GetStatId() const
		{
			return Tag.GetStatId();
		}

	private:
		const FRenderCommandTag& Tag;
		LambdaType Lambda;
	};

	class FRenderCommandListTask : public FRenderCommandTaskBase
	{
	public:
		FRenderCommandListTask(FRenderCommandList* InCommandList)
			: CommandList(InCommandList)
		{}

		void DoTask(ENamedThreads::Type, const FGraphEventRef&)
		{
			FRenderThreadCommandPipe::ExecuteCommands(CommandList);
		}

	private:
		FRenderCommandList* CommandList;
	};

public:
	template <typename RenderCommandTag, typename LambdaType>
	static void Enqueue(LambdaType&& Lambda)
	{
		const FRenderCommandTag& Tag = RenderCommandTag::Get();

		if (!IsInRenderingThread() && ShouldExecuteOnRenderThread())
		{
			CheckNotBlockedOnRenderThread();

			if (GRenderCommandPipeMode != ERenderCommandPipeMode::None)
			{
				Instance.EnqueueAndLaunch(MoveTemp(Lambda), Tag);
			}
			else
			{
				TGraphTask<TRenderCommandTask<LambdaType>>::CreateTask().ConstructAndDispatchWhenReady(MoveTemp(Lambda), Tag);
			}
		}
		else if (Tag.GetStatId().IsValidStat())
		{
			TRACE_CPUPROFILER_EVENT_SCOPE_USE_ON_CHANNEL(Tag.GetSpecId(), Tag.GetName(), EventScope, RenderCommandsChannel, !GCycleStatsShouldEmitNamedEvents);
			FScopeCycleCounter CycleScope(Tag.GetStatId());
			Lambda(GetImmediateCommandList_ForRenderCommand());
		}
		else
		{
			Lambda(GetImmediateCommandList_ForRenderCommand());
		}
	}

	static void Enqueue(FRenderCommandList* CommandList)
	{
		if (CommandList)
		{
			if (!IsInRenderingThread() && ShouldExecuteOnRenderThread())
			{
				CheckNotBlockedOnRenderThread();

				if (GRenderCommandPipeMode != ERenderCommandPipeMode::None)
				{
					Instance.EnqueueAndLaunch(CommandList);
				}
				else
				{
					TGraphTask<FRenderCommandListTask>::CreateTask().ConstructAndDispatchWhenReady(CommandList);
				}
			}
			else
			{
				ExecuteCommands(CommandList);
			}
		}
	}

private:
	static RENDERCORE_API FRenderThreadCommandPipe Instance;

	RENDERCORE_API void EnqueueAndLaunch(FRenderCommandList* CommandList);
	RENDERCORE_API void EnqueueAndLaunch(TUniqueFunction<void(FRHICommandListImmediate&)>&& Function, const FRenderCommandTag& Tag);
	RENDERCORE_API void Enqueue(TUniqueFunction<void(FRHICommandListImmediate&)>&& Function, const FRenderCommandTag& Tag);
};

class FRenderCommandPipe : public FRenderCommandPipeBase
{
public:
	using FCommandListFunction = TUniqueFunction<void(FRHICommandList&)>;
	using FEmptyFunction = TUniqueFunction<void()>;

	RENDERCORE_API FRenderCommandPipe(const TCHAR* Name, ERenderCommandPipeFlags Flags, const TCHAR* CVarName, const TCHAR* CVarDescription);

	inline const TCHAR* GetName() const
	{
		return Name;
	}

	inline bool IsValid() const
	{
		return Index != INDEX_NONE;
	}

	inline int32 GetIndex() const
	{
		check(IsValid());
		return Index;
	}

	inline bool IsReplaying() const
	{
		ensure(IsInParallelRenderingThread());
		return bReplaying;
	}

	inline bool IsRecording() const
	{
		return bRecording;
	}

	inline bool IsEmpty() const
	{
		return NumInFlightCommands.load(std::memory_order_relaxed) == 0 && NumInFlightCommandLists.load(std::memory_order_relaxed) == 0;
	}

	void SetEnabled(bool bInIsEnabled)
	{
		check(IsInGameThread());
		bEnabled = bInIsEnabled;
	}

	//////////////////////////////////////////////////////////////////////////

	bool Enqueue(FRenderCommandList* RenderCommandList)
	{
		if (RenderCommandList)
		{
			UE::TScopeLock Lock(Mutex);

			checkf(!UE::RenderCommandPipe::IsReplaying(*this), TEXT("Enqueuing command queues from the render command pipe replay task is not allowed."));

			if (RecordTask.IsValid())
			{
				EnqueueAndLaunch(RenderCommandList);
				return true;
			}
		}
		return false;
	}

	template <typename RenderCommandTag>
	bool Enqueue(FCommandListFunction& Function)
	{
		// Execute the function directly if this is being called recursively from within another pipe command.
		if (UE::RenderCommandPipe::IsReplaying(*this))
		{
			ExecuteCommand(MoveTemp(Function), RenderCommandTag::Get());
			return true;
		}

		UE::TScopeLock Lock(Mutex);

		if (RecordTask.IsValid())
		{
			EnqueueAndLaunch(MoveTemp(Function), RenderCommandTag::Get());
			return true;
		}

		return false;
	}

	template <typename RenderCommandTag>
	bool Enqueue(FEmptyFunction& Function)
	{
		// Execute the function directly if this is being called recursively from within another pipe command.
		if (UE::RenderCommandPipe::IsReplaying(*this))
		{
			ExecuteCommand(MoveTemp(Function), RenderCommandTag::Get());
			return true;
		}

		UE::TScopeLock Lock(Mutex);

		if (RecordTask.IsValid())
		{
			EnqueueAndLaunch(MoveTemp(Function), RenderCommandTag::Get());
			return true;
		}

		return false;
	}

	//////////////////////////////////////////////////////////////////////////

private:
	friend class FRenderCommandPipeRegistry;
	friend class FRenderCommandDispatcher;

	RENDERCORE_API void EnqueueAndLaunch(FRenderCommandList* CommandList);
	RENDERCORE_API void EnqueueAndLaunch(FRenderCommandFunctionVariant&& FunctionVariant, const FRenderCommandTag& Tag);

	void EnqueueAndLaunch(FCommandListFunction&& Function, const FRenderCommandTag& Tag)
	{
		EnqueueAndLaunch(FRenderCommandFunctionVariant(TInPlaceType<FCommandListFunction>(), MoveTemp(Function)), Tag);
	}

	void EnqueueAndLaunch(FEmptyFunction&& Function, const FRenderCommandTag& Tag)
	{
		EnqueueAndLaunch(FRenderCommandFunctionVariant(TInPlaceType<FEmptyFunction>(), MoveTemp(Function)), Tag);
	}

	RENDERCORE_API void ExecuteCommand(FRenderCommandFunctionVariant&& FunctionVariant, const FRenderCommandTag& Tag);

	void ExecuteCommand(FCommandListFunction&& Function, const FRenderCommandTag& Tag)
	{
		ExecuteCommand(FRenderCommandFunctionVariant(TInPlaceType<FCommandListFunction>(), MoveTemp(Function)), Tag);
	}

	void ExecuteCommand(FEmptyFunction&& Function, const FRenderCommandTag& Tag)
	{
		ExecuteCommand(FRenderCommandFunctionVariant(TInPlaceType<FEmptyFunction>(), MoveTemp(Function)), Tag);
	}

	void ExecuteCommands(UE::RenderCommandPipe::FCommandList& CommandList);
	void ExecuteCommands(FRenderCommandList* CommandList);

	const TCHAR* Name;
	UE::Tasks::FTask RecordTask;
	FRHICommandList* RHICmdList = nullptr;
	TLinkedList<FRenderCommandPipe*> GlobalListLink;
	FAutoConsoleVariable ConsoleVariable;
	std::atomic_int32_t NumInFlightCommands{ 0 };
	std::atomic_int32_t NumInFlightCommandLists{ 0 };
	int32 Index = INDEX_NONE;
	bool bRecording = false;
	bool bReplaying = false;
	bool bEnabled = true;
};

enum class ERenderCommandListFlags : uint8
{
	None,

	// Closes the command list on a call to Submit. Enables an optimization to skip submitting empty lists.
	CloseOnSubmit = 1 << 0
};
ENUM_CLASS_FLAGS(ERenderCommandListFlags);

/*
	Represents a command list of render commands that can be recorded on a thread and submitted. Recording is done using the FRecordScope which
	sets the command list in TLS, diverting any render commands enqueued via ENQUEUE_RENDER_COMMAND into the command list. Command lists can
	submit into other command lists as well as the main render command pipes. Command lists are useful for a couple reasons. First, the cost
	of queuing commands into command lists is very light when recording into command lists as there are no locks, at the cost of deferring
	submission of the work. Second, command lists can be submitted and recorded asynchronously from each other.

	Command lists actually contain several sub-command lists, one for each render command pipe. At submission time the sub-command lists are
	submitted separately. It doesn't matter if commands are enqueued to a pipe, they all go into the same command list.

	Command lists have two operations, Close and Submit. Call Close when recording is complete. Call Submit to patch the command list into a parent
	command list or the global render command pipes. Use FRenderCommandDispatcher::Submit to submit command lists.

	Command lists support a fast-path with ERenderCommandListFlags::CloseOnSubmit. This fuses the Close / Submit operations but enables an optimization
	to skip empty lists at the end, which is helpful when managing a large number of command lists (see FParallelForContext for a concrete use case).
*/
class FRenderCommandList final : public TConcurrentLinearObject<FRenderCommandList>
{
	friend class FRenderCommandPipe;
	friend class FRenderThreadCommandPipe;
	friend class FRenderCommandDispatcher;
	
	static thread_local FRenderCommandList* InstanceTLS;
	static RENDERCORE_API FRenderCommandList* GetInstanceTLS();
	static RENDERCORE_API FRenderCommandList* SetInstanceTLS(FRenderCommandList* CommandList);

public:
	using EPageSize = FMemStackBase::EPageSize;

	static FRenderCommandList* Create(ERenderCommandListFlags InFlags = ERenderCommandListFlags::None, EPageSize InPageSize = EPageSize::Small)
	{
		return new FRenderCommandList(InFlags, InPageSize);
	}

	RENDERCORE_API ~FRenderCommandList();

	enum class EStopRecordingAction
	{
		None,

		// Calls Close on the command list when the scope is complete.
		Close,

		// Calls Close and Submit on the command list when the scope is complete.
		Submit
	};

	// A scope to bind a command list for recording on the current thread.
	class FRecordScope
	{
	public:
		RENDERCORE_API ~FRecordScope();
		RENDERCORE_API FRecordScope(FRenderCommandList* InCommandList, EStopRecordingAction StopAction = EStopRecordingAction::None);

		FRecordScope(const FRecordScope&) = delete;

	private:
		FRenderCommandList* CommandList;
		FRenderCommandList* PreviousCommandList;
		EStopRecordingAction StopAction;
	};

	// A scope to unbind and flush the contents of the currently recording command list if there is one.
	class FFlushScope
	{
	public:
		RENDERCORE_API FFlushScope();
		RENDERCORE_API ~FFlushScope();

		FFlushScope(const FFlushScope&) = delete;

	private:
		FRenderCommandList* CommandList;
	};

	// Call when the command list recording is finished.
	RENDERCORE_API void Close();

	/*
		A task context for use with parallel for that allocates a command list for each task thread.

		Example Usage:

			FRenderCommandList* ParentCommandList = FRenderCommandList::Create();

			UE::Tasks::Launch(UE_SOURCE_LOCATION, [ParentCommandList]
			{
				// Closes recording of the command list on completion of the scope.
				FRenderCommandList::FRecordScope RecordScope(ParentCommandList, FRenderCommandList::EStopRecordingAction::Close);

				// Constructs a parallel for context with the command list as the root.
				FRenderCommandList::FParallelForContext ParallelForContext(ParentCommandList, NumContexts);

				// Issue a parallel with a command list per thread.
				ParallelForWithExistingTaskContext(TEXT("ParallelFor"), ParallelForContext.GetCommandLists(), ..., [] (FRenderCommandList* InRenderCommandList)
				{
					FRenderCommandList::FRecordScope RecordScope(InRenderCommandList);
				
					// Commands are recorded into InRenderCommandList.
				});
			});

			ENQUEUE_RENDER_COMMAND(... [] { ... }); // Command A

			// Submit the command list at any time. All included render commands are patched between commands A and C.
			ParentCommandList->Submit();

			ENQUEUE_RENDER_COMMAND(... [] { ... }); // Command C
	*/
	class FParallelForContext
	{
	public:
		~FParallelForContext()
		{
			Submit();
		}
		
		RENDERCORE_API FParallelForContext(FRenderCommandList* InRootCommandList, int32 NumContexts);
		RENDERCORE_API FParallelForContext(FRenderCommandList* InRootCommandList, int32 NumTasks, int32 BatchSize, EParallelForFlags Flags);

		FParallelForContext(const FParallelForContext&) = delete;

		FRenderCommandList* GetRootCommandList()
		{
			return RootCommandList;
		}

		TArrayView<FRenderCommandList*> GetCommandLists()
		{
			return TaskCommandLists;
		}

		RENDERCORE_API void Submit();

	private:
		FRenderCommandList* RootCommandList;
		TArray<FRenderCommandList*, FConcurrentLinearArrayAllocator> TaskCommandLists;
		bool bSubmitRootCommandList = false;
	};

	UE::Tasks::FTask GetDispatchTask() const
	{
		if (DispatchTaskEvent)
		{
			return *DispatchTaskEvent;
		}
		return {};
	}

private:
	RENDERCORE_API FRenderCommandList(ERenderCommandListFlags InFlags, EPageSize InPageSize);

	/** Use FRenderCommandDispatcher::Submit to submit command lists. */
	RENDERCORE_API void Submit(FRenderCommandList* InParentCommandList = nullptr);

	const UE::Tasks::FTask* TryGetDispatchTask() const
	{
		return DispatchTaskEvent.GetPtrOrNull();
	}

	void ReleasePipeRefs(int32 InNumRefs)
	{
		if (InNumRefs > 0)
		{
			int32 NumRefs = NumPipeRefs.fetch_sub(InNumRefs, std::memory_order_acq_rel) - InNumRefs;
			check(NumRefs >= 0);
			if (!NumRefs)
			{
				delete this;
			}
		}
	}

	void ReleasePipeRef()
	{
		ReleasePipeRefs(1);
	}

	bool HasDispatchTask()
	{
		return DispatchTaskEvent.IsSet();
	}

	void LazyInit()
	{
		if (!bInitialized)
		{
			Init();
		}
	}

	RENDERCORE_API void Init();

	RENDERCORE_API void Flush();

	template <typename RenderCommandTag, typename FunctionType>
	inline bool Enqueue(FunctionType&& Function)
	{
		return GetRenderThread().Enqueue<RenderCommandTag>(MoveTemp(Function));
	}

	template <typename RenderCommandTag, typename FunctionType>
	inline bool Enqueue(FRenderCommandPipe* Pipe, FunctionType&& Function)
	{
		return Get(Pipe).Enqueue<RenderCommandTag>(MoveTemp(Function));
	}

	inline UE::RenderCommandPipe::FCommandList& GetRenderThread()
	{
		LazyInit();
		return CommandLists.Last();
	}

	inline UE::RenderCommandPipe::FCommandList& Get(int32 PipeIndex)
	{
		LazyInit();
		return CommandLists[PipeIndex];
	}

	inline UE::RenderCommandPipe::FCommandList& Get(FRenderCommandPipe* Pipe)
	{
		LazyInit();
		return Pipe && Pipe->IsValid() ? CommandLists[Pipe->GetIndex()] : GetRenderThread();
	}

	FMemStackBase Allocator;
	TArray<UE::RenderCommandPipe::FCommandList, FConcurrentLinearArrayAllocator> CommandLists;
	TOptional<UE::Tasks::FTaskEvent> DispatchTaskEvent;

	std::atomic_int32_t NumPipeRefs = 0;

	const ERenderCommandListFlags Flags;
	uint8 bInitialized : 1 = false;
	uint8 bRecording   : 1 = true;
	uint8 bSubmitted   : 1 = false;

#if DO_CHECK
	uint8 NumRecordScopeRefs = 0;
#endif

	struct
	{
		FRenderCommandList* Head = nullptr;
		FRenderCommandList* Tail = nullptr;

	} Children;

	FRenderCommandList* Parent = nullptr;
	FRenderCommandList* NextSibling = nullptr;

	FRenderCommandPipeBitArray PipeEnqueueFailedBits;

	friend class FRenderCommandDispatcher;
};

class FRenderCommandDispatcher
{
public:
	/**
	 * Call to submit a command list into a parent command list or render command pipes. If the parent command list is null the recording instance
	 * is pulled from the currently bound render command list (set via FRecordScope). If both are null the commands are submitted to the global render
	 * command pipes.
	 */
	static void Submit(FRenderCommandList* RenderCommandList, FRenderCommandList* ParentCommandList = nullptr)
	{
		RenderCommandList->Submit(ParentCommandList);
	}

	template <typename RenderCommandTag>
	static void Enqueue(TUniqueFunction<void(FRHICommandListImmediate&)>&& Function)
	{
		if (AutoRTFM::IsClosed())
		{
			// We cannot move the `TUniqueFunction` into the `OnCommit` handler, so as a workaround we need to shove
			// it into a shared pointer and fire that in instead. Not ideal, but only affects the code when used under
			// transactions.
			AutoRTFM::OnCommit([SharedFunction = MakeShared<TUniqueFunction<void(FRHICommandListImmediate&)>>(MoveTemp(Function))]
			{
				TUniqueFunction<void(FRHICommandListImmediate&)> Function = MoveTemp(*SharedFunction);
				Enqueue<RenderCommandTag>(MoveTemp(Function));
			});

			return;
		}

		if (FRenderCommandList* CommandList = FRenderCommandList::GetInstanceTLS())
		{
			CommandList->Enqueue<RenderCommandTag>(MoveTemp(Function));
			return;
		}

		FRenderThreadCommandPipe::Enqueue<RenderCommandTag>(MoveTemp(Function));
	}

	template <typename RenderCommandTag>
	static void Enqueue(FRenderCommandPipe* Pipe, FRenderCommandPipe::FCommandListFunction&& Function)
	{
		if (FRenderCommandList* CommandList = FRenderCommandList::GetInstanceTLS())
		{
			CommandList->Enqueue<RenderCommandTag>(Pipe, MoveTemp(Function));
			return;
		}

		if (GRenderCommandPipeMode == ERenderCommandPipeMode::All && Pipe && Pipe->Enqueue<RenderCommandTag>(Function))
		{
			return;
		}

		FRenderThreadCommandPipe::Enqueue<RenderCommandTag>([Function = MoveTemp(Function)](FRHICommandListImmediate& RHICmdList) { Function(RHICmdList); });
	}

	template <typename RenderCommandTag>
	inline static void Enqueue(FRenderCommandPipe& Pipe, FRenderCommandPipe::FCommandListFunction&& Function)
	{
		Enqueue<RenderCommandTag>(&Pipe, MoveTemp(Function));
	}

	template <typename RenderCommandTag>
	static void Enqueue(FRenderCommandPipe* Pipe, FRenderCommandPipe::FEmptyFunction&& Function)
	{
		if (FRenderCommandList* CommandListSet = FRenderCommandList::GetInstanceTLS())
		{
			CommandListSet->Enqueue<RenderCommandTag>(Pipe, MoveTemp(Function));
			return;
		}

		if (GRenderCommandPipeMode == ERenderCommandPipeMode::All && Pipe && Pipe->Enqueue<RenderCommandTag>(Function))
		{
			return;
		}

		FRenderThreadCommandPipe::Enqueue<RenderCommandTag>([Function = MoveTemp(Function)](FRHICommandListImmediate&) { Function(); });
	}

	template <typename RenderCommandTag>
	inline  static void Enqueue(FRenderCommandPipe& Pipe, FRenderCommandPipe::FEmptyFunction&& Function)
	{
		Enqueue<RenderCommandTag>(&Pipe, MoveTemp(Function));
	}
};

/** Declares an extern reference to a render command pipe. */
#define DECLARE_RENDER_COMMAND_PIPE(Name, PrefixKeywords) \
	namespace UE::RenderCommandPipe { extern PrefixKeywords FRenderCommandPipe Name; }

/** Defines a render command pipe. */
#define DEFINE_RENDER_COMMAND_PIPE(Name, Flags) \
	namespace UE::RenderCommandPipe \
	{ \
		FRenderCommandPipe Name( \
			TEXT(#Name), \
			Flags, \
			TEXT("r.RenderCommandPipe." #Name), \
			TEXT("Whether to enable the " #Name " Render Command Pipe") \
			TEXT(" 0: off;") \
			TEXT(" 1: on (default)") \
		); \
	}

/** Enqueues a render command to a render pipe. The default implementation takes a lambda and schedules on the render thread.
 *  Alternative implementations accept either a reference or pointer to an FRenderCommandPipe instance to schedule on an async
 *  pipe, if enabled.
 */
#define ENQUEUE_RENDER_COMMAND(Type, ...) \
	DECLARE_RENDER_COMMAND_TAG(UE_JOIN(FRenderCommandTag_, Type, __LINE__), Type, __VA_ARGS__) \
	FRenderCommandDispatcher::Enqueue<UE_JOIN(FRenderCommandTag_, Type, __LINE__)>

#define ENQUEUE_RENDER_COMMAND_NOMARKER(Type, ...) \
	DECLARE_RENDER_COMMAND_TAG_NOMARKER(UE_JOIN(FRenderCommandTag_, Type, __LINE__), Type, __VA_ARGS__) \
	FRenderCommandDispatcher::Enqueue<UE_JOIN(FRenderCommandTag_, Type, __LINE__)>

////////////////////////////////////
// RenderThread scoped work
////////////////////////////////////

struct FRenderThreadStructBase
{
	FRenderThreadStructBase() = default;

	// Copy construction is not allowed. Used to avoid accidental copying in the command lambda.
	FRenderThreadStructBase(const FRenderThreadStructBase&) = delete;

	void InitRHI(FRHICommandListImmediate&) {}
	void ReleaseRHI(FRHICommandListImmediate&) {}
};

/**  Represents a struct with a lifetime that spans multiple render commands with scoped initialization
 *   and release on the render thread.
 * 
 *   Example:
 * 
 *   struct FMyStruct : public FRenderThreadStructBase
 *   {
 *       FInitializer { int32 Foo; int32 Bar; };
 * 
 *       FMyStruct(const FInitializer& InInitializer)
 *            : Initializer(InInitializer)
 *       {
 *            // Called immediately by TRenderThreadStruct when created.
 *       }
 * 
 *       ~FMyStruct()
 *       {
 *           // Called on the render thread when TRenderThreadStruct goes out of scope.
 *       }
 * 
 *       void InitRHI(FRHICommandListImmediate& RHICmdList)
 *       {
 *           // Called on the render thread by TRenderThreadStruct when created.
 *       }
 * 
 *       void ReleaseRHI(FRHICommandListImmediate& RHICmdList)
 *       {
 *           // Called on the render thread when TRenderThreadStruct goes out of scope.
 *       }
 * 
 *       FInitializer Initializer;
 *   };
 *
 *   // On Main Thread
 * 
 *   {
 *       TRenderThreadStruct<FMyStruct> MyStruct(FMyStruct::FInitializer{1, 2});
 * 
 *       ENQUEUE_RENDER_COMMAND(CommandA)[MyStruct = MyStruct.Get()](FRHICommandListImmediate& RHICmdList)
 *       {
 *           // Do something with MyStruct.
 *       };
 * 
 *       ENQUEUE_RENDER_COMMAND(CommandB)[MyStruct = MyStrucft.Get()](FRHICommandListImmediate& RHICmdList)
 *       {
 *           // Do something else with MyStruct.
 *       };
 * 
 *       // MyStruct instance is automatically released and deleted on the render thread.
 *   }
 */
template <typename StructType>
class TRenderThreadStruct
{
public:
	static_assert(TIsDerivedFrom<StructType, FRenderThreadStructBase>::IsDerived, "StructType must be derived from FRenderThreadStructBase.");

	template <typename... TArgs>
	TRenderThreadStruct(TArgs&&... Args)
		: Struct(new StructType(Forward<TArgs&&>(Args)...))
	{
		ENQUEUE_RENDER_COMMAND(InitStruct)([Struct = Struct](FRHICommandListImmediate& RHICmdList)
		{
			Struct->InitRHI(RHICmdList);
		});
	}

	~TRenderThreadStruct()
	{
		ENQUEUE_RENDER_COMMAND(DeleteStruct)([Struct = Struct](FRHICommandListImmediate& RHICmdList)
		{
			Struct->ReleaseRHI(RHICmdList);
			delete Struct;
		});
		Struct = nullptr;
	}

	TRenderThreadStruct(const TRenderThreadStruct&) = delete;

	const StructType* operator->() const
	{
		return Struct;
	}

	StructType* operator->()
	{
		return Struct;
	}

	const StructType& operator*() const
	{
		return *Struct;
	}

	StructType& operator*()
	{
		return *Struct;
	}

	const StructType* Get() const
	{
		return Struct;
	}

	StructType* Get()
	{
		return Struct;
	}

private:
	StructType* Struct;
};

DECLARE_MULTICAST_DELEGATE(FStopRenderingThread);
using FStopRenderingThreadDelegate = FStopRenderingThread::FDelegate;

extern RENDERCORE_API FDelegateHandle RegisterStopRenderingThreadDelegate(const FStopRenderingThreadDelegate& InDelegate);

extern RENDERCORE_API void UnregisterStopRenderingThreadDelegate(FDelegateHandle InDelegateHandle);

///////////////////////////////////////////////////////////////////////////////
// Deprecated Types

///////////////////////////////////////////////////////////////////////////////

#undef UE_API