// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GlobalShader.h"
#include "HAL/Platform.h"
#include "RHIDefinitions.h"
#include "Serialization/MemoryLayout.h"
#include "Shader.h"
#include "ShaderCore.h"
#include "ShaderParameters.h"

class FPointerTableBase;
class FRHICommandList;
struct FResolveRect;

struct FDummyResolveParameter {};

class FResolveDepthPS : public FGlobalShader
{
	DECLARE_EXPORTED_GLOBAL_SHADER(FResolveDepthPS, RENDERCORE_API);
public:
	typedef FDummyResolveParameter FParameter;
		
	FResolveDepthPS();
	FResolveDepthPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer);

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment);
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters);

	RENDERCORE_API void SetParameters(FRHIBatchedShaderParameters& BatchedParameters, FParameter);

	LAYOUT_FIELD(FShaderResourceParameter, UnresolvedSurface);
};

class FResolveDepth2XPS : public FResolveDepthPS
{
	DECLARE_EXPORTED_GLOBAL_SHADER(FResolveDepth2XPS, RENDERCORE_API);
public:

	typedef FDummyResolveParameter FParameter;

	FResolveDepth2XPS();
	FResolveDepth2XPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer);

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment);
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return true;
	}
};

class FResolveDepth4XPS : public FResolveDepthPS
{
	DECLARE_EXPORTED_GLOBAL_SHADER(FResolveDepth4XPS, RENDERCORE_API);
public:
	typedef FDummyResolveParameter FParameter;

	FResolveDepth4XPS();
	FResolveDepth4XPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer);

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment);
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return true;
	}
};

class FResolveDepth8XPS : public FResolveDepthPS
{
	DECLARE_EXPORTED_GLOBAL_SHADER(FResolveDepth8XPS, RENDERCORE_API);
public:
	typedef FDummyResolveParameter FParameter;

	FResolveDepth8XPS();
	FResolveDepth8XPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer);

	static bool ShouldCache(EShaderPlatform Platform);
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment);
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return true;
	}
};

class FResolveDepthArray2XPS : public FResolveDepthPS
{
	DECLARE_EXPORTED_GLOBAL_SHADER(FResolveDepthArray2XPS, RENDERCORE_API);
public:
	typedef FDummyResolveParameter FParameter;

	FResolveDepthArray2XPS();
	FResolveDepthArray2XPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer);

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters);
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment);
};

class FResolveDepthArray4XPS : public FResolveDepthPS
{
	DECLARE_EXPORTED_GLOBAL_SHADER(FResolveDepthArray4XPS, RENDERCORE_API);
public:
	typedef FDummyResolveParameter FParameter;

	FResolveDepthArray4XPS();
	FResolveDepthArray4XPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer);

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters);
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment);
};

class FResolveDepthArray8XPS : public FResolveDepthPS
{
	DECLARE_EXPORTED_GLOBAL_SHADER(FResolveDepthArray8XPS, RENDERCORE_API);
public:
	typedef FDummyResolveParameter FParameter;

	FResolveDepthArray8XPS();
	FResolveDepthArray8XPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer);

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters);
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment);
};

class FResolveSingleSamplePS : public FGlobalShader
{
	DECLARE_EXPORTED_GLOBAL_SHADER(FResolveSingleSamplePS, RENDERCORE_API);
public:
	typedef uint32 FParameter;

	FResolveSingleSamplePS();
	FResolveSingleSamplePS(const ShaderMetaType::CompiledShaderInitializerType& Initializer);
	
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters);

	RENDERCORE_API void SetParameters(FRHIBatchedShaderParameters& BatchedParameters, uint32 SingleSampleIndexValue);
	
	LAYOUT_FIELD(FShaderResourceParameter, UnresolvedSurface);
	LAYOUT_FIELD(FShaderParameter, SingleSampleIndex);
};

/**
 * A vertex shader for rendering a textured screen element.
 */
class FResolveVS : public FGlobalShader
{
	DECLARE_EXPORTED_GLOBAL_SHADER(FResolveVS, RENDERCORE_API);
public:
	FResolveVS();
	FResolveVS(const ShaderMetaType::CompiledShaderInitializerType& Initializer);
	
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters);

	RENDERCORE_API void SetParameters(FRHIBatchedShaderParameters& BatchedParameters, const FResolveRect& SrcBounds, const FResolveRect& DstBounds, uint32 DstSurfaceWidth, uint32 DstSurfaceHeight);

	LAYOUT_FIELD(FShaderParameter, PositionMinMax);
	LAYOUT_FIELD(FShaderParameter, UVMinMax);
};

class FResolveArrayVS : public FResolveVS
{
	DECLARE_EXPORTED_GLOBAL_SHADER(FResolveArrayVS, RENDERCORE_API);
public:
	FResolveArrayVS();
	FResolveArrayVS(const ShaderMetaType::CompiledShaderInitializerType& Initializer);

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters);
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment);
};
