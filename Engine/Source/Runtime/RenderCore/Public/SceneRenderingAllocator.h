// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Experimental/ConcurrentLinearAllocator.h"

struct FSceneRenderingBlockAllocationTag
{
	static constexpr uint32 BlockSize = 64 * 1024;		// Blocksize used to allocate from
	static constexpr bool AllowOversizedBlocks = true;  // The allocator supports oversized Blocks and will store them in a seperate Block with counter 1
	static constexpr bool RequiresAccurateSize = true;  // GetAllocationSize returning the accurate size of the allocation otherwise it could be relaxed to return the size to the end of the Block
	static constexpr bool InlineBlockAllocation = false;  // Inline or Noinline the BlockAllocation which can have an impact on Performance
	static constexpr const char* TagName = "SceneRenderingAllocator";

	using Allocator = TBlockAllocationLockFreeCache<BlockSize, FAlignedAllocator>;
};

using FSceneRenderingBulkObjectAllocator = TConcurrentLinearBulkObjectAllocator<FSceneRenderingBlockAllocationTag>;

template <typename T>
using FSceneRenderingAllocatorObject = TConcurrentLinearObject<T, FSceneRenderingBlockAllocationTag>;

using FSceneRenderingAllocator = TConcurrentLinearAllocator<FSceneRenderingBlockAllocationTag>;
using FSceneRenderingArrayAllocator = TConcurrentLinearArrayAllocator<FSceneRenderingBlockAllocationTag>;

using SceneRenderingAllocator = TConcurrentLinearArrayAllocator<FSceneRenderingBlockAllocationTag>;
using SceneRenderingBitArrayAllocator = TConcurrentLinearBitArrayAllocator<FSceneRenderingBlockAllocationTag>;
using SceneRenderingSparseArrayAllocator = TConcurrentLinearSparseArrayAllocator<FSceneRenderingBlockAllocationTag>;
using SceneRenderingSetAllocator = TConcurrentLinearSetAllocator<FSceneRenderingBlockAllocationTag>;
