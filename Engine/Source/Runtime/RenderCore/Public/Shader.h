// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	Shader.h: Shader definitions.
=============================================================================*/

#pragma once

#include "Algo/BinarySearch.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/ContainersFwd.h"
#include "Containers/EnumAsByte.h"
#include "Containers/HashTable.h"
#include "Containers/List.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Containers/StringFwd.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "HAL/CriticalSection.h"
#include "HAL/PlatformCrt.h"
#include "HAL/ThreadSafeBool.h"
#include "HAL/UnrealMemory.h"
#include "Math/NumericLimits.h"
#include "Misc/AssertionMacros.h"
#include "Misc/CString.h"
#include "Misc/EnumClassFlags.h"
#include "Misc/ScopeLock.h"
#include "Misc/SecureHash.h"
#include "RHI.h"
#include "RHIDefinitions.h"
#include "RHIMemoryLayout.h"
#include "RenderResource.h"
#include "RenderDeferredCleanup.h"
#include "Serialization/Archive.h"
#include "Serialization/ArchiveProxy.h"
#include "Serialization/MemoryImage.h"
#include "Serialization/MemoryLayout.h"
#include "ShaderCore.h"
#include "ShaderParameterMetadata.h"
#include "ShaderParameters.h"
#include "ShaderPermutation.h"
#include "ShaderPermutationUtils.h"
#include "ShaderMapAssetAssociation.h" // For FShaderMapAssetPaths
#include "Templates/RefCounting.h"
#include "Templates/TypeHash.h"
#include "Templates/UniquePtr.h"
#include "Templates/UnrealTemplate.h"
#include "Templates/UnrealTypeTraits.h"
#include "UObject/NameTypes.h"
#include "UObject/RenderingObjectVersion.h"

#include <atomic>

// For FShaderUniformBufferParameter

#if WITH_EDITOR
#include "UObject/DebugSerializationFlags.h"
#endif

class ITargetPlatform;
class FCbFieldView;
class FCbWriter;
class FComputeKernelShaderType;
class FNNERuntimeIREEShaderType;
class FGlobalShaderType;
class FMaterialShaderType;
class FMemoryImageWriter;
class FMemoryUnfreezeContent;
class FMeshMaterialShaderType;
class FNiagaraShaderType;
class FOpenColorIOShaderType;
class FRHIComputeCommandList;
class FShader;
class FShaderKeyGenerator;
class FShaderMapBase;
class FShaderMapPointerTable;
class FShaderParametersMetadata;
class FShaderPipelineType;
class FShaderType;
class FVertexFactoryType;
class FRHIShaderBindingLayout;
struct FShaderCompiledShaderInitializerType;
struct FShaderCompileJobKey;
struct FShaderCompilerOutput;
struct FShaderSerializeContext;

/** Define a shader permutation uniquely according to its type, and permutation id.*/
template<typename MetaShaderType>
struct TShaderTypePermutation
{
	MetaShaderType* const Type;
	const int32 PermutationId;

	TShaderTypePermutation(MetaShaderType* InType, int32 InPermutationId)
		: Type(InType)
		, PermutationId(InPermutationId)
	{
	}

	inline bool operator==(const TShaderTypePermutation& Other)const
	{
		return Type == Other.Type && PermutationId == Other.PermutationId;
	}

	inline bool operator!=(const TShaderTypePermutation& Other)const
	{
		return !(*this == Other);
	}

	friend inline uint32 GetTypeHash(const TShaderTypePermutation& Var)
	{
		return HashCombine(GetTypeHash(Var.Type), (uint32)Var.PermutationId);
	}
};

using FShaderPermutation = TShaderTypePermutation<FShaderType>;

inline const int32 kUniqueShaderPermutationId = 0;

/** Used to compare order shader types permutation deterministically. */
template<typename MetaShaderType>
class TCompareShaderTypePermutation
{
public:
	inline bool operator()(const TShaderTypePermutation<MetaShaderType>& A, const TShaderTypePermutation<MetaShaderType>& B) const
	{
		int32 AL = FCString::Strlen(A.Type->GetName());
		int32 BL = FCString::Strlen(B.Type->GetName());
		if (AL == BL)
		{
			int32 StrCmp = FCString::Strncmp(A.Type->GetName(), B.Type->GetName(), AL);
			if (StrCmp != 0)
			{
				return StrCmp > 0;
			}
			return A.PermutationId > B.PermutationId;
		}
		return AL > BL;
	}
};

class FShaderUniformBufferParameterInfo
{
	DECLARE_EXPORTED_TYPE_LAYOUT(FShaderUniformBufferParameterInfo, RENDERCORE_API, NonVirtual);
public:
	LAYOUT_FIELD(uint16, BaseIndex);

	FShaderUniformBufferParameterInfo() = default;

	FShaderUniformBufferParameterInfo(uint16 InBaseIndex)
	{
		BaseIndex = InBaseIndex;
		checkf(BaseIndex == InBaseIndex, TEXT("Tweak FShaderUniformBufferParameterInfo type sizes"));
	}

	friend FArchive& operator<<(FArchive& Ar, FShaderUniformBufferParameterInfo& Info)
	{
		Ar << Info.BaseIndex;
		return Ar;
	}

	inline bool operator==(const FShaderUniformBufferParameterInfo& Rhs) const
	{
		return BaseIndex == Rhs.BaseIndex;
	}

	inline bool operator<(const FShaderUniformBufferParameterInfo& Rhs) const
	{
		return BaseIndex < Rhs.BaseIndex;
	}
};

class FShaderResourceParameterInfo
{
	DECLARE_EXPORTED_TYPE_LAYOUT(FShaderResourceParameterInfo, RENDERCORE_API, NonVirtual);
public:
	LAYOUT_FIELD(uint16, BaseIndex);
	LAYOUT_FIELD(uint8, BufferIndex);
	LAYOUT_FIELD(EShaderParameterType, Type);

	FShaderResourceParameterInfo() = default;

	FShaderResourceParameterInfo(uint16 InBaseIndex, uint8 InBufferIndex, EShaderParameterType InType)
	{
		BaseIndex = InBaseIndex;
		BufferIndex = InBufferIndex;
		Type = InType;
		checkf(BaseIndex == InBaseIndex && BufferIndex == InBufferIndex && Type == InType, TEXT("Tweak FShaderResourceParameterInfo type sizes"));
	}

	friend FArchive& operator<<(FArchive& Ar, FShaderResourceParameterInfo& Info)
	{
		Ar << Info.BaseIndex;
		Ar << Info.BufferIndex;
		Ar << Info.Type;
		return Ar;
	}

	inline bool operator==(const FShaderResourceParameterInfo& Rhs) const
	{
		return BaseIndex == Rhs.BaseIndex
			&& BufferIndex == Rhs.BufferIndex
			&& Type == Rhs.Type;
	}

	inline bool operator<(const FShaderResourceParameterInfo& Rhs) const
	{
		return BaseIndex < Rhs.BaseIndex;
	}
};

class FShaderLooseParameterInfo
{
	DECLARE_EXPORTED_TYPE_LAYOUT(FShaderLooseParameterInfo, RENDERCORE_API, NonVirtual);
public:
	LAYOUT_FIELD(uint16, BaseIndex);
	LAYOUT_FIELD(uint16, Size);

	FShaderLooseParameterInfo() = default;

	FShaderLooseParameterInfo(uint16 InBaseIndex, uint16 InSize)
	{
		BaseIndex = InBaseIndex;
		Size = InSize;
		checkf(BaseIndex == InBaseIndex && Size == InSize, TEXT("Tweak FShaderLooseParameterInfo type sizes"));
	}

	friend FArchive& operator<<(FArchive& Ar, FShaderLooseParameterInfo& Info)
	{
		Ar << Info.BaseIndex;
		Ar << Info.Size;
		return Ar;
	}

	inline bool operator==(const FShaderLooseParameterInfo& Rhs) const
	{
		return BaseIndex == Rhs.BaseIndex
			&& Size == Rhs.Size;
	}

	inline bool operator<(const FShaderLooseParameterInfo& Rhs) const
	{
		return BaseIndex < Rhs.BaseIndex;
	}
};

class FShaderLooseParameterBufferInfo
{
	DECLARE_EXPORTED_TYPE_LAYOUT(FShaderLooseParameterBufferInfo, RENDERCORE_API, NonVirtual);
public:
	LAYOUT_FIELD(uint16, BaseIndex);
	LAYOUT_FIELD(uint16, Size);

	LAYOUT_FIELD(TMemoryImageArray<FShaderLooseParameterInfo>, Parameters);

	FShaderLooseParameterBufferInfo() {}

	FShaderLooseParameterBufferInfo(uint16 InBufferIndex, uint16 InBufferSize)
	{
		BaseIndex = InBufferIndex;
		Size = InBufferSize;
		checkf(BaseIndex == InBufferIndex, TEXT("Tweak FShaderLooseParameterBufferInfo type sizes"));
	}

	friend FArchive& operator<<(FArchive& Ar,FShaderLooseParameterBufferInfo& Info)
	{
		Ar << Info.BaseIndex;
		Ar << Info.Size;
		Ar << Info.Parameters;
		return Ar;
	}

	inline bool operator==(const FShaderLooseParameterBufferInfo& Rhs) const
	{
		return BaseIndex == Rhs.BaseIndex
			&& Size == Rhs.Size
			&& Parameters == Rhs.Parameters;
	}

	inline bool operator<(const FShaderLooseParameterBufferInfo& Rhs) const
	{
		return BaseIndex < Rhs.BaseIndex;
	}
};

class FShaderParameterMapInfo
{
	DECLARE_EXPORTED_TYPE_LAYOUT(FShaderParameterMapInfo, RENDERCORE_API, NonVirtual);
public:
	LAYOUT_FIELD(TMemoryImageArray<FShaderUniformBufferParameterInfo>, UniformBuffers);
	LAYOUT_FIELD(TMemoryImageArray<FShaderResourceParameterInfo>, TextureSamplers);
	LAYOUT_FIELD(TMemoryImageArray<FShaderResourceParameterInfo>, SRVs);
	LAYOUT_FIELD(TMemoryImageArray<FShaderLooseParameterBufferInfo>, LooseParameterBuffers);
	LAYOUT_FIELD(uint64, Hash);

	friend FArchive& operator<<(FArchive& Ar,FShaderParameterMapInfo& Info)
	{
		Ar << Info.UniformBuffers;
		Ar << Info.TextureSamplers;
		Ar << Info.SRVs;
		Ar << Info.LooseParameterBuffers;
		Ar << Info.Hash;
		return Ar;
	}

	inline bool operator==(const FShaderParameterMapInfo& Rhs) const
	{
		return Hash == Rhs.Hash;
	}
};

enum class ERayTracingPayloadType : uint32; // actual enum is defined in /Shaders/Shared/RayTracingPayloadType.h
typedef uint32(*TRaytracingPayloadSizeFunction)();
ENUM_CLASS_FLAGS(ERayTracingPayloadType);

RENDERCORE_API uint32 GetRayTracingPayloadTypeMaxSize(ERayTracingPayloadType PayloadType);
RENDERCORE_API void RegisterRayTracingPayloadType(ERayTracingPayloadType PayloadType, uint32 PayloadSize, TRaytracingPayloadSizeFunction PayloadSizeFunction);

struct FRegisterRayTracingPayloadTypeHelper
{
	FRegisterRayTracingPayloadTypeHelper(ERayTracingPayloadType PayloadType, uint32 PayloadSize, TRaytracingPayloadSizeFunction PayloadSizeFunction)
	{
		RegisterRayTracingPayloadType(PayloadType, PayloadSize, PayloadSizeFunction);
	}
};

#define IMPLEMENT_RT_PAYLOAD_TYPE_CONCAT2( x, y ) x##y
#define IMPLEMENT_RT_PAYLOAD_TYPE_CONCAT( x, y ) IMPLEMENT_RT_PAYLOAD_TYPE_CONCAT2( x, y )
#define IMPLEMENT_RT_PAYLOAD_TYPE(PayloadType, PayloadSize)  static FRegisterRayTracingPayloadTypeHelper IMPLEMENT_RT_PAYLOAD_TYPE_CONCAT(PayloadHelper, __COUNTER__) = FRegisterRayTracingPayloadTypeHelper(PayloadType, PayloadSize, nullptr);
#define IMPLEMENT_RT_PAYLOAD_TYPE_FUNCTION(PayloadType, PayloadSizeFunction)  static FRegisterRayTracingPayloadTypeHelper IMPLEMENT_RT_PAYLOAD_TYPE_CONCAT(PayloadHelper, __COUNTER__) = FRegisterRayTracingPayloadTypeHelper(PayloadType, 0u, PayloadSizeFunction);

class FShaderMapResource : public FRenderResource, public FDeferredCleanupInterface
{
public:
	static RENDERCORE_API bool ArePlatformsCompatible(EShaderPlatform CurrentPlatform, EShaderPlatform TargetPlatform);

	EShaderPlatform GetPlatform() const { return Platform; }

	RENDERCORE_API void AddRef();
	RENDERCORE_API void Release();
	inline int32 GetNumRefs() const { return NumRefs.load(std::memory_order_relaxed); }

	// FRenderResource interface.
	RENDERCORE_API virtual void ReleaseRHI() override;

	inline int32 GetNumShaders() const
	{
		return NumRHIShaders;
	}

	inline bool IsValidShaderIndex(int32 ShaderIndex) const
	{
		return static_cast<uint32>(ShaderIndex) < NumRHIShaders;
	}

	inline bool HasShader(int32 ShaderIndex) const
	{
		return RHIShaders[ShaderIndex].load(std::memory_order_acquire) != nullptr;
	}

	virtual uint32 GetShaderSizeBytes(int32 ShaderIndex) const;

	virtual void PreloadShader(int32 ShaderIndex, FGraphEventArray& OutCompletionEvents)
	{
		/* no-op when not using shader library */
	};

	virtual void PreloadShaderMap(FGraphEventArray& OutCompletionEvents)
	{
		/* no-op when not using shader library */
	};

	virtual int32 GetGroupIndexForShader(int32 ShaderIndex) const
	{
		/* return invalid index when not using shader library */
		return INDEX_NONE;
	};

	virtual int32 GetLibraryId() const
	{
		/* return invalid index when not using shader library */
		return INDEX_NONE;
	}

	virtual int32 GetLibraryShaderIndex(int32 ShaderIndex) const
	{
		/* return invalid index when not using shader library */
		return INDEX_NONE;
	}

	/** Returns whether all shader chunks for this shader map are resolved and available. Used to guard against PSO precaching on shaders whose IoStore chunks are not yet installed. */
	virtual bool IsResolved() const
	{
		return true;
	}

	inline bool ContainsAtLeastOneRHIShaderCreated() const
	{
		return bAtLeastOneRHIShaderCreated;
	}

	inline FRHIShader* GetShader(int32 ShaderIndex, bool bRequired = true)
	{
		// This is a double checked locking. This trickery arises from the fact that we're
		// synchronizing two threads: one that takes a lock and another that doesn't.
		// Without fences, there is a race between storing the shader pointer and accessing it
		// on the other (lockless) thread.
		FRHIShader* Shader = RHIShaders[ShaderIndex].load(std::memory_order_acquire);
		if (UNLIKELY(Shader == nullptr))
		{
			Shader = CreateShaderOrCrash(ShaderIndex, bRequired);
		}
		return Shader;
	}

	/** Return shader hash for a particular shader without creating it. */
	virtual FShaderHash GetShaderHash(int32 ShaderIndex) = 0;

	virtual FShaderHash GetShaderMapHash() const
	{
		/* return invalid hash when not using shader library */
		return FShaderHash();
	}

	RENDERCORE_API void BeginCreateAllShaders();

#if RHI_RAYTRACING
	static RENDERCORE_API void GetRayTracingHitGroupLibrary(EShaderPlatform ShaderPlatform, TArray<FRHIRayTracingShader*>& RayTracingHitGroupShaders, FRHIRayTracingShader* DefaultShader);
	static RENDERCORE_API void GetRayTracingCallableShaderLibrary(EShaderPlatform ShaderPlatform, TArray<FRHIRayTracingShader*>& RayTracingCallableShaders, FRHIRayTracingShader* DefaultShader);
	static RENDERCORE_API void GetRayTracingMissShaderLibrary(EShaderPlatform ShaderPlatform, TArray<FRHIRayTracingShader*>& RayTracingMissShaders, FRHIRayTracingShader* DefaultShader);

	inline uint32 GetRayTracingHitGroupLibraryIndex(int32 ShaderIndex)
	{
		FRHIShader* Shader = GetShader(ShaderIndex);	// make sure the shader is created
		checkSlow(Shader->GetFrequency() == SF_RayHitGroup);
		return RayTracingLibraryIndices[ShaderIndex];
	}

	inline uint32 GetRayTracingCallableShaderLibraryIndex(int32 ShaderIndex)
	{
		FRHIShader* Shader = GetShader(ShaderIndex);	// make sure the shader is created
		checkSlow(Shader->GetFrequency() == SF_RayCallable);
		return RayTracingLibraryIndices[ShaderIndex];
	}

	inline uint32 GetRayTracingMissShaderLibraryIndex(int32 ShaderIndex)
	{
		FRHIShader* Shader = GetShader(ShaderIndex);	// make sure the shader is created
		checkSlow(Shader->GetFrequency() == SF_RayMiss);
		return RayTracingLibraryIndices[ShaderIndex];
	}
#endif // RHI_RAYTRACING

	virtual uint32 GetSizeBytes() const = 0;

protected:
	RENDERCORE_API explicit FShaderMapResource(EShaderPlatform InPlatform, int32 NumShaders);
	RENDERCORE_API virtual ~FShaderMapResource();

	SIZE_T GetAllocatedSize() const
	{
		SIZE_T Size = NumRHIShaders * sizeof(std::atomic<FRHIShader*>);
#if RHI_RAYTRACING
		Size += RayTracingLibraryIndices.GetAllocatedSize();
#endif
		return Size;
	}

	/** Creates RHI shader, with a reference (so the caller can release). Never returns nullptr (inability to create is Fatal) */
	virtual FRHIShader*	CreateRHIShaderOrCrash(int32 ShaderIndex, bool bRequired) = 0;

	/** Signal the shader library that it can release compressed shader code for a shader that it keeps preloaded in memory. */
	virtual void ReleasePreloadedShaderCode(int32 ShaderIndex) { /* no-op when not using shader library */ };

	virtual bool TryRelease() { return true; }

	/** Releases all RHI shaders. This retains the internal array so that new RHI shaders can be created again, for example when a shadermap resource is being transferred to a new shader library instance. */
	RENDERCORE_API void ReleaseShaders();

	/** Exchanges an RHI shader. This is meant to be used by a shadermap resource that is transferred to a new shader library instance. */
	inline FRHIShader* ExchangeShader(int32 ShaderIndex, FRHIShader* Shader)
	{
		return RHIShaders[ShaderIndex].exchange(Shader, std::memory_order_acquire);
	}

private:

	/** Creates an entry in RHIShaders array and registers it among the raytracing libs if needed. Created shader is returned. */
	RENDERCORE_API FRHIShader* CreateShaderOrCrash(int32 ShaderIndex, bool bRequired);

	/** This lock is to prevent two threads creating the same RHIShaders element. It is only taken if the element is to be created. */
	FCriticalSection RHIShadersCreationGuard;

#if RHI_RAYTRACING
	TArray<uint32> RayTracingLibraryIndices;
#endif // RHI_RAYTRACING

	/** An array of shader pointers (refcount is managed manually). */
	TUniquePtr<std::atomic<FRHIShader*>[]> RHIShaders;

	/** Since the shaders are no longer a TArray, this is their count (the size of the RHIShaders array). This does not count actually created RHI shaders, just the size of the above array of pointers to them. */
	uint32 NumRHIShaders : 31;

	/** Whether we have at least one RHI shader created. */
	uint32 bAtLeastOneRHIShaderCreated : 1;

	EShaderPlatform Platform;

	/** The number of references to this shader map. */
	std::atomic<int32> NumRefs;
};

class FShaderMapResourceCode : public FRefCountedObject
{
public:
#if WITH_EDITORONLY_DATA
	struct FShaderEditorOnlyDataEntry
	{
		// Sets the symbols for this entry to the given buffer if either unset or the hash of the new buffer is "less" than the hash of the stored buffer.
		// This is done purely for determinism's sake (we always want to store the same copy of symbols in the editoronly data for any given shader bytecode).
		void ConditionalSetSymbolBuffer(FCompressedBuffer InSymbols);
	
		uint64 SymbolHash = 0u; // Used only to ensure the copy of symbols saved is stable for determinism's sake; does not need to be serialized.
		FCompressedBuffer SymbolBuffer;

		/** This field contains a debug string stored in a ShaderSymbols.info file when using r.Shaders.SymbolInfo=1
		*   Used to facilitate reverse lookup of a shader from a platform shader hash when no full shader symbol 
		*   information is available */
		FString DebugInfo;

		UE_DEPRECATED(5.8, "Warnings are no longer serialized with shadermaps to DDC due to significant overhead. If you need to see shader compile warnings, please enable shader dev mode and force a recompile.")
		TArray<FString> CompilerWarnings;
		
		/** An array of the hash of all FShaderType names which generate this particular shader (i.e. the return value
		 * of FShaderType::GetHashedName().GetHash()). */
		TArray<uint64> ShaderTypeHashes;

		/** Generic, data-driven key/value pairs of statistics. */
		TArray<FGenericShaderStat> ShaderStatistics;

		friend FArchive& operator<<(FArchive& Ar, FShaderEditorOnlyDataEntry& Entry)
		{
			return Ar << Entry.DebugInfo << Entry.ShaderTypeHashes << Entry.ShaderStatistics;
		}

		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		// Explicitly-defaulted copy/move ctors & assignment operators are needed temporarily due to deprecation
		// of PlatformDebugData field. These can be removed once the deprecation window for said field ends.
		FShaderEditorOnlyDataEntry() = default;
		FShaderEditorOnlyDataEntry(FShaderEditorOnlyDataEntry&&) = default;
		FShaderEditorOnlyDataEntry(const FShaderEditorOnlyDataEntry&) = default;
		FShaderEditorOnlyDataEntry& operator=(FShaderEditorOnlyDataEntry&&) = default;
		FShaderEditorOnlyDataEntry& operator=(const FShaderEditorOnlyDataEntry&) = default;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	};
#endif // WITH_EDITORONLY_DATA

	FShaderMapResourceCode() {}
	RENDERCORE_API FShaderMapResourceCode(const FShaderMapResourceCode& Other);
    RENDERCORE_API ~FShaderMapResourceCode();

	RENDERCORE_API void Finalize();

	RENDERCORE_API void Serialize(FShaderSerializeContext& Ctx);
#if WITH_EDITORONLY_DATA
	RENDERCORE_API void NotifyShadersCompiled(FName FormatName);
#endif // WITH_EDITORONLY_DATA

	RENDERCORE_API uint32 GetSizeBytes() const;

	RENDERCORE_API void AddShaderCompilerOutput(const FShaderCompilerOutput& Output, const FShaderCompileJobKey& Key, FString DebugInfo = FString());

	int32 FindShaderIndex(const FShaderHash& InHash) const;

#if WITH_EDITORONLY_DATA
	void AddEditorOnlyData(int32 Index, const FString& DebugName, uint64 ShaderTypeHash, FCompressedBuffer InSymbols, TArray<FGenericShaderStat>&& ShaderStatistics, const FString& DebugInfo = FString());
	void UpdateEditorOnlyData(int32 Index, const FString& DebugName, uint64 ShaderTypeHash, FCompressedBuffer InSymbols, const FString& DebugInfo);

	UE_DEPRECATED(5.8, "Shader compiler warnings are no longer stored with shadermap data, and so cannot be logged via this API; please force a recompile of any shaders you wish to see warnings for.")
	inline void LogShaderCompilerWarnings() {}
#endif

	RENDERCORE_API void ToString(FStringBuilderBase& OutString) const;

	/** A hash describing the total contents of *this. Constructed from the contents of ShaderHashes during Finalize. */
	FShaderHash ResourceHash;
	TArray<FShaderHash> ShaderHashes;

	TArray<FShaderCodeResource> ShaderCodeResources;
#if WITH_EDITORONLY_DATA
	// Optional array of editor-only data indexed in the same order as ShaderEntries (sorted by the shader hash)
	// Empty in the cases where the editor-only data is not serialized.
	TArray<FShaderEditorOnlyDataEntry> ShaderEditorOnlyDataEntries;

private:
	// Given an existing shader entry and a new set of shader statistics determine if the shader stats match.
	// This is to prevent non-determinism issues.
	void ValidateShaderStatisticsEditorOnlyData(int32 Index, const TArray<FGenericShaderStat>& ShaderStatistics);
#endif // WITH_EDITORONLY_DATA
};
	
class FShaderMapResource_InlineCode : public FShaderMapResource
{
public:
	FShaderMapResource_InlineCode(EShaderPlatform InPlatform, FShaderMapResourceCode* InCode)
		: FShaderMapResource(InPlatform, InCode->ShaderCodeResources.Num())
		, Code(InCode)
	{}

	// FShaderMapResource interface
	RENDERCORE_API virtual FShaderHash GetShaderHash(int32 ShaderIndex) override;
	RENDERCORE_API virtual FRHIShader* CreateRHIShaderOrCrash(int32 ShaderIndex, bool bRequired) override;
	virtual FString GetFriendlyName() const override { return TEXT("InlineCode"); }
	virtual uint32 GetSizeBytes() const override;

	virtual uint32 GetShaderSizeBytes(int32 ShaderIndex) const override;

	TRefCountPtr<FShaderMapResourceCode> Code;
};

class FShaderMapPointerTable : public FPointerTableBase
{
public:
	RENDERCORE_API virtual int32 AddIndexedPointer(const FTypeLayoutDesc& TypeDesc, void* Ptr) override;
	RENDERCORE_API virtual void* GetIndexedPointer(const FTypeLayoutDesc& TypeDesc, uint32 i) const override;

	virtual FShaderMapPointerTable* Clone() const { return new FShaderMapPointerTable(*this); }

	RENDERCORE_API virtual void SaveToArchive(FArchive& Ar, const FPlatformTypeLayoutParameters& LayoutParams, const void* FrozenObject) const override;
	RENDERCORE_API virtual bool LoadFromArchive(FArchive& Ar, const FPlatformTypeLayoutParameters& LayoutParams, void* FrozenObject) override;

	TPtrTable<FShaderType> ShaderTypes;
	TPtrTable<FVertexFactoryType> VFTypes;
};

/** Encapsulates information about a shader's serialization behavior, used to detect when C++ serialization changes to auto-recompile. */

class FShaderKey
{
public:
	inline FShaderKey(const FShaderHash& InMaterialShaderMapHash, const FShaderPipelineType* InShaderPipeline, FVertexFactoryType* InVertexFactoryType, int32 PermutationId, EShaderPlatform InPlatform)
		: VertexFactoryType(InVertexFactoryType)
		, ShaderPipeline(InShaderPipeline)
		, MaterialShaderMapHash(InMaterialShaderMapHash)
		, PermutationId(PermutationId)
		, Platform(InPlatform)
	{}

	friend inline uint32 GetTypeHash(const FShaderKey& Id)
	{
		return
			HashCombine(
				HashCombine(GetTypeHash(Id.MaterialShaderMapHash), GetTypeHash(Id.Platform)),
				HashCombine(GetTypeHash(Id.VertexFactoryType), uint32(Id.PermutationId)));
	}

	friend bool operator==(const FShaderKey& X, const FShaderKey& Y)
	{
		return X.MaterialShaderMapHash == Y.MaterialShaderMapHash
			&& X.ShaderPipeline == Y.ShaderPipeline
			&& X.VertexFactoryType == Y.VertexFactoryType
			&& X.PermutationId == Y.PermutationId
			&& X.Platform == Y.Platform;
	}

	RENDERCORE_API friend FArchive& operator<<(FArchive& Ar, FShaderKey& Ref);

	FVertexFactoryType* VertexFactoryType;
	const FShaderPipelineType* ShaderPipeline;
	FShaderHash MaterialShaderMapHash;
	int32 PermutationId;
	uint32 Platform : SP_NumBits;
};

/** 
 * Uniquely identifies an FShader instance.  
 * Used to link FMaterialShaderMaps and FShaders on load. 
 */
class FShaderId
{
public:
	inline FShaderId() {}
	inline FShaderId(const FShaderType* InType, uint64 InMaterialShaderMapHash, const FHashedName& InShaderPipeline, const FVertexFactoryType* InVertexFactoryType, int32 InPermutationId, EShaderPlatform InPlatform)
		: Type(InType)
		, VFType(InVertexFactoryType)
		, ShaderPipelineName(InShaderPipeline)
		, MaterialShaderMapHash(InMaterialShaderMapHash)
		, PermutationId(InPermutationId)
		, Platform(InPlatform)
	{}

	const FShaderType* Type;
	const FVertexFactoryType* VFType;
	FHashedName ShaderPipelineName;
	uint64 MaterialShaderMapHash;
	int32 PermutationId;
	uint32 Platform : SP_NumBits;

	friend inline uint32 GetTypeHash( const FShaderId& Id )
	{
		return HashCombine(
			GetTypeHash(Id.Type),
			HashCombine(GetTypeHash(Id.VFType),
				HashCombine(GetTypeHash(Id.ShaderPipelineName),
					HashCombine(GetTypeHash(Id.MaterialShaderMapHash),
						HashCombine(GetTypeHash(Id.PermutationId), GetTypeHash(Id.Platform))))));
	}

	friend bool operator==(const FShaderId& X, const FShaderId& Y)
	{
		return X.Type == Y.Type
			&& X.MaterialShaderMapHash == Y.MaterialShaderMapHash
			&& X.ShaderPipelineName == Y.ShaderPipelineName
			&& X.VFType == Y.VFType
			&& X.PermutationId == Y.PermutationId 
			&& X.Platform == Y.Platform;
	}

	friend bool operator!=(const FShaderId& X, const FShaderId& Y)
	{
		return !(X == Y);
	}
};

class FMaterial;

/**
 * Stores all shader parameter bindings and their corresponding offset and size in the shader's parameters struct.
 */
class FShaderParameterBindings
{
public:
	struct FParameter
	{
		DECLARE_INLINE_TYPE_LAYOUT(FParameter, NonVirtual);

		LAYOUT_FIELD(uint16, BufferIndex);
		LAYOUT_FIELD(uint16, BaseIndex);
		LAYOUT_FIELD(uint16, ByteOffset);
		LAYOUT_FIELD(uint16, ByteSize);
	};

	struct FResourceParameter
	{
		DECLARE_INLINE_TYPE_LAYOUT(FResourceParameter, NonVirtual);		
		LAYOUT_FIELD(uint16, ByteOffset);
		LAYOUT_FIELD(uint8, BaseIndex);
		LAYOUT_FIELD(EUniformBufferBaseType, BaseType);
	};

	struct FBindlessResourceParameter
	{
		DECLARE_INLINE_TYPE_LAYOUT(FBindlessResourceParameter, NonVirtual);
		LAYOUT_FIELD(uint16, ByteOffset);
		LAYOUT_FIELD(uint16, GlobalConstantOffset);
		LAYOUT_FIELD(EUniformBufferBaseType, BaseType);
	};

	struct FParameterStructReference
	{
		DECLARE_INLINE_TYPE_LAYOUT(FParameterStructReference, NonVirtual);
		LAYOUT_FIELD(uint16, BufferIndex);
		LAYOUT_FIELD(uint16, ByteOffset);
	};

	DECLARE_EXPORTED_TYPE_LAYOUT(FShaderParameterBindings, RENDERCORE_API, NonVirtual);
public:
	static constexpr uint16 kInvalidBufferIndex = 0xFFFF;

	RENDERCORE_API void BindForLegacyShaderParameters(const FShader* Shader, int32 PermutationId, const FShaderParameterMap& ParameterMaps, const FShaderParametersMetadata& StructMetaData, bool bShouldBindEverything = false);
	RENDERCORE_API void BindForRootShaderParameters(const FShader* Shader, int32 PermutationId, const FShaderParameterMap& ParameterMaps);

	LAYOUT_FIELD(TMemoryImageArray<FParameter>, Parameters);
	LAYOUT_FIELD(TMemoryImageArray<FResourceParameter>, ResourceParameters);
	LAYOUT_FIELD(TMemoryImageArray<FBindlessResourceParameter>, BindlessResourceParameters);
	LAYOUT_FIELD(TMemoryImageArray<FParameterStructReference>, GraphUniformBuffers);
	LAYOUT_FIELD(TMemoryImageArray<FParameterStructReference>, ParameterReferences);

	// Hash of the shader parameter structure when doing the binding.
	LAYOUT_FIELD_INITIALIZED(uint32, StructureLayoutHash, 0);

	// Buffer index of FShaderParametersMetadata::kRootUniformBufferBindingName
	LAYOUT_FIELD_INITIALIZED(uint16, RootParameterBufferIndex, FShaderParameterBindings::kInvalidBufferIndex);

}; // FShaderParameterBindings

inline int16 GetParameterIndex(const FShaderParameterBindings::FResourceParameter& Parameter) { return Parameter.BaseIndex; }
inline int16 GetParameterIndex(const FShaderParameterBindings::FBindlessResourceParameter& Parameter) { return Parameter.GlobalConstantOffset; }
inline int16 GetParameterIndex(const FShaderParameterBindings::FParameterStructReference& Parameter) { return Parameter.BufferIndex; }

enum class EShaderPermutationPrecacheRequest : uint8
{
	Precached,			//< Permutation should be precached because it can be used at runtime
	NotPrecached,		//< Permutation doesn't have to be precached (debug feature only for example)
	NotUsed,			//< Permutation is not used with current cvar/configuration/feature level
};

/** Get the shader permutation flags associated with a platform layout. */
RENDERCORE_API EShaderPermutationFlags GetShaderPermutationFlags(const FPlatformTypeLayoutParameters& LayoutParams);
/** Set global additional flags that are always added in GetShaderPermutationFlags(). Expected to be called once at startup. */
RENDERCORE_API void SetAdditionalShaderPermutationFlags(EShaderPermutationFlags AdditionalFlags);

namespace Freeze
{
	RENDERCORE_API void IntrinsicToString(const TIndexedPtr<FShaderType>& Object, const FTypeLayoutDesc& TypeDesc, const FPlatformTypeLayoutParameters& LayoutParams, FMemoryToStringContext& OutContext);
	RENDERCORE_API void IntrinsicToString(const TIndexedPtr<FVertexFactoryType>& Object, const FTypeLayoutDesc& TypeDesc, const FPlatformTypeLayoutParameters& LayoutParams, FMemoryToStringContext& OutContext);
}

DECLARE_EXPORTED_TEMPLATE_INTRINSIC_TYPE_LAYOUT(template<>, TIndexedPtr<FShaderType>, RENDERCORE_API);
DECLARE_EXPORTED_TEMPLATE_INTRINSIC_TYPE_LAYOUT(template<>, TIndexedPtr<FVertexFactoryType>, RENDERCORE_API);

/** A compiled shader and its parameter bindings. */
class FShader
{
	friend class FShaderType;
	DECLARE_EXPORTED_TYPE_LAYOUT(FShader, RENDERCORE_API, NonVirtual);
public:
	using FPermutationDomain = FShaderPermutationNone;
	using FPermutationParameters = FShaderPermutationParameters;
	using CompiledShaderInitializerType = FShaderCompiledShaderInitializerType;
	using ShaderMetaType = FShaderType;
	using ShaderStatKeyType = FMemoryImageName;
	using FShaderStatisticMap = TMemoryImageMap<ShaderStatKeyType, FShaderStatVariant>;

	/** 
	 * Used to construct a shader for deserialization.
	 * This still needs to initialize members to safe values since FShaderType::GenerateSerializationHistory uses this constructor.
	 */
	RENDERCORE_API FShader();

	/**
	 * Construct a shader from shader compiler output.
	 */
	RENDERCORE_API FShader(const CompiledShaderInitializerType& Initializer);

	RENDERCORE_API ~FShader();

	/** Can be overridden by FShader subclasses to modify their compile environment just before compilation occurs. */
	static void ModifyCompilationEnvironment(const FShaderPermutationParameters&, FShaderCompilerEnvironment&) {}

	/**	Get shader binding layout used by the shader */
	static const FShaderBindingLayout* GetShaderBindingLayout(const FShaderPermutationParameters&) { return nullptr; }

	/** Can be overridden by FShader subclasses to determine whether a specific permutation should be compiled. */
	static bool ShouldCompilePermutation(const FShaderPermutationParameters&) { return true; }

	/** Can be overridden by FShader subclasses to determine whether a specific permutation should be precached. */
	static EShaderPermutationPrecacheRequest ShouldPrecachePermutation(const FShaderPermutationParameters& Parameters) { return EShaderPermutationPrecacheRequest::Precached; }

	/** Can be overridden by FShader subclasses to determine whether compilation is valid. */
	static bool ValidateCompiledResult(EShaderPlatform InPlatform, const FShaderParameterMap& InParameterMap, TArray<FString>& OutError) { return true; }

	/** Can be overridden by FShader subclasses to set a custom job priority (EShaderCompileJobPriority)
		or EShaderCompileJobPriority::None to ignore custom priority. This can only increase priority. */
	static EShaderCompileJobPriority GetOverrideJobPriority() { return EShaderCompileJobPriority::None; }

	/** Can be overriden by FShader subclasses to specify which raytracing payload should be used. This method is only called for raytracing shaders. */
	static ERayTracingPayloadType GetRayTracingPayloadType(const int32 PermutationId) { return static_cast<ERayTracingPayloadType>(0); }

	/** Returns the hash of the shader file that this shader was compiled with. */
	RENDERCORE_API const FSHAHash& GetHash() const;

	RENDERCORE_API const FSHAHash& GetVertexFactoryHash() const;
	
	RENDERCORE_API const FShaderHash& GetOutputHash() const;

	/** Returns an identifier suitable for deterministic sorting of shaders between sessions. */
	uint32 GetSortKey() const { return SortKey; }

	RENDERCORE_API void Finalize(const FShaderMapResourceCode* Code);

	// Accessors.
	inline FShaderType* GetType(const FShaderMapPointerTable& InPointerTable) const { return Type.Get(InPointerTable.ShaderTypes); }
	inline FShaderType* GetType(const FPointerTableBase* InPointerTable) const { return Type.Get(InPointerTable); }
	inline FVertexFactoryType* GetVertexFactoryType(const FShaderMapPointerTable& InPointerTable) const { return VFType.Get(InPointerTable.VFTypes); }
	inline FVertexFactoryType* GetVertexFactoryType(const FPointerTableBase* InPointerTable) const { return VFType.Get(InPointerTable); }
	inline FShaderType* GetTypeUnfrozen() const { return Type.GetUnfrozen(); }
	inline int32 GetResourceIndex() const { checkSlow(ResourceIndex != INDEX_NONE); return ResourceIndex; }
	inline EShaderPlatform GetShaderPlatform() const { return Target.GetPlatform(); }
	inline EShaderFrequency GetFrequency() const { return Target.GetFrequency(); }
	inline const FShaderTarget GetTarget() const { return Target; }
	inline bool IsFrozen() const { return Type.IsFrozen(); }

#if WITH_EDITORONLY_DATA
	inline uint32 GetNumInstructions() const { return NumInstructions; }
	inline uint32 GetNumTextureSamplers() const { return NumTextureSamplers; }
	inline uint32 GetCodeSize() const { return CodeSize; }
	inline void SetNumInstructions(uint32 Value) { NumInstructions = Value; }
#else
	inline uint32 GetNumInstructions() const { return 0u; }
	inline uint32 GetNumTextureSamplers() const { return 0u; }
	inline uint32 GetCodeSize() const { return 0u; }
	inline void SetNumInstructions(uint32 Value) { }
#endif

	/** Finds an automatically bound uniform buffer matching the given uniform buffer type if one exists, or returns an unbound parameter. */
	template<typename UniformBufferStructType>
	inline const TShaderUniformBufferParameter<UniformBufferStructType>& GetUniformBufferParameter() const
	{
		const FShaderUniformBufferParameter& FoundParameter = GetUniformBufferParameter(UniformBufferStructType::FTypeInfo::GetStructMetadata());
		return static_cast<const TShaderUniformBufferParameter<UniformBufferStructType>&>(FoundParameter);
	}

	/** Finds an automatically bound uniform buffer matching the given uniform buffer struct if one exists, or returns an unbound parameter. */
	inline const FShaderUniformBufferParameter& GetUniformBufferParameter(const FShaderParametersMetadata* SearchStruct) const
	{
		const FHashedName SearchName = SearchStruct->GetShaderVariableHashedName();
		
		return GetUniformBufferParameter(SearchName);
	}

	/** Finds an automatically bound uniform buffer matching the HashedName if one exists, or returns an unbound parameter. */
	inline const FShaderUniformBufferParameter& GetUniformBufferParameter(const FHashedName SearchName) const
	{
		int32 FoundIndex = INDEX_NONE;
		TArrayView<const FHashedName> UniformBufferParameterStructsView(UniformBufferParameterStructs);
		for (int32 StructIndex = 0, Count = UniformBufferParameterStructsView.Num(); StructIndex < Count; StructIndex++)
		{
			if (UniformBufferParameterStructsView[StructIndex] == SearchName)
			{
				FoundIndex = StructIndex;
				break;
			}
		}

		if (FoundIndex != INDEX_NONE)
		{
			const FShaderUniformBufferParameter& FoundParameter = UniformBufferParameters[FoundIndex];
			return FoundParameter;
		}
		else
		{
			// This can happen if the uniform buffer was not bound
			// There's no good way to distinguish not being bound due to temporary debugging / compiler optimizations or an actual code bug,
			// Hence failing silently instead of an error message
			static FShaderUniformBufferParameter UnboundParameter;
			return UnboundParameter;
		}
	}

	RENDERCORE_API const FShaderParametersMetadata* FindAutomaticallyBoundUniformBufferStruct(int32 BaseIndex) const;

	RENDERCORE_API void DumpDebugInfo(const FShaderMapPointerTable& InPtrTable);

#if WITH_EDITOR
	RENDERCORE_API void SaveShaderStableKeys(const FShaderMapPointerTable& InPtrTable, EShaderPlatform TargetShaderPlatform, int32 PermutationId, const struct FStableShaderKeyAndValue& SaveKeyVal);
#endif // WITH_EDITOR

	/** Returns the meta data for the root shader parameter struct. */
	static inline const FShaderParametersMetadata* GetRootParametersMetadata()
	{
		return nullptr;
	}

private:
	RENDERCORE_API void BuildParameterMapInfo(const TMap<FString, FParameterAllocation>& ParameterMap);

public:
	LAYOUT_FIELD(FShaderParameterBindings, Bindings);
	LAYOUT_FIELD(FShaderParameterMapInfo, ParameterMapInfo);

protected:
	LAYOUT_FIELD(TMemoryImageArray<FHashedName>, UniformBufferParameterStructs);
	LAYOUT_FIELD(TMemoryImageArray<FShaderUniformBufferParameter>, UniformBufferParameters);

	/**
	* Hash of the compiled output from this shader and the resulting parameter map.
	* This is used to find a matching resource.
	*/
	LAYOUT_FIELD_EDITORONLY(FShaderHash, OutputHash);

	/** Vertex factory source hash, stored so that an FShaderId can be constructed from this shader. */
	LAYOUT_FIELD_EDITORONLY(FSHAHash, VFSourceHash);

	/** Hash of this shader's source files generated at compile time, and stored to allow creating an FShaderId. */
	LAYOUT_FIELD_EDITORONLY(FSHAHash, SourceHash);

private:
	LAYOUT_FIELD(TIndexedPtr<FShaderType>, Type);

	LAYOUT_FIELD(TIndexedPtr<FVertexFactoryType>, VFType);

	/** Target platform and frequency. */
	LAYOUT_FIELD(FShaderTarget, Target);

	/** Index of this shader within the FShaderMapResource */
	LAYOUT_FIELD(int32, ResourceIndex);

	/** The number of instructions the shader takes to execute. */
	LAYOUT_FIELD_EDITORONLY(uint32, NumInstructions);

	/** Truncated version of OutputHash, intended for sorting. Not suitable for unique shader identification. */
	LAYOUT_FIELD(uint32, SortKey);

	/** Number of texture samplers the shader uses. */
	LAYOUT_FIELD_EDITORONLY(uint32, NumTextureSamplers);

	/** Size of shader's compiled code */
	LAYOUT_FIELD_EDITORONLY(uint32, CodeSize);
};

RENDERCORE_API const FTypeLayoutDesc& GetTypeLayoutDesc(const FPointerTableBase* PtrTable, const FShader& Shader);

template<typename ShaderType, typename PointerTableType>
class TShaderRefBase
{
public:
	TShaderRefBase() : ShaderContent(nullptr), ShaderMap(nullptr) {}
	TShaderRefBase(ShaderType* InShader, const FShaderMapBase& InShaderMap) : ShaderContent(InShader), ShaderMap(&InShaderMap) { checkSlow(InShader); }
	TShaderRefBase(ShaderType* InShader, const FShaderMapBase& InShaderMap, const FSpecializationContainerType& InSpecConsts)
		: ShaderContent(InShader), ShaderMap(&InShaderMap), SpecConsts(InSpecConsts) { checkSlow(InShader); }
	TShaderRefBase(const TShaderRefBase&) = default;

	template<typename OtherShaderType, typename OtherPointerTableType>
	TShaderRefBase(const TShaderRefBase<OtherShaderType, OtherPointerTableType>& Rhs) : ShaderContent(Rhs.GetShader()), ShaderMap(Rhs.GetShaderMap()), SpecConsts(Rhs.GetSpecializationConstants()) {}

	TShaderRefBase& operator=(const TShaderRefBase&) = default;

	template<typename OtherShaderType, typename OtherPointerTableType>
	TShaderRefBase& operator=(const TShaderRefBase<OtherShaderType, OtherPointerTableType>& Rhs)
	{
		ShaderContent = Rhs.GetShader();
		ShaderMap = Rhs.GetShaderMap();
		SpecConsts = Rhs.GetSpecializationConstants();
		return *this;
	}

	template<typename OtherShaderType, typename OtherPointerTableType>
	static TShaderRefBase<ShaderType, PointerTableType> Cast(const TShaderRefBase<OtherShaderType, OtherPointerTableType>& Rhs)
	{
		return TShaderRefBase<ShaderType, PointerTableType>(static_cast<ShaderType*>(Rhs.GetShader()), Rhs.GetShaderMap(), Rhs.GetSpecializationConstants());
	}

	template<typename OtherShaderType, typename OtherPointerTableType>
	static TShaderRefBase<ShaderType, PointerTableType> ReinterpretCast(const TShaderRefBase<OtherShaderType, OtherPointerTableType>& Rhs)
	{
		return TShaderRefBase<ShaderType, PointerTableType>(reinterpret_cast<ShaderType*>(Rhs.GetShader()), Rhs.GetShaderMap(), Rhs.GetSpecializationConstants());
	}

	inline bool IsValid() const { return ShaderContent != nullptr; }
	inline bool IsNull() const { return ShaderContent == nullptr; }

	inline void Reset() { ShaderContent = nullptr; ShaderMap = nullptr; }

	inline ShaderType* GetShader() const { return ShaderContent; }
	inline const FShaderMapBase* GetShaderMap() const { return ShaderMap; }
	inline const FShaderMapBase& GetShaderMapChecked() const { check(ShaderMap); return *ShaderMap; }
	inline FShaderType* GetType() const { return ShaderContent->GetType(GetPointerTable()); }
	inline FVertexFactoryType* GetVertexFactoryType() const { return ShaderContent->GetVertexFactoryType(GetPointerTable()); }
	inline FShaderMapResource& GetResourceChecked() const { FShaderMapResource* Resource = GetResource(); check(Resource); return *Resource; }
	const PointerTableType& GetPointerTable() const;
	FShaderMapResource* GetResource() const;

	inline ShaderType* operator->() const { return ShaderContent; }

	inline FRHIShader* GetRHIShaderBase(EShaderFrequency Frequency, bool bRequired = true) const
	{
		FRHIShader* RHIShader = nullptr;
		if(ShaderContent)
		{
			checkSlow(ShaderContent->GetFrequency() == Frequency);
			RHIShader = GetResourceChecked().GetShader(ShaderContent->GetResourceIndex(), bRequired);
			if (RHIShader == nullptr)
			{
				UE_LOGF(LogShaders, Log, "Failed to create shader for type %ls with resource index %d.", GetType()->GetName(), ShaderContent->GetResourceIndex());
				return nullptr;
			}
			checkSlow(RHIShader->GetFrequency() == Frequency);
		}
		return RHIShader;
	}

	inline FRHIGraphicsShader* GetGraphicsShader(bool bRequired = true) const
	{
		FRHIGraphicsShader* RHIShader = nullptr;
		if(ShaderContent)
		{
			checkSlow(IsValidGraphicsFrequency(ShaderContent->GetFrequency()));
			RHIShader = static_cast<FRHIGraphicsShader*>(GetResourceChecked().GetShader(ShaderContent->GetResourceIndex(), bRequired));
			if (RHIShader == nullptr)
			{
				UE_LOGF(LogShaders, Log, "Failed to create shader for type %ls with resource index %d.", GetType()->GetName(), ShaderContent->GetResourceIndex());
				return nullptr;
			}
			checkSlow(IsValidGraphicsFrequency(RHIShader->GetFrequency()));
		}
		return RHIShader;
	}

	/** @return the shader's vertex shader */
	inline FRHIVertexShader* GetVertexShader(bool bRequired = true) const
	{
		return static_cast<FRHIVertexShader*>(GetRHIShaderBase(SF_Vertex, bRequired));
	}
	/** @return the shader's mesh shader */
	inline FRHIMeshShader* GetMeshShader() const
	{
		return static_cast<FRHIMeshShader*>(GetRHIShaderBase(SF_Mesh));
	}
	/** @return the shader's aplification shader */
	inline FRHIAmplificationShader* GetAmplificationShader() const
	{
		return static_cast<FRHIAmplificationShader*>(GetRHIShaderBase(SF_Amplification));
	}
	/** @return the shader's pixel shader */
	inline FRHIPixelShader* GetPixelShader(bool bRequired = true) const
	{
		return static_cast<FRHIPixelShader*>(GetRHIShaderBase(SF_Pixel, bRequired));
	}
	/** @return the shader's geometry shader */
	inline FRHIGeometryShader* GetGeometryShader() const
	{
		return static_cast<FRHIGeometryShader*>(GetRHIShaderBase(SF_Geometry));
	}
	/** @return the shader's compute shader */
	inline FRHIComputeShader* GetComputeShader(bool bRequired = true) const
	{
		return static_cast<FRHIComputeShader*>(GetRHIShaderBase(SF_Compute, bRequired));
	}
	/** @return the shader's work graph shader */
	inline FRHIWorkGraphShader* GetWorkGraphShader() const
	{
		FRHIWorkGraphShader* RHIShader = nullptr;
		if (ShaderContent)
		{
			const EShaderFrequency Frequency = ShaderContent->GetFrequency();
			checkSlow(IsWorkGraphShaderFrequency(Frequency));
			RHIShader = static_cast<FRHIWorkGraphShader*>(GetRHIShaderBase(Frequency));
		}
		return RHIShader;
	}

#if RHI_RAYTRACING
	inline FRHIRayTracingShader* GetRayTracingShader() const
	{
		FRHIRayTracingShader* RHIShader = nullptr;
		if(ShaderContent)
		{
			const EShaderFrequency Frequency = ShaderContent->GetFrequency();
			checkSlow(Frequency == SF_RayGen
				|| Frequency == SF_RayMiss
				|| Frequency == SF_RayHitGroup
				|| Frequency == SF_RayCallable);
			RHIShader = static_cast<FRHIRayTracingShader*>(GetResourceChecked().GetShader(ShaderContent->GetResourceIndex()));
			checkSlow(RHIShader->GetFrequency() == Frequency);
		}
		return RHIShader;
	}

	inline uint32 GetRayTracingHitGroupLibraryIndex() const
	{
		checkSlow(ShaderContent);
		checkSlow(ShaderContent->GetFrequency() == SF_RayHitGroup);
		return GetResourceChecked().GetRayTracingHitGroupLibraryIndex(ShaderContent->GetResourceIndex());
	}

	inline uint32 GetRayTracingCallableShaderLibraryIndex() const
	{
		checkSlow(ShaderContent);
		checkSlow(ShaderContent->GetFrequency() == SF_RayCallable);
		return GetResourceChecked().GetRayTracingCallableShaderLibraryIndex(ShaderContent->GetResourceIndex());
	}

	inline uint32 GetRayTracingMissShaderLibraryIndex() const
	{
		checkSlow(ShaderContent);
		checkSlow(ShaderContent->GetFrequency() == SF_RayMiss);
		return GetResourceChecked().GetRayTracingMissShaderLibraryIndex(ShaderContent->GetResourceIndex());
	}
#endif // RHI_RAYTRACING

	inline const FSpecializationContainerType& GetSpecializationConstants() const
	{
		return SpecConsts;
	}

	inline void SetSpecializationConstants(FSpecializationContainerType& InSpecConsts)
	{
		SpecConsts = InSpecConsts;
	}

private:
	TShaderRefBase(ShaderType* InShader, const FShaderMapBase* InShaderMap, const FSpecializationContainerType& InSpecConsts)
		: ShaderContent(InShader), ShaderMap(InShaderMap), SpecConsts(InSpecConsts)
	{
		checkSlow((!InShader && !InShaderMap) || (InShader && InShaderMap));
	}

	ShaderType* ShaderContent;
	const FShaderMapBase* ShaderMap;

	template<typename ShaderType1>
	friend inline bool operator==(const TShaderRefBase& Lhs, const TShaderRefBase<ShaderType1, PointerTableType>& Rhs)
	{
		if (Lhs.GetShader() == Rhs.GetShader())
		{
			check(Lhs.GetShaderMap() == Rhs.GetShaderMap());
			return true;
		}
		return false;
	}

	template<typename ShaderType1>
	friend inline bool operator!=(const TShaderRefBase& Lhs, const TShaderRefBase<ShaderType1, PointerTableType>& Rhs)
	{
		return !operator==(Lhs, Rhs);
	}

protected:
	FSpecializationContainerType SpecConsts;
};

template<typename ShaderType>
using TShaderRef = TShaderRefBase<ShaderType, FShaderMapPointerTable>;

#define SHADER_TYPE_LIST \
	SHADER_TYPE_ENTRY(Global), \
	SHADER_TYPE_ENTRY(Material), \
	SHADER_TYPE_ENTRY(MeshMaterial), \
	SHADER_TYPE_ENTRY(Niagara), \
	SHADER_TYPE_ENTRY(OCIO), \
	SHADER_TYPE_ENTRY(ComputeKernel), \
	SHADER_TYPE_ENTRY(NNERuntimeIREE),

/**
 * An object which is used to serialize/deserialize, compile, and cache a particular shader class.
 *
 * A shader type can manage multiple instance of FShader across mutiple dimensions such as EShaderPlatform, or permutation id.
 * The number of permutation of a shader type is simply given by GetPermutationCount().
 */
class FShaderType
{
public:
#define SHADER_TYPE_ENTRY(Name) Name
	enum class EShaderTypeForDynamicCast : uint32
	{
		SHADER_TYPE_LIST
		NumShaderTypes,
	};
#undef SHADER_TYPE_ENTRY

	/**
	 * Derived FShaderTypes should derive from this class to pass params to FShader constructor
	 */
	class FParameters
	{
	public:
		virtual ~FParameters()= default;
	};

	typedef class FShader* (*ConstructSerializedType)();
	typedef FShader* (*ConstructCompiledType)(const FShader::CompiledShaderInitializerType& Initializer);
	typedef bool (*ShouldCompilePermutationType)(const FShaderPermutationParameters&);
	typedef EShaderPermutationPrecacheRequest (*ShouldPrecachePermutationType)(const FShaderPermutationParameters&);
	typedef int32(*GetUnspecializedIdType)(const int32 PermutationId);
	typedef FSpecializationContainerType (*GetSpecializationValuesType)(const int32 PermutationId);
	typedef ERayTracingPayloadType(*GetRayTracingPayloadTypeType)(const int32 PermutationId);
	typedef const FShaderBindingLayout*(*GetShaderBindingLayoutType)(const FShaderPermutationParameters&);
#if WITH_EDITOR
	typedef void (*ModifyCompilationEnvironmentType)(const FShaderPermutationParameters&, FShaderCompilerEnvironment&);
	typedef bool (*ValidateCompiledResultType)(EShaderPlatform, const FShaderParameterMap&, TArray<FString>&);
	typedef EShaderCompileJobPriority(*GetOverrideJobPriorityType)();
	typedef void (*GetPermutationIdStringType)(int32 PermutationId, FString&, bool);
#endif // WITH_EDITOR

	/** @return The global shader factory list. */
	static RENDERCORE_API TLinkedList<FShaderType*>*& GetTypeList();

	static RENDERCORE_API FShaderType* GetShaderTypeByName(const TCHAR* Name);
	static RENDERCORE_API TArray<const FShaderType*> GetShaderTypesByFilename(const TCHAR* Filename, bool bSearchAsRegexFilter = false);
	static RENDERCORE_API TArray<const FShaderType*> GetShaderTypesByFilenameFilter(const TFunction<bool(const TCHAR*)>& InFilenameFilter);

	/** @return The global shader name to type map */
	static RENDERCORE_API TMap<FHashedName, FShaderType*>& GetNameToTypeMap();

	static RENDERCORE_API const TArray<FShaderType*>& GetSortedTypes(EShaderTypeForDynamicCast Type);

	/** Initialize FShaderType static members, this must be called before any shader types are created. */
	static RENDERCORE_API void Initialize(const TMap<FString, TArray<const TCHAR*> >& ShaderFileToUniformBufferVariables);

	/** Uninitializes FShaderType cached data. */
	static RENDERCORE_API void Uninitialize();

	/** Minimal initialization constructor. */
	RENDERCORE_API FShaderType(
		EShaderTypeForDynamicCast InShaderTypeForDynamicCast,
		FTypeLayoutDesc& InTypeLayout,
		const TCHAR* InName,
		const TCHAR* InSourceFilename,
		const TCHAR* InFunctionName,
		uint32 InFrequency,
		int32 TotalPermutationCount,
		int32 TotalSpecializationCount,
		ConstructSerializedType InConstructSerializedRef,
		ConstructCompiledType InConstructCompiledRef,
		ShouldCompilePermutationType InShouldCompilePermutationRef,
		ShouldPrecachePermutationType InShouldPrecachePermutationRef,
		GetUnspecializedIdType InGetUnspecializedIdRef,
		GetSpecializationValuesType InGetSpecializationValuesRef,
		GetRayTracingPayloadTypeType InGetRayTracingPayloadTypeRef,
		GetShaderBindingLayoutType InGetShaderBindingLayoutTypeRef,
#if WITH_EDITOR
		ModifyCompilationEnvironmentType InModifyCompilationEnvironmentRef,
		ValidateCompiledResultType InValidateCompiledResultRef,
		GetOverrideJobPriorityType InGetOverrideJobPriorityRef,
#endif // WITH_EDITOR
		uint32 InTypeSize,
		const FShaderParametersMetadata* InRootParametersMetadata
#if WITH_EDITOR
		, GetPermutationIdStringType InGetPermutationIdStringRef = nullptr
#endif
	);

	RENDERCORE_API virtual ~FShaderType();

	/** Constructs a new instance of the shader type for deserialization. */
	RENDERCORE_API FShader* ConstructForDeserialization() const;
	RENDERCORE_API FShader* ConstructCompiled(const FShader::CompiledShaderInitializerType& Initializer) const;

	RENDERCORE_API bool ShouldCompilePermutation(const FShaderPermutationParameters& Parameters) const;
	RENDERCORE_API EShaderPermutationPrecacheRequest ShouldPrecachePermutation(const FShaderPermutationParameters& Parameters) const;
	RENDERCORE_API bool IsSpecialization(const int32 PermutationId) const;
	RENDERCORE_API int32 GetUnspecializedId(const int32 PermutationId) const;
	RENDERCORE_API FSpecializationContainerType GetSpecializationValues(const int32 PermutationId) const;
	RENDERCORE_API const FShaderBindingLayout* GetShaderBindingLayout(const FShaderPermutationParameters& Parameters) const;

#if WITH_EDITOR
	RENDERCORE_API void ModifyCompilationEnvironment(const FShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment) const;

	/**
	* Checks if the shader type should pass compilation for a particular set of parameters.
	* @param Platform - Platform to validate.
	* @param ParameterMap - Shader parameters to validate.
	* @param OutError - List for appending validation errors.
	*/
	RENDERCORE_API bool ValidateCompiledResult(EShaderPlatform Platform, const FShaderParameterMap& ParameterMap, TArray<FString>& OutError) const;

	/** Returns a custom compile job priority or EShaderCompileJobPriority::None to ignore the customization. */
	RENDERCORE_API EShaderCompileJobPriority GetOverrideJobPriority() const;

	/**
	 * Returns a string representation for the specified permutation Id.
	 * @param PermutationId Zero based index in the half-open range [0, GetPermutationCount()).
	 * @param bFullNames Specifies whether the full shader defines are meant to be included. If false, a short form is returned as comma-separated integers.
	 */
	RENDERCORE_API FString GetPermutationIdString(int32 PermutationId, bool bFullNames) const;
#endif // WITH_EDITOR

	RENDERCORE_API ERayTracingPayloadType GetRayTracingPayloadType(const int32 PermutationId) const;

	/** Calculates a Hash based on this shader type's source code and includes */
	RENDERCORE_API const FSHAHash& GetSourceHash(EShaderPlatform ShaderPlatform) const;

	/** Serializes a shader type reference by name. */
	RENDERCORE_API friend FArchive& operator<<(FArchive& Ar,FShaderType*& Ref);
	
	/** Hashes a pointer to a shader type. */
	friend uint32 GetTypeHash(FShaderType* Ref)
	{
		return Ref ? GetTypeHash(Ref->HashedName) : 0u;
	}

	// Dynamic casts.
	inline FGlobalShaderType* GetGlobalShaderType() 
	{ 
		return (ShaderTypeForDynamicCast == EShaderTypeForDynamicCast::Global) ? reinterpret_cast<FGlobalShaderType*>(this) : nullptr;
	}
	inline const FGlobalShaderType* GetGlobalShaderType() const
	{
		return (ShaderTypeForDynamicCast == EShaderTypeForDynamicCast::Global) ? reinterpret_cast<const FGlobalShaderType*>(this) : nullptr;
	}
	inline FMaterialShaderType* GetMaterialShaderType()
	{
		return (ShaderTypeForDynamicCast == EShaderTypeForDynamicCast::Material) ? reinterpret_cast<FMaterialShaderType*>(this) : nullptr;
	}
	inline const FMaterialShaderType* GetMaterialShaderType() const
	{
		return (ShaderTypeForDynamicCast == EShaderTypeForDynamicCast::Material) ? reinterpret_cast<const FMaterialShaderType*>(this) : nullptr;
	}
	inline FMeshMaterialShaderType* GetMeshMaterialShaderType()
	{
		return (ShaderTypeForDynamicCast == EShaderTypeForDynamicCast::MeshMaterial) ? reinterpret_cast<FMeshMaterialShaderType*>(this) : nullptr;
	}
	inline const FMeshMaterialShaderType* GetMeshMaterialShaderType() const
	{
		return (ShaderTypeForDynamicCast == EShaderTypeForDynamicCast::MeshMaterial) ? reinterpret_cast<const FMeshMaterialShaderType*>(this) : nullptr;
	}
	inline const FNiagaraShaderType* GetNiagaraShaderType() const
	{
		return (ShaderTypeForDynamicCast == EShaderTypeForDynamicCast::Niagara) ? reinterpret_cast<const FNiagaraShaderType*>(this) : nullptr;
	}
	inline FNiagaraShaderType* GetNiagaraShaderType()
	{
		return (ShaderTypeForDynamicCast == EShaderTypeForDynamicCast::Niagara) ? reinterpret_cast<FNiagaraShaderType*>(this) : nullptr;
	}
	inline const FOpenColorIOShaderType* GetOpenColorIOShaderType() const
	{
		return (ShaderTypeForDynamicCast == EShaderTypeForDynamicCast::OCIO) ? reinterpret_cast<const FOpenColorIOShaderType*>(this) : nullptr;
	}
	inline FOpenColorIOShaderType* GetOpenColorIOShaderType()
	{
		return (ShaderTypeForDynamicCast == EShaderTypeForDynamicCast::OCIO) ? reinterpret_cast<FOpenColorIOShaderType*>(this) : nullptr;
	}
	inline const FComputeKernelShaderType* GetComputeKernelShaderType() const
	{
		return (ShaderTypeForDynamicCast == EShaderTypeForDynamicCast::ComputeKernel) ? reinterpret_cast<const FComputeKernelShaderType*>(this) : nullptr;
	}
	inline FComputeKernelShaderType* GetComputeKernelShaderType()
	{
		return (ShaderTypeForDynamicCast == EShaderTypeForDynamicCast::ComputeKernel) ? reinterpret_cast<FComputeKernelShaderType*>(this) : nullptr;
	}
	inline const FNNERuntimeIREEShaderType* GetNNERuntimeIREEShaderType() const
	{
		return (ShaderTypeForDynamicCast == EShaderTypeForDynamicCast::NNERuntimeIREE) ? reinterpret_cast<const FNNERuntimeIREEShaderType*>(this) : nullptr;
	}
	inline FNNERuntimeIREEShaderType* GetNNERuntimeIREEShaderType()
	{
		return (ShaderTypeForDynamicCast == EShaderTypeForDynamicCast::NNERuntimeIREE) ? reinterpret_cast<FNNERuntimeIREEShaderType*>(this) : nullptr;
	}

	inline const FGlobalShaderType* AsGlobalShaderType() const
	{
		checkf(ShaderTypeForDynamicCast == EShaderTypeForDynamicCast::Global, TEXT("ShaderType %s is not Global"), GetName());
		return reinterpret_cast<const FGlobalShaderType*>(this);
	}

	inline const FMaterialShaderType* AsMaterialShaderType() const
	{
		checkf(ShaderTypeForDynamicCast == EShaderTypeForDynamicCast::Material, TEXT("ShaderType %s is not Material"), GetName());
		return reinterpret_cast<const FMaterialShaderType*>(this);
	}

	inline const FMeshMaterialShaderType* AsMeshMaterialShaderType() const
	{
		checkf(ShaderTypeForDynamicCast == EShaderTypeForDynamicCast::MeshMaterial, TEXT("ShaderType %s is not MeshMaterial"), GetName());
		return reinterpret_cast<const FMeshMaterialShaderType*>(this);
	}

	inline const FNiagaraShaderType* AsNiagaraShaderType() const
	{
		checkf(ShaderTypeForDynamicCast == EShaderTypeForDynamicCast::Niagara, TEXT("ShaderType %s is not Niagara"), GetName());
		return reinterpret_cast<const FNiagaraShaderType*>(this);
	}

	inline const FOpenColorIOShaderType* AsOpenColorIOShaderType() const
	{
		checkf(ShaderTypeForDynamicCast == EShaderTypeForDynamicCast::OCIO, TEXT("ShaderType %s is not OCIO"), GetName());
		return reinterpret_cast<const FOpenColorIOShaderType*>(this);
	}

	inline EShaderTypeForDynamicCast GetTypeForDynamicCast() const
	{
		return ShaderTypeForDynamicCast;
	}

	// Accessors.
	inline const FTypeLayoutDesc& GetLayout() const
	{
		return *TypeLayout;
	}
	inline EShaderFrequency GetFrequency() const
	{ 
		return (EShaderFrequency)Frequency; 
	}
	inline const TCHAR* GetName() const
	{ 
		return Name; 
	}
	inline const FName& GetFName() const
	{
		return TypeName;
	}
	inline const FHashedName& GetHashedName() const
	{
		return HashedName;
	}
	inline const TCHAR* GetShaderFilename() const
	{ 
		return SourceFilename; 
	}
	inline const FHashedName& GetHashedShaderFilename() const
	{
		return HashedSourceFilename;
	}
	inline const TCHAR* GetFunctionName() const
	{
		return FunctionName;
	}
	inline uint32 GetTypeSize() const
	{
		return TypeSize;
	}

	inline int32 GetNumShaders() const
	{
		// TODO count this
		return 0;
	}

	inline int32 GetPermutationCount() const
	{
		return TotalPermutationCount;
	}

	inline int32 GetSpecializationCount() const
	{
		return TotalSpecializationCount;
	}

	/** Returns the meta data for the root shader parameter struct. */
	inline const FShaderParametersMetadata* GetRootParametersMetadata() const
	{
		return RootParametersMetadata;
	}

#if WITH_EDITOR
	const TSet<const FShaderParametersMetadata*>& GetReferencedUniformBuffers() const
	{
		return ReferencedUniformBuffers;
	}

	/** Adds include statements for uniform buffers that this shader type references. */
	RENDERCORE_API void AddUniformBufferIncludesToEnvironment(FShaderCompilerEnvironment& OutEnvironment, EShaderPlatform Platform) const;

	RENDERCORE_API void UpdateReferencedUniformBufferNames(const TMap<FString, TArray<const TCHAR*>>& ShaderFileToUniformBufferVariables);

	RENDERCORE_API void GetShaderStableKeyParts(struct FStableShaderKeyAndValue& SaveKeyVal);
#endif // WITH_EDITOR

	RENDERCORE_API void DumpDebugInfo();

private:
	EShaderTypeForDynamicCast ShaderTypeForDynamicCast;
	const FTypeLayoutDesc* TypeLayout;
	const TCHAR* Name;
	FName TypeName;
	FHashedName HashedName;
	FHashedName HashedSourceFilename;
	const TCHAR* SourceFilename;
	const TCHAR* FunctionName;
	uint32 Frequency;
	uint32 TypeSize;
	int32 TotalPermutationCount;
	int32 TotalSpecializationCount;

	ConstructSerializedType ConstructSerializedRef;
	ConstructCompiledType ConstructCompiledRef;
	ShouldCompilePermutationType ShouldCompilePermutationRef;
	ShouldPrecachePermutationType ShouldPrecachePermutationRef;
	GetUnspecializedIdType GetUnspecializedIdRef;
	GetSpecializationValuesType GetSpecializationValuesRef;
	GetRayTracingPayloadTypeType GetRayTracingPayloadTypeRef;
	GetShaderBindingLayoutType GetShaderBindingLayoutTypeRef;
#if WITH_EDITOR
	ModifyCompilationEnvironmentType ModifyCompilationEnvironmentRef;
	ValidateCompiledResultType ValidateCompiledResultRef;
	GetOverrideJobPriorityType GetOverrideJobPriorityRef;
	GetPermutationIdStringType GetPermutationIdStringRef;
#endif
	const FShaderParametersMetadata* const RootParametersMetadata;

	TLinkedList<FShaderType*> GlobalListLink;

	friend void RENDERCORE_API DumpShaderStats( EShaderPlatform Platform, EShaderFrequency Frequency );

#if WITH_EDITOR
protected:
	/**
	* Cache of referenced uniform buffer structs.
	* These are derived from source files so they need to be flushed when editing and recompiling shaders on the fly.
	* FShaderType::Initialize will add the referenced uniform buffers, but this set may be updated by FlushShaderFileCache.
	*/
	TSet<const FShaderParametersMetadata*> ReferencedUniformBuffers;
#endif // WITH_EDITOR
};

inline FString LexToString(FShaderType::EShaderTypeForDynamicCast ShaderType)
{
#define SHADER_TYPE_ENTRY(Name) TEXT(#Name)
	static const TCHAR* TypeStrings[] =
	{
		SHADER_TYPE_LIST
	};
#undef SHADER_TYPE_ENTRY
	checkf(ShaderType < FShaderType::EShaderTypeForDynamicCast::NumShaderTypes, TEXT("Invalid shader type"));
	static_assert((sizeof(TypeStrings) / sizeof(TCHAR*)) == (int)FShaderType::EShaderTypeForDynamicCast::NumShaderTypes);
	return TypeStrings[static_cast<uint32>(ShaderType)];
}

/**
 * Registers a shader type in various systems. Should be created as a static field/global.
 * 
 * Each shader type is collected here, not as an instance but as an accessor, so the actual construction can be deferred.
 * The collection happens during static init, the actual construction happens later during launch.
 * The purpose of collecting the types is the CommitAll function, called in LaunchEngineLoop, to ensure all type instances are constructed and registered before other systems start iterating them.
 */
class FShaderTypeRegistration
{
public:
	RENDERCORE_API FShaderTypeRegistration(TFunctionRef<FShaderType& ()> LazyShaderTypeAccessor);

	static RENDERCORE_API TArray<const FShaderTypeRegistration*>& GetInstances();

	// Actually register all the types and clear the array
	static RENDERCORE_API void CommitAll();
	static RENDERCORE_API bool AreShaderTypesInitialized();

private:
	static bool bShaderTypesInitialized;
	TFunctionRef<FShaderType& ()> LazyShaderTypeAccessor;
};

struct FShaderCompiledShaderInitializerType
{
	const FShaderType* Type;
	const FShaderType::FParameters* Parameters;
	FShaderTarget Target;
	TConstArrayView<uint8> Code;
	const FShaderParameterMap& ParameterMap;
	const FShaderHash& OutputHash;
	uint64 MaterialShaderMapHash;
	const FShaderPipelineType* ShaderPipeline;
	const FVertexFactoryType* VertexFactoryType;
	uint32 NumInstructions;
	uint32 NumTextureSamplers;
	uint32 CodeSize;
	int32 PermutationId;
	TArray<FGenericShaderStat> ShaderStatistics;

	RENDERCORE_API FShaderCompiledShaderInitializerType(
		const FShaderType* InType,
		const FShaderType::FParameters* InParameters,
		int32 InPermutationId,
		const FShaderCompilerOutput& CompilerOutput,
		uint64 InMaterialShaderMapHash,
		const FShaderPipelineType* InShaderPipeline,
		const FVertexFactoryType* InVertexFactoryType
	);
};

#if WITH_EDITOR
	#define SHADER_DECLARE_EDITOR_VTABLE(ShaderClass) \
		static void ModifyCompilationEnvironmentImpl(const FShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment) \
		{ \
			const typename ShaderClass::FPermutationDomain PermutationVector(Parameters.PermutationId); \
			PermutationVector.ModifyCompilationEnvironment(OutEnvironment); \
			ShaderClass::ModifyCompilationEnvironment(static_cast<const typename ShaderClass::FPermutationParameters&>(Parameters), OutEnvironment); \
		} \
		static void GetPermutationIdStringImpl(int32 PermutationId, FString& OutString, bool bFullNames) \
		{ \
			const typename ShaderClass::FPermutationDomain PermutationVector(PermutationId); \
			UE::ShaderPermutationUtils::FormatPermutationDomain(PermutationVector, OutString, bFullNames, nullptr); \
		}
#else
	#define SHADER_DECLARE_EDITOR_VTABLE(ShaderClass)
#endif // WITH_EDITOR

#define SHADER_DECLARE_VTABLE(ShaderClass) \
	static FShader* ConstructSerializedInstance() { return new ShaderClass(); } \
	static FShader* ConstructCompiledInstance(const typename FShader::CompiledShaderInitializerType& Initializer) \
	{ return new ShaderClass(static_cast<const typename ShaderMetaType::CompiledShaderInitializerType&>(Initializer)); }\
	static bool ShouldCompilePermutationImpl(const FShaderPermutationParameters& Parameters) \
	{ return ShaderClass::ShouldCompilePermutation(static_cast<const typename ShaderClass::FPermutationParameters&>(Parameters)); } \
	static EShaderPermutationPrecacheRequest ShouldPrecachePermutationImpl(const FShaderPermutationParameters& Parameters) \
	{ return ShaderClass::ShouldPrecachePermutation(static_cast<const typename ShaderClass::FPermutationParameters&>(Parameters)); } \
	static int32 GetUnspecializedIdImpl(const int32 PermutationID) \
	{ return typename ShaderClass::FPermutationDomain(PermutationID).ToDimensionUnspecializedId(); } \
	static FSpecializationContainerType GetSpecializationValuesImpl(const int32 PermutationID) \
	{ \
		if constexpr (ShaderClass::FPermutationDomain::SpecializationCount > 0) {\
			FSpecializationContainerType SpecConsts; \
			SpecConsts.Reserve(ShaderClass::FPermutationDomain::SpecializationCount); \
			typename ShaderClass::FPermutationDomain(PermutationID).ToSpecializationConstantArray(SpecConsts); \
			return SpecConsts; \
		} else { return FSpecializationContainerType(); } \
	} \
	SHADER_DECLARE_EDITOR_VTABLE(ShaderClass)


#define INTERNAL_DECLARE_SHADER_TYPE_COMMON(ShaderClass,ShaderMetaTypeShortcut,RequiredAPI) \
	public: \
	using ShaderMetaType = F##ShaderMetaTypeShortcut##ShaderType; \
	using ShaderMapType = F##ShaderMetaTypeShortcut##ShaderMap; \
	\
	static RequiredAPI ShaderMetaType& GetStaticType(); \
	private: \
	static FShaderTypeRegistration ShaderTypeRegistration; \
	public: \
	\
	SHADER_DECLARE_VTABLE(ShaderClass)

/**
 * A macro to declare a new shader type.  This should be called in the class body of the new shader type.
 * @param ShaderClass - The name of the class representing an instance of the shader type.
 * @param ShaderMetaTypeShortcut - The shortcut for the shader meta type: simple, material, meshmaterial, etc.  The shader meta type
 *	controls 
 */
#define DECLARE_EXPORTED_SHADER_TYPE(ShaderClass,ShaderMetaTypeShortcut,RequiredAPI, ...) \
	INTERNAL_DECLARE_SHADER_TYPE_COMMON(ShaderClass, ShaderMetaTypeShortcut, RequiredAPI); \
	DECLARE_EXPORTED_TYPE_LAYOUT(ShaderClass, RequiredAPI, NonVirtual); \
	public:

#define DECLARE_SHADER_TYPE(ShaderClass,ShaderMetaTypeShortcut,...) \
	DECLARE_EXPORTED_SHADER_TYPE(ShaderClass,ShaderMetaTypeShortcut,, ##__VA_ARGS__)

#define DECLARE_SHADER_TYPE_EXPLICIT_BASES(ShaderClass,ShaderMetaTypeShortcut,...) \
	INTERNAL_DECLARE_SHADER_TYPE_COMMON(ShaderClass, ShaderMetaTypeShortcut,); \
	DECLARE_EXPORTED_TYPE_LAYOUT_EXPLICIT_BASES(ShaderClass,, NonVirtual, __VA_ARGS__); \
	public:

#if WITH_EDITOR

#define SHADER_TYPE_EDITOR_VTABLE(ShaderClass) \
	, ShaderClass::ModifyCompilationEnvironmentImpl \
	, ShaderClass::ValidateCompiledResult \
	, ShaderClass::GetOverrideJobPriority

#define SHADER_TYPE_EDITOR_PERMUTATION_METADATA(ShaderClass) \
	, ShaderClass::GetPermutationIdStringImpl

#else

#define SHADER_TYPE_EDITOR_VTABLE(ShaderClass)
#define SHADER_TYPE_EDITOR_PERMUTATION_METADATA(ShaderClass)

#endif

#define SHADER_TYPE_VTABLE(ShaderClass) \
	ShaderClass::ConstructSerializedInstance, \
	ShaderClass::ConstructCompiledInstance, \
	ShaderClass::ShouldCompilePermutationImpl, \
	ShaderClass::ShouldPrecachePermutationImpl, \
	ShaderClass::GetUnspecializedIdImpl, \
	ShaderClass::GetSpecializationValuesImpl, \
	ShaderClass::GetRayTracingPayloadType, \
	ShaderClass::GetShaderBindingLayout \
	SHADER_TYPE_EDITOR_VTABLE(ShaderClass)

#if !UE_BUILD_DOCS
/** A macro to implement a shader type. */
#define IMPLEMENT_SHADER_TYPE(TemplatePrefix,ShaderClass,SourceFilename,FunctionName,Frequency) \
	IMPLEMENT_UNREGISTERED_TEMPLATE_TYPE_LAYOUT(TemplatePrefix, ShaderClass); \
	TemplatePrefix \
	ShaderClass::ShaderMetaType& ShaderClass::GetStaticType() \
	{ \
		static ShaderClass::ShaderMetaType StaticType( \
			ShaderClass::StaticGetTypeLayout(), \
			TEXT(#ShaderClass), \
			SourceFilename, \
			FunctionName, \
			Frequency, \
			ShaderClass::FPermutationDomain::PermutationCount, \
			ShaderClass::FPermutationDomain::SpecializationCount, \
			SHADER_TYPE_VTABLE(ShaderClass), \
			sizeof(ShaderClass), \
			ShaderClass::GetRootParametersMetadata() \
			SHADER_TYPE_EDITOR_PERMUTATION_METADATA(ShaderClass) \
		); \
		return StaticType; \
	} \
	TemplatePrefix FShaderTypeRegistration ShaderClass::ShaderTypeRegistration{TFunctionRef<::FShaderType&()>{ShaderClass::GetStaticType}};

/** A macro to implement a shader type. Shader name is got from GetDebugName(), which is helpful for templated shaders. */
#define IMPLEMENT_SHADER_TYPE_WITH_DEBUG_NAME(TemplatePrefix,ShaderClass,SourceFilename,FunctionName,Frequency) \
	IMPLEMENT_UNREGISTERED_TEMPLATE_TYPE_LAYOUT(TemplatePrefix, ShaderClass); \
	TemplatePrefix \
	typename ShaderClass::ShaderMetaType& ShaderClass::GetStaticType() \
	{ \
		static typename ShaderClass::ShaderMetaType StaticType( \
			ShaderClass::StaticGetTypeLayout(), \
			ShaderClass::GetDebugName(), \
			SourceFilename, \
			FunctionName, \
			Frequency, \
			ShaderClass::FPermutationDomain::PermutationCount, \
			ShaderClass::FPermutationDomain::SpecializationCount, \
			SHADER_TYPE_VTABLE(ShaderClass), \
			sizeof(ShaderClass), \
			ShaderClass::GetRootParametersMetadata() \
			SHADER_TYPE_EDITOR_PERMUTATION_METADATA(ShaderClass) \
		); \
		return StaticType; \
	} \
	TemplatePrefix FShaderTypeRegistration ShaderClass::ShaderTypeRegistration{TFunctionRef<::FShaderType&()>{ShaderClass::GetStaticType}};

/** A macro to implement a templated shader type, the function name and the source filename comes from the class. */
#define IMPLEMENT_SHADER_TYPE2_WITH_TEMPLATE_PREFIX(TemplatePrefix,ShaderClass,Frequency) \
	IMPLEMENT_UNREGISTERED_TEMPLATE_TYPE_LAYOUT(TemplatePrefix, ShaderClass); \
	TemplatePrefix \
	ShaderClass::ShaderMetaType& ShaderClass::GetStaticType() \
	{ \
		static ShaderClass::ShaderMetaType StaticType( \
			ShaderClass::StaticGetTypeLayout(), \
			TEXT(#ShaderClass), \
			ShaderClass::GetSourceFilename(), \
			ShaderClass::GetFunctionName(), \
			Frequency, \
			ShaderClass::FPermutationDomain::PermutationCount, \
			ShaderClass::FPermutationDomain::SpecializationCount, \
			SHADER_TYPE_VTABLE(ShaderClass), \
			sizeof(ShaderClass), \
			ShaderClass::GetRootParametersMetadata() \
			SHADER_TYPE_EDITOR_PERMUTATION_METADATA(ShaderClass) \
		); \
		return StaticType; \
	} \
	TemplatePrefix FShaderTypeRegistration ShaderClass::ShaderTypeRegistration{TFunctionRef<::FShaderType&()>{ShaderClass::GetStaticType}};

#define IMPLEMENT_SHADER_TYPE2(ShaderClass,Frequency) \
	IMPLEMENT_SHADER_TYPE2_WITH_TEMPLATE_PREFIX(template<>, ShaderClass, Frequency)

/** todo: this should replace IMPLEMENT_SHADER_TYPE */
#define IMPLEMENT_SHADER_TYPE3(ShaderClass,Frequency) \
	IMPLEMENT_UNREGISTERED_TEMPLATE_TYPE_LAYOUT(,ShaderClass); \
	ShaderClass::ShaderMetaType& ShaderClass::GetStaticType() \
	{ \
		static ShaderClass::ShaderMetaType StaticType( \
			ShaderClass::StaticGetTypeLayout(), \
			TEXT(#ShaderClass), \
			ShaderClass::GetSourceFilename(), \
			ShaderClass::GetFunctionName(), \
			Frequency, \
			ShaderClass::FPermutationDomain::PermutationCount, \
			ShaderClass::FPermutationDomain::SpecializationCount, \
			SHADER_TYPE_VTABLE(ShaderClass), \
			sizeof(ShaderClass), \
			ShaderClass::GetRootParametersMetadata() \
			SHADER_TYPE_EDITOR_PERMUTATION_METADATA(ShaderClass) \
		); \
		return StaticType; \
	} \
	FShaderTypeRegistration ShaderClass::ShaderTypeRegistration{TFunctionRef<::FShaderType&()>{ShaderClass::GetStaticType}};
#endif // !UE_BUILD_DOCS

#define IMPLEMENT_SHADER_TYPE4_WITH_TEMPLATE_PREFIX(TemplatePrefix,RequiredAPI,ShaderClass,Frequency) \
	IMPLEMENT_UNREGISTERED_TEMPLATE_TYPE_LAYOUT(TemplatePrefix, ShaderClass); \
	TemplatePrefix RequiredAPI \
	ShaderClass::ShaderMetaType& ShaderClass::GetStaticType() \
	{ \
		static ShaderClass::ShaderMetaType StaticType( \
			ShaderClass::StaticGetTypeLayout(), \
			TEXT(#ShaderClass), \
			ShaderClass::GetSourceFilename(), \
			ShaderClass::GetFunctionName(), \
			Frequency, \
			ShaderClass::FPermutationDomain::PermutationCount, \
			ShaderClass::FPermutationDomain::SpecializationCount, \
			SHADER_TYPE_VTABLE(ShaderClass), \
			sizeof(ShaderClass), \
			ShaderClass::GetRootParametersMetadata() \
			SHADER_TYPE_EDITOR_PERMUTATION_METADATA(ShaderClass) \
		); \
		return StaticType; \
	} \
	TemplatePrefix FShaderTypeRegistration ShaderClass::ShaderTypeRegistration{TFunctionRef<::FShaderType&()>{ShaderClass::GetStaticType}};

#if WITH_EDITOR

#define IMPLEMENT_SHADER_TYPE_CONSTRUCTOR(ShaderClass, InShaderTypeForDynamicCast, InShaderClassUserFriendlyName) \
	ShaderClass( \
		FTypeLayoutDesc& InTypeLayout, \
		const TCHAR* InName, \
		const TCHAR* InSourceFilename, \
		const TCHAR* InFunctionName, \
		uint32 InFrequency, \
		int32 InTotalPermutationCount, \
		int32 InTotalSpecializationCount, \
		ConstructSerializedType InConstructSerializedRef, \
		ConstructCompiledType InConstructCompiledRef, \
		ShouldCompilePermutationType InShouldCompilePermutationRef, \
		ShouldPrecachePermutationType InShouldPrecachePermutationRef, \
		GetUnspecializedIdType InGetUnspecializedIdRef, \
		GetSpecializationValuesType InGetSpecializationValuesRef, \
		GetRayTracingPayloadTypeType InGetRayTracingPayloadTypeRef, \
		GetShaderBindingLayoutType InGetShaderBindingLayoutTypeRef, \
		ModifyCompilationEnvironmentType InModifyCompilationEnvironmentRef, \
		ValidateCompiledResultType InValidateCompiledResultRef, \
		GetOverrideJobPriorityType InGetOverrideJobPriorityRef, \
		uint32 InTypeSize, \
		const FShaderParametersMetadata* InRootParametersMetadata = nullptr, \
		GetPermutationIdStringType InGetPermutationIdStringRef = nullptr \
		): \
		FShaderType( \
			InShaderTypeForDynamicCast, \
			InTypeLayout, \
			InName, \
			InSourceFilename, \
			InFunctionName, \
			InFrequency, \
			InTotalPermutationCount, \
			InTotalSpecializationCount, \
			InConstructSerializedRef, \
			InConstructCompiledRef, \
			InShouldCompilePermutationRef, \
			InShouldPrecachePermutationRef, \
			InGetUnspecializedIdRef, \
			InGetSpecializationValuesRef, \
			InGetRayTracingPayloadTypeRef, \
			InGetShaderBindingLayoutTypeRef, \
			InModifyCompilationEnvironmentRef, \
			InValidateCompiledResultRef, \
			InGetOverrideJobPriorityRef, \
			InTypeSize, \
			InRootParametersMetadata, \
			InGetPermutationIdStringRef \
		) \
	{ \
		checkf(FPaths::GetExtension(InSourceFilename) == TEXT("usf"), \
			TEXT("Incorrect virtual shader path extension for %s '%s': Only .usf files should be compiled."), \
			InShaderClassUserFriendlyName, InSourceFilename); \
	}

#else

#define IMPLEMENT_SHADER_TYPE_CONSTRUCTOR(ShaderClass, InShaderTypeForDynamicCast, InShaderClassUserFriendlyName) \
	ShaderClass( \
		FTypeLayoutDesc& InTypeLayout, \
		const TCHAR* InName, \
		const TCHAR* InSourceFilename, \
		const TCHAR* InFunctionName, \
		uint32 InFrequency, \
		int32 InTotalPermutationCount, \
		int32 InTotalSpecializationCount, \
		ConstructSerializedType InConstructSerializedRef, \
		ConstructCompiledType InConstructCompiledRef, \
		ShouldCompilePermutationType InShouldCompilePermutationRef, \
		ShouldPrecachePermutationType InShouldPrecachePermutationRef, \
		GetUnspecializedIdType InGetUnspecializedIdRef, \
		GetSpecializationValuesType InGetSpecializationValuesRef, \
		GetRayTracingPayloadTypeType InGetRayTracingPayloadTypeRef, \
		GetShaderBindingLayoutType InGetShaderBindingLayoutTypeRef, \
		uint32 InTypeSize, \
		const FShaderParametersMetadata* InRootParametersMetadata = nullptr \
		): \
		FShaderType( \
			InShaderTypeForDynamicCast, \
			InTypeLayout, \
			InName, \
			InSourceFilename, \
			InFunctionName, \
			InFrequency, \
			InTotalPermutationCount, \
			InTotalSpecializationCount, \
			InConstructSerializedRef, \
			InConstructCompiledRef, \
			InShouldCompilePermutationRef, \
			InShouldPrecachePermutationRef, \
			InGetUnspecializedIdRef, \
			InGetSpecializationValuesRef, \
			InGetRayTracingPayloadTypeRef, \
			InGetShaderBindingLayoutTypeRef, \
			InTypeSize, \
			InRootParametersMetadata \
		) \
	{ \
		checkf(FPaths::GetExtension(InSourceFilename) == TEXT("usf"), \
			TEXT("Incorrect virtual shader path extension for %s '%s': Only .usf files should be compiled."), \
			InShaderClassUserFriendlyName, InSourceFilename); \
	}

#endif // WITH_EDITOR

// Binding of a set of shader stages in a single pipeline
class FShaderPipelineType
{
public:
	// Set bShouldOptimizeUnusedOutputs to true if we want unique FShaders for each shader pipeline
	// Set bShouldOptimizeUnusedOutputs to false if the FShaders will point to the individual shaders in the map
	RENDERCORE_API FShaderPipelineType(
		const TCHAR* InName,
		const FShaderType* InVertexOrMeshShader,
		const FShaderType* InGeometryOrAmplificationShader,
		const FShaderType* InPixelShader,
		bool bInIsMeshPipeline,
		bool bInShouldOptimizeUnusedOutputs);
	RENDERCORE_API ~FShaderPipelineType();

	inline bool HasMeshShader() const { return AllStages[SF_Mesh] != nullptr; }
	inline bool HasGeometry() const { return AllStages[SF_Geometry] != nullptr; }
	inline bool HasPixelShader() const { return AllStages[SF_Pixel] != nullptr; }

	inline const FShaderType* GetShader(EShaderFrequency Frequency) const
	{
		check(Frequency < SF_NumFrequencies);
		return AllStages[Frequency];
	}

	inline FName GetFName() const { return TypeName; }
	inline TCHAR const* GetName() const { return Name; }
	inline const FHashedName& GetHashedName() const { return HashedName; }
	inline const FHashedName& GetHashedPrimaryShaderFilename() const { return HashedPrimaryShaderFilename; }

	// Returns an array of valid stages, sorted from PS->GS->DS->HS->VS, no gaps if missing stages
	inline const TArray<const FShaderType*>& GetStages() const { return Stages; }

	static RENDERCORE_API TLinkedList<FShaderPipelineType*>*& GetTypeList();

	static RENDERCORE_API const TArray<FShaderPipelineType*>& GetSortedTypes(FShaderType::EShaderTypeForDynamicCast Type);

	/** @return The global shader pipeline name to type map */
	static RENDERCORE_API TMap<FHashedName, FShaderPipelineType*>& GetNameToTypeMap();
	static RENDERCORE_API const FShaderPipelineType* GetShaderPipelineTypeByName(const FHashedName& Name);

	/** Initialize static members, this must be called before any shader types are created. */
	static RENDERCORE_API void Initialize();

	static RENDERCORE_API TArray<const FShaderPipelineType*> GetShaderPipelineTypesByFilename(const TCHAR* Filename, bool bSearchAsRegexFilter = false);
	static RENDERCORE_API TArray<const FShaderPipelineType*> GetShaderPipelineTypesByFilenameFilter(const TFunction<bool(const TCHAR*)>& InFilenameFilter);

	/** Serializes a shader type reference by name. */
	RENDERCORE_API friend FArchive& operator<<(FArchive& Ar, const FShaderPipelineType*& Ref);

	/** Hashes a pointer to a shader type. */
	friend uint32 GetTypeHash(FShaderPipelineType* Ref) { return Ref ? Ref->HashIndex : 0; }
	friend uint32 GetTypeHash(const FShaderPipelineType* Ref) { return Ref ? Ref->HashIndex : 0; }

	// Check if this pipeline is built of specific types
	bool IsGlobalTypePipeline() const { return Stages[0]->GetGlobalShaderType() != nullptr; }
	bool IsMaterialTypePipeline() const { return Stages[0]->GetMaterialShaderType() != nullptr; }
	bool IsMeshMaterialTypePipeline() const { return Stages[0]->GetMeshMaterialShaderType() != nullptr; }

	RENDERCORE_API bool ShouldOptimizeUnusedOutputs(EShaderPlatform Platform) const;

	/** Calculates a Hash based on this shader pipeline type stages' source code and includes */
	RENDERCORE_API const FSHAHash& GetSourceHash(EShaderPlatform ShaderPlatform) const;

	RENDERCORE_API bool ShouldCompilePermutation(const FShaderPermutationParameters& Parameters) const;

	RENDERCORE_API EShaderPermutationPrecacheRequest ShouldPrecachePermutation(const FShaderPermutationParameters& Parameters) const;

protected:
	const TCHAR* const Name;
	FName TypeName;
	FHashedName HashedName;
	FHashedName HashedPrimaryShaderFilename;

	// Pipeline Stages, ordered from lowest (usually PS) to highest (VS). Guaranteed at least one stage (for VS).
	TArray<const FShaderType*> Stages;

	const FShaderType* AllStages[SF_NumFrequencies];

	TLinkedList<FShaderPipelineType*> GlobalListLink;

	uint32 HashIndex;
	bool bShouldOptimizeUnusedOutputs;

	static RENDERCORE_API bool bInitialized;
};

#if !UE_BUILD_DOCS
// Vertex+Pixel
#define IMPLEMENT_SHADERPIPELINE_TYPE_VSPS(PipelineName, VertexShaderType, PixelShaderType, bRemoveUnused)	\
	static FShaderPipelineType PipelineName(TEXT(UE_STRINGIZE(PipelineName)), &VertexShaderType::GetStaticType(), nullptr, &PixelShaderType::GetStaticType(), false, bRemoveUnused);
// Only VS
#define IMPLEMENT_SHADERPIPELINE_TYPE_VS(PipelineName, VertexShaderType, bRemoveUnused)	\
	static FShaderPipelineType PipelineName(TEXT(UE_STRINGIZE(PipelineName)), &VertexShaderType::GetStaticType(), nullptr, nullptr, false, bRemoveUnused);
// Vertex+Geometry+Pixel
#define IMPLEMENT_SHADERPIPELINE_TYPE_VSGSPS(PipelineName, VertexShaderType, GeometryShaderType, PixelShaderType, bRemoveUnused)	\
	static FShaderPipelineType PipelineName(TEXT(UE_STRINGIZE(PipelineName)), &VertexShaderType::GetStaticType(), &GeometryShaderType::GetStaticType(), &PixelShaderType::GetStaticType(), false, bRemoveUnused);
// Vertex+Geometry
#define IMPLEMENT_SHADERPIPELINE_TYPE_VSGS(PipelineName, VertexShaderType, GeometryShaderType, bRemoveUnused)	\
	static FShaderPipelineType PipelineName(TEXT(UE_STRINGIZE(PipelineName)), &VertexShaderType::GetStaticType(), &GeometryShaderType::GetStaticType(), nullptr, false, bRemoveUnused);

// Mesh+Pixel
#define IMPLEMENT_SHADERPIPELINE_TYPE_MSPS(PipelineName, MeshShaderType, PixelShaderType, bRemoveUnused)	\
	static FShaderPipelineType PipelineName(TEXT(UE_STRINGIZE(PipelineName)), &MeshShaderType::GetStaticType(), nullptr, &PixelShaderType::GetStaticType(), true, bRemoveUnused);
// Mesh+Amplification+Pixel
#define IMPLEMENT_SHADERPIPELINE_TYPE_MSASPS(PipelineName, MeshShaderType, AmplificationShaderType, PixelShaderType, bRemoveUnused)	\
	static FShaderPipelineType PipelineName(TEXT(UE_STRINGIZE(PipelineName)), &MeshShaderType::GetStaticType(), &AmplificationShaderType::GetStaticType(), &PixelShaderType::GetStaticType(), true, bRemoveUnused);
#endif

/** Encapsulates a dependency on a shader type and saved state from that shader type. */
class FShaderTypeDependency
{
	DECLARE_EXPORTED_TYPE_LAYOUT(FShaderTypeDependency, RENDERCORE_API, NonVirtual);
public:

	FShaderTypeDependency()
		: PermutationId(0)
	{}

	FShaderTypeDependency(FShaderType* InShaderType, EShaderPlatform ShaderPlatform)
		: ShaderTypeName(InShaderType->GetHashedName())
		, PermutationId(0)
	{
		if (InShaderType)
		{
			SourceHash = InShaderType->GetSourceHash(ShaderPlatform);
		}
	}

	friend FArchive& operator<<(FArchive& Ar,class FShaderTypeDependency& Ref)
	{
		Ar.UsingCustomVersion(FRenderingObjectVersion::GUID);

		Ar << Ref.ShaderTypeName;
		Ar << Ref.SourceHash;

		if (Ar.CustomVer(FRenderingObjectVersion::GUID) >= FRenderingObjectVersion::ShaderPermutationId)
		{
			Ar << Ref.PermutationId;
		}

		return Ar;
	}

	bool operator==(const FShaderTypeDependency& Reference) const
	{
		return ShaderTypeName == Reference.ShaderTypeName && PermutationId == Reference.PermutationId && SourceHash == Reference.SourceHash;
	}

	bool operator!=(const FShaderTypeDependency& Reference) const
	{
		return !(*this == Reference);
	}

	/** Call GetShaderFileHash to get the cached value for the filename's hash in the current process. */
	RENDERCORE_API void RefreshCachedSourceHash(EShaderPlatform ShaderPlatform);

	/** Shader type */
	LAYOUT_FIELD(FHashedName, ShaderTypeName);

	/** Unique permutation identifier of the global shader type. */
	LAYOUT_FIELD(int32, PermutationId);

	/** Used to detect changes to the shader source files. This is always present, as this type is sometimes frozen. */
	LAYOUT_FIELD(FSHAHash, SourceHash);

private:
#if WITH_EDITOR
	// Compact binary API with hidden friend operator<<
	RENDERCORE_API void Save(FCbWriter& Writer) const;
	bool TryLoad(FCbFieldView Field);
	friend inline FCbWriter& operator<<(FCbWriter& Writer, const FShaderTypeDependency& Value)
	{
		Value.Save(Writer);
		return Writer;
	}
	friend RENDERCORE_API bool LoadFromCompactBinary(FCbFieldView Field, FShaderTypeDependency& OutValue);
#endif
};


class FShaderPipelineTypeDependency
{
public:
	FShaderPipelineTypeDependency() {}
	FShaderPipelineTypeDependency(const FShaderPipelineType* InShaderPipelineType, EShaderPlatform ShaderPlatform) :
		ShaderPipelineTypeName(InShaderPipelineType->GetHashedName())
	{
		if (InShaderPipelineType)
		{
			StagesSourceHash = InShaderPipelineType->GetSourceHash(ShaderPlatform);
		}
	}

	/** Shader Pipeline type */
	FHashedName ShaderPipelineTypeName;

	/** Used to detect changes to the shader source files. */
	FSHAHash StagesSourceHash;

	friend FArchive& operator<<(FArchive& Ar, class FShaderPipelineTypeDependency& Ref)
	{
		Ar << Ref.ShaderPipelineTypeName;
		Ar << Ref.StagesSourceHash;
		return Ar;
	}

	bool operator==(const FShaderPipelineTypeDependency& Reference) const
	{
		return ShaderPipelineTypeName == Reference.ShaderPipelineTypeName && StagesSourceHash == Reference.StagesSourceHash;
	}

	bool operator!=(const FShaderPipelineTypeDependency& Reference) const
	{
		return !(*this == Reference);
	}

	/** Call GetShaderFileHash to get the cached value for the filename's hash in the current process. */
	RENDERCORE_API void RefreshCachedSourceHash(EShaderPlatform ShaderPlatform);

private:
#if WITH_EDITOR
	// Compact binary API with hidden friend operator<<
	RENDERCORE_API void Save(FCbWriter& Writer) const;
	bool TryLoad(FCbFieldView Field);
	friend inline FCbWriter& operator<<(FCbWriter& Writer, const FShaderPipelineTypeDependency& Value)
	{
		Value.Save(Writer);
		return Writer;
	}
	friend RENDERCORE_API bool LoadFromCompactBinary(FCbFieldView Field, FShaderPipelineTypeDependency& OutValue);
#endif
};

/** Used to compare two shader types by name. */
class FCompareShaderTypes
{																				
public:
	inline bool operator()(const FShaderType& A, const FShaderType& B ) const
	{
		int32 AL = FCString::Strlen(A.GetName());
		int32 BL = FCString::Strlen(B.GetName());
		if ( AL == BL )
		{
			return FCString::Strncmp(A.GetName(), B.GetName(), AL) > 0;
		}
		return AL > BL;
	}
};


// A Shader Pipeline instance with compiled stages
class FShaderPipeline
{
	DECLARE_EXPORTED_TYPE_LAYOUT(FShaderPipeline, RENDERCORE_API, NonVirtual);
public:
	explicit FShaderPipeline(const FShaderPipelineType* InType) : TypeName(InType->GetHashedName()) { FMemory::Memzero(&PermutationIds, sizeof(PermutationIds)); }
	RENDERCORE_API ~FShaderPipeline();

	RENDERCORE_API void AddShader(FShader* Shader, int32 PermutationId);
	RENDERCORE_API FShader* FindOrAddShader(FShader* Shader, int32 PermutationId);

	inline uint32 GetNumShaders() const
	{
		uint32 NumShaders = 0u;
		for (uint32 i = 0u; i < SF_NumGraphicsFrequencies; ++i)
		{
			if (Shaders[i].IsValid())
			{
				++NumShaders;
			}
		}
		return NumShaders;
	}

	// Find a shader inside the pipeline
	template<typename ShaderType>
	ShaderType* GetShader(const FShaderMapPointerTable& InPtrTable)
	{
		const FShaderType& Type = ShaderType::GetStaticType();
		const EShaderFrequency Frequency = Type.GetFrequency();
		if (Frequency < SF_NumGraphicsFrequencies && Shaders[Frequency].IsValid())
		{
			FShader* Shader = Shaders[Frequency].GetChecked();
			if (Shader->GetType(InPtrTable) == &Type)
			{
				return static_cast<ShaderType*>(Shader);
			}
		}
		return nullptr;
	}

	FShader* GetShader(EShaderFrequency Frequency)
	{
		check(Frequency < SF_NumGraphicsFrequencies);
		return Shaders[Frequency];
	}

	const FShader* GetShader(EShaderFrequency Frequency) const
	{
		check(Frequency < SF_NumGraphicsFrequencies);
		return Shaders[Frequency];
	}

	inline TArray<TShaderRef<FShader>> GetShaders(const FShaderMapBase& InShaderMap) const
	{
		TArray<TShaderRef<FShader>> Result;
		for (uint32 i = 0u; i < SF_NumGraphicsFrequencies; ++i)
		{
			if (Shaders[i].IsValid())
			{
				Result.Add(TShaderRef<FShader>(Shaders[i].GetChecked(), InShaderMap));
			}
		}
		return Result;
	}

	RENDERCORE_API void Validate(const FShaderPipelineType* InPipelineType) const;

	RENDERCORE_API void Finalize(const FShaderMapResourceCode* Code);

	enum EFilter
	{
		EAll,			// All pipelines
		EOnlyShared,	// Only pipelines with shared shaders
		EOnlyUnique,	// Only pipelines with unique shaders
	};

	/** Saves stable keys for the shaders in the pipeline */
#if WITH_EDITOR
	RENDERCORE_API void SaveShaderStableKeys(const FShaderMapPointerTable& InPtrTable, EShaderPlatform TargetShaderPlatform, const struct FStableShaderKeyAndValue& SaveKeyVal) const;
#endif // WITH_EDITOR

	LAYOUT_FIELD(FHashedName, TypeName);
	LAYOUT_ARRAY(TMemoryImagePtr<FShader>, Shaders, SF_NumGraphicsFrequencies);
	LAYOUT_ARRAY(int32, PermutationIds, SF_NumGraphicsFrequencies);
};

inline bool operator<(const FShaderPipeline& Lhs, const FShaderPipeline& Rhs)
{
	return Lhs.TypeName.GetHash() < Rhs.TypeName.GetHash();
}

class FShaderPipelineRef
{
public:
	FShaderPipelineRef() : ShaderPipeline(nullptr), ShaderMap(nullptr) {}
	FShaderPipelineRef(FShaderPipeline* InPipeline, const FShaderMapBase& InShaderMap) : ShaderPipeline(InPipeline), ShaderMap(&InShaderMap) { checkSlow(InPipeline); }

	inline bool IsValid() const { return ShaderPipeline != nullptr; }
	inline bool IsNull() const { return ShaderPipeline == nullptr; }

	template<typename ShaderType>
	TShaderRef<ShaderType> GetShader() const
	{
		return TShaderRef<ShaderType>(ShaderPipeline->GetShader<ShaderType>(GetPointerTable()), *ShaderMap);
	}

	TShaderRef<FShader> GetShader(EShaderFrequency Frequency) const
	{
		return TShaderRef<FShader>(ShaderPipeline->GetShader(Frequency), *ShaderMap);
	}

	inline TArray<TShaderRef<FShader>> GetShaders() const
	{
		return ShaderPipeline->GetShaders(*ShaderMap);
	}

	inline FShaderPipeline* GetPipeline() const { return ShaderPipeline; }
	FShaderMapResource* GetResource() const;
	const FShaderMapPointerTable& GetPointerTable() const;

	inline FShaderPipeline* operator->() const { check(ShaderPipeline); return ShaderPipeline; }

private:
	FShaderPipeline* ShaderPipeline;
	const FShaderMapBase* ShaderMap;
};

DECLARE_DELEGATE_EightParams(FShaderListReport, const TCHAR*, const TCHAR*, const TCHAR*, const TCHAR*, uint32, uint32, const TCHAR*, void*);

/** A collection of shaders of different types */
class FShaderMapContent
{
	DECLARE_EXPORTED_TYPE_LAYOUT(FShaderMapContent, RENDERCORE_API, NonVirtual);
public:
	struct FProjectShaderPipelineToKey
	{
		inline FHashedName operator()(const FShaderPipeline* InShaderPipeline) { return InShaderPipeline->TypeName; }
	};

	/** Default constructor. */
	RENDERCORE_API explicit FShaderMapContent(EShaderPlatform InPlatform);

	/** Destructor ensures pipelines cleared up. */
	RENDERCORE_API ~FShaderMapContent();

	RENDERCORE_API EShaderPlatform GetShaderPlatform() const;

	RENDERCORE_API void Validate(const FShaderMapBase& InShaderMap) const;

	/** Finds the shader with the given type.  Asserts on failure. */
	template<typename ShaderType>
	ShaderType* GetShader(int32 PermutationId = 0) const
	{
		FShader* Shader = GetShader(&ShaderType::GetStaticType(), PermutationId);
		checkf(Shader != nullptr, TEXT("Failed to find shader type %s in Platform %s"), ShaderType::GetStaticType().GetName(), *LegacyShaderPlatformToShaderFormat(GetShaderPlatform()).ToString());
		return static_cast<ShaderType*>(Shader);
	}

	/** Finds the shader with the given type.  Asserts on failure. */
	template<typename ShaderType>
	ShaderType* GetShader( const typename ShaderType::FPermutationDomain& PermutationVector ) const
	{
		return GetShader<ShaderType>(UE::ShaderPermutationUtils::ConvertToUnspecializedId(GetShaderPlatform(), PermutationVector));
	}

	/** Finds the shader with the given type.  May return NULL. */
	FShader* GetShader(const FShaderType* ShaderType, int32 PermutationId = 0) const
	{
		return GetShader(ShaderType->GetHashedName(), PermutationId);
	}

	/** Finds the shader with the given type name.  May return NULL. */
	RENDERCORE_API FShader* GetShader(const FHashedName& TypeName, int32 PermutationId = 0) const;

	/** Finds the shader with the given type. */
	bool HasShader(const FHashedName& TypeName, int32 PermutationId) const
	{
		const FShader* Shader = GetShader(TypeName, PermutationId);
		return Shader != nullptr;
	}

	bool HasShader(const FShaderType* Type, int32 PermutationId) const
	{
		return HasShader(Type->GetHashedName(), PermutationId);
	}

	inline TArrayView<const TMemoryImagePtr<FShader>> GetShaders() const
	{
		return Shaders;
	}

	inline TArrayView<const TMemoryImagePtr<FShaderPipeline>> GetShaderPipelines() const
	{
		return ShaderPipelines;
	}

	RENDERCORE_API void AddShader(const FHashedName& TypeName, int32 PermutationId, FShader* Shader);

	RENDERCORE_API FShader* FindOrAddShader(const FHashedName& TypeName, int32 PermutationId, FShader* Shader);

	RENDERCORE_API void AddShaderPipeline(FShaderPipeline* Pipeline);

	RENDERCORE_API FShaderPipeline* FindOrAddShaderPipeline(FShaderPipeline* Pipeline);

	/**
	 * Removes the shader of the given type from the shader map
	 * @param Type Shader type to remove the entry for 
	 */
	RENDERCORE_API void RemoveShaderTypePermutaion(const FHashedName& TypeName, int32 PermutationId);

	inline void RemoveShaderTypePermutaion(const FShaderType* Type, int32 PermutationId)
	{
		RemoveShaderTypePermutaion(Type->GetHashedName(), PermutationId);
	}

	RENDERCORE_API void RemoveShaderPipelineType(const FShaderPipelineType* ShaderPipelineType);

#if !UE_BUILD_SHIPPING
	RENDERCORE_API void DumpShaderList(const FShaderMapBase& InShaderMap, FShaderListReport& Out) const;
#endif // !UE_BUILD_SHIPPING


	UE_DEPRECATED(5.8, "Use overload accepting uint64 shadermap hash")
	inline void GetShaderList(const FShaderMapBase& InShaderMap, const FSHAHash& InMaterialShaderMapHash, TMap<FShaderId, TShaderRef<FShader>>& OutShaders) const
	{
		uint64 TruncatedShaHash;
		memcpy(&TruncatedShaHash, InMaterialShaderMapHash.Hash, sizeof(uint64));
		GetShaderList(InShaderMap, TruncatedShaHash, OutShaders);
	}
	
	/** Builds a list of the shaders in a shader map. */
	RENDERCORE_API void GetShaderList(const FShaderMapBase& InShaderMap, uint64 InMaterialShaderMapHash, TMap<FShaderId, TShaderRef<FShader>>& OutShaders) const;
	/** Builds a list of the shaders in a shader map. Key is FShaderType::TypeName */
	RENDERCORE_API void GetShaderList(const FShaderMapBase& InShaderMap, TMap<FHashedName, TShaderRef<FShader>>& OutShaders) const;

	/** Builds a list of the shader pipelines in a shader map. */
	RENDERCORE_API void GetShaderPipelineList(const FShaderMapBase& InShaderMap, TArray<FShaderPipelineRef>& OutShaderPipelines, FShaderPipeline::EFilter Filter) const;

#if WITH_EDITOR
	RENDERCORE_API uint32 GetMaxTextureSamplersShaderMap(const FShaderMapBase& InShaderMap) const;

	RENDERCORE_API void GetOutdatedTypes(const FShaderMapBase& InShaderMap, TArray<const FShaderType*>& OutdatedShaderTypes, TArray<const FShaderPipelineType*>& OutdatedShaderPipelineTypes, TArray<const FVertexFactoryType*>& OutdatedFactoryTypes) const;

	RENDERCORE_API void SaveShaderStableKeys(const FShaderMapBase& InShaderMap, EShaderPlatform TargetShaderPlatform, const struct FStableShaderKeyAndValue& SaveKeyVal);
#endif // WITH_EDITOR

#if WITH_EDITORONLY_DATA
	RENDERCORE_API TArray<FGenericShaderStat> GetShaderStatistics(const FShaderMapBase& InShaderMap, FShaderType* ShaderType) const;
#endif // WITH_EDITORONLY_DATA

	/** @return true if the map is empty */
	inline bool IsEmpty() const
	{
		return Shaders.Num() == 0;
	}

	/** @return The number of shaders in the map. */
	RENDERCORE_API uint32 GetNumShaders() const;

	/** @return The number of shader pipelines in the map. */
	inline uint32 GetNumShaderPipelines() const
	{
		return ShaderPipelines.Num();
	}

	/** clears out all shaders and deletes shader pipelines held in the map */
	RENDERCORE_API void Empty();

	inline FShaderPipeline* GetShaderPipeline(const FHashedName& PipelineTypeName) const
	{
		const int32 Index = Algo::BinarySearchBy(ShaderPipelines, PipelineTypeName, FProjectShaderPipelineToKey());
		return (Index != INDEX_NONE) ? ShaderPipelines[Index].Get() : nullptr;
	}

	inline FShaderPipeline* GetShaderPipeline(const FShaderPipelineType* PipelineType) const
	{
		return GetShaderPipeline(PipelineType->GetHashedName());
	}

	inline bool HasShaderPipeline(const FHashedName& PipelineTypeName) const { return GetShaderPipeline(PipelineTypeName) != nullptr; }
	inline bool HasShaderPipeline(const FShaderPipelineType* PipelineType) const { return (GetShaderPipeline(PipelineType) != nullptr); }

	RENDERCORE_API uint32 GetMaxNumInstructionsForShader(const FShaderMapBase& InShaderMap, FShaderType* ShaderType) const;

	RENDERCORE_API void Finalize(const FShaderMapResourceCode* Code);

	template <typename THashBuilder>
	void UpdateHash(THashBuilder& Hasher) const
	{
		for (int32 ShaderIndex = 0; ShaderIndex < Shaders.Num(); ++ShaderIndex)
		{
			const uint64 TypeNameHash = ShaderTypes[ShaderIndex].GetHash();
			const int32 PermutationId = ShaderPermutations[ShaderIndex];
			Hasher.Update((uint8*)&TypeNameHash, sizeof(TypeNameHash));
			Hasher.Update((uint8*)&PermutationId, sizeof(PermutationId));
		}
		for (const FShaderPipeline* Pipeline : GetShaderPipelines())
		{
			const uint64 TypeNameHash = Pipeline->TypeName.GetHash();
			Hasher.Update((uint8*)&TypeNameHash, sizeof(TypeNameHash));
		}
	}

protected:
	RENDERCORE_API void EmptyShaderPipelines();

	using FMemoryImageHashTable = THashTable<FMemoryImageAllocator>;

	LAYOUT_FIELD(FMemoryImageHashTable, ShaderHash);
	LAYOUT_FIELD(TMemoryImageArray<FHashedName>, ShaderTypes);
	LAYOUT_FIELD(TMemoryImageArray<int32>, ShaderPermutations);
	LAYOUT_FIELD(TMemoryImageArray<TMemoryImagePtr<FShader>>, Shaders);
	LAYOUT_FIELD(TMemoryImageArray<TMemoryImagePtr<FShaderPipeline>>, ShaderPipelines);
	/** The ShaderPlatform Name this shader map was compiled with */
	LAYOUT_FIELD(FMemoryImageName, ShaderPlatformName);
};

class FShaderMapBase
{
public:
	RENDERCORE_API virtual ~FShaderMapBase();

	RENDERCORE_API FShaderMapResourceCode* GetResourceCode();

	inline FShaderMapResource* GetResource() const { return Resource; }
	inline FShaderMapResource* GetResourceChecked() const { check(Resource); return Resource; }
	inline const FShaderMapPointerTable& GetPointerTable() const { check(PointerTable); return *PointerTable; }
	inline const FShaderMapContent* GetContent() const { return Content.Object; }
	inline FShaderMapContent* GetMutableContent()
	{
		UnfreezeContent();
		return Content.Object;
	}
	/** Builds a list of the shaders in a shader map. Key is FShaderType::TypeName */
	virtual void GetShaderList(TMap<FHashedName, TShaderRef<FShader>>& OutShaders) const = 0;
	virtual void GetShaderPipelineList(TArray<FShaderPipelineRef>& OutShaderPipelines) const = 0;
	
#if !UE_BUILD_SHIPPING
	virtual void DumpShaderList(FShaderListReport& Out) const
	{
		GetContent()->DumpShaderList(*this, Out);
	}
#endif // !UE_BUILD_SHIPPING

	inline EShaderPlatform GetShaderPlatform() const { return Content.Object ? Content.Object->GetShaderPlatform() : SP_NumPlatforms; }
	inline uint32 GetFrozenContentSize() const { return Content.FrozenSize; }

	RENDERCORE_API void AssignContent(TMemoryImageObject<FShaderMapContent> InContent);

	RENDERCORE_API void FinalizeContent();
	RENDERCORE_API void UnfreezeContent();
	RENDERCORE_API bool Serialize(FShaderSerializeContext& Ctx);

	EShaderPermutationFlags GetPermutationFlags() const
	{
		return PermutationFlags;
	}

	RENDERCORE_API FString ToString() const;

#if WITH_EDITORONLY_DATA
	RENDERCORE_API TArray<FGenericShaderStat> GetShaderStatistics(FShaderType* ShaderType) const;
	RENDERCORE_API TArray<FGenericShaderStat> GetShaderStatistics(FShader& Shader) const;
#endif

#if WITH_EDITOR
	inline void GetOutdatedTypes(TArray<const FShaderType*>& OutdatedShaderTypes, TArray<const FShaderPipelineType*>& OutdatedShaderPipelineTypes, TArray<const FVertexFactoryType*>& OutdatedFactoryTypes) const
	{
		Content.Object->GetOutdatedTypes(*this, OutdatedShaderTypes, OutdatedShaderPipelineTypes, OutdatedFactoryTypes);
	}
	void SaveShaderStableKeys(EShaderPlatform TargetShaderPlatform, const struct FStableShaderKeyAndValue& SaveKeyVal)
	{
		Content.Object->SaveShaderStableKeys(*this, TargetShaderPlatform, SaveKeyVal);
	}

	/** Associates a shadermap with an asset (note: one shadermap can be used by several assets, e.g. MIs). 
	 * This helps cooker lay out the shadermaps (and shaders) in the file open order, if provided. Maps not associated with any assets
	 * may be placed after all maps associated with known assets. Global shadermaps need to be associated with a "Global" asset */
	void AssociateWithAsset(const FName& AssetPath)
	{
		AssociatedAssets.Add(AssetPath);
	}

	void AssociateWithAssets(const FShaderMapAssetPaths& AssetPaths)
	{
		AssociatedAssets.Append(AssetPaths);
	}

	const FShaderMapAssetPaths& GetAssociatedAssets() const
	{
		return AssociatedAssets;
	}
#endif // WITH_EDITOR

protected:
	RENDERCORE_API FShaderMapBase();

	RENDERCORE_API void AssignCopy(const FShaderMapBase& Source);

	RENDERCORE_API void InitResource();
	RENDERCORE_API void DestroyContent();

protected:
	virtual const FTypeLayoutDesc& GetContentTypeDesc() const = 0;
	virtual FShaderMapPointerTable* CreatePointerTable() const = 0;
	virtual void PostFinalizeContent() { }

private:
#if WITH_EDITOR
	/** List of the assets that are using this shadermap. This is only available in the editor (cooker) to influence ordering of shader libraries. */
	FShaderMapAssetPaths AssociatedAssets;
#endif
	TRefCountPtr<FShaderMapResource> Resource;
	TRefCountPtr<FShaderMapResourceCode> Code;
	FShaderMapPointerTable* PointerTable;
	TMemoryImageObject<FShaderMapContent> Content;
	uint32 NumFrozenShaders;
	EShaderPermutationFlags PermutationFlags;
};

template<typename ContentType, typename PointerTableType = FShaderMapPointerTable>
class TShaderMap : public FShaderMapBase
{
public:
	inline const PointerTableType& GetPointerTable() const { return static_cast<const PointerTableType&>(FShaderMapBase::GetPointerTable()); }
	inline const ContentType* GetContent() const { return static_cast<const ContentType*>(FShaderMapBase::GetContent()); }
	inline ContentType* GetMutableContent() { return static_cast<ContentType*>(FShaderMapBase::GetMutableContent()); }

	void FinalizeContent()
	{
		ContentType* LocalContent = this->GetMutableContent();
		check(LocalContent);
		LocalContent->Finalize(this->GetResourceCode());
		LocalContent->Validate(*this);
		FShaderMapBase::FinalizeContent();
	}

protected:
	virtual const FTypeLayoutDesc& GetContentTypeDesc() const final override { return StaticGetTypeLayoutDesc<ContentType>(); }
	virtual FShaderMapPointerTable* CreatePointerTable() const final override { return new PointerTableType(); }
};

template<typename ShaderType, typename PointerTableType>
inline const PointerTableType& TShaderRefBase<ShaderType, PointerTableType>::GetPointerTable() const
{
	checkSlow(ShaderMap);
	return static_cast<const PointerTableType&>(ShaderMap->GetPointerTable());
}

template<typename ShaderType, typename PointerTableType>
inline FShaderMapResource* TShaderRefBase<ShaderType, PointerTableType>::GetResource() const
{
	checkSlow(ShaderMap);
	return ShaderMap->GetResource();
}

inline const FShaderMapPointerTable& FShaderPipelineRef::GetPointerTable() const
{
	checkSlow(ShaderMap);
	return ShaderMap->GetPointerTable();
}

inline FShaderMapResource* FShaderPipelineRef::GetResource() const
{
	checkSlow(ShaderMap);
	return ShaderMap->GetResource();
}

/** A reference which is initialized with the requested shader type from a shader map. */
template<typename ShaderType>
class TShaderMapRef : public TShaderRef<ShaderType>
{
public:
	TShaderMapRef(const typename ShaderType::ShaderMapType* ShaderIndex)
		: TShaderRef<ShaderType>(ShaderIndex->template GetShader<ShaderType>(/* PermutationId = */ 0)) // gcc3 needs the template quantifier so it knows the < is not a less-than
	{
		static_assert(
			std::is_same_v<typename ShaderType::FPermutationDomain, FShaderPermutationNone>,
			"Missing permutation vector argument for shader that have a permutation domain.");
	}

	TShaderMapRef(
		const typename ShaderType::ShaderMapType* ShaderIndex,
		const typename ShaderType::FPermutationDomain& PermutationVector)
		: TShaderRef<ShaderType>(ShaderIndex->template GetShader<ShaderType>(PermutationVector.ToDimensionValueId())) // gcc3 needs the template quantifier so it knows the < is not a less-than
	{
		checkf(TShaderRef<ShaderType>::SpecConsts.Num() == ShaderType::FPermutationDomain::SpecializationCount, TEXT("Shader returned without specialization constant values."));
	}
};

/** A reference to an optional shader, initialized with a shader type from a shader map if it is available or nullptr if it is not. */
template<typename ShaderType>
class TOptionalShaderMapRef : public TShaderRef<ShaderType>
{
public:
	TOptionalShaderMapRef(const typename ShaderType::ShaderMapType* ShaderIndex):
		TShaderRef<ShaderType>(TShaderRef<ShaderType>::Cast(ShaderIndex->GetShader(&ShaderType::GetStaticType()))) // gcc3 needs the template quantifier so it knows the < is not a less-than
	{}

	TOptionalShaderMapRef(
		const typename ShaderType::ShaderMapType* ShaderIndex,
		const typename ShaderType::FPermutationDomain& PermutationVector)
		: TShaderRef<ShaderType>(ShaderIndex->template GetShader<ShaderType>(PermutationVector.ToDimensionValueId())) // gcc3 needs the template quantifier so it knows the < is not a less-than
	{
	}
};

/**
 * Dumps shader stats to the log. Will also print some shader pipeline information.
 * @param Platform  - Platform to dump shader info for, use SP_NumPlatforms for all
 * @param Frequency - Whether to dump PS or VS info, use SF_NumFrequencies to dump both
 */
extern RENDERCORE_API void DumpShaderStats( EShaderPlatform Platform, EShaderFrequency Frequency );

/**
 * Dumps shader pipeline stats to the log. Does not include material (eg shader pipeline instance) information.
 * @param Platform  - Platform to dump shader info for, use SP_NumPlatforms for all
 */
extern RENDERCORE_API void DumpShaderPipelineStats(EShaderPlatform Platform);

/**
 * Finds the shader type with a given name.
 * @param ShaderTypeName - The name of the shader type to find.
 * @return The shader type, or NULL if none matched.
 */
extern RENDERCORE_API FShaderType* FindShaderTypeByName(const FHashedName& ShaderTypeName);

/** Helper function to dispatch a compute shader while checking that parameters have been set correctly. */
extern RENDERCORE_API void DispatchComputeShader(
	FRHIComputeCommandList& RHICmdList,
	FShader* Shader,
	uint32 ThreadGroupCountX,
	uint32 ThreadGroupCountY,
	uint32 ThreadGroupCountZ);

/** Helper function to dispatch a compute shader indirectly while checking that parameters have been set correctly. */
extern RENDERCORE_API void DispatchIndirectComputeShader(
	FRHIComputeCommandList& RHICmdList,
	FShader* Shader,
	FRHIBuffer* ArgumentBuffer,
	uint32 ArgumentOffset);

inline void DispatchComputeShader(
	FRHIComputeCommandList& RHICmdList,
	const TShaderRef<FShader>& Shader,
	uint32 ThreadGroupCountX,
	uint32 ThreadGroupCountY,
	uint32 ThreadGroupCountZ)
{
	DispatchComputeShader(RHICmdList, Shader.GetShader(), ThreadGroupCountX, ThreadGroupCountY, ThreadGroupCountZ);
}

/** Returns whether the platform is using emulated uniform buffers */
extern RENDERCORE_API bool IsUsingEmulatedUniformBuffers(EShaderPlatform Platform);

/** Returns whether DirectXShaderCompiler (DXC) is enabled for the specified shader platform. See console variables "r.OpenGL.ForceDXC", "r.D3D.ForceDXC". */
extern RENDERCORE_API bool IsDxcEnabledForPlatform(EShaderPlatform Platform, bool bHlslVersion2021 = false);

/** Appends to KeyString for all shaders. */
extern RENDERCORE_API void ShaderMapAppendKeyString(EShaderPlatform Platform, FString& KeyString);
extern RENDERCORE_API void ShaderMapAppendKey(EShaderPlatform Platform, FShaderKeyGenerator& KeyGen);
