// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GlobalShader.h"

/** Global shader to fill a shader bundle. */
class FDispatchShaderBundleCS : public FGlobalShader
{
	DECLARE_EXPORTED_GLOBAL_SHADER(FDispatchShaderBundleCS, RENDERCORE_API);

	// TODO: Dependency issues
	// class FBundleMode : SHADER_PERMUTATION_ENUM_CLASS("BUNDLE_MODE", ERHIShaderBundleMode);
	class FBundleMode : SHADER_PERMUTATION_INT("BUNDLE_MODE", 3);
	using FPermutationDomain = TShaderPermutationDomain<FBundleMode>;

public:

	FDispatchShaderBundleCS() = default;
	FDispatchShaderBundleCS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FGlobalShader(Initializer)
	{
		RecordArgBufferParam.Bind(Initializer.ParameterMap, TEXT("RecordArgBuffer"), SPF_Optional);
		RecordDataBufferParam.Bind(Initializer.ParameterMap, TEXT("RecordDataBuffer"), SPF_Optional);
		RWExecutionBufferParam.Bind(Initializer.ParameterMap, TEXT("RWExecutionBuffer"), SPF_Optional);
	}

	static const uint32 ThreadGroupSizeX = 64;

	LAYOUT_FIELD(FShaderResourceParameter, RecordArgBufferParam);
	LAYOUT_FIELD(FShaderResourceParameter, RecordDataBufferParam);
	LAYOUT_FIELD(FShaderResourceParameter, RWExecutionBufferParam);

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters);
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment);
};


/** Global work graph shader used to dispatch a shader bundle. */
class FDispatchShaderBundleWorkGraph : public FGlobalShader
{
	DECLARE_EXPORTED_GLOBAL_SHADER(FDispatchShaderBundleWorkGraph, RENDERCORE_API);

	FDispatchShaderBundleWorkGraph() = default;
	FDispatchShaderBundleWorkGraph(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FGlobalShader(Initializer)
	{
		RecordArgBufferParam.Bind(Initializer.ParameterMap, TEXT("RecordArgBuffer"), SPF_Optional);
	}

	static const uint32 ThreadGroupSizeX = 64;

	LAYOUT_FIELD(FShaderResourceParameter, RecordArgBufferParam);

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters);
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment);

	RENDERCORE_API static int32 GetMaxShaderBundleSize();

	// Input record structure should match the one declared in the shader.
	struct FEntryNodeRecord
	{
		uint32 DispatchGridSize;
		uint32 RecordCount;
		FUintVector PlatformData;
	};
	RENDERCORE_API static FEntryNodeRecord MakeInputRecord(uint32 RecordCount, uint32 ArgOffset, uint32 ArgStride, uint32 ArgsBindlessHandle);
};
