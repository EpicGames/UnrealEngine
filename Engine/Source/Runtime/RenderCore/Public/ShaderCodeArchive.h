// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/TaskGraphInterfaces.h"
#include "Containers/Array.h"
#include "Containers/HashTable.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "FileCache/FileCache.h"
#include "HAL/CriticalSection.h"
#include "HAL/Platform.h"
#include "IO/IoDispatcher.h"
#include "Misc/AssertionMacros.h"
#include "Misc/CoreDelegates.h"
// USE_MMAPPED_SHADERARCHIVE is defined in Misc/CoreDelegates.h
#if USE_MMAPPED_SHADERARCHIVE
#include "Async/MappedFileHandle.h"
#endif
#include "Misc/MemoryReadStream.h"
#include "Misc/SecureHash.h"
#include "RHI.h"
#include "RHIDefinitions.h"
#include "Serialization/Archive.h"
#include "Shader.h"
#include "ShaderCodeLibrary.h"
#include "ShaderMapAssetAssociation.h"
#include "Templates/RefCounting.h"
#include "UObject/NameTypes.h"

#include <type_traits>

#if WITH_EDITOR
class FCbFieldView;
class FCbWriter;
#endif

struct FShaderHash;

// enable visualization in the desktop Development builds only as it has a memory hit and writes files
#define UE_SCA_VISUALIZE_SHADER_USAGE			(!WITH_EDITOR && !IS_PROGRAM && UE_BUILD_DEVELOPMENT && PLATFORM_DESKTOP)

// enable deep, manual debugging of leaked preload groups. This level of information slows the engine down and is only needed when chasing tricky bugs
#define UE_SCA_DEBUG_PRELOADING					(0)

struct FShaderMapEntry
{
	uint32 ShaderIndicesOffset = 0u;
	uint32 NumShaders = 0u;
	uint32 FirstPreloadIndex = 0u;
	uint32 NumPreloadEntries = 0u;

	friend FArchive& operator <<(FArchive& Ar, FShaderMapEntry& Ref)
	{
		return Ar << Ref.ShaderIndicesOffset << Ref.NumShaders << Ref.FirstPreloadIndex << Ref.NumPreloadEntries;
	}
};

static FArchive& operator <<(FArchive& Ar, FFileCachePreloadEntry& Ref)
{
	return Ar << Ref.Offset << Ref.Size;
}


#if USE_MMAPPED_SHADERARCHIVE
#pragma pack(push, 1)
#endif
struct FShaderCodeEntry
{
	// If the corresponding shader is part of the library,
	// it is stored uncompressed and assigned an index within
	// the library.
	static constexpr uint8  HasLibraryIndexTag   = 1u << SF_NumBits;
	static constexpr uint32 FrequencyPayloadMask = HasLibraryIndexTag - 1u;

	uint64 Offset = 0;
	uint32 Size   = 0;
	
	// The active member is specified by the tag stored
	// in the TagAndFrequency field. Rely on access methods.
	struct
	{
		union
		{
			uint32 UncompressedSize;
			uint32 LibraryIndex;
		};
	} Payload {};	
	uint8 TagAndFrequency = 0;
	
	EShaderFrequency GetFrequency() const
	{
		return (EShaderFrequency)(TagAndFrequency & FrequencyPayloadMask);
	}
	
	void SetFrequency(EShaderFrequency Frequency)
	{
		check(Frequency <= FrequencyPayloadMask);
		TagAndFrequency = (TagAndFrequency & HasLibraryIndexTag) | Frequency;
	}
	
	uint32 GetUncompressedSize() const
	{
		return IsPartOfLibrary() ? Size : Payload.UncompressedSize;
	}
	
	void SetUncompressedSize(uint32 InUncompressedSize)
	{
		check(!IsPartOfLibrary());
		Payload.UncompressedSize = InUncompressedSize;
	}
	
	int32 GetLibraryIndex() const
	{
		return IsPartOfLibrary() ? Payload.LibraryIndex : INDEX_NONE;
	}
	
	void SetLibraryIndex(int32 Index)
	{
		check(Index >= 0);
		Payload.LibraryIndex  = Index;
		TagAndFrequency      |= HasLibraryIndexTag;
	}
	
	bool IsCompressed() const
	{
		return !IsPartOfLibrary() && Size != GetUncompressedSize();
	}
	
	bool IsPartOfLibrary() const
	{
		return (TagAndFrequency & HasLibraryIndexTag) != 0;
	}
	
	friend FArchive& operator <<(FArchive& Ar, FShaderCodeEntry& Ref)
	{
		return Ar << Ref.Offset << Ref.Size << (Ref.IsPartOfLibrary() ? Ref.Payload.LibraryIndex : Ref.Payload.UncompressedSize) << Ref.TagAndFrequency;
	}
};
#if USE_MMAPPED_SHADERARCHIVE
#pragma pack(pop)
#endif
static_assert(std::is_standard_layout_v<FShaderCodeEntry>);
static_assert(std::is_trivially_copyable_v<FShaderCodeEntry>);

/**
 * Stores information about a shader library name.
 * This splits the filename into names for chunk, shader format, and platform format.
 * For example, "Global-SF_VULKAN_SM6-VULKAN_SM6" is split into chunk "Global", shader format "SF_VULKAN_SM6", and platform format "VULKAN_SM6".
 * Note that some platforms only have a single string for the format-platform combination, in which case ShaderFormatName can be empty.
 */
struct FShaderLibraryNameInfo
{
	// Shader library chunk name, e.g. "Global", "Lyra_Chunk0", "MyPlugin" etc.
	FString ChunkName;

	// Shader format nane, e.g, "SF_VULKAN_SM6".
	FName ShaderFormatName;

	// Shader platform format name, e.g. "VULKAN_SM6".
	FName ShaderPlatformName;

	/** Initializes all name sections from the specified filename (usually a *.ushaderbytecode file). */
	RENDERCORE_API bool ParseFromFilename(const TCHAR* Filename);

	/** Returns the shader asset info filename (*.assetinfo.json) that belongs to the respective *.ushaderbytecode file. */
	RENDERCORE_API FString GetAssetInfoFilename() const;

	/** Returns the shader type info filename (*.stinfo) that belongs to the respective *.ushaderbytecode file. */
	RENDERCORE_API FString GetTypeInfoFilename() const;

	/** Returns the name for this shader library, which is a combination of chunk and format names, e.g. "Global-SF_VULKAN_SM6". Used for debugging. */
	FORCEINLINE FString GetLibraryName() const
	{
		return FString::Printf(TEXT("%s-%s"), *ChunkName, *ShaderFormatName.ToString());
	}

	/** Returns the combined format and platform names. If shader format name is empty, only the platform name is returned. Used for FEditorShaderCodeArchive::FormatAndPlatformName. */
	FORCEINLINE FString GetFormatAndPlatformName() const
	{
		return ShaderFormatName.IsNone()
			? ShaderPlatformName.ToString()
			: FString::Printf(TEXT("%s-%s"), *ShaderFormatName.ToString(), *ShaderPlatformName.ToString());
	}

	/** Returns the full name with chunk, shader format, and platform format names. */
	FORCEINLINE FString GetFullName() const
	{
		return FString::Printf(TEXT("%s-%s"), *ChunkName, *GetFormatAndPlatformName());
	}

	/** Helper function to parse the ID and optional suffix from a pakchunk name, e.g. "pakchunk100iad-Windows" -> 100 and "iad" */
	RENDERCORE_API static bool ParsePakChunkId(const FStringView& PakChunkName, int32& OutChunkId, FString& OutChunkIdSuffix);
};

/**
 * Header portion of shader code archive that's serialized to disk.
 * 
 * The shader codes themselves are stored separately, either with individual buffers (one for each shader group),
 * composite buffers, or a single consecutive buffer (those can become quite large).
 * 
 * @see FEditorShaderCodeArchive, FSerializedShaderArchiveBuffer
 */
class FSerializedShaderArchive
{
public:
#if WITH_EDITORONLY_DATA
	using ECookShaderLibrarySource = ShaderCodeArchive::ECookShaderLibrarySource;
#endif
#if USE_MMAPPED_SHADERARCHIVE
	template<typename T> using TArrayType = TArrayView<T>;
	// Ensure use of the const Get functions when mmapping is used.
	private:
#else
	template<typename T> using TArrayType = TArray<T>;
#endif

	/** Hashes of all shadermaps in the library */
	TArrayType<FShaderHash> ShaderMapHashes;

	/** Output hashes of all shaders in the library */
	TArrayType<FShaderHash> ShaderHashes;

	/** An array of a shadermap descriptors. Each shadermap can reference an arbitrary number of shaders */
	TArrayType<FShaderMapEntry> ShaderMapEntries;

	/** An array of all shaders descriptors, deduplicated */
	TArrayType<FShaderCodeEntry> ShaderEntries;
	/** An array of entries for the bytes of shadercode that need to be preloaded for a shadermap.
	  * Each shadermap has a range in this array, beginning of which is stored in FShaderMapEntry.FirstPreloadIndex. */
	TArrayType<FFileCachePreloadEntry> PreloadEntries;

	/** Flat array of shaders referenced by all shadermaps. Each shadermap has a range in this array, beginning of which is
	  * stored as ShaderIndicesOffset in the shadermap's descriptor (FShaderMapEntry).
	  */
	TArrayType<uint32> ShaderIndices;

public:
	const TArrayView<const uint32> GetShaderIndices() const { return ShaderIndices; }
	const TArrayView<const FFileCachePreloadEntry> GetPreloadEntries() const { return PreloadEntries; }
	const TArrayView<const FShaderCodeEntry> GetShaderEntries() const { return ShaderEntries; }
	const TArrayView<const FShaderMapEntry> GetShaderMapEntries() const { return ShaderMapEntries; }
	const TArrayView<const FShaderHash> GetShaderHashes() const { return ShaderHashes; }
	const TArrayView<const FShaderHash> GetShaderMapHashes() const { return ShaderMapHashes; }

	FHashTable ShaderMapHashTable;
	FHashTable ShaderHashTable;

#if WITH_EDITORONLY_DATA

private:
	FShaderMapAssetAssociations ShaderMapAssetAssociations;

public:
	FORCEINLINE FShaderMapAssetAssociations& GetShaderMapAssetAssociations()
	{
		return ShaderMapAssetAssociations;
	}
	FORCEINLINE const FShaderMapAssetAssociations& GetShaderMapAssetAssociations() const
	{
		return ShaderMapAssetAssociations;
	}

	UE_DEPRECATED(5.8, "No longer referenced. Use GetShaderMapAssetAssociations() instead.")
	TMap<FSHAHash, FShaderMapAssetPaths> ShaderCodeToAssets;

	struct FShaderTypeHashes
	{
		/** Most of the time, this will only contain one element. But some shaders are shared across two or more shader types. */
		TArray<uint64> Data;

		friend FArchive& operator<<(FArchive& Ar, FShaderTypeHashes& Ref)
		{
			Ar << Ref.Data;
			return Ar;
		}
	};
	
	/** Array of all shader type hashes which deduplicated to each individual shader; indexed as ShaderEntries array */
	TArray<FShaderTypeHashes> ShaderTypes;

	enum class EAssetInfoVersion : uint8
	{
		CurrentVersion = 2
	};

	struct FDebugStats
	{
		int32 NumAssets;
		int64 ShadersSize;
		int64 ShadersUniqueSize;
		int32 NumShaders;
		int32 NumUniqueShaders;
		int32 NumShaderMaps;
	};

	struct FExtendedDebugStats
	{
		/** Textual contents, should match the binary layout in terms of order */
		FString TextualRepresentation;

		/** Minimum number of shaders in any given shadermap */
		uint32 MinNumberOfShadersPerSM;

		/** Median number of shaders in shadermaps */
		uint32 MedianNumberOfShadersPerSM;

		/** Maximum number of shaders in any given shadermap */
		uint32 MaxNumberofShadersPerSM;

		/** For the top shaders (descending), number of shadermaps in which they are used. Expected to be limited to a small number (10) */
		TArray<int32> TopShaderUsages;

		/** Number of shaers per frequency. */
		int32 NumShadersPerFrequency[SF_NumFrequencies];

		/** Uncompressed size of all shaders of a given frequency. */
		uint64 UncompressedSizePerFrequency[SF_NumFrequencies];

		/** Compressed (individually) size of all shaders of a given frequency. */
		uint64 CompressedSizePerFrequency[SF_NumFrequencies];
	};
#endif // WITH_EDITORONLY_DATA

	FSerializedShaderArchive()
	{
	}
	FSerializedShaderArchive(FSerializedShaderArchive&&);
	FSerializedShaderArchive(const FSerializedShaderArchive&);
	FSerializedShaderArchive& operator=(FSerializedShaderArchive&&);
	FSerializedShaderArchive& operator=(const FSerializedShaderArchive&);

	int64 GetAllocatedSize() const
	{
#if USE_MMAPPED_SHADERARCHIVE
		return ShaderHashes.Num() * ShaderHashes.GetTypeSize() +
			ShaderEntries.Num() * ShaderEntries.GetTypeSize() +
			ShaderMapHashes.Num() * ShaderMapHashes.GetTypeSize() +
			ShaderMapEntries.Num() * ShaderMapEntries.GetTypeSize() +
			PreloadEntries.Num() * PreloadEntries.GetTypeSize() +
			ShaderIndices.Num() * ShaderIndices.GetTypeSize()
#else
		return ShaderHashes.GetAllocatedSize() +
			ShaderEntries.GetAllocatedSize() +
			ShaderMapHashes.GetAllocatedSize() +
			ShaderMapEntries.GetAllocatedSize() +
			PreloadEntries.GetAllocatedSize() +
			ShaderIndices.GetAllocatedSize()
#endif
#if WITH_EDITORONLY_DATA
			+ ShaderMapAssetAssociations.GetAllocatedSize()
			+ ShaderTypes.GetAllocatedSize()
#endif // WITH_EDITORONLY_DATA
			;
	}

	void Empty()
	{
		EmptyShaderMaps();
#if !USE_MMAPPED_SHADERARCHIVE
		ShaderHashes.Empty();
		ShaderEntries.Empty();
#endif
		ShaderHashTable.Clear();
#if WITH_EDITORONLY_DATA
		ShaderTypes.Empty();
#endif
	}

	void EmptyShaderMaps()
	{
#if !USE_MMAPPED_SHADERARCHIVE
		ShaderMapHashes.Empty();
		ShaderMapEntries.Empty();
		PreloadEntries.Empty();
		ShaderIndices.Empty();
#endif
		ShaderMapHashTable.Clear();
#if WITH_EDITORONLY_DATA
		ShaderMapAssetAssociations.Empty();
#endif
	}

	int32 GetNumShaderMaps() const
	{
		return ShaderMapEntries.Num();
	}

	int32 GetNumShaders() const
	{
		return ShaderEntries.Num();
	}

	bool IsEmpty() const
	{
		return ShaderMapEntries.IsEmpty() && ShaderEntries.IsEmpty() && PreloadEntries.IsEmpty()
#if WITH_EDITORONLY_DATA
			&& ShaderMapAssetAssociations.IsEmpty()
			&& ShaderTypes.IsEmpty()
#endif
			;
	}

	RENDERCORE_API int32 FindShaderMapWithKey(const FShaderHash& Hash, uint32 Key) const;
	RENDERCORE_API int32 FindShaderMap(const FShaderHash& Hash) const;
	RENDERCORE_API int32 FindShaderWithKey(const FShaderHash& Hash, uint32 Key) const;
	RENDERCORE_API int32 FindShader(const FShaderHash& Hash) const;

#if !USE_MMAPPED_SHADERARCHIVE
	RENDERCORE_API bool FindOrAddShader(const FShaderHash& Hash, int32& OutIndex);
	UE_DEPRECATED(5.8, "Call FindOrAddShaderMapNonEditor or FindOrAddShaderMapEditor.")
	RENDERCORE_API bool FindOrAddShaderMap(const FShaderHash& Hash, int32& OutIndex, const FShaderMapAssetPaths* AssociatedAssets);
#if WITH_EDITORONLY_DATA
	RENDERCORE_API bool FindOrAddShaderMapEditor(const FShaderHash& Hash, int32& OutIndex,
		const FShaderMapAssetPaths* AssociatedAssets, ECookShaderLibrarySource CookSource);
#else
	RENDERCORE_API bool FindOrAddShaderMapNonEditor(const FShaderHash& Hash, int32& OutIndex);
#endif
	RENDERCORE_API void RemoveLastAddedShader();
	RENDERCORE_API void Finalize();
#endif // !USE_MMAPPED_SHADERARCHIVE

#if WITH_EDITORONLY_DATA
	/** Sorts Shaders and ShaderMaps by their Hash values. Used to make cooked data deterministic. */
	RENDERCORE_API void SortByHashes(TArray<FSharedBuffer>& InOutShaderCodeForShaderIndex);
#endif

	RENDERCORE_API void DecompressShader(int32 Index, const TArray<FSharedBuffer>& ShaderCode, TArray<uint8>& OutDecompressedShader) const;
	RENDERCORE_API static void DecompressShaderEntryAndAppend(const FShaderCodeEntry& Entry, const uint8* CompressedShaderCode, TArray<uint8>& OutDecompressedShader);

	RENDERCORE_API static FString MakeDebugDirectory(const FString LibraryName = TEXT(""), const FName& FormatAndPlatformName = NAME_None, const TCHAR* BasePath = nullptr);
	
	RENDERCORE_API static bool SerializeHeaderVersion(FArchive& Ar);
	RENDERCORE_API void Serialize(FArchive& Ar);
#if WITH_EDITORONLY_DATA
	RENDERCORE_API void SaveAssetInfo(FArchive& Ar);
	RENDERCORE_API bool LoadAssetInfo(const FString& Filename, ECookShaderLibrarySource CookSource = ECookShaderLibrarySource::CurrentCook);
	RENDERCORE_API bool LoadAssetInfo(FArchive* Ar, ECookShaderLibrarySource CookSource = ECookShaderLibrarySource::CurrentCook);
	UE_DEPRECATED(5.8, "Use CreateFromFilteredAssets instead")
	RENDERCORE_API void CreateAsChunkFrom(const FSerializedShaderArchive& Parent, const TSet<FName>& PackagesInChunk, TArray<int32>& OutShaderCodeEntriesNeeded);

	RENDERCORE_API void CreateFromFilteredAssets(const FSerializedShaderArchive& Parent, TArray<int32>& OutShaderCodeEntriesNeeded,
		FShaderMapAssetAssociations::FAssetFilterFunctionRef ShouldKeepAsset,
		FShaderMapAssetAssociations::FShaderMapFilterFunction ShouldExcludeShaderMap = nullptr);
	RENDERCORE_API void CollectStatsAndDebugInfo(FDebugStats& OutDebugStats, FExtendedDebugStats* OutExtendedDebugStats);
	RENDERCORE_API void DumpContentsInPlaintext(FString& OutText) const;
	RENDERCORE_API friend FCbWriter& operator<<(FCbWriter& Writer, const FSerializedShaderArchive& Archive);
	RENDERCORE_API friend bool LoadFromCompactBinary(FCbFieldView Field, FSerializedShaderArchive& OutArchive);
#endif

	friend FArchive& operator<<(FArchive& Ar, FSerializedShaderArchive& Ref)
	{
		Ref.Serialize(Ar);
		return Ar;
	}

private:
#if !USE_MMAPPED_SHADERARCHIVE
	bool FindOrAddShaderMapNonEditorInternal(const FShaderHash& Hash, int32& OutIndex);
#endif
};

// run-time only debugging facility
struct FShaderUsageVisualizer
{
#if UE_SCA_VISUALIZE_SHADER_USAGE
	/** Lock guarding access to visualization structures. */
	FCriticalSection VisualizeLock;

	/** Total number of shaders. */
	int32 NumShaders;

	/** Shader indices that we explicitly preloaded (does not include shaders preloaded as part of a compressed group). */
	TSet<int32> ExplicitlyPreloadedShaders;

	/** Shader indices that we preloaded (either explicitly or because they were a part of a compressed group). */
	TSet<int32> PreloadedShaders;

	/** Shader indices that we decompressed. */
	TSet<int32> DecompressedShaders;

	/** Shader indices that were created. */
	TSet<int32> CreatedShaders;

	void Initialize(const int32 InNumShaders);

	inline void MarkExplicitlyPreloadedForVisualization(int32 ShaderIndex)
	{
		extern int32 GShaderCodeLibraryVisualizeShaderUsage;
		if (LIKELY(GShaderCodeLibraryVisualizeShaderUsage))
		{
			FScopeLock Lock(&VisualizeLock);
			ExplicitlyPreloadedShaders.Add(ShaderIndex);
		}
	}

	inline void MarkPreloadedForVisualization(int32 ShaderIndex)
	{
		extern int32 GShaderCodeLibraryVisualizeShaderUsage;
		if (LIKELY(GShaderCodeLibraryVisualizeShaderUsage))
		{
			FScopeLock Lock(&VisualizeLock);
			PreloadedShaders.Add(ShaderIndex);
		}
	}

	inline void MarkDecompressedForVisualization(int32 ShaderIndex)
	{
		extern int32 GShaderCodeLibraryVisualizeShaderUsage;
		if (LIKELY(GShaderCodeLibraryVisualizeShaderUsage))
		{
			FScopeLock Lock(&VisualizeLock);
			DecompressedShaders.Add(ShaderIndex);
		}
	}

	inline void MarkCreatedForVisualization(int32 ShaderIndex)
	{
		extern int32 GShaderCodeLibraryVisualizeShaderUsage;
		if (LIKELY(GShaderCodeLibraryVisualizeShaderUsage))
		{
			FScopeLock Lock(&VisualizeLock);
			CreatedShaders.Add(ShaderIndex);
		}
	}

	void SaveShaderUsageBitmap(const FString& Name, EShaderPlatform ShaderPlatform);
#else
	inline void Initialize(const int32 InNumShaders) {}
	inline void MarkPreloadedForVisualization(int32 ShaderIndex) {}
	inline void MarkExplicitlyPreloadedForVisualization(int32 ShaderIndex) {}
	inline void MarkDecompressedForVisualization(int32 ShaderIndex) {}
	inline void MarkCreatedForVisualization(int32 ShaderIndex) {}
	inline void SaveShaderUsageBitmap(const FString& Name, EShaderPlatform ShaderPlatform) {}
#endif // UE_SCA_VISUALIZE_SHADER_USAGE
};

/**
 * Implements FRHIShaderLibrary and reads shader code from a single file cache handle.
 * @todo-lh - Consider renaming to FFileCacheRHIShaderLibrary
 */
class FShaderCodeArchive : public FRHIShaderLibrary
{
public:
	static FShaderCodeArchive* Create(EShaderPlatform InPlatform, FArchive& Ar, const FString& InDestFilePath, const FString& InLibraryDir, const FString& InLibraryName);

	virtual ~FShaderCodeArchive();

	virtual bool IsNativeLibrary() const override { return false; }

	virtual uint32 GetSizeBytes() const override
	{
		return sizeof(*this) +
			SerializedShaders.GetAllocatedSize() +
			ShaderPreloads.GetAllocatedSize();
	}

	virtual int32 GetNumShaders() const override { return SerializedShaders.GetShaderEntries().Num(); }
	virtual int32 GetNumShaderMaps() const override { return SerializedShaders.GetShaderMapEntries().Num(); }
	virtual int32 GetNumShadersForShaderMap(int32 ShaderMapIndex) const override { return SerializedShaders.GetShaderMapEntries()[ShaderMapIndex].NumShaders; }

	virtual int32 GetShaderIndex(int32 ShaderMapIndex, int32 i) const override
	{
		const FShaderMapEntry& ShaderMapEntry = SerializedShaders.GetShaderMapEntries()[ShaderMapIndex];
		return SerializedShaders.GetShaderIndices()[ShaderMapEntry.ShaderIndicesOffset + i];
	}

	virtual void GetAllShaderIndices(int32 ShaderMapIndex, TArray<int32>& OutShaderIndices) const override
	{
		const FShaderMapEntry& ShaderMapEntry = SerializedShaders.GetShaderMapEntries()[ShaderMapIndex];
		for (uint32 i = 0u; i < ShaderMapEntry.NumShaders; ++i)
		{
			OutShaderIndices.AddUnique(SerializedShaders.GetShaderIndices()[ShaderMapEntry.ShaderIndicesOffset + i]);
		}
	}

	virtual int32 FindShaderMapIndex(const FShaderHash& Hash) override
	{
		return SerializedShaders.FindShaderMap(Hash);
	}

	virtual int32 FindShaderIndex(const FShaderHash& Hash) override
	{
		return SerializedShaders.FindShader(Hash);
	}

	virtual FShaderHash GetShaderHash(int32 ShaderMapIndex, int32 ShaderIndex) override
	{
		return SerializedShaders.GetShaderHashes()[GetShaderIndex(ShaderMapIndex, ShaderIndex)];
	};

	virtual FShaderHash GetShaderMapHash(int32 ShaderMapIndex) const override
	{
		return SerializedShaders.GetShaderMapHashes()[ShaderMapIndex];
	};

	virtual bool PreloadShader(int32 ShaderIndex, FGraphEventArray& OutCompletionEvents) override;
	virtual bool PreloadShaderMap(int32 ShaderMapIndex, FGraphEventArray& OutCompletionEvents) override;
	virtual void ReleasePreloadedShader(int32 ShaderIndex) override;

	virtual TRefCountPtr<FRHIShader> CreateShader(int32 Index, bool bRequired = true) override;
	virtual void Teardown() override;

	void OnShaderPreloadFinished(int32 ShaderIndex, const IMemoryReadStreamRef& PreloadData);

protected:
	FShaderCodeArchive(EShaderPlatform InPlatform, const FString& InLibraryDir, const FString& InLibraryName);

	FORCENOINLINE void CheckShaderCreation(void* ShaderPtr, int32 Index)
	{
	}

	struct FShaderPreloadEntry
	{
		FGraphEventRef PreloadEvent;
		void* Code = nullptr;
		uint32 FramePreloadStarted = ~0u;
		uint32 NumRefs : 31;
		uint32 bNeverToBePreloaded : 1;

		FShaderPreloadEntry()
			: NumRefs(0)
			, bNeverToBePreloaded(0)
		{
		}
	};

	bool WaitForPreload(FShaderPreloadEntry& ShaderPreloadEntry);

	// Library directory
	FString LibraryDir;

	// Offset at where shader code starts in a code library
	int64 LibraryCodeOffset;

	// Library file handle for async reads
	IFileCacheHandle* FileCacheHandle;

	// The shader code present in the library
	FSerializedShaderArchive SerializedShaders;

	TArray<FShaderPreloadEntry> ShaderPreloads;
	FRWLock ShaderPreloadLock;

	/** debug visualizer - in Shipping compiles out to an empty struct with no-op functions */
	FShaderUsageVisualizer DebugVisualizer;
};



namespace ShaderCodeArchive
{
	/** Decompresses the shader into caller-provided memory. Caller is assumed to allocate at least ShaderEntry uncompressed size value.
	 * The engine will crash (LogFatal) if this function fails.
	 */
	RENDERCORE_API void DecompressShaderWithOodle(uint8* OutDecompressedShader, int64 UncompressedSize, const uint8* CompressedShaderCode, int64 CompressedSize);
	
	// We decompression, group, and recompress shaders when they are added to iostore containers in UnrealPak, where we don't have access to cvars - so there's no way 
	// to configure - so we force Oodle and allow specification of the parameters here.
	RENDERCORE_API bool CompressShaderWithOodle(uint8* OutCompressedShader, int64& OutCompressedSize, const uint8* InUncompressedShaderCode, int64 InUncompressedSize, FOodleDataCompression::ECompressor InOodleCompressor, FOodleDataCompression::ECompressionLevel InOodleLevel);
}

/** Descriptor of a shader map. This concept exists in run time, so this class describes the information stored in the library for a particular FShaderMap */
struct FIoStoreShaderMapEntry
{
	/** Offset to the first shader index referenced by this shader map in the array of shader indices. */
	uint32 ShaderIndicesOffset = 0u;
	/** Number of shaders in this shader map. */
	uint32 NumShaders = 0u;

	friend FArchive& operator <<(FArchive& Ar, FIoStoreShaderMapEntry& Ref)
	{
		return Ar << Ref.ShaderIndicesOffset << Ref.NumShaders;
	}
};

/** Descriptor of an individual shader. */
struct FIoStoreShaderCodeEntry
{
	union
	{
		uint64 Packed;
		struct
		{
			/** Shader type aka frequency (vertex, pixel, etc) */
			uint64 Frequency : SF_NumBits;	// 4 as of now

			/** Each shader belongs to a (one and only) shader group (even if it is the only one shader in that group) that is compressed and decompressed together. */
			uint64 ShaderGroupIndex : 30;

			/** Offset of the uncompressed shader in a group of shaders that are compressed / decompressed together. */
			uint64 UncompressedOffsetInGroup : 30;
		};
	};

	FIoStoreShaderCodeEntry()
		: Packed(0)
	{
	}

	EShaderFrequency GetFrequency() const
	{
		return (EShaderFrequency)Frequency;
	}

	friend FArchive& operator <<(FArchive& Ar, FIoStoreShaderCodeEntry& Ref)
	{
		return Ar << Ref.Packed;
	}
};

static_assert(sizeof(FIoStoreShaderCodeEntry) == sizeof(uint64), "To reduce memory footprint, shader code entries should be as small as possible");

/** Descriptor of a group of shaders compressed together. This groups already deduplicated, and possibly unrelated, shaders, so this is a distinct concept from a shader map. */
struct FIoStoreShaderGroupEntry
{
	/** Offset to the first shader index referenced by this group in the array of shader indices. This extra level of indirection allows arbitrary grouping. */
	uint32 ShaderIndicesOffset = 0u;
	/** Number of shaders in this group. */
	uint32 NumShaders = 0u;

	/** Uncompressed size of the whole group. */
	uint32 UncompressedSize = 0;
	/** Compressed size of the whole group. */
	uint32 CompressedSize = 0;

	friend FArchive& operator <<(FArchive& Ar, FIoStoreShaderGroupEntry& Ref)
	{
		return Ar << Ref.ShaderIndicesOffset << Ref.NumShaders << Ref.UncompressedSize << Ref.CompressedSize;
	}

	/** Some groups can be stored uncompressed if their compression wasn't beneficial (this is very vell possible, for groups that contain only one small shader. */
	inline bool IsGroupCompressed() const
	{
		return CompressedSize != UncompressedSize;
	}
};

struct FIoStoreShaderCodeArchiveHeader
{
private:
	template <typename TGroupEntry>
	void ForEachShaderInternal(const TArray<TGroupEntry>& InGroupEntries, int32 InEntryIndex, TFunctionRef<void(int32 ShaderIndex)> InCallback) const
	{
		const TGroupEntry& GroupEntry = InGroupEntries[InEntryIndex];
		for (uint32 IndirectShaderIndex = 0; IndirectShaderIndex < GroupEntry.NumShaders; ++IndirectShaderIndex)
		{
			const uint32 MappedShaderIndex = this->ShaderIndices[GroupEntry.ShaderIndicesOffset + IndirectShaderIndex];
			InCallback(static_cast<int32>(MappedShaderIndex));
		}
	}

public:

	/** Hashes of all shadermaps in the library */
	TArray<FShaderHash> ShaderMapHashes;

	/** Output hashes of all shaders in the library */
	TArray<FShaderHash> ShaderHashes;

	/** Chunk Ids (essentially hashes) of the shader groups - needed to be serialized as they are used for preloading. */
	TArray<FIoChunkId> ShaderGroupIoHashes;

	/** An array of a shadermap descriptors. Each shadermap can reference an arbitrary number of shaders */
	TArray<FIoStoreShaderMapEntry> ShaderMapEntries;

	/** An array of all shaders descriptors, deduplicated */
	TArray<FIoStoreShaderCodeEntry> ShaderEntries;

	/** An array of shader group descriptors */
	TArray<FIoStoreShaderGroupEntry> ShaderGroupEntries;

	/** Flat array of shaders referenced by all shadermaps. Each shadermap has a range in this array, beginning of which is
	  * stored as ShaderIndicesOffset in the shadermap's descriptor (FIoStoreShaderMapEntry).
	  * This is also used by the shader groups.
	  */
	TArray<uint32> ShaderIndices;

	friend RENDERCORE_API FArchive& operator <<(FArchive& Ar, FIoStoreShaderCodeArchiveHeader& Ref);

	inline uint64 GetShaderUncompressedSize(int ShaderIndex) const
	{
		const FIoStoreShaderCodeEntry& ThisShaderEntry = ShaderEntries[ShaderIndex];
		const FIoStoreShaderGroupEntry& GroupEntry = ShaderGroupEntries[ThisShaderEntry.ShaderGroupIndex];

		for (uint32 ShaderIdxIdx = GroupEntry.ShaderIndicesOffset, StopBeforeIdxIdx = GroupEntry.ShaderIndicesOffset + GroupEntry.NumShaders; ShaderIdxIdx < StopBeforeIdxIdx; ++ShaderIdxIdx)
		{
			int32 GroupMemberShaderIndex = ShaderIndices[ShaderIdxIdx];
			if (ShaderIndex == GroupMemberShaderIndex)
			{
				// found ourselves, now find our size by subtracting from the next neighbor or the group size
				if (LIKELY(ShaderIdxIdx < StopBeforeIdxIdx - 1))
				{
					const FIoStoreShaderCodeEntry& NextShaderEntry = ShaderEntries[ShaderIndices[ShaderIdxIdx + 1]];
					return NextShaderEntry.UncompressedOffsetInGroup - ThisShaderEntry.UncompressedOffsetInGroup;
				}
				else
				{
					return GroupEntry.UncompressedSize - ThisShaderEntry.UncompressedOffsetInGroup;
				}
			}
		}

		checkf(false, TEXT("Could not find shader index %d in its own group %" UINT64_FMT " - library is corrupted."), ShaderIndex, ThisShaderEntry.ShaderGroupIndex);
		return 0;
	}

	/** Helper function to iterate through all shaders of the specified shadermap and resolve the indirection of shader indices. */
	FORCEINLINE void ForEachShaderInShaderMap(int32 ShaderMapIndex, TFunctionRef<void(int32 ShaderIndex)> InCallback) const
	{
		ForEachShaderInternal(this->ShaderMapEntries, ShaderMapIndex, InCallback);
	}

	/** Helper function to iterate through all shaders of the specified shader group and resolve the indirection of shader indices. */
	FORCEINLINE void ForEachShaderInGroup(int32 GroupIoIndex, TFunctionRef<void(int32 ShaderIndex)> InCallback) const
	{
		ForEachShaderInternal(this->ShaderGroupEntries, GroupIoIndex, InCallback);
	}

	uint64 GetAllocatedSize() const
	{
		return sizeof(*this) +
			ShaderMapHashes.GetAllocatedSize() +
			ShaderHashes.GetAllocatedSize() +
			ShaderGroupIoHashes.GetAllocatedSize() +
			ShaderMapEntries.GetAllocatedSize() +
			ShaderEntries.GetAllocatedSize() +
			ShaderGroupEntries.GetAllocatedSize() +
			ShaderIndices.GetAllocatedSize();
	}
};

/**
 * Implements FRHIShaderLibrary and reads shader code through IoStore interface (either a ZenServer or CDN).
 * This class stores its shader code via IoBuffers, one per group whereas each shadermap references one or more groups and these groups can be shared between shadermaps.
 * @todo-lh - Consider renaming to FIoStoreRHIShaderLibrary
 */
class FIoStoreShaderCodeArchive : public FRHIShaderLibrary
{
public:
	UE_DEPRECATED(5.8, "GetShaderCodeArchiveChunkId is internal engine functionality and so is no longer publically exposed")
	RENDERCORE_API static FIoChunkId GetShaderCodeArchiveChunkId(const FString& LibraryName, FName FormatName);
	UE_DEPRECATED(5.8, "GetShaderCodeChunkId is internal engine functionality and so is no longer publically exposed")
	RENDERCORE_API static FIoChunkId GetShaderCodeChunkId(const FSHAHash& ShaderHash);
	/** This function creates the archive header, including splitting shaders into groups. */
	RENDERCORE_API static void CreateIoStoreShaderCodeArchiveHeader(const FName& Format, const FSerializedShaderArchive& SerializedShaders, FIoStoreShaderCodeArchiveHeader& OutHeader);
	RENDERCORE_API static void SaveIoStoreShaderCodeArchive(const FIoStoreShaderCodeArchiveHeader& Header, FArchive& OutLibraryAr);
	RENDERCORE_API static FIoStoreShaderCodeArchive* Create(EShaderPlatform InPlatform, const FString& InLibraryName, FIoDispatcher& InIoDispatcher);

	using FCreateIoStoreShaderCodeArchiveDelegate = TFunction<FIoStoreShaderCodeArchive*(EShaderPlatform, const FString&, FIoDispatcher&)>;
	RENDERCORE_API static void RegisterIoStoreShaderCodeArchiveFactory(FCreateIoStoreShaderCodeArchiveDelegate InFactory);

	RENDERCORE_API virtual ~FIoStoreShaderCodeArchive();

	virtual bool IsNativeLibrary() const override { return false; }

	virtual uint32 GetSizeBytes() const override
	{
		return sizeof(*this) +
			Header.GetAllocatedSize() +
			PreloadedShaderGroups.GetAllocatedSize();
	}

	RENDERCORE_API virtual uint32 GetShaderSizeBytes(int32 ShaderIndex) const override;

	virtual int32 GetNumShaders() const override { return Header.ShaderEntries.Num(); }
	virtual int32 GetNumShaderMaps() const override { return Header.ShaderMapEntries.Num(); }
	virtual int32 GetNumShadersForShaderMap(int32 ShaderMapIndex) const override { return Header.ShaderMapEntries[ShaderMapIndex].NumShaders; }

	virtual int32 GetShaderIndex(int32 ShaderMapIndex, int32 i) const override
	{
		const FIoStoreShaderMapEntry& ShaderMapEntry = Header.ShaderMapEntries[ShaderMapIndex];
		return Header.ShaderIndices[ShaderMapEntry.ShaderIndicesOffset + i];
	}

	virtual void GetAllShaderIndices(int32 ShaderMapIndex, TArray<int32>& OutShaderIndices) const override
	{
		const FIoStoreShaderMapEntry& ShaderMapEntry = Header.ShaderMapEntries[ShaderMapIndex];
		for (uint32 i = 0u; i < ShaderMapEntry.NumShaders; ++i)
		{
			OutShaderIndices.AddUnique(Header.ShaderIndices[ShaderMapEntry.ShaderIndicesOffset + i]);
		}
	}

	RENDERCORE_API virtual int32 FindShaderMapIndex(const FShaderHash& Hash) override;
	RENDERCORE_API virtual int32 FindShaderIndex(const FShaderHash& Hash) override;
	virtual FShaderHash GetShaderHash(int32 ShaderMapIndex, int32 ShaderIndex) override
	{
		return Header.ShaderHashes[GetShaderIndex(ShaderMapIndex, ShaderIndex)];
	}

	virtual FShaderHash GetShaderMapHash(int32 ShaderMapIndex) const override
	{
		return Header.ShaderMapHashes[ShaderMapIndex];
	}

	RENDERCORE_API virtual bool IsPreloading(int32 ShaderIndex, FGraphEventArray& OutCompletionEvents) override;
	RENDERCORE_API virtual bool PreloadShader(int32 ShaderIndex, FGraphEventArray& OutCompletionEvents) override;
	RENDERCORE_API virtual void AddRefPreloadedShaderGroup(int32 ShaderGroupIndex) override;
	RENDERCORE_API virtual void ReleasePreloadedShaderGroup(int32 ShaderGroupIndex) override;

	/** Returns the index of shader group that a given shader belongs to. */
	virtual int32 GetGroupIndexForShader(int32 ShaderIndex) const override
	{
		return Header.ShaderEntries[ShaderIndex].ShaderGroupIndex;
	}
	
	RENDERCORE_API virtual bool ResolveShaderMap(int32 ShaderMapIndex, FCoreDelegates::FIoChunkIdResolvedFunc IoChunkIdResolvedFunc) const override;
	RENDERCORE_API virtual bool PreloadShaderMap(int32 ShaderMapIndex, FGraphEventArray& OutCompletionEvents) override;
	RENDERCORE_API virtual bool PreloadShaderMap(int32 ShaderMapIndex, FCoreDelegates::FAttachShaderReadRequestFunc AttachShaderReadRequestFunc) override;
	RENDERCORE_API virtual void ReleasePreloadedShader(int32 ShaderIndex) override;

	/** Returns true if all IoChunks associated with the specified shadermap have been resolved and are available for use without any further delay.
	    Otherwise, at least one IoChunk is either unknown or has not been installed yet (when coming from an on-demand container). */
	RENDERCORE_API virtual bool IsShaderMapResolved(int32 ShaderMapIndex) const override;

	RENDERCORE_API virtual TRefCountPtr<FRHIShader> CreateShader(int32 Index, bool bRequired = true) override;
	RENDERCORE_API virtual void Teardown() override;
	RENDERCORE_API virtual void OnCloseShaderCode() override;

protected:
	struct FShaderGroupData
	{
		FIoRequest IoRequest;
#if USE_MMAPPED_SHADERARCHIVE
		TIoStatusOr<FIoMappedRegion> MappedRegionStatus;
#endif

		~FShaderGroupData()
		{
			IoRequest.Cancel();
#if USE_MMAPPED_SHADERARCHIVE
			if (MappedRegionStatus.IsOk())
			{
				FIoMappedRegion MappedRegion = MappedRegionStatus.ConsumeValueOrDie();
				delete MappedRegion.MappedFileRegion;
				delete MappedRegion.MappedFileHandle;
			}
#endif
		}
	};

	struct FShaderGroupPreloadEntry
	{
		FGraphEventRef PreloadEvent;
		TSharedPtr<FShaderGroupData> ShaderGroupData;

		uint32 FramePreloadStarted = ~0u;
		uint32 NumRefs : 31;
		uint32 bNeverToBePreloaded : 1;

#if UE_SCA_DEBUG_PRELOADING
		FString DebugInfo;
#endif

		FShaderGroupPreloadEntry()
			: ShaderGroupData(MakeShared<FShaderGroupData>())
			, NumRefs(0)
			, bNeverToBePreloaded(0)
		{
		}

		~FShaderGroupPreloadEntry() = default;
	};
	RENDERCORE_API FIoStoreShaderCodeArchive(EShaderPlatform InPlatform, const FString& InLibraryName, FIoDispatcher& InIoDispatcher);

	/** Called when a shader group data owner with a custom deleter is about to be created. */
	virtual void OnShaderGroupDataOwnerCreated() {}

	/** Called when a shader group data owner's deleter has been invoked (the data is no longer in use). */
	virtual void OnShaderGroupDataOwnerReleased() {}

	FIoDispatcher& IoDispatcher;

	/** Archive header with all the metadata */
	FIoStoreShaderCodeArchiveHeader Header;

	/** Hash tables for faster searching for shader and shadermap hashes. */
	FHashTable ShaderMapHashTable;
	FHashTable ShaderHashTable;

	/** Mapping between the group index and preloaded groups. Should be only modified when lock is taken. */
	TMap<int32, FShaderGroupPreloadEntry*> PreloadedShaderGroups;
	/** Lock guarding access to the book-keeping info above.*/
	FRWLock PreloadedShaderGroupsLock;

	/** debug visualizer - in Shipping compiles out to an empty struct with no-op functions */
	FShaderUsageVisualizer DebugVisualizer;

	static constexpr uint32 CurrentVersion = 1;

private:

	static FCreateIoStoreShaderCodeArchiveDelegate IoStoreShaderCodeArchiveFactory;

	/** Preloads a given shader group. */
	bool PreloadShaderGroup(int32 ShaderGroupIndex, FGraphEventArray& OutCompletionEvents, 
#if UE_SCA_DEBUG_PRELOADING
		const FString& CallsiteInfo,
#endif
		FCoreDelegates::FAttachShaderReadRequestFunc* AttachShaderReadRequestFuncPtr = nullptr);

	/** Sets up a new preload entry for preload.*/
	void SetupPreloadEntryForLoading(int32 ShaderGroupIndex, FShaderGroupPreloadEntry& PreloadEntry);

	/** Sets up a preload entry for groups that shouldn't be preloaded.*/
	void MarkPreloadEntrySkipped(int32 ShaderGroupIndex
#if UE_SCA_DEBUG_PRELOADING
		, const FString& CallsiteInfo
#endif
	);

	/** Releases a reference to a preloaded shader group, potentially deleting it. */
	void ReleasePreloadEntry(int32 ShaderGroupIndex
#if UE_SCA_DEBUG_PRELOADING
		, const FString& CallsiteInfo
#endif
	);

	/** Finds or adds preload info for a shader group. Assumes lock guarding access to the info taken, never returns nullptr (except when new failed and we're already broken beyond repair)*/
	inline FShaderGroupPreloadEntry* FindOrAddPreloadEntry(int32 ShaderGroupIndex)
	{
		FShaderGroupPreloadEntry*& Ptr = PreloadedShaderGroups.FindOrAdd(ShaderGroupIndex);
		if (UNLIKELY(Ptr == nullptr))
		{
			Ptr = new FShaderGroupPreloadEntry;
		}
		return Ptr;
	}

	/** Finds existing preload info for a shader group. Assumes lock guarding access to the info is taken */
	inline FShaderGroupPreloadEntry* FindExistingPreloadEntry(int32 ShaderGroupIndex)
	{
		FShaderGroupPreloadEntry** PtrPtr = PreloadedShaderGroups.Find(ShaderGroupIndex);
		checkf(PtrPtr != nullptr, TEXT("The preload entry for a shader group we assumed to exist does not exist!"));
		return *PtrPtr;
	}

	/** Returns true if the group contains only RTX shaders. We can avoid preloading it when running with RTX off. */
	bool GroupOnlyContainsRaytracingShaders(int32 ShaderGroupIndex) const;
};


///////////////////////////////////////////////////////
// Inline implementations
///////////////////////////////////////////////////////


PRAGMA_DISABLE_DEPRECATION_WARNINGS
inline FSerializedShaderArchive::FSerializedShaderArchive(FSerializedShaderArchive&&) = default;
inline FSerializedShaderArchive::FSerializedShaderArchive(const FSerializedShaderArchive&) = default;
inline FSerializedShaderArchive& FSerializedShaderArchive::operator=(FSerializedShaderArchive&&) = default;
inline FSerializedShaderArchive& FSerializedShaderArchive::operator=(const FSerializedShaderArchive&) = default;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

