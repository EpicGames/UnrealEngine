// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ShaderCodeLibrary.h: 
=============================================================================*/

#pragma once

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Containers/SparseArray.h"
#include "Containers/StringFwd.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "Logging/LogMacros.h"
#include "RHI.h"
#include "RHIDefinitions.h"
#include "Serialization/Archive.h"
#include "ShaderMapAssetAssociation.h"
#include "Templates/Function.h"
#include "Templates/RefCounting.h"
#include "UObject/NameTypes.h"

DECLARE_LOG_CATEGORY_EXTERN(LogShaderLibrary, Log, All);

class FShaderMapResource;
class FShaderMapResourceCode;
class FShaderPipeline;

class FIoBuffer;
class FIoChunkId;
class IPlugin;
class UObject;

#if WITH_EDITOR
class FCbFieldView;
class FCbWriter;
namespace UE::Cook { class ICookArtifactReader; }
namespace UE::Cook::Artifact { struct FUpdateOplogPackagesContext; }
#endif

struct FShaderHash;

struct FShaderCodeLibraryPipeline
{
	FShaderHash Shaders[SF_NumGraphicsFrequencies];
	mutable uint32 Hash;

	/** Fills the hashes from the pipeline stage shaders */
	RENDERCORE_API void Initialize(const FShaderPipeline* Pipeline);

	FShaderCodeLibraryPipeline() : Hash(0) {}

	friend bool operator ==(const FShaderCodeLibraryPipeline& A,const FShaderCodeLibraryPipeline& B)
	{
		for (uint32 Frequency = 0u; Frequency < SF_NumGraphicsFrequencies; ++Frequency)
		{
			if (A.Shaders[Frequency] != B.Shaders[Frequency])
			{
				return false;
			}
		}
		return true;
	}

	friend uint32 GetTypeHash(const FShaderCodeLibraryPipeline &Key)
	{
		if(!Key.Hash)
		{
			for (uint32 Frequency = 0u; Frequency < SF_NumGraphicsFrequencies; ++Frequency)
			{
				Key.Hash = HashCombine(Key.Hash, GetTypeHash(Key.Shaders[Frequency]));
			}
		}
		return Key.Hash;
	}

	/** Computes a longer hash that uniquely identifies the whole pipeline, used in FStableShaderKeyAndValue */
	RENDERCORE_API void GetPipelineHash(FShaderHash& Output);

	friend FArchive& operator<<( FArchive& Ar, FShaderCodeLibraryPipeline& Info )
	{
		for (uint32 Frequency = 0u; Frequency < SF_NumGraphicsFrequencies; ++Frequency)
		{
			Ar << Info.Shaders[Frequency];
		}
		return Ar << Info.Hash;
	}
};

struct FCompactFullName
{
	TArray<FName, TInlineAllocator<16>> ObjectClassAndPath;

	bool operator==(const FCompactFullName& Other) const
	{
		return ObjectClassAndPath == Other.ObjectClassAndPath;
	}

	RENDERCORE_API FString ToString() const;
	RENDERCORE_API FString ToStringPathOnly() const;
	RENDERCORE_API void AppendString(FStringBuilderBase& Out) const;
	RENDERCORE_API void AppendString(FAnsiStringBuilderBase& Out) const;
	RENDERCORE_API void ParseFromString(const FStringView& Src);
	friend RENDERCORE_API uint32 GetTypeHash(const FCompactFullName& A);

#if WITH_EDITOR
	/** Used to set up some compact FName paths for the FCompactFullName */
	RENDERCORE_API void SetCompactFullNameFromObject(UObject* InDepObject);
	RENDERCORE_API FString GetPackageNameOnly() const;
#endif
};


struct FStableShaderKeyAndValue
{
	FCompactFullName ClassNameAndObjectPath;
	FName ShaderType;
	FName ShaderClass;
	FName MaterialDomain;
	FName FeatureLevel;
	FName QualityLevel;
	FName TargetFrequency;
	FName TargetPlatform;
	FName VFType;
	FName PermutationId;
	FShaderHash PipelineHash;

	uint32 KeyHash;

	FShaderHash OutputHash;

	FStableShaderKeyAndValue()
		: KeyHash(0)
	{
	}

	RENDERCORE_API void ComputeKeyHash();
	RENDERCORE_API void ParseFromString(const FStringView& Src);
	RENDERCORE_API void ParseFromStringCached(const FStringView& Src, class TMap<uint32, FName>& NameCache);
	RENDERCORE_API FString ToString() const;
	RENDERCORE_API void ToString(FString& OutResult) const;
	RENDERCORE_API void AppendString(FAnsiStringBuilderBase& Out) const;
	static RENDERCORE_API FString HeaderLine();

	/** Computes pipeline hash from the passed pipeline. Pass nullptr to clear */
	RENDERCORE_API void SetPipelineHash(const FShaderPipeline* Pipeline);

	friend bool operator ==(const FStableShaderKeyAndValue& A, const FStableShaderKeyAndValue& B)
	{
		return
			A.ClassNameAndObjectPath == B.ClassNameAndObjectPath &&
			A.ShaderType == B.ShaderType &&
			A.ShaderClass == B.ShaderClass &&
			A.MaterialDomain == B.MaterialDomain &&
			A.FeatureLevel == B.FeatureLevel &&
			A.QualityLevel == B.QualityLevel &&
			A.TargetFrequency == B.TargetFrequency &&
			A.TargetPlatform == B.TargetPlatform &&
			A.VFType == B.VFType &&
			A.PermutationId == B.PermutationId &&
			A.PipelineHash == B.PipelineHash;
	}

	friend uint32 GetTypeHash(const FStableShaderKeyAndValue &Key)
	{
		return Key.KeyHash;
	}
};

struct FStableShaderKeyFuncs : BaseKeyFuncs<TUniquePtr<FStableShaderKeyAndValue>, FStableShaderKeyAndValue>
{
	[[nodiscard]] static FORCEINLINE const FStableShaderKeyAndValue& GetSetKey(const TUniquePtr<FStableShaderKeyAndValue>& Element)
	{
		return *Element;
	}

	[[nodiscard]] static FORCEINLINE bool Matches(const FStableShaderKeyAndValue& A, const FStableShaderKeyAndValue& B)
	{
		return A == B;
	}

	/** Calculates a hash index for a key. */
	[[nodiscard]] static FORCEINLINE uint32 GetKeyHash(const FStableShaderKeyAndValue& Key)
	{
		return GetTypeHash(Key);
	}
};

using FStableShaderSet = TSet<TUniquePtr<FStableShaderKeyAndValue>, FStableShaderKeyFuncs>;

#if WITH_EDITOR
RENDERCORE_API void WriteToCompactBinary(FCbWriter& Writer, const FStableShaderKeyAndValue& Key, 
	const TMap<FShaderHash, int32>& HashToIndex);
RENDERCORE_API bool LoadFromCompactBinary(FCbFieldView Field, FStableShaderKeyAndValue& Key,
	const TArray<FShaderHash>& IndexToHash);
#endif


DECLARE_DELEGATE_OneParam(FSharedShaderMapResourceExplicitRelease, const FShaderMapResource*);
RENDERCORE_API extern FSharedShaderMapResourceExplicitRelease OnSharedShaderMapResourceExplicitRelease;

using FSharedShaderCodeRequest = TTSMulticastDelegate<void(const FShaderHash&, FArchive*)>; // thread-safe because it's "broadcasted" concurrently
using FSharedShaderCodeRelease = TMulticastDelegate<void(const FShaderHash&)>;

// Collection of unique shader code
// Populated at cook time
struct FShaderCodeLibrary
{
	/** Adds the hooks for OnPakFileMounted, since pak opening normally happens earlier. */
	static RENDERCORE_API void PreInit();

	/** This is the real initialization function. */
	static RENDERCORE_API void InitForRuntime(EShaderPlatform ShaderPlatform);
	static RENDERCORE_API void Shutdown();

	static RENDERCORE_API bool IsEnabled();

	static RENDERCORE_API bool AreShaderMapsPreloadedAtLoadTime();

	/**
	 * Makes a number of ChunkIDs known to the library.
	 * 
	 * Normally the library is tracking chunk IDs runtime itself by listening to the pak mounted notifications.
	 * However, sometimes shaderbytecode files are moved between the chunks by the packaging code can can end up being
	 * preloaded.
	 * Takes C array and not TArray because that makes easier to hardcode the chunk IDs (which is the intended use case for this function).
	 */
	static RENDERCORE_API void AddKnownChunkIDs(const int32* IDs, const int32 NumChunkIDs);

	/** 
	 * Open a named library.
	 * 
	 * At runtime this will open the shader library with this name.
	 * @param Name is a high level description of the library (usually a project name or "Global")
	 * @param Directory location of the .ushadercode file
	 * @param bMonolithicOnly If true, only attempt to open a monolithic library (no chunks)
	 * @return true if successful
	 */
	static RENDERCORE_API bool OpenLibrary(FString const& Name, FString const& Directory, bool bMonolithicOnly = false);

	/**
	 * Close a named library.
	 *
	 * At runtime this will release the library data and further requests for shaders from this library will fail.
	 */
	static RENDERCORE_API void CloseLibrary(FString const& Name);

	static RENDERCORE_API bool ContainsShaderCode(const FShaderHash& Hash);
    static RENDERCORE_API bool ContainsShaderCode(const FShaderHash& Hash, const FString& LogicalLibraryName);

	static RENDERCORE_API TRefCountPtr<FShaderMapResource> LoadResource(const FShaderHash& Hash, FArchive* Ar);

	static RENDERCORE_API bool PreloadShader(const FShaderHash& Hash, FArchive* Ar);
	static RENDERCORE_API bool ReleasePreloadedShader(const FShaderHash& Hash);

	static RENDERCORE_API FVertexShaderRHIRef CreateVertexShader(EShaderPlatform Platform, const FShaderHash& Hash);
	static RENDERCORE_API FPixelShaderRHIRef CreatePixelShader(EShaderPlatform Platform, const FShaderHash& Hash);
	static RENDERCORE_API FGeometryShaderRHIRef CreateGeometryShader(EShaderPlatform Platform, const FShaderHash& Hash);
	static RENDERCORE_API FComputeShaderRHIRef CreateComputeShader(EShaderPlatform Platform, const FShaderHash& Hash);
	static RENDERCORE_API FMeshShaderRHIRef CreateMeshShader(EShaderPlatform Platform, const FShaderHash& Hash);
	static RENDERCORE_API FAmplificationShaderRHIRef CreateAmplificationShader(EShaderPlatform Platform, const FShaderHash& Hash);
	static RENDERCORE_API FRayTracingShaderRHIRef CreateRayTracingShader(EShaderPlatform Platform, const FShaderHash& Hash, EShaderFrequency Frequency);

	// Total number of shader entries in the library
	static RENDERCORE_API uint32 GetShaderCount(void);
	
	// The shader platform that the library manages - at runtime this will only be one
	static RENDERCORE_API EShaderPlatform GetRuntimeShaderPlatform(void);

	// Safely assign the hash to a shader object
	static RENDERCORE_API void SafeAssignHash(FRHIShader* InShader, const FShaderHash& Hash);

	// Delegate called whenever shader code is requested.
	static RENDERCORE_API FDelegateHandle RegisterSharedShaderCodeRequestDelegate_Handle(const FSharedShaderCodeRequest::FDelegate& Delegate);
	static RENDERCORE_API void UnregisterSharedShaderCodeRequestDelegate_Handle(FDelegateHandle Handle);

	// Disables opening the specified plugin's shader library on mount
	static RENDERCORE_API void DontOpenPluginShaderLibraryOnMount(const FString& PluginName);
	
	// Open the plugin's shader library
	// @param bMonolithicOnly If true, only attempt to open a monolithic library (no chunks) - which is a default behavior for DLC plugins, see FShaderLibraryChunkDataGenerator.
	static RENDERCORE_API void OpenPluginShaderLibrary(IPlugin& Plugin, bool bMonolithicOnly = true);
};

#if WITH_EDITOR
class ITargetPlatform;

namespace UE::Cook
{

enum class EPackagingSystem
{
	IoStore,
	Loose,
};

} // namespace UE::Cook

struct FShaderLibraryCooker
{
	// Initialize the library cooker
	static RENDERCORE_API void InitForCooking(bool bNativeFormat, UE::Cook::ICookArtifactReader* CookArtifactReader);
	// Shutdown the library cooker
	static RENDERCORE_API void Shutdown();

	// Clean the cook directories
	static RENDERCORE_API void CleanDirectories(TArray<FName> const& ShaderFormats);

	struct FShaderFormatDescriptor
	{
		FName ShaderFormat;
		bool bNeedsStableKeys;
		bool bNeedsDeterministicOrder;
	};

	/**
	 * Opens a named library for cooking and sets it as the default.
	 *
	 * This will place all added shaders & pipelines into the library file with this name.
	 * @param Name is a high level description of the library (usually a project name or "Global")
	 * @return true if successful
	 */
	static RENDERCORE_API bool BeginCookingLibrary(FString const& Name);

	/**
	 * Close a named library.
	 *  For cooking, after this point any AddShaderCode/AddShaderPipeline calls will be invalid until OpenLibrary is called again.
	 */
	static RENDERCORE_API void EndCookingLibrary(FString const& Name);

	/**
	 * Whether storing shaders in the shader library is enabled
	 */
	static RENDERCORE_API bool IsShaderLibraryEnabled();

	// Specify the shader formats to cook and which ones needs stable keys. Provide an array of FShaderFormatDescriptors
	static RENDERCORE_API void CookShaderFormats(TArray<FShaderFormatDescriptor> const& ShaderFormats);

	// At cook time, add shader code to collection
	static RENDERCORE_API bool AddShaderCode(EShaderPlatform ShaderPlatform, const FShaderMapResourceCode* Code, const FShaderMapAssetPaths& AssociatedAssets);

	/** Called from a CookWorker to send all contents of the ShaderLibrary to the CookDirector */
	static RENDERCORE_API void CopyToCompactBinaryAndClear(FCbWriter& Writer, bool& bOutHasData, bool& bOutRanOutOfRoom, int64 MaxShaderSize);

	/** Called On the CookDirector to receive ShaderLibrary contents from a CookWorker */
	static RENDERCORE_API bool AppendFromCompactBinary(FCbFieldView Field);

	// We check this early in the callstack to avoid creating a bunch of FName and keys and things we will never save anyway. 
	// Pass the shader platform to check or EShaderPlatform::SP_NumPlatforms to check if any of the registered types require
	// stable keys.
	static RENDERCORE_API bool NeedsShaderStableKeys(EShaderPlatform ShaderPlatform);

	// At cook time, add the human readable key value information
	static RENDERCORE_API void AddShaderStableKeyValue(EShaderPlatform ShaderPlatform, FStableShaderKeyAndValue& StableKeyValue);

	/** Finishes collection of data that should be in the named code library. This includes loading data from a previous iterative cook. */
	static RENDERCORE_API void FinishPopulateShaderLibrary(const ITargetPlatform* TargetPlatform, FString const& Name, FString const& SandboxDestinationPath,
		FString const& SandboxMetadataPath, bool bInitializeReferenceTracking = false);

	/** Called by the cooker to report which assets are referenced in the current cook or in incrementalcook oplog. */
	static RENDERCORE_API void UpdateOplogPackages(UE::Cook::Artifact::FUpdateOplogPackagesContext& Context);

	/**
	 * Called by the cooker to ask the library to prune the no-longer referenced shaders from its incremental cook library,
	 * called only after UpdateOplogPackages has been called on every platform.
	 */
	static RENDERCORE_API void PruneShaderLibrary(const ITargetPlatform* TargetPlatform, const FString& Name);

	/**
	 * Given multiple Cooked Metadata directories will attempt to merge the ShaderByteCode and the ShaderStableInfo into the given OutputDir.
	 * It would be expected that the OutputDir is another MetaData directory but this can be any dir.
	 * Sub directories for ShaderLibrarySource and PipelineCaches will be automatically generated.
	 * 
	 * @param CookedMetadataDirs - the cooked metadata directories to merge the shader archives from
	 * @param OutputDir - where to place the union of the shader archives
	 * @param OutWrittenFiles - full path to all the files written
	 */
	static RENDERCORE_API bool MergeShaderCodeArchive(const TArray<FString>& CookedMetadataDirs, const FString& OutputDir, TArray<FString>& OutWrittenFiles);

	/**
	 * Saves collected shader code to a single file per shader platform
	 * When chunking is enabled, this call will not write the shader code, only the SCL.CSV file with the stable shader info.
	 * 
	 * @param TargetPlatform target platform
	 * @param Name shader library name
	 * @param SandboxDestinationPath where to put the .ushaderbytecode file(s)
	 * @param SandboxMetadataPath path for the metadata (not a part of the build itself, but produced together with the build)
	 * @param PlatformSCLCSVPaths path for where to put the information about the shader hashes
	 * @param PackagingSystem Which packaging system is being used by the project's data build. This impacts
	 *        some internal format storage decisions for ShaderLibraries due to runtime performance requirements.
	 * @param OutErrorMessage used to return the details of the failure (if failed)
	 * @param bOutHasData Reports whether any files were written to PlatformSCLCSVPaths
	 * @return true if successful
	 */
	static RENDERCORE_API bool SaveShaderLibraryWithoutChunking(const ITargetPlatform* TargetPlatform,
		FString const& Name, FString const& SandboxDestinationPath, FString const& SandboxMetadataPath,
		TArray<FString>& PlatformSCLCSVPaths, UE::Cook::EPackagingSystem PackagingSystem,
		FString& OutErrorMessage, bool& bOutHasData);

	/** 
	 * Saves a single chunk of the collected shader code (per shader platform). Does not save SCL.CSV info.
	 * This code path is only called if we're chunking.
	 * FIXME: this function does not write build metadata
	 * 
	 * @param ChunkId the chunk id
	 * @param InPackagesInChunk packages that belong to the chunk
	 * @param TargetPlatform target platform
	 * @param SandboxDestinationPath where to put the .ushaderbytecode file(s)
	 * @param SandboxMetadataPath path for the metadata (not a part of the build itself, but produced together with the build)
	 * @param PackagingSystem Which packaging system is being used by the project's data build. This impacts
	 *        some internal format storage decisions for ShaderLibraries due to runtime performance requirements.
	 * @param OutChunkFilenames array where the function will append the full paths of the written files
	 * @param bOutHasData Reports whether any files were written to OutChunkFilenames
	 * @return true if successful
	 */
	static RENDERCORE_API bool SaveShaderLibraryChunk(int32 ChunkId, const TSet<FName>& InPackagesInChunk,
		const ITargetPlatform* TargetPlatform, const FString& SandboxDestinationPath,
		const FString& SandboxMetadataPath, UE::Cook::EPackagingSystem PackagingSystem,
		TArray<FString>& OutChunkFilenames, bool& bOutHasData);

	/**
	 * Delete the chunk-specific shaderlibrary files that are still present on disk from a previous incremental cook.
	 * They will be rewritten by SaveShaderLibraryChunk calls.
	 */
	static RENDERCORE_API void DeleteShaderLibraryChunks(const ITargetPlatform* TargetPlatform,
		const FString& SandboxDestinationPath, const FString& SandboxMetadataPath);

	static RENDERCORE_API void DumpShaderTypeStats(const FString& DebugInfoPath, const FString& MetaDataPath,
		TFunctionRef<void(FName AssetName, bool& bImplemented, bool& bOutReferenced, bool& bOutRecooked)> GetCookStatus);

	// Create a smaller 'patch' library that only contains data from 'NewMetaDataDir' not contained in any of 'OldMetaDataDirs'
	static RENDERCORE_API bool CreatePatchLibrary(TArray<FString> const& OldMetaDataDirs, FString const& NewMetaDataDir, FString const& OutDir, bool bNativeFormat, bool bNeedsDeterministicOrder);
};
#endif // WITH_EDITOR
