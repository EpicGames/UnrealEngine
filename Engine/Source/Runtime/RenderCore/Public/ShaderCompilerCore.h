// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ShaderCompilerCore.h: Shader Compiler core module definitions.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "HAL/FileManager.h"
#include "Hash/Blake3.h"
#include "Hash/ShaderHash.h"
#include "Stats/Stats.h"
#include "Templates/RefCounting.h"
#include "Misc/EnumClassFlags.h"
#include "Misc/SecureHash.h"
#include "Misc/Paths.h"
#include "Misc/PathViews.h"
#include "Misc/CoreStats.h"
#include "ShaderCore.h"
#include "ShaderParameterMetadata.h"

class Error;
class IShaderFormat;
class FShaderCommonCompileJob;
class FShaderCompileJob;
class FShaderPipelineCompileJob;

struct FShaderCompilerInput;

using FShaderSharedStringPtr = TSharedPtr<FString, ESPMode::ThreadSafe>;
typedef TSharedPtr<TArray<ANSICHAR>, ESPMode::ThreadSafe> FShaderSharedAnsiStringPtr;

// this is for the protocol, not the data, bump if FShaderCompilerInput/FShaderPreprocessOutput serialization, SerializeWorkerInput or ProcessInputFromArchive changes.
inline const int32 ShaderCompileWorkerInputVersion = 33;
// this is for the protocol, not the data, bump if FShaderCompilerOutput or WriteToOutputArchive changes.
inline const int32 ShaderCompileWorkerOutputVersion = 31;
// this is for the protocol, not the data.
inline const int32 ShaderCompileWorkerSingleJobHeader = 'S';
// this is for the protocol, not the data.
inline const int32 ShaderCompileWorkerPipelineJobHeader = 'P';

// Modify this in ShaderCompilerCore.cpp to invalidate _just_ the cache/DDC entries for individual shaders (will not cause
// shadermaps to rebuild if they are not otherwise out-of-date).
// This should be bumped for changes to the FShaderCompilerOutput data structure (in addition to ShaderCompileWorkerOutputVersion)
extern RENDERCORE_API const FGuid UE_SHADER_CACHE_VERSION;

namespace UE::ShaderCompiler
{
	UE_DEPRECATED(5.8, "UE::ShaderCompiler::GetBindlessConfiguration has been replaced by FShaderPlatformConfig::GetBindlessConfiguration")
	RENDERCORE_API ERHIBindlessConfiguration GetBindlessConfiguration(EShaderPlatform ShaderPlatform);

	RENDERCORE_API bool ShouldCompileWithBindlessEnabled(EShaderPlatform ShaderPlatform, const FShaderCompilerInput& Input);

	UE_DEPRECATED(5.7, "GetBindlessResourcesConfiguration is now GetBindlessConfiguration")
	RENDERCORE_API ERHIBindlessConfiguration GetBindlessResourcesConfiguration(FName ShaderFormat);

	UE_DEPRECATED(5.7, "GetBindlessSamplersConfiguration is now GetBindlessConfiguration")
	RENDERCORE_API ERHIBindlessConfiguration GetBindlessSamplersConfiguration(FName ShaderFormat);
}

/** Returns the path where shader compilation related artifacts should be stored when running on a build machine */
extern RENDERCORE_API const FString& GetBuildMachineArtifactBasePath();

/** Returns the base path where any shader debug information should be written to */
extern RENDERCORE_API const FString& GetShaderDebugInfoPath();

/** Returns true if shader symbols should be kept for a given platform. */
extern RENDERCORE_API bool ShouldGenerateShaderSymbols(FName ShaderFormat);

/** Returns true if shader symbol minimal info files should be generated for a given platform. */
extern RENDERCORE_API bool ShouldGenerateShaderSymbolsInfo(FName ShaderFormat);

/** Returns true if shader symbols should be exported to separate files for a given platform. */
extern RENDERCORE_API bool ShouldWriteShaderSymbols(FName ShaderFormat);

/** Returns true if the shader symbol path is overridden and OutPathOverride contains the override path. */
extern RENDERCORE_API bool GetShaderSymbolPathOverride(FString& OutPathOverride, FName ShaderFormat);

/** Returns true if the shader file name is overridden by the Cvar and OutFileNameOverride contains the override filename.
    ShaderFormat used for format specific cvar, PlatformName used for {Platform} string substitution, if requested.
    Returns false otherwise and leaves OutFileNameOverride unchanged */
extern RENDERCORE_API bool GetShaderFileNameOverride(FString& OutFileNameOverride, const TCHAR* Cvar, FName ShaderFormat, FName PlatformName);

/** Returns true if (external) shader symbols should be specific to each shader rather than be de-duplicated. */
extern RENDERCORE_API bool ShouldAllowUniqueShaderSymbols(FName ShaderFormat);

enum class EWriteShaderSymbols : uint8
{
	Disable		= 0,
	UnCompress	= 1,
	Compress	= 2
};

/** Returns true if shaders should be combined into a single zip file instead of individual files. */
extern RENDERCORE_API EWriteShaderSymbols GetWriteShaderSymbolsOptions(FName ShaderFormat);

/** Returns true if the user wants more runtime shader data (names, extra info) */
extern RENDERCORE_API bool ShouldEnableExtraShaderData(FName ShaderFormat);

extern RENDERCORE_API bool ShouldOptimizeShaders(FName ShaderFormat);

/** Returns true is shader compiling is allowed */
extern RENDERCORE_API bool AllowShaderCompiling();

/** Returns true if the global shader cache should be loaded (and potentially compiled if allowed/needed */
extern RENDERCORE_API bool AllowGlobalShaderLoad();


enum ECompilerFlags
{
	#define SHADER_COMPILER_FLAGS_ENTRY(Name) CFLAG_##Name,
	#define SHADER_COMPILER_FLAGS_ENTRY_DEPRECATED(Name, Version, Message) CFLAG_##Name UE_DEPRECATED(Version, Message),
	#include "ShaderCompilerFlags.inl"
	CFLAG_Max
};
static_assert(CFLAG_PreferFlowControl == 0, "First entry in ECompilerFlags must be 'CFLAG_PreferFlowControl' and assigned to 0");
static_assert(CFLAG_Max < 64, "Out of bitfield space! Modify FShaderCompilerFlags");

extern RENDERCORE_API void LexFromString(ECompilerFlags& OutValue, const TCHAR* InString);
extern RENDERCORE_API const TCHAR* LexToString(ECompilerFlags InValue);

struct FShaderCompilerResourceTable
{
	/** Bits indicating which resource tables contain resources bound to this shader. */
	uint32 ResourceTableBits;

	/** The max index of a uniform buffer from which resources are bound. */
	uint32 MaxBoundResourceTable;

	/** Mapping of bound Textures to their location in resource tables. */
	TArray<uint32> TextureMap;

	/** Mapping of bound SRVs to their location in resource tables. */
	TArray<uint32> ShaderResourceViewMap;

	/** Mapping of bound sampler states to their location in resource tables. */
	TArray<uint32> SamplerMap;

	/** Mapping of bound UAVs to their location in resource tables. */
	TArray<uint32> UnorderedAccessViewMap;

	/** Mapping of bound respource collections to their location in resource tables. */
	TArray<uint32> ResourceCollectionMap;

	/** Hash of the layouts of resource tables at compile time, used for runtime validation. */
	TArray<uint32> ResourceTableLayoutHashes;

	FShaderCompilerResourceTable()
		: ResourceTableBits(0)
		, MaxBoundResourceTable(0)
	{
	}
};

/** enumeration of offline shader compiler for the material editor */
enum class EOfflineShaderCompilerType : uint8
{
	Mali,
	Adreno,

	Num
};

/** Additional compilation settings that can be configured by each FMaterial instance before compilation */
struct FExtraShaderCompilerSettings
{
	bool bExtractShaderSource = false;
	FString OfflineCompilerPath;
	EOfflineShaderCompilerType OfflineCompiler = EOfflineShaderCompilerType::Mali;
	FString GPUTarget;
	bool bDumpAll = false;
	bool bSaveCompilerStatsFiles = false;
	bool bMobileMultiView = false;

	friend FArchive& operator<<(FArchive& Ar, FExtraShaderCompilerSettings& StatsSettings)
	{
		// Note: this serialize is used to pass between UE and the shader compile worker, recompile both when modifying
		return Ar << StatsSettings.bExtractShaderSource << StatsSettings.OfflineCompilerPath 
			<< StatsSettings.OfflineCompiler << StatsSettings.GPUTarget << StatsSettings.bDumpAll << StatsSettings.bSaveCompilerStatsFiles << StatsSettings.bMobileMultiView;
	}
};

namespace FOodleDataCompression
{
	enum class ECompressor : uint8;
	enum class ECompressionLevel : int8;
}

enum class EShaderDebugInfoFlags : uint8
{
	Default = 0,
	DirectCompileCommandLine UE_DEPRECATED(5.7, "DirectCompile has been renamed DebugCompile and DebugCompileArgs.txt is dumped by default") = 1 << 0,
	InputHash = 1 << 1,
	Diagnostics = 1 << 2,
	ShaderCodeBinary = 1 << 3,
	DetailedSource = 1 << 4,
	CompileFromDebugUSF = 1 << 5,
	ShaderCodePlatformHashes = 1 << 6,
};
ENUM_CLASS_FLAGS(EShaderDebugInfoFlags)

using FShaderCompilerInputHash = FBlake3Hash;

struct FShaderDebugDataContext
{
	bool bIsPipeline = false;
	TArray<TPair<EShaderFrequency, FString>> DebugSourceFiles;
};

/** Struct that gathers all readonly inputs needed for the compilation of a single shader. */
struct FShaderCompilerInput
{
	FShaderTarget Target{ SF_NumFrequencies, SP_NumPlatforms };
	
	FName ShaderFormat;
	FName CompressionFormat;
	FName ShaderPlatformName;

	// Preferred output format for FShaderCode, e.g. when DXIL is preferred over DXBC while both would be supported.
	FName PreferredShaderCodeFormat;
	
	FString VirtualSourceFilePath;
	FString EntryPointName;
	FString ShaderName;

	uint32 SupportedHardwareMask = 0;

	// Indicates which additional debug outputs should be written for this compile job.
	EShaderDebugInfoFlags DebugInfoFlags = EShaderDebugInfoFlags::Default;

	// Array of symbols that should be maintained when deadstripping. If this is empty, entry
	// point name alone will be used.
	TArray<FString> RequiredSymbols;
	
	// Shader pipeline information
	TArray<FString> UsedOutputs;

	// Uniform buffer that are (dis)allowed to be used for (non) VS frequencies: relies on reflection data to fail 
	// compilation even though the shader may actually compile. This ensures we don't start using a uniform buffer by mistake
	TArray<FString> VSDisallowedUniformBufferNames;
	TArray<FString> NonVSDisallowedUniformBufferNames;

	// Dump debug path (up to platform) e.g. "D:/Project/Saved/ShaderDebugInfo/PCD3D_SM5"
	FString DumpDebugInfoRootPath;
	// only used if enabled by r.DumpShaderDebugInfo (platform/groupname) e.g. ""
	FString DumpDebugInfoPath;
	// materialname or "Global" "for debugging and better error messages
	FString DebugGroupName;

	FString DebugExtension;

	// Description of the configuration used when compiling. 
	FString DebugDescription;

	// Hash of this input (used as the key for the shader job cache)
	FShaderCompilerInputHash Hash;

	// Compilation Environment
	FShaderCompilerEnvironment Environment;
	TRefCountPtr<FSharedShaderCompilerEnvironment> SharedEnvironment;

	// The root of the shader parameter structures / uniform buffers bound to this shader to generate shader resource table from.
	// This only set if a shader class is defining the 
	const FShaderParametersMetadata* RootParametersStructure = nullptr;


	// Additional compilation settings that can be filled by FMaterial::SetupExtraCompilationSettings
	// FMaterial::SetupExtraCompilationSettings is usually called by each (*)MaterialShaderType::BeginCompileShader() function
	FExtraShaderCompilerSettings ExtraSettings;

	/** Oodle-specific compression algorithm - used if CompressionFormat is set to NAME_Oodle. */
	FOodleDataCompression::ECompressor OodleCompressor;

	/** Oodle-specific compression level - used if CompressionFormat is set to NAME_Oodle. */
	FOodleDataCompression::ECompressionLevel OodleLevel;

	// Marks the compile job for being used with a ShaderPipeline. This tells the shader backend to remove unused interpolators from the Pixel input and Vertex output (in that order).
	// Therefore, ShaderPipelines compile their Pixel shader first, to determine what input is actually used and then compile the Vertex shader, to remove output interpolators that are no longer needed by the Pixel input.
	// Only the first stage in a ShaderPipeline job enables this attribute when the job is constructed and the next stage propagates its value in CompileShaderPipeline().
	bool bCompilingForShaderPipeline = false;
	bool bIncludeUsedOutputs = false;

	// Internal only flag to signal that bindless is enabled
	bool bBindlessEnabled = false;

	bool DumpDebugInfoEnabled() const 
	{
		return DumpDebugInfoPath != TEXT("") && IFileManager::Get().DirectoryExists(*DumpDebugInfoPath);
	}

	bool NeedsOriginalShaderSource() const
	{
		return DumpDebugInfoEnabled() || ExtraSettings.bExtractShaderSource;
	}

	// generate human readable name for debugging
	FString GenerateShaderName() const
	{
		FString Name;

		if(DebugGroupName == TEXT("Global"))
		{
			Name = VirtualSourceFilePath + TEXT("|") + EntryPointName;
		}
		else
		{
			Name = DebugGroupName + TEXT(":") + VirtualSourceFilePath + TEXT("|") + EntryPointName;
		}

		return Name;
	}

	FStringView GetSourceFilenameView() const
	{
		return FPathViews::GetCleanFilename(VirtualSourceFilePath);
	}

	FString GetSourceFilename() const
	{
		return FPaths::GetCleanFilename(VirtualSourceFilePath);
	}

	// Common code to generate a debug string to associate with platform-specific shader symbol files and hashes
	// Currently uses DebugGroupName, but can be updated to contain other important information as needed
	FString GenerateDebugInfo() const
	{
		return Environment.CompilerFlags.Contains(CFLAG_GenerateSymbolsInfo) ? DebugGroupName : FString();
	}

	void GatherSharedInputs(
		TArray<TRefCountPtr<FSharedShaderCompilerEnvironment>>& SharedEnvironments,
		TArray<const FShaderParametersMetadata*>& ParametersStructures)
	{
		check(!SharedEnvironment || SharedEnvironment->IncludeVirtualPathToSharedContentsMap.Num() == 0);

		if (SharedEnvironment)
		{
			SharedEnvironments.AddUnique(SharedEnvironment);
		}

		if (RootParametersStructure)
		{
			ParametersStructures.AddUnique(RootParametersStructure);
		}
	}

	void SerializeSharedInputs(FArchive& Ar, const TArray<TRefCountPtr<FSharedShaderCompilerEnvironment>>& SharedEnvironments, const TArray<const FShaderParametersMetadata*>& ParametersStructures)
	{
		check(Ar.IsSaving());

		int32 SharedEnvironmentIndex = SharedEnvironments.Find(SharedEnvironment);
		Ar << SharedEnvironmentIndex;

		int32 ShaderParameterStructureIndex = INDEX_NONE;
		if (RootParametersStructure)
		{
			ShaderParameterStructureIndex = ParametersStructures.Find(RootParametersStructure);
			check(ShaderParameterStructureIndex != INDEX_NONE);
		}
		Ar << ShaderParameterStructureIndex;
	}

	void DeserializeSharedInputs(
		FArchive& Ar,
		const TArray<FShaderCompilerEnvironment>& SharedEnvironments,
		const TArray<TUniquePtr<FShaderParametersMetadata>>& ShaderParameterStructures)
	{
		check(Ar.IsLoading());

		int32 SharedEnvironmentIndex = 0;
		Ar << SharedEnvironmentIndex;

		if (SharedEnvironments.IsValidIndex(SharedEnvironmentIndex))
		{
			Environment.Merge(SharedEnvironments[SharedEnvironmentIndex]);
		}

		int32 ShaderParameterStructureIndex = INDEX_NONE;
		Ar << ShaderParameterStructureIndex;
		if (ShaderParameterStructureIndex != INDEX_NONE)
		{
			RootParametersStructure = ShaderParameterStructures[ShaderParameterStructureIndex].Get();
		}
	}

	friend RENDERCORE_API FArchive& operator<<(FArchive& Ar, FShaderCompilerInput& Input);

	bool IsRayTracingShader() const
	{
		return IsRayTracingShaderFrequency(Target.GetFrequency());
	}

	bool IsWorkGraphShader() const
	{
		return IsWorkGraphShaderFrequency(Target.GetFrequency());
	}

	bool ShouldUseStableConstantBuffer() const
	{
		// stable constant buffer is for the FShaderParameterBindings::BindForLegacyShaderParameters() code path.
		// Ray tracing shaders use FShaderParameterBindings::BindForRootShaderParameters instead.
		if (IsRayTracingShader())
		{
			return false;
		}

		return RootParametersStructure != nullptr;
	}

	bool IsBindlessEnabled() const
	{
		return bBindlessEnabled;
	}

	/** Returns the shader debug info path for this shader compiler input and create the directory if it doesn't exist yet. */
	RENDERCORE_API FString GetOrCreateShaderDebugInfoPath() const;
};

/** A shader compiler error or warning. */
struct FShaderCompilerError
{
	FShaderCompilerError(const TCHAR* InStrippedErrorMessage = TEXT(""))
		: StrippedErrorMessage(InStrippedErrorMessage)
	{}

	FShaderCompilerError(const TCHAR* InVirtualFilePath, const TCHAR* InLineString, const TCHAR* InStrippedErrorMessage)
		: ErrorVirtualFilePath(InVirtualFilePath)
		, ErrorLineString(InLineString)
		, StrippedErrorMessage(InStrippedErrorMessage)
		, HighlightedLine(TEXT(""))
		, HighlightedLineMarker(TEXT(""))
	{}

	FShaderCompilerError(FString&& InStrippedErrorMessage)
		: StrippedErrorMessage(MoveTemp(InStrippedErrorMessage))
	{}

	FShaderCompilerError(FString&& InStrippedErrorMessage, FString&& InHighlightedLine, FString&& InHighlightedLineMarker)
		: StrippedErrorMessage(MoveTemp(InStrippedErrorMessage))
		, HighlightedLine(MoveTemp(InHighlightedLine))
		, HighlightedLineMarker(MoveTemp(InHighlightedLineMarker))
	{}

	FString ErrorVirtualFilePath;
	FString ErrorLineString;
	FString StrippedErrorMessage;
	FString HighlightedLine;
	FString HighlightedLineMarker;

	/** Returns the error message with source file and source line (if present), as well as a line marker separated with a LINE_TERMINATOR. */
	FString RENDERCORE_API GetErrorStringWithSourceLocation() const;
	
	/** Returns the error message with source file and source line (if present), as well as a line marker separated with a LINE_TERMINATOR. */
	FString RENDERCORE_API GetErrorStringWithLineMarker() const;

	/** Returns the error message with source file and source line (if present). */
	FString RENDERCORE_API GetErrorString(bool bOmitLineMarker = false) const;

	/**
	Returns true if this error message has a marker string for the highlighted source line where the error occurred. Example:
		/Engine/Private/MySourceFile.usf(120): error: undeclared identifier 'a'
		float b = a;
				  ^
	*/
	inline bool HasLineMarker() const
	{
		return !HighlightedLine.IsEmpty() && !HighlightedLineMarker.IsEmpty();
	}

	/** Extracts the file path and source line from StrippedErrorMessage to ErrorVirtualFilePath and ErrorLineString. */
	bool RENDERCORE_API ExtractSourceLocation();

	/** Extracts the file path and source line for each error from the error message. Propagates highlighted line and marker to
	 * all errors pertaining to the same source location. */
	static void RENDERCORE_API ExtractSourceLocations(TArray<FShaderCompilerError>& InOutErrors);

	/** Returns the path of the underlying source file relative to the process base dir. */
	FString RENDERCORE_API GetShaderSourceFilePath(TArray<FShaderCompilerError>* InOutErrors = nullptr) const;

	friend FArchive& operator<<(FArchive& Ar,FShaderCompilerError& Error)
	{
		return Ar << Error.ErrorVirtualFilePath << Error.ErrorLineString << Error.StrippedErrorMessage << Error.HighlightedLine << Error.HighlightedLineMarker;
	}
};

/**
 *	The output of the shader compiler.
 *	Bump UE_SHADER_CACHE_VERSION and ShaderCompileWorkerOutputVersion if FShaderCompilerOutput changes
 */
struct FShaderCompilerOutput
{
	FShaderCompilerOutput()
	:	NumInstructions(0)
	,	NumTextureSamplers(0)
	,	CompileTime(0.0)
	,	PreprocessTime(0.0)
	,	bSucceeded(false)
	,	bSupportsQueryingUsedAttributes(false)
	,	bSerializingForCache(false)
	,	bInterpolatorRemovalFailed(false)
	,	bForceDumpDebugInfo(false)
	,	bForceLogDiagnosticMessages(false)
	{
	}

	FShaderParameterMap ParameterMap;
	TArray<FShaderCompilerError> Errors;
	FShaderTarget Target;
	FShaderCode ShaderCode;
	FShaderHash OutputHash;
	FShaderCompilerInputHash ValidateInputHash;
	uint32 NumInstructions;
	uint32 NumTextureSamplers;
	double CompileTime;
	double PreprocessTime;
	bool bSucceeded;
	bool bSupportsQueryingUsedAttributes;
	bool bSerializingForCache;

	/** Specifies that the interpolator removal optimization failed for this job.
	 * If the job still succeeded, then all other shaders for the associated pipeline job
	 * must also skip interpolator removal to avoid mismatch between interpolators of shader stages.
	 */
	bool bInterpolatorRemovalFailed;

	/** Specifies to always dump debug information for this job output.
	 * If the compile job was successful, this allows to dump debug information to diagnose failed intermediate results, regardless of r.DumpShaderDebugInfo configuration.
	 * This is primarily used when the cross-compilation pipeline is triggered due to internal compiler errors (usually with FXC).
	 */
	bool bForceDumpDebugInfo;

	/** Specifies to always log diagnostic messages.
	 * If the compile job was successful, this will be printed with Display verbosity.
	 * If r.ShaderDevelopmentMode is enabled, this will be printed with Warning verbosity.
	 */
	bool bForceLogDiagnosticMessages;

	TArray<FString> UsedAttributes;

	TArray<FShaderCodeValidationStride> ParametersStrideToValidate;
	TArray<FShaderCodeValidationType> ParametersSRVTypeToValidate;
	TArray<FShaderCodeValidationType> ParametersUAVTypeToValidate;
	TArray<FShaderCodeValidationUBSize> ParametersUBSizeToValidate;

	TArray<FShaderDiagnosticData> ShaderDiagnosticDatas;

	/** Use this field to store the shader source code if it's modified as part of the shader format's compilation process. This should only be set when 
	 * additional manipulation of source code  is required that is not part of the implementation of PreprocessShader. This version of the source, if set, 
	 * will be what is written as part of the debug dumps of preprocessed source, as well as used for upstream code which explicitly requests the final
	 * source code for other purposes (i.e. when ExtraSettings.bExtractShaderSource is set on the FShaderCompilerInput struct)
	 */
	FString ModifiedShaderSource;

	/** Use this field to store the entry point name if it's modified as part of the shader format's compilation process. This field is only 
	 * currently required for shader formats which implement the independent preprocessing API And should only be set when compilation requires
	 * a different entry point than was set on the FShaderCompilerInput struct. */
	FString ModifiedEntryPointName;

	TArray<FGenericShaderStat> ShaderStatistics;

	/** Generates OutputHash from the compiler output. */
	RENDERCORE_API void GenerateOutputHash();

	/** Calls GenerateOutputHash() before the compression, replaces FShaderCode with the compressed data (if compression result was smaller). */
	RENDERCORE_API void CompressOutput(FName ShaderCompressionFormat, FOodleDataCompression::ECompressor OodleCompressor, FOodleDataCompression::ECompressionLevel OodleLevel);

	/** Add optional data in ShaderCode to perform additional shader input validation at runtime*/
	RENDERCORE_API void SerializeShaderCodeValidation();

	/** Add optional diagnostic data in ShaderCode to perform assert translation at runtime*/
	RENDERCORE_API void SerializeShaderDiagnosticData();

	template<typename... Types>
	void AddErrorf(UE::Core::TCheckedFormatString<FString::FmtCharType, Types...> Fmt, Types... Args)
	{
		Errors.Emplace(FString::Printf(Fmt, Args...));
	}

	template<typename TValue>
	void AddStatistic(const TCHAR* Name, TValue Value, FGenericShaderStat::EFlags Flags = FGenericShaderStat::EFlags::None, FName TagName = NAME_None)
	{
		// handle the case where a stat with the given name already exists; some backends compile iteratively and this is a simpler way to handle that
		// than requiring each backend to manually avoid setting until the "final" compilation. linear search is in practice fine here; there are low 
		// single digit numbers of stats for all existing shader formats at the time of writing.
		FName StatName(Name);
		FGenericShaderStat* Stat = nullptr;
		for (FGenericShaderStat& ExistingStat : ShaderStatistics)
		{
			if (ExistingStat.StatName == StatName)
			{
				Stat = &ExistingStat;
				break;
			}
		}

		if (!Stat)
		{
			Stat = &ShaderStatistics.AddZeroed_GetRef();
			Stat->StatName = StatName;
		}

		Stat->Value = FShaderStatVariant(TInPlaceType<TValue>(), Value);
		Stat->Flags = Flags;
		Stat->TagName = TagName;
	}

	const FShaderCodeResource& GetFinalizedCodeResource() const
	{
		return ShaderCode.GetFinalizedResource(Target.GetFrequency(), OutputHash);
	}

	void SetCodeFromResource(FShaderCodeResource&& Resource)
	{
		return ShaderCode.SetFromResource(MoveTemp(Resource));
	}

	// Bump ShaderCompileWorkerOutputVersion if FShaderCompilerOutput changes
	friend FArchive& operator<<(FArchive& Ar, FShaderCompilerOutput& Output)
	{
		// Note: this serialize is used to pass between UE and the shader compile worker, recompile both when modifying
		Ar << Output.ParameterMap;
		Ar << Output.Target;
		Ar << Output.bSerializingForCache;
		if (!Output.bSerializingForCache)
		{
			// skip serializing these fields when saving to cache/DDC; only needed when reading back results from workers
			Ar << Output.Errors;
			Ar << Output.ShaderCode;
			Ar << Output.ValidateInputHash;
			Ar << Output.CompileTime;
		}
		Ar << Output.bInterpolatorRemovalFailed;
		Ar << Output.bForceDumpDebugInfo;
		Ar << Output.bForceLogDiagnosticMessages;
		Ar << Output.OutputHash;
		Ar << Output.NumInstructions;
		Ar << Output.NumTextureSamplers;
		Ar << Output.bSucceeded;
		Ar << Output.ModifiedShaderSource;
		Ar << Output.ModifiedEntryPointName;
		Ar << Output.ShaderStatistics;

		// note: intentionally never serializing the following fields:
		// - PreprocessTime - it is always set in the cooker since we no longer run preprocessing in SCW
		// - bSupportsQueryingUsedAttributes - only used when compiling pipelines by subsequent stage compile steps, these are always executed in order in a single SCW job invocation
		// - UsedAttributes - as above

		return Ar;
	}
};

/** Serializable structure of diagnostic output from a SCW process. Include error code and timing statistics for the duration of a job batch. */
struct FShaderCompileWorkerDiagnostics
{
	/** Error code return from a ShaderCompileWorker process that terminated abnormally (not for shader syntax/semantic errors). See FSCWErrorCode::ECode for valid values. */
	int32 ErrorCode = 0;

	/** Timestamp when the ShaderCompileWorker entered the main entry point. */
	double EntryPointTimestamp = 0.0;

	/** Time in seconds before this batch was being processed. This either starts from when the process launched or since the last batch was finished. */
	double BatchPreparationTime = 0.0;

	/** Time in seconds it took the ShaderCompileWorker to process the entire job batch. */
	double BatchProcessTime = 0.0;

	/** Index of the batch that was processed by the same worker process. Helps diagnose how many batches a worker has processed. */
	int32 BatchIndex = 0;

	friend FArchive& operator<<(FArchive& Ar, FShaderCompileWorkerDiagnostics& Output)
	{
		Ar << Output.ErrorCode << Output.EntryPointTimestamp << Output.BatchPreparationTime << Output.BatchProcessTime << Output.BatchIndex;
		return Ar;
	}
};

#if PLATFORM_WINDOWS
extern RENDERCORE_API int HandleShaderCompileException(Windows::LPEXCEPTION_POINTERS Info, FString& OutExMsg, FString& OutCallStack);
#endif
extern RENDERCORE_API const IShaderFormat* FindShaderFormat(FName Format, const TArray<const IShaderFormat*>& ShaderFormats);

// Executes preprocessing for the given job, if the job is marked to be preprocessed independently prior to compilation.
extern RENDERCORE_API bool PreprocessShader(FShaderCommonCompileJob* Job);
extern RENDERCORE_API void CompileShader(const TArray<const IShaderFormat*>& ShaderFormats, FShaderCompileJob& Job, int32* CompileCount = nullptr);
extern RENDERCORE_API void CompileShaderPipeline(const TArray<const IShaderFormat*>& ShaderFormats, FShaderPipelineCompileJob* PipelineJob, int32* CompileCount = nullptr);

/**
 * Validates the format of a virtual shader file path.
 * Meant to be use as such: check(CheckVirtualShaderFilePath(VirtualFilePath));
 * CompileErrors output array is optional. If this is non-null, all validation errors are returned to this array instead of logging them to LogShaders.
 */
extern RENDERCORE_API bool CheckVirtualShaderFilePath(FStringView VirtualPath, TArray<FShaderCompilerError>* CompileErrors = nullptr);

/**
 * Fixes up the given virtual file path (substituting virtual platform path/autogen path for the given platform)
 */
extern RENDERCORE_API void FixupShaderFilePath(FString& VirtualFilePath, EShaderPlatform ShaderPlatform, const FName* ShaderPlatformName);

enum class EConvertAndStripFlags : uint8
{
	None = 0,
	NoSimdPadding = 1 << 0,
};
ENUM_CLASS_FLAGS(EConvertAndStripFlags);
/**
 * Utility function to strip comments and convert source to ANSI, useful for preprocessing
 */
extern RENDERCORE_API void ShaderConvertAndStripComments(const FString& ShaderSource, TArray<ANSICHAR>& OutStripped, EConvertAndStripFlags Flags = EConvertAndStripFlags::None);

/**
 * Loads the shader file with the given name.
 * @param VirtualFilePath - The virtual path of shader file to load.
 * @param OutFileContents - If true is returned, will contain the contents of the shader file. Can be null.
 * @return True if the file was successfully loaded.
 */
extern RENDERCORE_API bool LoadShaderSourceFile(const TCHAR* VirtualFilePath, EShaderPlatform ShaderPlatform, FString* OutFileContents, TArray<FShaderCompilerError>* OutCompileErrors, const FName* ShaderPlatformName = nullptr, FShaderSharedAnsiStringPtr* OutStrippedContents = nullptr);

extern RENDERCORE_API bool LoadCachedShaderSourceFile(const TCHAR* VirtualFilePath, EShaderPlatform ShaderPlatform, FShaderSharedStringPtr* OutFileContents, TArray<FShaderCompilerError>* OutCompileErrors, const FName* ShaderPlatformName = nullptr, FShaderSharedAnsiStringPtr* OutStrippedContents = nullptr);


struct FShaderPreprocessDependency
{
	FShaderHash					PathInSourceHash;		// PathInSourceHash doesn't include PathInSource's null terminator, so hash computation can use a string view
	TArray<ANSICHAR>			PathInSource;			// Path as it appears in include directive in original shader source, allowing faster case sensitive hash
	TArray<ANSICHAR>			ParentPath;				// For relative paths, ResultPath is dependent on the parent file the include directive is found in
	TArray<ANSICHAR>			ResultPath;
	uint32						ResultPathHash;			// Case insensitive hash of ResultPath (compatible with hash of corresponding FString)
	uint32						ResultPathUniqueIndex;	// Index of first instance of a given result path in Dependencies array
	FShaderSharedAnsiStringPtr	StrippedSource;			// Source with comments stripped out, and converted to ANSICHAR (output of ShaderConvertAndStripComments)

	inline bool EqualsPathInSource(const ANSICHAR* InPathInSource, int32 InPathInSourceLen, FShaderHash InPathInSourceHash, const ANSICHAR* InParentPath) const
	{
		// PathInSource is case sensitive, ParentPath is case insensitive.
		// If the path is absolute (starts with '/'), then the parent path isn't relevant, and shouldn't be checked.
		return
			PathInSourceHash == InPathInSourceHash &&
			(PathInSource[0] == '/' || !FCStringAnsi::Stricmp(ParentPath.GetData(), InParentPath)) &&
			!FCStringAnsi::Strncmp(PathInSource.GetData(), InPathInSource, InPathInSourceLen);
	}

	inline bool EqualsResultPath(const FString& InResultPath, uint32 InResultPathHash) const
	{
		return (ResultPathHash == InResultPathHash) && InResultPath.Equals(ResultPath.GetData(), ESearchCase::IgnoreCase);
	}

	inline bool EqualsResultPath(const ANSICHAR* InResultPath, uint32 InResultPathHash) const
	{
		return (ResultPathHash == InResultPathHash) && !FCStringAnsi::Stricmp(ResultPath.GetData(), InResultPath);
	}
};


// Structure that provides an array of #include dependencies for a given root shader file, including not just immediate
// dependencies, but recursive dependencies from children as well.  Not exhaustive, as it does not include platform
// specific or generated files, although it does include children of "/Engine/Generated/Material.ush", as derived from
// "/Engine/Private/MaterialTemplate.ush".  Take the example of ClearUAV.usf:
//
// /Engine/Private/Tools/ClearUAV.usf    #include "../Common.ush"
// /Engine/Private/Common.ush            #include "/Engine/Public/Platform.ush"
//                                       #include "PackUnpack.ush"
// /Engine/Public/Platform.ush           #include "FP16Math.ush"
//
// The above is a small subset, but the above (and many more) would all show up as elements in Dependencies:
//
//      PathInSource                   ParentPath                                ResultPath
//      --------------------------------------------------------------------------------------------------------
//      ../Common.ush                  /Engine/Private/Tools/ClearUAV.usf        /Engine/Private/Common.ush
//      /Engine/Public/Platform.ush    /Engine/Private/Common.ush                /Engine/Public/Platform.ush
//      PackUnpack.ush                 /Engine/Private/Common.ush                /Engine/Private/PackUnpack.ush
//      FP16Math.ush                   /Engine/Public/Platform.ush               /Engine/Public/FP16Math.ush
//
// The goal of this structure is to allow a shader preprocessor implementation to fetch most of the source dependencies in a
// single query of the loaded shader cache, and then efficiently search for dependencies encountered in the shader source
// code, without needing to do string operations to resolve paths (such as converting relative paths like "../Common.ush" to
// "/Engine/Private/Common.ush").  Besides that, the array organization can be used to manage encountered source files
// by index, rather than needing a map, and the "ResultPath" strings from this structure can be referenced by pointer,
// rather than needing to dynamically allocate a copy of the resolved path.  Lookups by PathInSource can use a much faster
// case sensitive hash, because PathInSource has verbatim capitalization from the source code files.  Altogether, this
// utility structure saves a bunch of shader cache query, hash, map, string, and memory allocation overhead.
//
struct FShaderPreprocessDependencies
{
	// First item in array contains stripped source for root file, and is not in the hash tables
	TArray<FShaderPreprocessDependency> Dependencies;
	FHashTable BySource;								// Hash table by PathInSource
	FHashTable ByResult;								// Hash table by ResultPath
};

typedef TSharedPtr<FShaderPreprocessDependencies, ESPMode::ThreadSafe> FShaderPreprocessDependenciesShared;

/**
 * Utility function that returns a root shader file plus all non-platform include dependencies in a single batch call, useful for preprocessing.
 */
extern RENDERCORE_API bool GetShaderPreprocessDependencies(const TCHAR* VirtualFilePath, EShaderPlatform ShaderPlatform, FShaderPreprocessDependenciesShared& OutDependencies);

enum class EShaderCompilerWorkerType : uint8
{
	None,
	LocalThread,
	Distributed,
};

enum class EShaderCompileJobType : uint8
{
	Single,
	Pipeline,
	Num,
};
inline constexpr int32 NumShaderCompileJobTypes = (int32)EShaderCompileJobType::Num;


