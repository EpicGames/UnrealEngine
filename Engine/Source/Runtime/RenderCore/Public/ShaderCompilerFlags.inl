// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ShaderCompilerFlags.inl: Flags used in ECompilerFlags enumeration.
=============================================================================*/

#if !defined(SHADER_COMPILER_FLAGS_ENTRY) 
#error SHADER_COMPILER_FLAGS_ENTRY must be defined before including ShaderCompilerFlags.inl
#endif

#if !defined(SHADER_COMPILER_FLAGS_ENTRY_DEPRECATED)
#define SHADER_COMPILER_FLAGS_ENTRY_DEPRECATED(Name, Version, Message)
#endif

// To add a flag:
// SHADER_COMPILER_FLAGS_ENTRY(FlagName) - generates CFLAG_FlagName
// For deprecated flags:
// SHADER_COMPILER_FLAGS_ENTRY_DEPRECATED(FlagName, X.X, "Deprecation Message") - generates CFLAG_FlagName UE_DEPRECATED(X.X, "Deprecation Message")

SHADER_COMPILER_FLAGS_ENTRY(PreferFlowControl)
SHADER_COMPILER_FLAGS_ENTRY(Debug)
SHADER_COMPILER_FLAGS_ENTRY(AvoidFlowControl)
// Disable shader validation
SHADER_COMPILER_FLAGS_ENTRY(SkipValidation)
// Only allows standard optimizations, not the longest compile times.
SHADER_COMPILER_FLAGS_ENTRY(StandardOptimization)
// Always optimize even whenDebug is set. Required for some complex shaders and features.
SHADER_COMPILER_FLAGS_ENTRY(ForceOptimization)
// Shader should generate symbols for debugging.
SHADER_COMPILER_FLAGS_ENTRY(GenerateSymbols)
// Shader should insert debug/name info at the risk of generating non-deterministic libraries
SHADER_COMPILER_FLAGS_ENTRY(ExtraShaderData)
// Allows the (external) symbols to be specific to each shader rather than trying to deduplicate.
SHADER_COMPILER_FLAGS_ENTRY(AllowUniqueSymbols)
SHADER_COMPILER_FLAGS_ENTRY(NoFastMath)
// Explicitly enforce zero initialization on shader platforms that may omit it.
SHADER_COMPILER_FLAGS_ENTRY(ZeroInitialise)
// Explicitly enforce bounds checking on shader platforms that may omit it.
SHADER_COMPILER_FLAGS_ENTRY(BoundsChecking)
// Force removing unused interpolators for platforms that can opt out
SHADER_COMPILER_FLAGS_ENTRY(ForceRemoveUnusedInterpolators)
// Hint that it is a vertex to geometry shader
SHADER_COMPILER_FLAGS_ENTRY(VertexToGeometryShader)
// Hint that it is a vertex to primitive shader
SHADER_COMPILER_FLAGS_ENTRY(VertexToPrimitiveShader)
// Hint that a vertex shader should use automatic culling on certain platforms.
SHADER_COMPILER_FLAGS_ENTRY(VertexUseAutoCulling)
// Prepare the shader for archiving in the native binary shader cache format
SHADER_COMPILER_FLAGS_ENTRY(Archive)
// Shaders uses external texture so may need special runtime handling
SHADER_COMPILER_FLAGS_ENTRY(UsesExternalTexture)
// Use emulated uniform buffers on supported platforms
SHADER_COMPILER_FLAGS_ENTRY(UseEmulatedUB)
// Enable wave operation intrinsics (requires DX12 and DXC/DXIL on PC).
// Check GRHISupportsWaveOperations before using shaders compiled with this flag at runtime.
// https://github.com/Microsoft/DirectXShaderCompiler/wiki/Wave-Intrinsics
SHADER_COMPILER_FLAGS_ENTRY(WaveOperations)
// Use DirectX Shader Compiler (DXC) to compile all shaders - intended for compatibility testing.
SHADER_COMPILER_FLAGS_ENTRY(ForceDXC)
SHADER_COMPILER_FLAGS_ENTRY(SkipOptimizations)
// Temporarily disable optimizations with DXC compiler only - intended to workaround shader compiler bugs until they can be resolved with 1st party
SHADER_COMPILER_FLAGS_ENTRY(SkipOptimizationsDXC)
// Typed UAV loads are disallowed by default as Windows 7 D3D 11.0 does not support them; this flag allows a shader to use them.
SHADER_COMPILER_FLAGS_ENTRY(AllowTypedUAVLoads)
// Prefer shader execution in wave32 mode if possible.
SHADER_COMPILER_FLAGS_ENTRY(Wave32)
// Enable support of inline raytracing in compute shader.
SHADER_COMPILER_FLAGS_ENTRY(InlineRayTracing)
// Enable support of C-style data types for platforms that can. Check for PLATFORM_SUPPORTS_REAL_TYPES and FDataDrivenShaderPlatformInfo::GetSupportsRealTypes()
SHADER_COMPILER_FLAGS_ENTRY(AllowRealTypes)
// Precompile HLSL to optimized HLSL then forward to FXC. Speeds up some shaders that take longer with FXC and works around crashes in FXC.
SHADER_COMPILER_FLAGS_ENTRY(PrecompileWithDXC)
// Enable HLSL 2021 version. Enables templates, operator overloading, and C++ style function overloading. Contains breaking change with short-circuiting evaluation.
SHADER_COMPILER_FLAGS_ENTRY(HLSL2021)
// Allow warnings to be treated as errors
SHADER_COMPILER_FLAGS_ENTRY(WarningsAsErrors)
SHADER_COMPILER_FLAGS_ENTRY_DEPRECATED(BindlessResources, 5.7, "This flag is now internal to the shader compiler.")
SHADER_COMPILER_FLAGS_ENTRY_DEPRECATED(BindlessSamplers, 5.7, "This flag is now internal to the shader compiler.")
// Force disable bindless resources and samplers on a shader
SHADER_COMPILER_FLAGS_ENTRY(ForceBindful)
// EXPERIMENTAL: Run the shader re-writer that removes any unused functions/resources/types from source code before compilation.
SHADER_COMPILER_FLAGS_ENTRY(RemoveDeadCode)
// Enable CullBeforeFetch optimization on supported platforms
SHADER_COMPILER_FLAGS_ENTRY(CullBeforeFetch)
// Enable WarpCulling optimization on supported platforms
SHADER_COMPILER_FLAGS_ENTRY(WarpCulling)
// Shader should generate minimal symbols info
SHADER_COMPILER_FLAGS_ENTRY(GenerateSymbolsInfo)
// Enabled root constants optimization on supported platforms
SHADER_COMPILER_FLAGS_ENTRY(RootConstants)
// Specifies that a shader provides derivatives, and the compiler should look in the compiled ISA for any instructions requiring
// auto derivatives. If none are found, the shader will be marked with EShaderResourceUsageFlags::NoDerivativeOps, meaning that
// calling code can safely assume only provided derivatives are used.
SHADER_COMPILER_FLAGS_ENTRY(CheckForDerivativeOps)
// Shader is used with indirect draws. This flag is currently used to fix a platform specific problem with certain (rare) indirect draw setups, but it is intended to be set for all indirect draw shaders in the future.
// Must not be used on shaders that are used with direct draws. Doing so might cause crashes or visual corruption on certain platforms.
SHADER_COMPILER_FLAGS_ENTRY(IndirectDraw)
// Shader is used with shader bundles.
SHADER_COMPILER_FLAGS_ENTRY(ShaderBundle)
// Shader code should not be stripped of comments/whitespace/line directives at the end of preprocessing
SHADER_COMPILER_FLAGS_ENTRY(DisableSourceStripping)
// Shader uses RHI Shader Binding Layout for global shader binding
SHADER_COMPILER_FLAGS_ENTRY(ShaderBindingLayout)
// Request full shader analysis artifacts in output statistics. This may contain multiple compilation steps in full text form.
SHADER_COMPILER_FLAGS_ENTRY(OutputAnalysisArtifacts)
// Force to generate debug info, i.e. FShaderConductorOptions::bEnableDebugInfo.
SHADER_COMPILER_FLAGS_ENTRY(ForceSpirvDebugInfo)

// Include in the "Minimal" bindless configuration.
SHADER_COMPILER_FLAGS_ENTRY(SupportsMinimalBindless)

#undef SHADER_COMPILER_FLAGS_ENTRY
#undef SHADER_COMPILER_FLAGS_ENTRY_DEPRECATED
