// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <atomic>
#include "HAL/PlatformTime.h"
#include "Hash/Blake3.h"
#include "Shader.h"
#include "ShaderCompilerCore.h"
#include "ShaderPreprocessTypes.h"
#include "VertexFactory.h"
#include "Templates/Function.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"

class FShaderCommonCompileJob;
class FShaderCompileJob;
class FShaderPipelineCompileJob;
struct FShaderCacheSerializeContext;

namespace UE::DerivedData { class FRequestOwner; }

/** Results for a single compiled shader map. */
struct FShaderMapCompileResults
{
	FShaderMapCompileResults() :
		bAllJobsSucceeded(true),
		bSkipResultProcessing(false),
		TimeStarted(FPlatformTime::Seconds()),
		bIsHung(false)
	{}

	void CheckIfHung();

	TArray<TRefCountPtr<class FShaderCommonCompileJob>> FinishedJobs;
	FThreadSafeCounter NumPendingJobs;
	bool bAllJobsSucceeded;
	bool bSkipResultProcessing;
	double TimeStarted;
	bool bIsHung;
};

struct FPendingShaderMapCompileResults
	: public FShaderMapCompileResults
	, public FRefCountedObject
{};
using FPendingShaderMapCompileResultsPtr = TRefCountPtr<FPendingShaderMapCompileResults>;


/**
 * Cached reference to the location of an in-flight job's FShaderJobData in the FShaderJobDataMap, used by the private FShaderJobCache class.
 *
 * Caching the reference avoids the need to do additional map lookups to find the entry again, potentially avoiding a lock of the container for
 * the lookup.  Heap allocation of blocks is used by the cache to allow map entries to have a persistent location in memory.  The persistent
 * memory allows modifications of map entry data for a given job, without needing locks to protect against container resizing.
 *
 * In-flight jobs and their duplicates reference the same FShaderJobData.  Client code should treat this structure as opaque.
 */
struct FShaderJobCacheRef
{
	/** Pointer to block the private FShaderJobData is stored in */
	struct FShaderJobDataBlock* Block = nullptr;

	/** Index of FShaderJobData in the block */
	int32 IndexInBlock = INDEX_NONE;

	/** If job is a duplicate, index of pointer to job in DuplicateJobs array in FShaderJobCache, used for clearing the pointer when the in-flight job completes */
	int32 DuplicateIndex = INDEX_NONE;

	void Clear()
	{
		Block = nullptr;
		IndexInBlock = INDEX_NONE;
		DuplicateIndex = INDEX_NONE;
	}
};

enum class EShaderCompileJobStatus : uint8
{
	/* Initial status, created but not yet manipulated */
	Unset,
	/* Ready for execution, set after preprocessing and input hash computation is completed */
	Ready,
	/* Job execution was skipped due to a preprocessing failure */
	Skipped,
	/* Job was cancelled by owning system */
	Cancelled,
	/* DDC query for job has been submitted and is waiting on results */
	PendingDDC,
	/* Job is queued in the shader compilation system, but not yet picked up by a compilation thread (i.e. not a cache or DDC hit) */
	Queued,
	/* Job has been picked up by a compilation thread, has been submitted for compilation is pending execution completion */
	Pending,
	/* Job has been completed and results are available for processing */
	Completed,
	/* Job results have been read and processed into the owning shadermap, and job has been released by the shader compilation system.
	*  Note that the job may remain alive if other systems are holding references to it at this point.
	*/
	Released
};

enum class EShaderCompileJobCompletionMethod
{
	/* Initial value, execution method not yet determined */
	Unset,
	/* Job was found in the in-memory job cache */
	CacheHit,
	/* Job was found in DDC */
	DdcHit,
	/* Job was a duplicate of another and shared its results */
	Duplicate,
	/* Job was executed by a local shader compilation mechanism (local shader compile thread or local fallback) */
	LocalExecution,
	/* Job was executed by a distributed shader compilation mechanism (note that this could still have executed on the local machine, but coordinated by a distributed compiler) */
	DistributedExecution,
};

RENDERCORE_API const TCHAR* LexToString(EShaderCompileJobStatus Status);
RENDERCORE_API const TCHAR* LexToString(EShaderCompileJobCompletionMethod Method);

class FShaderCompileJobStatus
{
public:
	void Reset()
	{
		// Unset status before the input hash since GetInputHash() uses it as a substitute for lock guards
		Status.store(EShaderCompileJobStatus::Unset);
		CompletionMethod.store(EShaderCompileJobCompletionMethod::Unset);

		InputHash = nullptr;
	}

	/* Returns true if the job has been completed (i.e. is in "Completed" or "Released" phase) */
	bool IsCompleted() const
	{
		EShaderCompileJobStatus LocalStatus = GetStatus();
		return
			(LocalStatus == EShaderCompileJobStatus::Completed) ||
			(LocalStatus == EShaderCompileJobStatus::Released);
	}

	/* Returns true if the owning job is completed but the compile step was not executed. */
	bool WasCompilationSkipped() const
	{
		EShaderCompileJobStatus LocalStatus = GetStatus();
		EShaderCompileJobCompletionMethod LocalCompletionMethod = GetCompletionMethod();
		bool bIsCompleted = IsCompleted();

		return
			(LocalStatus == EShaderCompileJobStatus::Skipped) ||
			(bIsCompleted &&
			(LocalCompletionMethod == EShaderCompileJobCompletionMethod::Duplicate ||
			LocalCompletionMethod == EShaderCompileJobCompletionMethod::CacheHit ||
			LocalCompletionMethod == EShaderCompileJobCompletionMethod::DdcHit));
	}

	void SetStatus(EShaderCompileJobStatus InStatus)
	{
		Status.store(InStatus);
	}

	void SetStatus(EShaderCompileJobStatus InStatus, EShaderCompileJobCompletionMethod InCompletionMethod)
	{
		SetCompletionMethod(InCompletionMethod);
		SetStatus(InStatus);
	}

	void SetCompletionMethod(EShaderCompileJobCompletionMethod InCompletionMethod)
	{
		CompletionMethod.store(InCompletionMethod);
	}

	EShaderCompileJobCompletionMethod GetCompletionMethod() const
	{
		return CompletionMethod.load();
	}

	EShaderCompileJobStatus GetStatus() const
	{
		return Status.load();
	}

	void SetInputHash(const FShaderCompilerInputHash* InInputHash)
	{
		InputHash = InInputHash;
	}

	const FShaderCompilerInputHash& GetInputHash() const
	{
		return InputHash ? *InputHash : FShaderCompilerInputHash::Zero;
	}

private:
	const FShaderCompilerInputHash* InputHash = nullptr; // set when input hash is computed during submission
	std::atomic<EShaderCompileJobStatus> Status{ EShaderCompileJobStatus::Unset };
	std::atomic<EShaderCompileJobCompletionMethod> CompletionMethod{ EShaderCompileJobCompletionMethod::Unset };
};

/** Stores all of the common information used to compile a shader or pipeline. */
class FShaderCommonCompileJob
{
public:
	/** Linked list support -- not using TIntrusiveLinkedList because we want lock free insertion not supported by the core class */
	FShaderCommonCompileJob* NextLink = nullptr;
	FShaderCommonCompileJob** PrevLink = nullptr;

	FPendingShaderMapCompileResultsPtr PendingShaderMap;

	mutable FThreadSafeCounter NumRefs;
	int32 JobIndex;
	uint32 Hash;

	/** Id of the shader map this shader belongs to. */
	uint32 Id;

	EShaderCompileJobType Type;

	/** Priority that is requested for this job. */
	EShaderCompileJobPriority Priority;

	/**
	 * Priority bucket this job has been assigned to while it's in the pending queue.
	 * Will be reset to none once the job is picked up from the queue by one of the workers.
	 */
	EShaderCompileJobPriority PendingPriority;

	EShaderCompilerWorkerType CurrentWorker;

	TPimplPtr<UE::DerivedData::FRequestOwner> RequestOwner;

	/** True if the compile job executed successfully, false if errors were encountered */
	uint8 bSucceeded : 1;
	/** True if errors encountered by this job are likely to be caused by code problems (inferred from shader type and whether or not shader dev mode is enabled) */
	uint8 bErrorsAreLikelyToBeCode : 1;
	/** Whether or not we are a default material. */
	uint8 bIsDefaultMaterial : 1;
	/** Whether or not we are a global shader. */
	uint8 bIsGlobalShader : 1;
	/** Whether or not to bypass the job/ddc caches when executing this job */
	uint8 bBypassCache : 1;
	/** Whether this job has been transferred after submitted to a remote worker. If this is true, don't register this as an assigned job again. */
	uint8 bTransferred : 1;


	/** Whether inputs have been hash and the below input hash has been set (note: intentionally separate from above bitfield to avoid data race due
		to code accessing other bitfield members from game thread while submission task is running in parallel on a worker thread) */
	uint8 bInputHashSet;
	/** Hash of all the job inputs */
	FShaderCompilerInputHash InputHash;

	/** In-engine timestamp of being added to a pending queue. Not set for jobs that are satisfied from the jobs cache */
	double TimeAddedToPendingQueue = 0.0;
	/** In-engine timestamp of being assigned to a worker. Not set for jobs that are satisfied from the jobs cache */
	double TimeAssignedToExecution = 0.0;
	/** In-engine timestamp of job being completed. Encompasses the compile time. Not set for jobs that are satisfied from the jobs cache */
	double TimeExecutionCompleted = 0.0;

	FShaderJobCacheRef JobCacheRef;

	TSharedPtr<FShaderCompileJobStatus> JobStatusPtr;

	void UpdateInputHash()
	{
		(void)GetInputHash();
		check(bInputHashSet);
		JobStatusPtr->SetInputHash(&InputHash);
	}

	void UpdateStatus(EShaderCompileJobStatus NewStatus)
	{
		JobStatusPtr->SetStatus(NewStatus);
	}

	void UpdateStatus(EShaderCompileJobStatus InStatus, EShaderCompileJobCompletionMethod InCompletionMethod)
	{
		// InCompletionMethod defaults to Unset if not explicitly passed; intended behaviour is to keep the 
		// existing value for execution method rather than overwriting with "unset".
		if (InCompletionMethod != EShaderCompileJobCompletionMethod::Unset)
		{
			JobStatusPtr->SetStatus(InStatus, InCompletionMethod);
		}
		else
		{
			JobStatusPtr->SetStatus(InStatus);
		}
	}

	EShaderCompileJobStatus GetStatus() const
	{
		return JobStatusPtr->GetStatus();
	}
	
	uint32 AddRef() const
	{
		return uint32(NumRefs.Increment());
	}

	uint32 Release() const
	{
		uint32 Refs = uint32(NumRefs.Decrement());
		if (Refs == 0)
		{
			Destroy();
		}
		return Refs;
	}
	uint32 GetRefCount() const
	{
		return uint32(NumRefs.GetValue());
	}

	/** Returns hash of all inputs for this job (needed for caching). */
	virtual FShaderCompilerInputHash GetInputHash() { return FShaderCompilerInputHash(); }

	/** Serializes (and deserializes) the output for caching purposes. */
	virtual void SerializeOutput(FShaderCacheSerializeContext& Ctx) {}

	/** Generates a diagnostics string for this compile job suitable for the output log. */
	virtual void AppendDiagnostics(FString& OutDiagnostics, int32 InJobIndex, int32 InNumJobs, const TCHAR* Indentation = nullptr) const = 0;

	FShaderCompileJob* GetSingleShaderJob();
	const FShaderCompileJob* GetSingleShaderJob() const;
	FShaderPipelineCompileJob* GetShaderPipelineJob();
	const FShaderPipelineCompileJob* GetShaderPipelineJob() const;

	UE_DEPRECATED(5.7, "Use overload of OnComplete accepting a context object")
	virtual void OnComplete() = 0;

	// Executed for all jobs (including those read from cache) on completion.
	virtual void OnComplete(FShaderDebugDataContext& Ctx) = 0;

	virtual void AppendDebugName(FStringBuilderBase& OutName) const = 0;

	bool Equals(const FShaderCommonCompileJob& Rhs) const;

	/** Calls the specified predicate for each single compile job, i.e. FShaderCompileJob and each stage of FShaderPipelineCompileJob. */
	void ForEachSingleShaderJob(const TFunction<void(const FShaderCompileJob& SingleJob)>& Predicate) const;
	void ForEachSingleShaderJob(const TFunction<void(FShaderCompileJob& SingleJob)>& Predicate);

	/** This returns a unique id for a shader compiler job */
	RENDERCORE_API static uint32 GetNextJobId();

protected:
	friend class FShaderCompilingManager;
	friend class FShaderPipelineCompileJob;

	FShaderCommonCompileJob(EShaderCompileJobType InType, uint32 InHash, uint32 InId, EShaderCompileJobPriority InPriority) :
		NumRefs(0),
		JobIndex(INDEX_NONE),
		Hash(InHash),
		Id(InId),
		Type(InType),
		Priority(InPriority),
		PendingPriority(EShaderCompileJobPriority::None),
		CurrentWorker(EShaderCompilerWorkerType::None),
		bSucceeded(false),
		bErrorsAreLikelyToBeCode(false),
		bIsDefaultMaterial(false),
		bIsGlobalShader(false),
		bBypassCache(false),
		bTransferred(false),
		bInputHashSet(false)
	{
		check(InPriority != EShaderCompileJobPriority::None);

		JobStatusPtr = MakeShared<FShaderCompileJobStatus>();
	}

	virtual ~FShaderCommonCompileJob() = default;

private:
	/** Value counter for job ids. */
	static FThreadSafeCounter JobIdCounter;

	void Destroy() const;
};
using FShaderCommonCompileJobPtr = TRefCountPtr<FShaderCommonCompileJob>;

struct FShaderCompileJobKey
{
	explicit FShaderCompileJobKey(const FShaderType* InType = nullptr, EShaderPlatform InPlatform = EShaderPlatform::SP_PCD3D_SM6, const FVertexFactoryType* InVFType = nullptr, int32 InPermutationId = 0)
		: ShaderType(InType), VFType(InVFType), PermutationId(InPermutationId), Platform(InPlatform)
	{}

	uint32 MakeHash(uint32 Id) const { return HashCombine(HashCombine(HashCombine(HashCombine(GetTypeHash(Id), GetTypeHash(VFType)), GetTypeHash(ShaderType)), GetTypeHash(PermutationId)), GetTypeHash(Platform)); }
	RENDERCORE_API FString ToString() const;
	const FShaderType* ShaderType;
	const FVertexFactoryType* VFType;
	int32 PermutationId;
	EShaderPlatform Platform;

	friend inline bool operator==(const FShaderCompileJobKey& Lhs, const FShaderCompileJobKey& Rhs)
	{
		return Lhs.VFType == Rhs.VFType && Lhs.ShaderType == Rhs.ShaderType && Lhs.PermutationId == Rhs.PermutationId && Lhs.Platform == Rhs.Platform;
	}
	friend inline bool operator!=(const FShaderCompileJobKey& Lhs, const FShaderCompileJobKey& Rhs)
	{
		return !operator==(Lhs, Rhs);
	}
};

/** Stores all of the input and output information used to compile a single shader. */
class FShaderCompileJob : public FShaderCommonCompileJob
{
public:
	static const EShaderCompileJobType Type = EShaderCompileJobType::Single;

	FShaderCompileJobKey Key;

	/** 
	 * Additional parameters that can be supplied to the compile job such 
	 * that it is available from the compilation begins to when the FShader is created.
	 */
	TSharedPtr<const FShaderType::FParameters, ESPMode::ThreadSafe> ShaderParameters;

	/** Input for the shader compile */
	FShaderCompilerInput Input;
	FShaderPreprocessOutput PreprocessOutput;
	TUniquePtr<FShaderPreprocessOutput> SecondaryPreprocessOutput{};
	FShaderCompilerOutput Output;
	TUniquePtr<FShaderCompilerOutput> SecondaryOutput{};

	// List of pipelines that are sharing this job.
	TMap<const FVertexFactoryType*, TArray<const FShaderPipelineType*>> SharingPipelines;

	virtual RENDERCORE_API FShaderCompilerInputHash GetInputHash() override;

	RENDERCORE_API void SerializeOutput(FShaderCacheSerializeContext& Ctx, int32 CodeIndex);
	virtual void SerializeOutput(FShaderCacheSerializeContext& Ctx)
	{
		SerializeOutput(Ctx, 0);
	}

	UE_DEPRECATED(5.7, "OnComplete now accepts a FDebugDataContext object")
	virtual void OnComplete() override {}

	virtual RENDERCORE_API void OnComplete(FShaderDebugDataContext& Ctx) override;
	
	virtual RENDERCORE_API void AppendDebugName(FStringBuilderBase& OutName) const override;

	// Serializes only the subset of data written by SCW/read back from ShaderCompiler when using worker processes.
	RENDERCORE_API void SerializeWorkerOutput(FArchive& Ar);
	
	// Serializes only the subset of data written by ShaderCompiler and read from SCW when using worker processes.
	RENDERCORE_API void SerializeWorkerInput(FArchive& Ar);

	void SerializeWorkerInputNoSource(FArchive& Ar);

	// Serializes the compile job for a cook artifact for later analysis in a commandlet.
	RENDERCORE_API void SerializeArtifact(FArchive& Ar);

	RENDERCORE_API FStringView GetFinalSourceView() const;

	virtual RENDERCORE_API void AppendDiagnostics(FString& OutDiagnostics, int32 InJobIndex, int32 InNumJobs, const TCHAR* Indentation = nullptr) const override final;

	FShaderCompileJob() : FShaderCommonCompileJob(Type, 0u, 0u, EShaderCompileJobPriority::Num)
	{}

	FShaderCompileJob(uint32 InHash, uint32 InId, EShaderCompileJobPriority InPriroity, const FShaderCompileJobKey& InKey) :
		FShaderCommonCompileJob(Type, InHash, InId, InPriroity),
		Key(InKey)
	{}

private:
	void SerializeWorkerOutputInner(FArchive& Ar, bool bSerializeForArtifact = false);

};

struct FShaderPipelineCompileJobKey
{
	explicit FShaderPipelineCompileJobKey(const FShaderPipelineType* InType = nullptr, EShaderPlatform InPlatform = EShaderPlatform::SP_PCD3D_SM6, const FVertexFactoryType* InVFType = nullptr, int32 InPermutationId = 0)
		: ShaderPipeline(InType), VFType(InVFType), PermutationId(InPermutationId), Platform(InPlatform)
	{}

	uint32 MakeHash(uint32 Id) const { return HashCombine(HashCombine(HashCombine(HashCombine(GetTypeHash(Id), GetTypeHash(ShaderPipeline)), GetTypeHash(VFType)), GetTypeHash(PermutationId)), GetTypeHash(Platform)); }

	const FShaderPipelineType* ShaderPipeline;
	const FVertexFactoryType* VFType;
	int32 PermutationId;
	EShaderPlatform Platform;

	friend inline bool operator==(const FShaderPipelineCompileJobKey& Lhs, const FShaderPipelineCompileJobKey& Rhs)
	{
		return Lhs.Platform == Rhs.Platform && Lhs.ShaderPipeline == Rhs.ShaderPipeline && Lhs.VFType == Rhs.VFType && Lhs.PermutationId == Rhs.PermutationId;
	}
	friend inline bool operator!=(const FShaderPipelineCompileJobKey& Lhs, const FShaderPipelineCompileJobKey& Rhs)
	{
		return !operator==(Lhs, Rhs);
	}
};

class FShaderPipelineCompileJob : public FShaderCommonCompileJob
{
public:
	static const EShaderCompileJobType Type = EShaderCompileJobType::Pipeline;

	FShaderPipelineCompileJobKey Key;
	TArray<TRefCountPtr<FShaderCompileJob>> StageJobs;

	virtual RENDERCORE_API FShaderCompilerInputHash GetInputHash() override;

	virtual RENDERCORE_API void SerializeOutput(FShaderCacheSerializeContext& Ctx) override;

	UE_DEPRECATED(5.7, "OnComplete now accepts a FDebugDataContext object")
	void OnComplete() override {};
	virtual RENDERCORE_API void OnComplete(FShaderDebugDataContext& Ctx) override;
	virtual RENDERCORE_API void AppendDebugName(FStringBuilderBase& OutName) const override;
	virtual RENDERCORE_API void AppendDiagnostics(FString& OutDiagnostics, int32 InJobIndex, int32 InNumJobs, const TCHAR* Indentation = nullptr) const override final;

	RENDERCORE_API FShaderPipelineCompileJob(int32 NumStages);
	RENDERCORE_API FShaderPipelineCompileJob(uint32 InHash, uint32 InId, EShaderCompileJobPriority InPriroity, const FShaderPipelineCompileJobKey& InKey);
};

inline FShaderCompileJob* FShaderCommonCompileJob::GetSingleShaderJob() { return Type == EShaderCompileJobType::Single ? static_cast<FShaderCompileJob*>(this) : nullptr; }
inline const FShaderCompileJob* FShaderCommonCompileJob::GetSingleShaderJob() const { return Type == EShaderCompileJobType::Single ? static_cast<const FShaderCompileJob*>(this) : nullptr; }
inline FShaderPipelineCompileJob* FShaderCommonCompileJob::GetShaderPipelineJob() { return Type == EShaderCompileJobType::Pipeline ? static_cast<FShaderPipelineCompileJob*>(this) : nullptr; }
inline const FShaderPipelineCompileJob* FShaderCommonCompileJob::GetShaderPipelineJob() const { return Type == EShaderCompileJobType::Pipeline ? static_cast<const FShaderPipelineCompileJob*>(this) : nullptr; }

inline bool FShaderCommonCompileJob::Equals(const FShaderCommonCompileJob& Rhs) const
{
	if (Type == Rhs.Type && Id == Rhs.Id)
	{
		switch (Type)
		{
		case EShaderCompileJobType::Single: return static_cast<const FShaderCompileJob*>(this)->Key == static_cast<const FShaderCompileJob&>(Rhs).Key;
		case EShaderCompileJobType::Pipeline: return static_cast<const FShaderPipelineCompileJob*>(this)->Key == static_cast<const FShaderPipelineCompileJob&>(Rhs).Key;
		default: checkNoEntry(); break;
		}
	}
	return false;
}

inline void FShaderCommonCompileJob::Destroy() const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FShaderCommonCompileJob::Destroy);

	switch (Type)
	{
	case EShaderCompileJobType::Single: delete static_cast<const FShaderCompileJob*>(this); break;
	case EShaderCompileJobType::Pipeline: delete static_cast<const FShaderPipelineCompileJob*>(this); break;
	default: checkNoEntry();
	}
}

inline void FShaderCommonCompileJob::ForEachSingleShaderJob(const TFunction<void(const FShaderCompileJob&)>& Function) const
{
	if (const FShaderCompileJob* SingleJob = GetSingleShaderJob())
	{
		Function(*SingleJob);
	}
	else if (const FShaderPipelineCompileJob* PipelineJob = GetShaderPipelineJob())
	{
		for (const TRefCountPtr<FShaderCompileJob>& StageJob : PipelineJob->StageJobs)
		{
			if (const FShaderCompileJob* SingleStageJob = StageJob->GetSingleShaderJob())
			{
				Function(*SingleStageJob);
			}
		}
	}
}

inline void FShaderCommonCompileJob::ForEachSingleShaderJob(const TFunction<void(FShaderCompileJob&)>& Function)
{
	if (FShaderCompileJob* SingleJob = GetSingleShaderJob())
	{
		Function(*SingleJob);
	}
	else if (FShaderPipelineCompileJob* PipelineJob = GetShaderPipelineJob())
	{
		for (TRefCountPtr<FShaderCompileJob>& StageJob : PipelineJob->StageJobs)
		{
			if (FShaderCompileJob* SingleStageJob = StageJob->GetSingleShaderJob())
			{
				Function(*SingleStageJob);
			}
		}
	}
}
