// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ShaderCore.h: Shader core module definitions.
=============================================================================*/

#pragma once

#include "Compression/OodleDataCompression.h"
#include "Compression/CompressedBuffer.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "Containers/Map.h"
#include "Containers/SortedMap.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "HAL/UnrealMemory.h"
#include "Logging/LogMacros.h"
#include "Memory/CompositeBuffer.h"
#include "Memory/SharedBuffer.h"
#include "Misc/AssertionMacros.h"
#include "Misc/CString.h"
#include "Misc/CoreStats.h"
#include "Misc/EnumClassFlags.h"
#include "Misc/Optional.h"
#include "Misc/Paths.h"
#include "Misc/SecureHash.h"
#include "Misc/TVariant.h"
#include "PixelFormat.h"
#include "RHIDefinitions.h"
#include "RHIShaderBindingLayout.h"
#include "Serialization/Archive.h"
#include "Serialization/MemoryLayout.h"
#include "ShaderParameterMetadata.h"
#include "Stats/Stats.h"
#include "Templates/Function.h"
#include "Templates/PimplPtr.h"
#include "Templates/RefCounting.h"
#include "Templates/SharedPointer.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealNames.h"
#include "UniformBuffer.h"

class Error;
class FMemoryImageWriter;
class FMemoryUnfreezeContent;
class FPointerTableBase;
class FShaderCompilerDefinitions;
class FShaderCompileUtilities;
class FShaderKeyGenerator;
class FShaderPreprocessorUtilities;
class ITargetPlatform;

using FShaderStatVariant = TVariant<bool, float, int32, uint32, FString>;

bool operator==(const FShaderStatVariant& LHS, const FShaderStatVariant& RHS);
bool operator<(const FShaderStatVariant& LHS, const FShaderStatVariant& RHS);

namespace FShaderStatTagNames
{
	/** Tag name for shader analysis artifacts. See CFLAG_OutputAnalysisArtifacts. */
	extern RENDERCORE_API const FName AnalysisArtifactsName;
}

struct FGenericShaderStat
{
public:
	enum class EFlags : uint8
	{
		None = 0,
		Hidden = 1 << 0, // If set this stat will not be shown to the user in the shader stats UI (i.e. stat is for internal use only)
	};
	FName StatName;
	FShaderStatVariant Value;
	EFlags Flags = EFlags::None;
	FName TagName;

	FGenericShaderStat() = default;

	friend FArchive& operator<<(FArchive& Ar, FGenericShaderStat& Stat) { Stat.StreamArchive(Ar); return Ar; }
	bool operator==(const FGenericShaderStat& RHS) const;
	bool operator<(const FGenericShaderStat& RHS) const;
	friend FStringBuilderBase& operator<<(FStringBuilderBase& Builder, const FGenericShaderStat& Stat);

	RENDERCORE_API void StreamArchive(FArchive& Ar);
};

CSV_DECLARE_CATEGORY_MODULE_EXTERN(RENDERCORE_API, Shaders);
/**
 * Controls whether shader related logs are visible.
 * Note: The runtime verbosity is driven by the console variable 'r.ShaderDevelopmentMode'
 */
#if UE_BUILD_DEBUG && (PLATFORM_UNIX)
RENDERCORE_API DECLARE_LOG_CATEGORY_EXTERN(LogShaders, Log, All);
#else
RENDERCORE_API DECLARE_LOG_CATEGORY_EXTERN(LogShaders, Error, All);
#endif

DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Num Total Niagara Shaders"), STAT_ShaderCompiling_NumTotalNiagaraShaders, STATGROUP_ShaderCompiling, RENDERCORE_API);
DECLARE_FLOAT_ACCUMULATOR_STAT_EXTERN(TEXT("Total Niagara Shader Compiling Time"), STAT_ShaderCompiling_NiagaraShaders, STATGROUP_ShaderCompiling, RENDERCORE_API);

DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Num Total OpenColorIO Shaders"), STAT_ShaderCompiling_NumTotalOpenColorIOShaders, STATGROUP_ShaderCompiling, RENDERCORE_API);
DECLARE_FLOAT_ACCUMULATOR_STAT_EXTERN(TEXT("Total OpenColorIO Shader Compiling Time"), STAT_ShaderCompiling_OpenColorIOShaders, STATGROUP_ShaderCompiling, RENDERCORE_API);

DECLARE_FLOAT_ACCUMULATOR_STAT_EXTERN(TEXT("Total Material Shader Compiling Time"),STAT_ShaderCompiling_MaterialShaders,STATGROUP_ShaderCompiling, RENDERCORE_API);
DECLARE_FLOAT_ACCUMULATOR_STAT_EXTERN(TEXT("Total Global Shader Compiling Time"),STAT_ShaderCompiling_GlobalShaders,STATGROUP_ShaderCompiling, RENDERCORE_API);
DECLARE_FLOAT_ACCUMULATOR_STAT_EXTERN(TEXT("RHI Compile Time"),STAT_ShaderCompiling_RHI,STATGROUP_ShaderCompiling, RENDERCORE_API);
DECLARE_FLOAT_ACCUMULATOR_STAT_EXTERN(TEXT("Loading Shader Files"),STAT_ShaderCompiling_LoadingShaderFiles,STATGROUP_ShaderCompiling, RENDERCORE_API);
DECLARE_FLOAT_ACCUMULATOR_STAT_EXTERN(TEXT("CRCing Shader Files"),STAT_ShaderCompiling_HashingShaderFiles,STATGROUP_ShaderCompiling, RENDERCORE_API);
DECLARE_FLOAT_ACCUMULATOR_STAT_EXTERN(TEXT("HLSL Translation"), STAT_ShaderCompiling_HLSLTranslation, STATGROUP_ShaderCompiling, RENDERCORE_API);
DECLARE_FLOAT_ACCUMULATOR_STAT_EXTERN(TEXT("DDC Loading"),STAT_ShaderCompiling_DDCLoading,STATGROUP_ShaderCompiling, RENDERCORE_API);
DECLARE_FLOAT_ACCUMULATOR_STAT_EXTERN(TEXT("Material Loading"),STAT_ShaderCompiling_MaterialLoading,STATGROUP_ShaderCompiling, RENDERCORE_API);
DECLARE_FLOAT_ACCUMULATOR_STAT_EXTERN(TEXT("Material Compiling"),STAT_ShaderCompiling_MaterialCompiling,STATGROUP_ShaderCompiling, RENDERCORE_API);
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Num Total Material Shaders"),STAT_ShaderCompiling_NumTotalMaterialShaders,STATGROUP_ShaderCompiling, RENDERCORE_API);
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Num Special Material Shaders"),STAT_ShaderCompiling_NumSpecialMaterialShaders,STATGROUP_ShaderCompiling, RENDERCORE_API);
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Num Particle Material Shaders"),STAT_ShaderCompiling_NumParticleMaterialShaders,STATGROUP_ShaderCompiling, RENDERCORE_API);
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Num Skinned Material Shaders"),STAT_ShaderCompiling_NumSkinnedMaterialShaders,STATGROUP_ShaderCompiling, RENDERCORE_API);
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Num Lit Material Shaders"),STAT_ShaderCompiling_NumLitMaterialShaders,STATGROUP_ShaderCompiling, RENDERCORE_API);
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Num Unlit Material Shaders"),STAT_ShaderCompiling_NumUnlitMaterialShaders,STATGROUP_ShaderCompiling, RENDERCORE_API);
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Num Transparent Material Shaders"),STAT_ShaderCompiling_NumTransparentMaterialShaders,STATGROUP_ShaderCompiling, RENDERCORE_API);
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Num Opaque Material Shaders"),STAT_ShaderCompiling_NumOpaqueMaterialShaders,STATGROUP_ShaderCompiling, RENDERCORE_API);
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Num Masked Material Shaders"),STAT_ShaderCompiling_NumMaskedMaterialShaders,STATGROUP_ShaderCompiling, RENDERCORE_API);

// Shaders that have been loaded in memory (in a compressed form). Can also be called "preloaded" shaders.
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Num Shaders Loaded"),STAT_Shaders_NumShadersLoaded,STATGROUP_Shaders, RENDERCORE_API);
// Shaders that have been created as an RHI shader (i.e. exist in terms of underlying graphics API). Can also be called "used" shaders, although in theory some code may create but not use.
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Num Shaders Created"), STAT_Shaders_NumShadersCreated, STATGROUP_Shaders, RENDERCORE_API);
// Shaders maps (essentially, material assets) that have been loaded in memory. This squarely correlates with the number of assets in memory (but assets can share SMs or have more than one SM - e.g. diff quality levels).
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Num ShaderMaps Loaded"), STAT_Shaders_NumShaderMaps,STATGROUP_Shaders, RENDERCORE_API);
// Shaders maps that has had RHI shaders created for them. This usually means that the asset referencing this shadermap was rendered at least in some pass.
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Num ShaderMaps Used"), STAT_Shaders_NumShaderMapsUsedForRendering, STATGROUP_Shaders, RENDERCORE_API);

DECLARE_CYCLE_STAT_EXTERN(TEXT("RT Shader Load Time"),STAT_Shaders_RTShaderLoadTime,STATGROUP_Shaders, RENDERCORE_API);

DECLARE_MEMORY_STAT_EXTERN(TEXT("Shader Memory"),STAT_Shaders_ShaderMemory,STATGROUP_Shaders, RENDERCORE_API);
DECLARE_MEMORY_STAT_EXTERN(TEXT("Shader Resource Mem"),STAT_Shaders_ShaderResourceMemory,STATGROUP_Shaders, RENDERCORE_API);
DECLARE_MEMORY_STAT_EXTERN(TEXT("Shader Preload Mem"), STAT_Shaders_ShaderPreloadMemory, STATGROUP_Shaders, RENDERCORE_API);

inline TStatId GetMemoryStatType(EShaderFrequency ShaderFrequency)
{
	static_assert(12 == SF_NumFrequencies, "EShaderFrequency has a bad size.");

	switch(ShaderFrequency)
	{
		case SF_Pixel:					return GET_STATID(STAT_PixelShaderMemory);
		case SF_Compute:				return GET_STATID(STAT_PixelShaderMemory);
		case SF_RayGen:					return GET_STATID(STAT_PixelShaderMemory);
		case SF_RayMiss:				return GET_STATID(STAT_PixelShaderMemory);
		case SF_RayHitGroup:			return GET_STATID(STAT_PixelShaderMemory);
		case SF_RayCallable:			return GET_STATID(STAT_PixelShaderMemory);
		case SF_WorkGraphRoot:			return GET_STATID(STAT_PixelShaderMemory);
		case SF_WorkGraphComputeNode:	return GET_STATID(STAT_PixelShaderMemory);
	}
	return GET_STATID(STAT_VertexShaderMemory);
}

/** Initializes shader hash cache from IShaderFormatModules. This must be called before reading any shader include. */
extern RENDERCORE_API void InitializeShaderHashCache();

/** Updates the PreviewPlatform's IncludeDirectory to match that of the Parent Platform*/
extern RENDERCORE_API void UpdateIncludeDirectoryForPreviewPlatform(EShaderPlatform PreviewPlatform, EShaderPlatform ActualPlatform);

/** Checks if shader include isn't skipped by a shader hash cache. */
extern RENDERCORE_API void CheckShaderHashCacheInclude(const FString& VirtualFilePath, EShaderPlatform ShaderPlatform, const FString& ShaderFormatName);

/** Initializes cached shader type data.  This must be called before creating any FShaderType. */
extern RENDERCORE_API void InitializeShaderTypes();

/** Returns true if debug viewmodes are allowed for the current platform. */
extern RENDERCORE_API bool AllowDebugViewmodes();

/** Returns true if debug viewmodes are allowed for the given platform. */
extern RENDERCORE_API bool AllowDebugViewmodes(EShaderPlatform Platform);

/** Returns the shader compression format. Oodle is used exclusively now. r.Shaders.SkipCompression configures Oodle to be uncompressed instead of returning NAME_None.*/
extern RENDERCORE_API FName GetShaderCompressionFormat();

/** Returns Oodle-specific shader compression format settings (passing ShaderFormat for future proofing, but as of now the setting is global for all formats). */
extern RENDERCORE_API void GetShaderCompressionOodleSettings(FOodleDataCompression::ECompressor& OutCompressor, FOodleDataCompression::ECompressionLevel& OutLevel, const FName& ShaderFormat = NAME_None);

struct FShaderTarget
{
	// The rest of uint32 holding the bitfields can be left unitialized. Union with a uint32 serves to prevent that to be able to set the whole uint32 value
	union
	{
		uint32 Packed;
		struct
		{
			uint32 Frequency : SF_NumBits;
			uint32 Platform : SP_NumBits;
		};
	};

	FShaderTarget()
		: Packed(0)
	{}

	FShaderTarget(EShaderFrequency InFrequency,EShaderPlatform InPlatform)
	:	Packed(0)
	{
		Frequency = InFrequency;
		Platform = InPlatform;
	}

	friend bool operator==(const FShaderTarget& X, const FShaderTarget& Y)
	{
		return X.Frequency == Y.Frequency && X.Platform == Y.Platform;
	}

	friend FArchive& operator<<(FArchive& Ar,FShaderTarget& Target)
	{
		uint32 TargetFrequency = Target.Frequency;
		uint32 TargetPlatform = Target.Platform;
		Ar << TargetFrequency << TargetPlatform;
		if (Ar.IsLoading())
		{
			Target.Packed = 0;
			Target.Frequency = TargetFrequency;
			Target.Platform = TargetPlatform;
		}
		return Ar;
	}

	EShaderPlatform GetPlatform() const
	{
		return (EShaderPlatform)Platform;
	}

	EShaderFrequency GetFrequency() const
	{
		return (EShaderFrequency)Frequency;
	}

	friend inline uint32 GetTypeHash(FShaderTarget Target)
	{
		return ((Target.Frequency << SP_NumBits) | Target.Platform);
	}
};
DECLARE_INTRINSIC_TYPE_LAYOUT(FShaderTarget);

static_assert(sizeof(FShaderTarget) == sizeof(uint32), "FShaderTarget is expected to be bit-packed into a single uint32.");

enum class EShaderParameterType : uint8
{
	LooseData,
	UniformBuffer,
	Sampler,
	SRV,
	UAV,

	BindlessSampler,
	BindlessSRV,
	BindlessUAV,

	DescriptorRange,

	Num
};

enum class EShaderParameterTypeMask : uint16
{
	LooseDataMask = 1 << uint16(EShaderParameterType::LooseData),
	UniformBufferMask = 1 << uint16(EShaderParameterType::UniformBuffer),
	SamplerMask = 1 << uint16(EShaderParameterType::Sampler),
	SRVMask = 1 << uint16(EShaderParameterType::SRV),
	UAVMask = 1 << uint16(EShaderParameterType::UAV),
	BindlessSamplerMask = 1 << uint16(EShaderParameterType::BindlessSampler),
	BindlessSRVMask = 1 << uint16(EShaderParameterType::BindlessSRV),
	BindlessUAVMask = 1 << uint16(EShaderParameterType::BindlessUAV),
	DescriptorRangeMask = 1 << uint16(EShaderParameterType::DescriptorRange),
};
ENUM_CLASS_FLAGS(EShaderParameterTypeMask);

inline bool IsParameterBindless(EShaderParameterType ParameterType)
{
	return ParameterType == EShaderParameterType::BindlessSampler
		|| ParameterType == EShaderParameterType::BindlessSRV
		|| ParameterType == EShaderParameterType::BindlessUAV
		;
}

struct FParameterAllocation
{
	uint16 BufferIndex = 0;
	uint16 BaseIndex = 0;
	uint16 Size = 0;
	EShaderParameterType Type{ EShaderParameterType::Num };
	mutable bool bBound = false;

	FParameterAllocation() = default;
	FParameterAllocation(uint16 InBufferIndex, uint16 InBaseIndex, uint16 InSize, EShaderParameterType InType)
		: BufferIndex(InBufferIndex)
		, BaseIndex(InBaseIndex)
		, Size(InSize)
		, Type(InType)
	{
	}

	friend FArchive& operator<<(FArchive& Ar,FParameterAllocation& Allocation)
	{
		Ar << Allocation.BufferIndex << Allocation.BaseIndex << Allocation.Size << Allocation.bBound;
		Ar << Allocation.Type;
		return Ar;
	}

	friend inline bool operator==(const FParameterAllocation& A, const FParameterAllocation& B)
	{
		return
			A.BufferIndex == B.BufferIndex && A.BaseIndex == B.BaseIndex && A.Size == B.Size && A.Type == B.Type && A.bBound == B.bBound;
	}

	friend inline bool operator!=(const FParameterAllocation& A, const FParameterAllocation& B)
	{
		return !(A == B);
	}
};

/**
 * A map of shader parameter names to registers allocated to that parameter.
 */
class FShaderParameterMap
{
public:

	FShaderParameterMap()
	{}

	RENDERCORE_API TOptional<FParameterAllocation> FindParameterAllocation(FStringView ParameterName) const;
	RENDERCORE_API TOptional<FParameterAllocation> FindAndRemoveParameterAllocation(FStringView ParameterName);
	RENDERCORE_API bool FindParameterAllocation(FStringView ParameterName, uint16& OutBufferIndex, uint16& OutBaseIndex, uint16& OutSize) const;
	RENDERCORE_API bool ContainsParameterAllocation(FStringView ParameterName) const;
	RENDERCORE_API void AddParameterAllocation(FStringView ParameterName, uint16 BufferIndex, uint16 BaseIndex, uint16 Size, EShaderParameterType ParameterType);
	RENDERCORE_API void RemoveParameterAllocation(FStringView ParameterName);

	/** Returns an array of all parameters with the given type. */
	RENDERCORE_API TArray<FStringView> GetAllParameterNamesOfType(EShaderParameterType InType) const;

	/** Returns a count of all parameters of the given type. */
	RENDERCORE_API uint32 CountParametersOfType(EShaderParameterType InType) const;

	/** Checks that all parameters are bound and asserts if any aren't in a debug build
	* @param InVertexFactoryType can be 0
	*/
	RENDERCORE_API void VerifyBindingsAreComplete(const TCHAR* ShaderTypeName, FShaderTarget Target, const class FVertexFactoryType* InVertexFactoryType) const;

	/** Updates the hash state with the contents of this parameter map. */
	template <typename THashBuilder>
	void UpdateHash(THashBuilder& HashState) const
	{
		for(TMap<FString,FParameterAllocation>::TConstIterator ParameterIt(ParameterMap);ParameterIt;++ParameterIt)
		{
			const FString& ParamName = ParameterIt.Key();
			const FParameterAllocation& ParamValue = ParameterIt.Value();
			HashState.Update((const uint8*)*ParamName, ParamName.Len() * sizeof(TCHAR));
			HashState.Update(&ParamValue.BufferIndex, sizeof(ParamValue.BufferIndex));
			HashState.Update(&ParamValue.BaseIndex, sizeof(ParamValue.BaseIndex));
			HashState.Update(&ParamValue.Size, sizeof(ParamValue.Size));
		}
	}

	friend FArchive& operator<<(FArchive& Ar,FShaderParameterMap& InParameterMap)
	{
		// Note: this serialize is used to pass between UE and the shader compile worker, recompile both when modifying
		Ar << InParameterMap.ParameterMap;
		return Ar;
	}

	void GetAllParameterNames(TArray<FString>& OutNames) const
	{
		ParameterMap.GenerateKeyArray(OutNames);
	}

	const TMap<FString, FParameterAllocation>& GetParameterMap() const
	{
		return ParameterMap;
	}

	TMap<FString,FParameterAllocation> ParameterMap;
};

inline FArchive& operator<<(FArchive& Ar, FUniformResourceEntry& Entry)
{
	if (Ar.IsLoading())
	{
		// Filled in later in FShaderResourceTableMap::FixupOnLoad
		Entry.UniformBufferMemberName = nullptr;
	}
	Ar << Entry.UniformBufferNameLength;
	Ar << Entry.Type;
	Ar << Entry.ResourceIndex;
	return Ar;
}

inline FArchive& operator<<(FArchive& Ar, FUniformBufferEntry& Entry)
{
	Ar << Entry.StaticSlotName;
	if (Ar.IsLoading())
	{
		Entry.MemberNameBuffer = MakeShareable(new TArray<TCHAR>());
	}
	Ar << *Entry.MemberNameBuffer.Get();
	Ar << Entry.LayoutHash;
	Ar << Entry.BindingFlags;
	Ar << Entry.Flags;
	return Ar;
}

using FThreadSafeSharedStringPtr = TSharedPtr<FString, ESPMode::ThreadSafe>;
using FThreadSafeNameBufferPtr = TSharedPtr<TArray<TCHAR>, ESPMode::ThreadSafe>;

// Simple wrapper for a uint64 bitfield; doesn't use TBitArray as it is fixed size and doesn't need dynamic memory allocations
class FShaderCompilerFlags
{
public:
	FShaderCompilerFlags(uint64 InData = 0)
		: Data(InData)
	{
	}

	inline void Append(const FShaderCompilerFlags& In)
	{
		Data |= In.Data;
	}

	inline void Add(uint32 InFlag)
	{
		const uint64 FlagBit = (uint64)1 << (uint64)InFlag;
		Data = Data | FlagBit;
	}

	inline void Remove(uint32 InFlag)
	{
		const uint64 FlagBit = (uint64)1 << (uint64)InFlag;
		Data = Data & ~FlagBit;
	}

	inline bool Contains(uint32 InFlag) const
	{
		const uint64 FlagBit = (uint64)1 << (uint64)InFlag;
		return (Data & FlagBit) == FlagBit;
	}

	inline void Iterate(TFunction<void(uint32)> Callback) const
	{
		uint64 Remaining = Data;
		uint32 Index = 0;
		while (Remaining)
		{
			if (Remaining & (uint64)1)
			{
				Callback(Index);
			}
			++Index;
			Remaining = Remaining >> (uint64)1;
		}
	}

	friend inline FArchive& operator << (FArchive& Ar, FShaderCompilerFlags& F)
	{
		Ar << F.Data;
		return Ar;
	}

	inline uint64 GetData() const
	{
		return Data;
	}

private:
	uint64 Data;
};

/**
RenderCore wrapper around FRHIShaderBindingLayout which can also cache the uniform buffer declarations used during shader code generation
*/
class FShaderBindingLayout
{
public:

	FRHIShaderBindingLayout RHILayout;

#if WITH_EDITOR
	void SetUniformBufferDeclarationAnsiPtr(const FShaderParametersMetadata* ShaderParametersMetadata, FThreadSafeSharedAnsiStringPtr UniformBufferDeclarationAnsi);
	const TMap<FString, FThreadSafeSharedAnsiStringPtr>& GetUniformBufferDeclarations() const { return UniformBufferMap; }

	RENDERCORE_API void AddRequiredSymbols(TArray<FString>& RequiredSymbols) const;

protected:
		
	TMap<FString, FThreadSafeSharedAnsiStringPtr> UniformBufferMap;
#endif
};

/**
Static shader binding layout object managing all possible binding type versions of the FShaderBindingLayout
*/
class FShaderBindingLayoutContainer
{
public:
	enum class EBindingType : uint8
	{
		Bindless,
		NotBindless,
		Num
	};

	const FShaderBindingLayout& GetLayout(EBindingType BindingType) const
	{
		return Layouts[(uint8)BindingType];
	}
	void SetLayout(EBindingType BindingType, const FShaderBindingLayout& InLayout)
	{
		Layouts[(uint8)BindingType] = InLayout;
	}

protected:

	FShaderBindingLayout Layouts[(uint8)EBindingType::Num];
};

struct FShaderResourceTableMap
{
	TArray<FUniformResourceEntry> Resources;

	RENDERCORE_API void Append(const FShaderResourceTableMap& Other);
	RENDERCORE_API void FixupOnLoad(const TMap<FString, FUniformBufferEntry>& UniformBufferMap);
};

class FShaderCompilerDefineNameCache
{
public:
	FShaderCompilerDefineNameCache(const TCHAR* InName)
		: Name(InName), MapIndex(INDEX_NONE)
	{}

	FString ToString() const 
	{
		return Name.ToString();
	}

	operator FName() const
	{
		return Name;
	}

private:
	FName Name;
	int32 MapIndex;

	friend class FShaderCompilerDefinitions;
};

/** The environment used to compile a shader. */
struct FShaderCompilerEnvironment
{
	// Map of the virtual file path -> content.
	// The virtual file paths are the ones that USF files query through the #include "<The Virtual Path of the file>"
	TMap<FString,FString> IncludeVirtualPathToContentsMap;

	TMap<FString, FThreadSafeSharedAnsiStringPtr> IncludeVirtualPathToSharedContentsMap;

	FShaderCompilerFlags CompilerFlags;
	TMap<uint32,uint8> RenderTargetOutputFormatsMap;
	FShaderResourceTableMap ResourceTableMap;
	TMap<FString, FUniformBufferEntry> UniformBufferMap;
	
	// Optional shader binding layout which can be used build the Uniform buffer map
	const FShaderBindingLayout* ShaderBindingLayout = nullptr;

	// Serialized version of the shader binding layout which can be used during platform specific shader code generation and serialization
	FRHIShaderBindingLayout RHIShaderBindingLayout;

	// Array of possible values for each specialization constants
	TArray<TArray<int32>> SpecializationConstantValues;

	const ITargetPlatform* TargetPlatform = nullptr;

	// Used for mobile platforms to allow per shader/material precision modes
	bool FullPrecisionInPS = 0;

	/** Default constructor. */
	RENDERCORE_API FShaderCompilerEnvironment();

	/** Constructor used when environment is constructed temporarily purely for the purpose of hashing for inclusion in DDC keys. */
	RENDERCORE_API FShaderCompilerEnvironment(FMemoryHasherBlake3& Hasher);

	/**
	 * Works for TCHAR
	 * e.g. SetDefine(TEXT("NAME"), TEXT("Test"));
	 * e.g. SetDefine(TEXT("NUM_SAMPLES"), 1);
	 * e.g. SetDefine(TEXT("DOIT"), true);
	 *
	 * Or use optimized macros, which can cache FName and map lookups to improve performance:
	 * e.g. SET_SHADER_DEFINE(NAME, TEXT("Test"));
	 * e.g. SET_SHADER_DEFINE(NUM_SAMPLES, 1);
	 * e.g. SET_SHADER_DEFINE(DOIT, true);
	 */
	RENDERCORE_API void SetDefine(const TCHAR* Name, const TCHAR* Value);
	RENDERCORE_API void SetDefine(const TCHAR* Name, const FString& Value);
	RENDERCORE_API void SetDefine(const TCHAR* Name, uint32 Value);
	RENDERCORE_API void SetDefine(const TCHAR* Name, int32 Value);
	RENDERCORE_API void SetDefine(const TCHAR* Name, bool Value);
	RENDERCORE_API void SetDefine(const TCHAR* Name, float Value);

	RENDERCORE_API void SetDefine(FName Name, const TCHAR* Value);
	RENDERCORE_API void SetDefine(FName Name, const FString& Value);
	RENDERCORE_API void SetDefine(FName Name, uint32 Value);
	RENDERCORE_API void SetDefine(FName Name, int32 Value);
	RENDERCORE_API void SetDefine(FName Name, bool Value);
	RENDERCORE_API void SetDefine(FName Name, float Value);

	RENDERCORE_API void SetDefine(FShaderCompilerDefineNameCache& Name, const TCHAR* Value);
	RENDERCORE_API void SetDefine(FShaderCompilerDefineNameCache& Name, const FString& Value);
	RENDERCORE_API void SetDefine(FShaderCompilerDefineNameCache& Name, uint32 Value);
	RENDERCORE_API void SetDefine(FShaderCompilerDefineNameCache& Name, int32 Value);
	RENDERCORE_API void SetDefine(FShaderCompilerDefineNameCache& Name, bool Value);
	RENDERCORE_API void SetDefine(FShaderCompilerDefineNameCache& Name, float Value);

	template <typename ValueType> void SetDefineIfUnset(const TCHAR* Name, ValueType Value)
	{
		FName NameKey(Name);
		if (!ContainsDefinition(NameKey))
		{
			SetDefine(NameKey, Value);
		}
	}


	// Sets a generic parameter which can be read in the various shader format backends to modify compilation
	// behaviour. Intended to replace any usage of definitions after shader preprocessing.
	template <typename ValueType> void SetCompileArgument(const TCHAR* Name, ValueType Value)
	{
		CompileArgs.Add(Name, TVariant<bool, float, int32, uint32, FString>(TInPlaceType<ValueType>(), Value));
	}
	
	// Like above, but this overload takes in the define value variant explicitly.
	void SetCompileArgument(const TCHAR* Name, TVariant<bool, float, int32, uint32, FString> Value)
	{
		CompileArgs.Add(Name, MoveTempIfPossible(Value));
	}

	// Helper to set both a define and a compile argument to the same value. Useful for various parameters which
	// need to be consumed both by preprocessing and in the shader format backends to modify compilation behaviour.
	template <typename ValueType> void SetDefineAndCompileArgument(const TCHAR* Name, ValueType Value)
	{
		SetDefine(Name, Value);
		SetCompileArgument(Name, Value);
	}

	// If a compile argument with the given name exists, returns true. 
	bool HasCompileArgument(const TCHAR* Name) const
	{
		if (CompileArgs.Contains(Name))
		{
			return true;
		}
		return false;
	}

	// If a compile argument with the given name exists and is of the specified type, returns its value. Otherwise, 
	// either the named argument doesn't exist or the type does not match, and the default value will be returned.
	template <typename ValueType> ValueType GetCompileArgument(const TCHAR* Name, const ValueType& DefaultValue) const
	{
		const TVariant<bool, float, int32, uint32, FString>* StoredValue = CompileArgs.Find(Name);
		if (StoredValue && StoredValue->IsType<ValueType>())
		{
			return StoredValue->Get<ValueType>();
		}
		return DefaultValue;
	}

	// If a compile argument with the given name exists and is of the specified type, its value will be assigned to OutValue
	// and the function will return true. Otherwise, either the named argument doesn't exist or the type does not match, the
	// OutValue will be left unmodified and the function will return false.
	template <typename ValueType> bool GetCompileArgument(const TCHAR* Name, ValueType& OutValue) const
	{
		const TVariant<bool, float, int32, uint32, FString>* StoredValue = CompileArgs.Find(Name);
		if (StoredValue && StoredValue->IsType<ValueType>())
		{
			OutValue = StoredValue->Get<ValueType>();
			return true;
		}
		return false;
	}

	void SetRenderTargetOutputFormat(uint32 RenderTargetIndex, EPixelFormat PixelFormat)
	{
		RenderTargetOutputFormatsMap.Add(RenderTargetIndex, UE_PIXELFORMAT_TO_UINT8(PixelFormat));
	}

	/** This "core" serialization is also used for the hashing the compiler job (where files are handled differently). Should stay in sync with the ShaderCompileWorker. */
	RENDERCORE_API void SerializeEverythingButFiles(FArchive& Ar);

	// Serializes the portions of the environment that are used as input to the backend compilation process (i.e. after all preprocessing)
	RENDERCORE_API void SerializeCompilationDependencies(FArchive& Ar);

	friend FArchive& operator<<(FArchive& Ar,FShaderCompilerEnvironment& Environment)
	{
		// Note: this serialize is used to pass between UE and the shader compile worker, recompile both when modifying
		Ar << Environment.IncludeVirtualPathToContentsMap;

		// Note: skipping Environment.IncludeVirtualPathToSharedContentsMap, which is handled by FShaderCompileUtilities::DoWriteTasks in order to maintain sharing

		Environment.SerializeEverythingButFiles(Ar);
		return Ar;
	}
	
	RENDERCORE_API void Merge(const FShaderCompilerEnvironment& Other);

	RENDERCORE_API FString GetDefinitionsAsCommentedCode() const;

private:
	RENDERCORE_API bool ContainsDefinition(FName Name) const;

	friend class FShaderCompileUtilities;
	friend class FShaderPreprocessorUtilities;

	TPimplPtr<FShaderCompilerDefinitions, EPimplPtrMode::DeepCopy> Definitions;

	FMemoryHasherBlake3* Hasher = nullptr;

	TMap<FString, TVariant<bool, float, int32, uint32, FString>> CompileArgs;

	/** Unused data kept around for deprecated FShaderCompilerEnvironment::GetDefinitions call */
	TMap<FString, FString> UnusedStringDefinitions;
};


/** Optimized define setting macros that cache the FName lookup, and potentially the map index. */
#define SET_SHADER_DEFINE(ENVIRONMENT, NAME, VALUE) \
	do {																	\
		static FShaderCompilerDefineNameCache Cache_##NAME(TEXT(#NAME));	\
		(ENVIRONMENT).SetDefine(Cache_##NAME, VALUE);						\
	} while(0)

#define SET_SHADER_DEFINE_AND_COMPILE_ARGUMENT(ENVIRONMENT, NAME, VALUE) \
	do {																	\
		static FShaderCompilerDefineNameCache Cache_##NAME(TEXT(#NAME));	\
		(ENVIRONMENT).SetDefine(Cache_##NAME, VALUE);						\
		(ENVIRONMENT).SetCompileArgument(TEXT(#NAME), VALUE);				\
	} while(0)


struct FSharedShaderCompilerEnvironment final : public FShaderCompilerEnvironment, public FRefCountedObject
{
	virtual ~FSharedShaderCompilerEnvironment() = default;
};

enum class EShaderOptionalDataKey : uint8
{
	AttributeInputs      = uint8('i'),
	AttributeOutputs     = uint8('o'),
	CompressedDebugCode  = uint8('z'),
	Diagnostic           = uint8('D'),
	Features             = uint8('x'),
	Name                 = uint8('n'),
	NativePath           = uint8('P'),
	ObjectFile           = uint8('O'),
	PackedResourceCounts = uint8('p'),
	ResourceMasks        = uint8('m'),
	ShaderModel6         = uint8('6'),
	SourceCode           = uint8('c'),
	UncompressedSize     = uint8('U'),
	UniformBuffers       = uint8('u'),
	Validation           = uint8('V'),
	VendorExtension      = uint8('v'),
	ShaderBindingLayout  = uint8('s'),
	EntryPoint           = uint8('e'),
	GlobalUniformBufferConfiguration = uint8('G'),
	SpecializationConstant = uint8('k'),
};

enum class EShaderResourceUsageFlags : uint8
{
	None                  = 0,
	GlobalUniformBuffer   = 1 << 0,
	BindlessResources     = 1 << 1,
	BindlessSamplers      = 1 << 2,
	RootConstants         = 1 << 3,
	NoDerivativeOps       = 1 << 4,
	ShaderBundle          = 1 << 5,
	DiagnosticBuffer      = 1 << 6,
};
ENUM_CLASS_FLAGS(EShaderResourceUsageFlags)

// if this changes you need to make sure all shaders get invalidated
struct FShaderCodePackedResourceCounts
{
	// for FindOptionalData() and AddOptionalData()
	static const EShaderOptionalDataKey Key = EShaderOptionalDataKey::PackedResourceCounts;

	EShaderResourceUsageFlags UsageFlags;
	uint8 NumSamplers;
	uint8 NumSRVs;
	uint8 NumCBs;
	uint8 NumUAVs;
};

// Configuration for a different 'global constant buffer' register assignment.
// Most compilers always force it to 0 or allow you to assign it, but some compilers do not.
struct FShaderCodeGlobalUniformBufferConfiguration
{
	static const EShaderOptionalDataKey Key = EShaderOptionalDataKey::GlobalUniformBufferConfiguration;

	uint32 Slot;
};

struct FShaderCodeResourceMasks
{
	// for FindOptionalData() and AddOptionalData()
	static const EShaderOptionalDataKey Key = EShaderOptionalDataKey::ResourceMasks;

	uint32 UAVMask; // Mask of UAVs bound
};


enum class EShaderCompileJobPriority : uint8
{
	None = 0xff,

	Low = 0u,
	Normal,
	High,		// All global shaders have at least High priorty.
	ExtraHigh,	// Above high priority for shaders known to be slow
	ForceLocal, // Force shader to skip distributed build and compile on local machine
	Num,
};
inline constexpr int32 NumShaderCompileJobPriorities = (int32)EShaderCompileJobPriority::Num;

inline const TCHAR* ShaderCompileJobPriorityToString(EShaderCompileJobPriority InPriority)
{
	switch (InPriority)
	{
	case EShaderCompileJobPriority::None: return TEXT("None");
	case EShaderCompileJobPriority::Low: return TEXT("Low");
	case EShaderCompileJobPriority::Normal: return TEXT("Normal");
	case EShaderCompileJobPriority::High: return TEXT("High");
	case EShaderCompileJobPriority::ExtraHigh: return TEXT("ExtraHigh");
	case EShaderCompileJobPriority::ForceLocal: return TEXT("ForceLocal");
	default: checkNoEntry(); return TEXT("");
	}
}


// if this changes you need to make sure all shaders get invalidated
enum class EShaderCodeFeatures : uint16
{
	None                    = 0,
	WaveOps                 = 1 << 0,
	SixteenBitTypes         = 1 << 1,
	TypedUAVLoadsExtended   = 1 << 2,
	Atomic64                = 1 << 3,
	UNUSED_BIT_4            = 1 << 4,
	BindlessResources       = 1 << 5,
	BindlessSamplers        = 1 << 6,
	StencilRef              = 1 << 7,
	BarycentricsSemantic    = 1 << 8,
};
ENUM_CLASS_FLAGS(EShaderCodeFeatures);

struct FShaderCodeFeatures
{
	// for FindOptionalData() and AddOptionalData()
	static const EShaderOptionalDataKey Key = EShaderOptionalDataKey::Features;

	EShaderCodeFeatures CodeFeatures = EShaderCodeFeatures::None;
};

// if this changes you need to make sure all shaders get invalidated
struct FShaderCodeName
{
	static const EShaderOptionalDataKey Key = EShaderOptionalDataKey::Name;

	// We store the straight ANSICHAR zero-terminated string
};

struct FShaderCodeUniformBuffers
{
	static const EShaderOptionalDataKey Key = EShaderOptionalDataKey::UniformBuffers;
	// We store an array of FString objects
};

struct FShaderCodeShaderResourceTableDataDesc
{
	static const EShaderOptionalDataKey Key = EShaderOptionalDataKey::ShaderBindingLayout;
	// We store FRHIShaderBindingLayout
};

// if this changes you need to make sure all shaders get invalidated
struct FShaderCodeVendorExtension
{
	// for FindOptionalData() and AddOptionalData()
	static const EShaderOptionalDataKey Key = EShaderOptionalDataKey::VendorExtension;

	EGpuVendorId VendorId = EGpuVendorId::NotQueried;
	FParameterAllocation Parameter;

	// When true, this vendor extension is included in the shader bytecode due to compile-time platform defines
	// but all code paths that access it are behind runtime dynamic branches, so the shader can safely be created on GPUs from other vendors.
	bool bIsOptional = false;

	FShaderCodeVendorExtension() = default;
	FShaderCodeVendorExtension(EGpuVendorId InVendorId, uint16 InBufferIndex, uint16 InBaseIndex, uint16 InSize, EShaderParameterType InType, bool InIsOptional = false)
		: VendorId(InVendorId)
		, Parameter(InBufferIndex, InBaseIndex, InSize, InType)
		, bIsOptional(InIsOptional)
	{
	}

	friend FArchive& operator<<(FArchive& Ar, FShaderCodeVendorExtension& Extension)
	{
		return Ar << Extension.VendorId << Extension.Parameter;
	}

	friend inline bool operator==(const FShaderCodeVendorExtension& A, const FShaderCodeVendorExtension& B)
	{
		return A.VendorId == B.VendorId && A.Parameter == B.Parameter;
	}

	friend inline bool operator!=(const FShaderCodeVendorExtension& A, const FShaderCodeVendorExtension& B)
	{
		return !(A == B);
	}

};

struct FSpecializationConstantData
{
	// Possible values for the specialization constants
	TArray<TArray<int32>> Values;

	// Pairs of SpecializationHash -> ArrayOfResourceExclusions
	TArray<TPair<uint32, TArray<uint32>>> Exclusions;

	void Sort()
	{
		for (TArray<int32>& Array : Values)
		{
			Array.Sort();
		}

		for (TPair<uint32, TArray<uint32>>& Pair : Exclusions)
		{
			Pair.Value.Sort();
		}
		Exclusions.Sort([](const TPair<uint32, TArray<uint32>>& PairA, const TPair<uint32, TArray<uint32>>& PairB) { return PairA.Key < PairB.Key; });
	}

	TMap<uint32, TArray<uint32>> GetExclusionMap() const
	{
		TMap<uint32, TArray<uint32>> ExclusionMap;
		for (const TPair<uint32, TArray<uint32>>& Pair : Exclusions)
		{
			ExclusionMap.Add(Pair);
		}
		return ExclusionMap;
	}

	void SetExclusionMap(TMap<uint32, TArray<uint32>>&& ExclusionMap)
	{
		for (TPair<uint32, TArray<uint32>>& Pair : ExclusionMap)
		{
			Exclusions.Emplace(Pair.Key, MoveTemp(Pair.Value));
		}
	}
};

inline FArchive& operator<<(FArchive& Ar, FSpecializationConstantData& SpecializationConstantData)
{
	return Ar << SpecializationConstantData.Values << SpecializationConstantData.Exclusions;
}


inline FArchive& operator<<(FArchive& Ar, FShaderCodeValidationStride& ShaderCodeValidationStride)
{
	return Ar << ShaderCodeValidationStride.BindPoint << ShaderCodeValidationStride.Stride;
}

inline FArchive& operator<<(FArchive& Ar, FShaderCodeValidationType& ShaderCodeValidationType)
{
	return Ar << ShaderCodeValidationType.BindPoint << ShaderCodeValidationType.Type;
}

inline FArchive& operator<<(FArchive& Ar, FShaderCodeValidationUBSize& ShaderCodeValidationSize)
{
	return Ar << ShaderCodeValidationSize.BindPoint << ShaderCodeValidationSize.Size;
}

struct FShaderCodeValidationExtension
{
	// for FindOptionalData() and AddOptionalData()
	static constexpr EShaderOptionalDataKey Key = EShaderOptionalDataKey::Validation;
	static constexpr uint16 StaticVersion = 0;

	TArray<FShaderCodeValidationStride> ShaderCodeValidationStride;
	TArray<FShaderCodeValidationType> ShaderCodeValidationSRVType;
	TArray<FShaderCodeValidationType> ShaderCodeValidationUAVType;
	TArray<FShaderCodeValidationUBSize> ShaderCodeValidationUBSize;
	uint16 Version = StaticVersion;

	friend FArchive& operator<<(FArchive& Ar, FShaderCodeValidationExtension& Extension)
	{
		Ar << Extension.Version;
		Ar << Extension.ShaderCodeValidationStride;
		Ar << Extension.ShaderCodeValidationSRVType;
		Ar << Extension.ShaderCodeValidationUAVType;
		Ar << Extension.ShaderCodeValidationUBSize;
		return Ar;
	}
};

struct FShaderDiagnosticData
{
	uint32 Hash;
	FString Message;
};

inline FArchive& operator<<(FArchive& Ar, FShaderDiagnosticData& ShaderCodeDiagnosticData)
{
	return Ar << ShaderCodeDiagnosticData.Hash << ShaderCodeDiagnosticData.Message;
}

struct FShaderDiagnosticExtension
{
	// for FindOptionalData() and AddOptionalData()
	static constexpr EShaderOptionalDataKey Key = EShaderOptionalDataKey::Diagnostic;
	static constexpr uint16 StaticVersion = 0;

	TArray<FShaderDiagnosticData> ShaderDiagnosticDatas;
	uint16 Version = StaticVersion;

	friend FArchive& operator<<(FArchive& Ar, FShaderDiagnosticExtension& Extension)
	{
		Ar << Extension.Version;
		Ar << Extension.ShaderDiagnosticDatas;
		return Ar;
	}
};

#ifndef RENDERCORE_ATTRIBUTE_UNALIGNED
// TODO find out if using GCC_ALIGN(1) instead of this new #define break on all kinds of platforms...
#define RENDERCORE_ATTRIBUTE_UNALIGNED
#endif
typedef int32  RENDERCORE_ATTRIBUTE_UNALIGNED unaligned_int32;
typedef uint32 RENDERCORE_ATTRIBUTE_UNALIGNED unaligned_uint32;

// later we can transform that to the actual class passed around at the RHI level
class FShaderCodeReader
{
	TConstArrayView<uint8> ShaderCode;

public:
	FShaderCodeReader(TConstArrayView<uint8> InShaderCode)
		: ShaderCode(InShaderCode)
	{
		check(ShaderCode.Num());
	}

	uint32 GetActualShaderCodeSize() const
	{
		return ShaderCode.Num() - GetOptionalDataSize();
	}

	TConstArrayView<uint8> GetOffsetShaderCode(int32 Offset)
	{
		return MakeConstArrayView(ShaderCode.GetData() + Offset, GetActualShaderCodeSize() - Offset);
	}

	// for convenience
	template <class T>
	const T* FindOptionalData() const
	{
		return (const T*)FindOptionalData(T::Key, sizeof(T));
	}


	// @param InKey e.g. FShaderCodePackedResourceCounts::Key
	// @return 0 if not found
	const uint8* FindOptionalData(EShaderOptionalDataKey InKey, uint8 ValueSize) const
	{
		check(ValueSize);

		const uint8* End = &ShaderCode[0] + ShaderCode.Num();

		int32 LocalOptionalDataSize = GetOptionalDataSize();

		const uint8* Start = End - LocalOptionalDataSize;
		// while searching don't include the optional data size
		End = End - sizeof(LocalOptionalDataSize);
		const uint8* Current = Start;

		while(Current < End)
		{
			EShaderOptionalDataKey Key = EShaderOptionalDataKey(*Current++);
			uint32 Size = *((const unaligned_uint32*)Current);
			Current += sizeof(Size);

			if(Key == InKey && Size == ValueSize)
			{
				return Current;
			}

			Current += Size;
		}

		return 0;
	}

	const ANSICHAR* FindOptionalData(EShaderOptionalDataKey InKey) const
	{
		check(ShaderCode.Num() >= 4);

		const uint8* End = &ShaderCode[0] + ShaderCode.Num();

		int32 LocalOptionalDataSize = GetOptionalDataSize();

		const uint8* Start = End - LocalOptionalDataSize;
		// while searching don't include the optional data size
		End = End - sizeof(LocalOptionalDataSize);
		const uint8* Current = Start;

		while(Current < End)
		{
			EShaderOptionalDataKey Key = EShaderOptionalDataKey(*Current++);
			uint32 Size = *((const unaligned_uint32*)Current);
			Current += sizeof(Size);

			if(Key == InKey)
			{
				return (ANSICHAR*)Current;
			}

			Current += Size;
		}

		return 0;
	}

	// Returns nullptr and Size -1 if key was not found
	const uint8* FindOptionalDataAndSize(EShaderOptionalDataKey InKey, int32& OutSize) const
	{
		check(ShaderCode.Num() >= 4);

		const uint8* End = &ShaderCode[0] + ShaderCode.Num();

		int32 LocalOptionalDataSize = GetOptionalDataSize();

		const uint8* Start = End - LocalOptionalDataSize;
		// while searching don't include the optional data size
		End = End - sizeof(LocalOptionalDataSize);
		const uint8* Current = Start;

		while (Current < End)
		{
			EShaderOptionalDataKey Key = EShaderOptionalDataKey(*Current++);
			uint32 Size = *((const unaligned_uint32*)Current);
			Current += sizeof(Size);

			if (Key == InKey)
			{
				OutSize = Size;
				return Current;
			}

			Current += Size;
		}

		OutSize = -1;
		return nullptr;
	}

	int32 GetOptionalDataSize() const
	{
		if(ShaderCode.Num() < sizeof(int32))
		{
			return 0;
		}

		const uint8* End = &ShaderCode[0] + ShaderCode.Num();

		int32 LocalOptionalDataSize = ((const unaligned_int32*)End)[-1];

		check(LocalOptionalDataSize >= 0);
		check(ShaderCode.Num() >= LocalOptionalDataSize);

		return LocalOptionalDataSize;
	}

	int32 GetShaderCodeSize() const
	{
		return ShaderCode.Num() - GetOptionalDataSize();
	}
};

class FShaderCode;

RENDERCORE_API FArchive& operator<<(FArchive& Ar, FSharedBuffer& Buffer);

class FShaderCodeResource
{
	struct FHeader
	{
		int32 UncompressedSize = 0;		// full size of code array before compression
		int32 ShaderCodeSize = 0;		// uncompressed size excluding optional data
		EShaderFrequency Frequency = EShaderFrequency::SF_NumFrequencies;
		uint8 _Pad0 = 0;
		uint16 _Pad1 = 0;
	};
	// header is cloned into shared buffer to avoid needing to determine what offsets FArchive serialization wrote everything at
	// as such it needs explicitly initialized padding, so we ensure no additional padding was added by the compiler
	static_assert(std::has_unique_object_representations_v<FHeader>);

	FSharedBuffer Header;		// The above FHeader struct persisted in a shared buffer
	FSharedBuffer Code;			// The bytecode buffer as constructed by FShaderCode::FinalizeShaderCode
	FCompressedBuffer Symbols;	// Buffer containing the symbols for this bytecode; will be empty if symbols are disabled

	friend class FShaderCode;
	friend RENDERCORE_API FArchive& operator<<(FArchive& Ar, FShaderCode& Code);
	friend FArchive& operator<<(FArchive& Ar, FShaderCodeResource& Resource);

public:

	/* Returns a uint8 array view representation of the Code FSharedBuffer, for compatibility's sake (much downstream
	 * usage of shader code expects an array of uint8)
	 */
	TConstArrayView<uint8> GetCodeView() const
	{
		return MakeConstArrayView(reinterpret_cast<const uint8*>(Code.GetData()), static_cast<int32>(Code.GetSize()));
	}

	/* Return the buffer storing just the shader code for this resource */
	FSharedBuffer GetCodeBuffer() const
	{
		return Code;
	}

	/* Return the buffer storing the (compressed) symbols for this resource */
	FCompressedBuffer GetSymbolsBuffer() const
	{
		return Symbols;
	}

	/* Returns a single composite buffer referencing both the header and code data to be cached. */
	FCompositeBuffer GetCacheBuffer() const
	{
		return FCompositeBuffer(Header, Code);
	}

	/* Unpacks the given FSharedBuffer into separate header/code buffer views and returns them as a 2-segment composite buffer. 
	 * Note that this is required since when pushing a composite buffer to DDC it does not maintain the segment structure.
	 */
	static FCompositeBuffer Unpack(FSharedBuffer MonolithicBuffer)
	{
		FMemoryView FullBufferView = MonolithicBuffer.GetView();

		return FCompositeBuffer(
			MonolithicBuffer.MakeView(FullBufferView.Left(sizeof(FHeader)), MonolithicBuffer),
			MonolithicBuffer.MakeView(FullBufferView.RightChop(sizeof(FHeader)), MonolithicBuffer));
	}
	
	/* Sets the Header and Code shared buffer references in this resource to the segments referenced
	 * by the given composite buffer.
	 */
	void PopulateFromComposite(FCompositeBuffer CacheBuffer, FCompressedBuffer SymbolsBuffer)
	{
		check(CacheBuffer.GetSegments().Num() == 2);
		Header = CacheBuffer.GetSegments()[0];
		check(Header.GetSize() == sizeof(FHeader));
		Code = CacheBuffer.GetSegments()[1];
		Symbols = SymbolsBuffer;
	}

	/* Populates the header for this code resource with the given sizes and frequency. 
	 * Note that this is done as a separate process from the construction of the Code buffer
	 * as the shader frequency is only known by the owning job, and not stored in the FShaderCode.
	 */
	void PopulateHeader(int32 UncompressedSize, int32 ShaderCodeSize, EShaderFrequency Frequency)
	{
		check(Code);
		FHeader HeaderData{ UncompressedSize, ShaderCodeSize, Frequency };
		Header = FSharedBuffer::Clone(&HeaderData, sizeof(HeaderData));
	}

	/* Retrieves the uncompressed size of the shader code as stored in the FHeader buffer. */
	int32 GetUncompressedSize() const
	{
		check(Header);
		return reinterpret_cast<const FHeader*>(Header.GetData())->UncompressedSize;
	}

	/* Retrieves the actual shader code size (excluding optional data) as stored in the FHeader buffer. */
	int32 GetShaderCodeSize() const
	{
		check(Header);
		return reinterpret_cast<const FHeader*>(Header.GetData())->ShaderCodeSize;
	}

	/* Retrieves the shader frequency as stored in the FHeader buffer. */
	EShaderFrequency GetFrequency() const
	{
		check(Header);
		return reinterpret_cast<const FHeader*>(Header.GetData())->Frequency;
	}

};

class FShaderCode
{
	// -1 if ShaderData was finalized
	mutable int32 OptionalDataSize;
	// access through class methods
	mutable TArray<uint8> ShaderCodeWithOptionalData;

	mutable TArray<uint8> SymbolData;

	mutable FShaderCodeResource ShaderCodeResource;

	/** ShaderCode may be compressed in SCWs on demand. If this value isn't null, the shader code is compressed. */
	mutable int32 UncompressedSize;

	/** Compression algo */
	mutable FName CompressionFormat;

	/** Oodle-specific compression algorithm - used if CompressionFormat is set to NAME_Oodle. */
	FOodleDataCompression::ECompressor OodleCompressor;

	/** Oodle-specific compression level - used if CompressionFormat is set to NAME_Oodle. */
	FOodleDataCompression::ECompressionLevel OodleLevel;

	/** We cannot get the code size after the compression, so store it here */
	mutable int32 ShaderCodeSize;

public:

	FShaderCode()
	: OptionalDataSize(0)
	, UncompressedSize(0)
	, CompressionFormat(NAME_None)
	, OodleCompressor(FOodleDataCompression::ECompressor::NotSet)
	, OodleLevel(FOodleDataCompression::ECompressionLevel::None)
	, ShaderCodeSize(0)
	{
	}

	// converts code and symbols into shared buffer representations, optionally overriding the symbols buffer with the given input
	void FinalizeShaderCode(FCompressedBuffer OverrideSymbolsBuffer = FCompressedBuffer()) const
	{
		if (OptionalDataSize != -1)
		{
			checkf(UncompressedSize == 0, TEXT("FShaderCode::FinalizeShaderCode() was called after compressing the code"));
			OptionalDataSize += sizeof(OptionalDataSize);
			ShaderCodeWithOptionalData.Append((const uint8*)&OptionalDataSize, sizeof(OptionalDataSize));
			OptionalDataSize = -1;

			// sanity check: the override symbol buffer is only used currently when merging multiple code outputs into a single one, and in this
			// case we expect the symbols to be empty (as merging the symbols currently needs to be handled differently in each shader format)
			check(OverrideSymbolsBuffer.IsNull() || SymbolData.IsEmpty());
			ShaderCodeResource.Code = MakeSharedBufferFromArray(MoveTemp(ShaderCodeWithOptionalData));
			ShaderCodeResource.Symbols = !OverrideSymbolsBuffer.IsNull() ? OverrideSymbolsBuffer : FCompressedBuffer::Compress(MakeSharedBufferFromArray(MoveTemp(SymbolData)));
		}
	}

	void Compress(FName ShaderCompressionFormat, FOodleDataCompression::ECompressor InOodleCompressor, FOodleDataCompression::ECompressionLevel InOodleLevel);

	// Write access for regular microcode: Optional Data must be added AFTER regular microcode and BEFORE Finalize
	TArray<uint8>& GetWriteAccess()
	{
		checkf(OptionalDataSize != -1, TEXT("Tried to add ShaderCode after being finalized!"));
		checkf(OptionalDataSize == 0, TEXT("Tried to add ShaderCode after adding Optional data!"));
		return ShaderCodeWithOptionalData;
	}

	TArray<uint8>& GetSymbolWriteAccess()
	{
		checkf(OptionalDataSize != -1, TEXT("Tried to add shader symbols after being finalized!"));
		return SymbolData;
	}

	TConstArrayView<uint8> GetSymbolReadView() const
	{
		checkf(OptionalDataSize != -1, TEXT("Tried to read uncompressed symbol data from bytecode after FinalizeShaderCode was called (which compresses the symbol data)"));
		return SymbolData;
	}

	int32 GetShaderCodeSize() const
	{
		// use the cached size whenever available
		if (ShaderCodeSize != 0)
		{
			return ShaderCodeSize;
		}
		else
		{
			FinalizeShaderCode();

			if (UncompressedSize != 0) // already compressed, get code size from resource
			{
				return ShaderCodeResource.GetShaderCodeSize();
			}
			else
			{
				// code buffer has been populated but not compressed, can still read additional fields from code buffer
				FShaderCodeReader Wrapper(ShaderCodeResource.GetCodeView());
				return Wrapper.GetShaderCodeSize();
			}
		}
	}

	// for read access, can have additional data attached to the end. Can also be compressed
	TConstArrayView<uint8> GetReadView() const
	{
		FinalizeShaderCode();

		return ShaderCodeResource.GetCodeView();
	}

	bool IsCompressed() const
	{
		return UncompressedSize != 0;
	}

	FName GetCompressionFormat() const
	{
		return CompressionFormat;
	}

	FOodleDataCompression::ECompressor GetOodleCompressor() const
	{
		return OodleCompressor;
	}

	FOodleDataCompression::ECompressionLevel GetOodleLevel() const
	{
		return OodleLevel;
	}

	int32 GetUncompressedSize() const
	{
		return UncompressedSize;
	}

	// for convenience
	template <class T>
	void AddOptionalData(const T &In)
	{
		AddOptionalData(T::Key, (uint8*)&In, sizeof(T));
	}

	// Note: we don't hash the optional attachments in GenerateOutputHash() as they would prevent sharing (e.g. many material share the save VS)
	// can be called after the non optional data was stored in ShaderData
	// @param Key uint8 to save memory so max 255, e.g. FShaderCodePackedResourceCounts::Key
	// @param Size >0, only restriction is that sum of all optional data values must be < 4GB
	void AddOptionalData(EShaderOptionalDataKey Key, const uint8* ValuePtr, uint32 ValueSize)
	{
		check(ValuePtr);

		// don't add after Finalize happened
		check(OptionalDataSize >= 0);

		ShaderCodeWithOptionalData.Add(uint8(Key));
		ShaderCodeWithOptionalData.Append((const uint8*)&ValueSize, sizeof(ValueSize));
		ShaderCodeWithOptionalData.Append(ValuePtr, ValueSize);
		OptionalDataSize += sizeof(uint8) + sizeof(ValueSize) + (uint32)ValueSize;
	}

	// Note: we don't hash the optional attachments in GenerateOutputHash() as they would prevent sharing (e.g. many material share the save VS)
	// convenience, silently drops the data if string is too long
	// @param e.g. 'n' for the ShaderSourceFileName
	void AddOptionalData(EShaderOptionalDataKey Key, const ANSICHAR* InString)
	{
		uint32 Size = FCStringAnsi::Strlen(InString) + 1;
		AddOptionalData(Key, (uint8*)InString, Size);
	}

	// Populates FShaderCodeResource's header buffer and returns the fully populated resource struct
	const FShaderCodeResource& GetFinalizedResource(EShaderFrequency Frequency, FShaderHash OutputHash) const;

	void SetFromResource(FShaderCodeResource&& Resource)
	{
		ShaderCodeResource = MoveTemp(Resource);
		// Set the internal state of this FShaderCode to that of a finalized (and possibly compressed) ShaderCode object
		OptionalDataSize = -1;
		ShaderCodeSize = ShaderCodeResource.GetShaderCodeSize();

		// as above, set UncompressedSize to 0 if not compressed, indicated by the resource uncompressed size matching the code buffer size.
		int32 ResourceUncompressedSize = ShaderCodeResource.GetUncompressedSize();
		UncompressedSize = ResourceUncompressedSize == ShaderCodeResource.Code.GetSize() ? 0 : ResourceUncompressedSize;

		// already validated that compression settings matched when serializing the resource, so we can just initialize them to the known-correct values
		CompressionFormat = GetShaderCompressionFormat();
		GetShaderCompressionOodleSettings(OodleCompressor, OodleLevel);
	}

	friend RENDERCORE_API FArchive& operator<<(FArchive& Ar, FShaderCode& Output);
};

/**
* Convert the virtual shader path to an actual file system path.
* CompileErrors output array is optional.
*/
extern RENDERCORE_API FString GetShaderSourceFilePath(const FString& VirtualFilePath, TArray<struct FShaderCompilerError>* CompileErrors = nullptr);

/**
 * Converts an absolute or relative shader filename to a filename relative to
 * the shader directory.
 * @param InFilename - The shader filename.
 * @returns a filename relative to the shaders directory.
 */
extern RENDERCORE_API FString ParseVirtualShaderFilename(const FString& InFilename);

/** Replaces virtual platform path with appropriate path for a given ShaderPlatform. Returns true if path was changed. */
extern RENDERCORE_API bool ReplaceVirtualFilePathForShaderPlatform(FString& InOutVirtualFilePath, EShaderPlatform ShaderPlatform);

/** Replaces virtual platform path with appropriate autogen path for a given ShaderPlatform. Returns true if path was changed. */
extern RENDERCORE_API bool ReplaceVirtualFilePathForShaderAutogen(FString& InOutVirtualFilePath, EShaderPlatform ShaderPlatform, const FName* InShaderPlatformName = nullptr);

/** Loads the shader file with the given name.  If the shader file couldn't be loaded, throws a fatal error. */
extern RENDERCORE_API void LoadShaderSourceFileChecked(const TCHAR* VirtualFilePath, EShaderPlatform ShaderPlatform, FString& OutFileContents, const FName* ShaderPlatformName = nullptr);

/**
 * Recursively populates IncludeFilenames with the include filenames from Filename
 */
extern RENDERCORE_API void GetShaderIncludes(const TCHAR* EntryPointVirtualFilePath, const TCHAR* VirtualFilePath, TArray<FString>& IncludeVirtualFilePaths, EShaderPlatform ShaderPlatform, uint32 DepthLimit=100, const FName* ShaderPlatformName = nullptr);
extern RENDERCORE_API void GetShaderIncludes(const TCHAR* EntryPointVirtualFilePath, const TCHAR* VirtualFilePath, const FString& FileContents, TArray<FString>& IncludeVirtualFilePaths, EShaderPlatform ShaderPlatform, uint32 DepthLimit = 100, const FName* ShaderPlatformName = nullptr);

/**
 * Calculates a Hash for the given filename if it does not already exist in the Hash cache.
 * @param Filename - shader file to Hash
 * @param ShaderPlatform - shader platform to Hash
 * @return Reference to the Hash created and stored for the file, or to an empty FSHAHash if not found.
 * Logs an error if the file is not loadable.
 */
extern RENDERCORE_API const class FSHAHash& GetShaderFileHash(const TCHAR* VirtualFilePath, EShaderPlatform ShaderPlatform);
/**
 * Calculates a Hash for the given filename if it does not already exist in the Hash cache.
 * @param Filename - shader file to Hash
 * @param ShaderPlatform - shader platform to Hash
 * @param OutErrorMessage - If non-null, receives the errormessage if nullptr is returned.
 * @return Pointer to the Hash created and stored for the file, or nullptr if not found.
 */
extern RENDERCORE_API const FSHAHash* TryGetShaderFileHash(const TCHAR* VirtualFilePath,
	EShaderPlatform ShaderPlatform, FString* OutErrorMessage = nullptr);

/**
 * Calculates a Hash for the list of filenames if it does not already exist in the Hash cache.
 */
extern RENDERCORE_API const class FSHAHash& GetShaderFilesHash(const TArray<FString>& VirtualFilePaths, EShaderPlatform ShaderPlatform);

/**
 * Primes the shader file hash cache for a batch of virtual file paths, acquiring each lock only
 * once for the entire batch rather than once per file. Prefer this over repeated TryGetShaderFileHash
 * calls when pre-warming the cache for a known set of files.
 * @param VirtualFilePaths - List of virtual shader file paths to prime.
 * @param ShaderPlatform   - Shader platform the hashes are for.
 */
extern RENDERCORE_API void PrimeShaderFileHashCache(TArrayView<const FString> VirtualFilePaths, EShaderPlatform ShaderPlatform);

/**
 * Flushes the shader file and CRC cache, and regenerates the binary shader files if necessary.
 * Allows shader source files to be re-read properly even if they've been modified since startup.
 */
extern RENDERCORE_API void FlushShaderFileCache();

/** Invalidates a single entry in the shader file and CRC caches. */
extern RENDERCORE_API void InvalidateShaderFileCacheEntry(const TCHAR* InVirtualFilePath, EShaderPlatform InShaderPlatform, const FName* InShaderPlatformName = nullptr);

extern RENDERCORE_API void VerifyShaderSourceFiles(EShaderPlatform ShaderPlatform);

#if WITH_EDITOR

class FShaderType;
class FVertexFactoryType;
class FShaderPipelineType;

// Text to use as line terminator for HLSL files (may differ from platform LINE_TERMINATOR)
#define HLSL_LINE_TERMINATOR TEXT("\n")

/** Force updates each shader/pipeline type provided to update their list of referenced uniform buffers. */
RENDERCORE_API void UpdateReferencedUniformBufferNames(
	TArrayView<const FShaderType*> OutdatedShaderTypes,
	TArrayView<const FVertexFactoryType*> OutdatedFactoryTypes,
	TArrayView<const FShaderPipelineType*> OutdatedShaderPipelineTypes);

/** Parses the given source file and its includes for references of uniform buffers. */
extern void GenerateReferencedUniformBuffers(
	const TCHAR* SourceFilename,
	const TCHAR* ShaderTypeName,
	const TMap<FString, TArray<const TCHAR*> >& ShaderFileToUniformBufferVariables,
	TSet<const FShaderParametersMetadata*>& UniformBuffers);

struct FUniformBufferNameSortOrder
{
	inline bool operator()(const TCHAR* Name1, const TCHAR* Name2) const
	{
		return FCString::Strcmp(Name1, Name2) < 0;
	}
};

/**
 * Return the hash of the given type layout for a partical platform type layout. This function employs caching to avoid re-hashing the same parameters several times.
 */
extern RENDERCORE_API FSHAHash GetShaderTypeLayoutHash(const FTypeLayoutDesc& TypeDesc, FPlatformTypeLayoutParameters LayoutParameters);

// Forward declarations
class FShaderTypeDependency;
class FShaderPipelineTypeDependency;
class FVertexFactoryTypeDependency;

/** Appends information to a KeyString for a given shader to reflect its dependencies */
extern RENDERCORE_API void AppendKeyStringShaderDependencies(
	TConstArrayView<FShaderTypeDependency> ShaderTypeDependencies,
	FPlatformTypeLayoutParameters LayoutParams,
	FString& OutKeyString,
	bool bIncludeSourceHashes = true);

extern RENDERCORE_API void AppendKeyStringShaderDependencies(
	TConstArrayView<FShaderTypeDependency> ShaderTypeDependencies,
	TConstArrayView<FShaderPipelineTypeDependency> ShaderPipelineTypeDependencies,
	TConstArrayView<FVertexFactoryTypeDependency> VertexFactoryTypeDependencies,
	FPlatformTypeLayoutParameters LayoutParams,
	FString& OutKeyString,
	bool bIncludeSourceHashes = true);
extern RENDERCORE_API void AppendShaderDependencies(
	FShaderKeyGenerator& KeyGen,
	TConstArrayView<FShaderTypeDependency> ShaderTypeDependencies,
	TConstArrayView<FShaderPipelineTypeDependency> ShaderPipelineTypeDependencies,
	TConstArrayView<FVertexFactoryTypeDependency> VertexFactoryTypeDependencies,
	FPlatformTypeLayoutParameters LayoutParams,
	bool bIncludeSourceHashes = true);
#endif // WITH_EDITOR

/** Create a block of source code to be injected in the preprocessed shader code. The Block will be put into a #line directive
 * to show up in case shader compilation failures happen in this code block.
 */
FString RENDERCORE_API MakeInjectedShaderCodeBlock(const TCHAR* BlockName, const FString& CodeToInject);


/**
 * Returns the map virtual shader directory path -> real shader directory path.
 */
extern RENDERCORE_API const TMap<FString, FString>& AllShaderSourceDirectoryMappings();

/** Hook for shader compile worker to reset the directory mappings. */
extern RENDERCORE_API void ResetAllShaderSourceDirectoryMappings();

/**
 * Maps a real shader directory existing on disk to a virtual shader directory.
 * @param VirtualShaderDirectory Unique absolute path of the virtual shader directory (ex: /Project).
 * @param RealShaderDirectory FPlatformProcess::BaseDir() relative path of the directory map.
 */
extern RENDERCORE_API void AddShaderSourceDirectoryMapping(const FString& VirtualShaderDirectory, const FString& RealShaderDirectory);
/**
 * Specifies that the virtual shader directory and all subdirectories should contain only .h files that are shared between C++ / shader
 * @param VirtualShaderDirectory Unique absolute path of the virtual shader directory (ex: /Project/Shared/).
 */
extern RENDERCORE_API void AddShaderSourceSharedVirtualDirectory(const FString& VirtualShaderDirectory);

extern RENDERCORE_API void AddShaderSourceFileEntry(TArray<FString>& OutVirtualFilePaths, FString VirtualFilePath, EShaderPlatform ShaderPlatform, const FName* ShaderPlatformName = nullptr);
extern RENDERCORE_API void GetAllVirtualShaderSourcePaths(TArray<FString>& OutVirtualFilePaths, EShaderPlatform ShaderPlatform, const FName* ShaderPlatformName = nullptr);
