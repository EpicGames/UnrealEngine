// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Hash/ShaderHash.h"
#include "Misc/CoreMiscDefines.h"
#include "Templates/Function.h"
#include "UObject/NameTypes.h"

using FShaderMapAssetPaths = TSet<FName>;

#if WITH_EDITORONLY_DATA

namespace ShaderCodeArchive
{

	/**
	 * Descriptor when loading a ShaderLibrary during cooking for which operation is loading the ShaderLibrary.
	 * When the operation is loading data from previous incremental cooks, we have more strict validation requirements
	 * and we ignore data for stale Assets that have been recooked.
	 * Smaller values are newer sources and override larger values which come from older (more stale) sources.
	 */
	enum class ECookShaderLibrarySource : uint8
	{
		/** The data is provided by compilation that executed during the current cook; this is the most up-to-date. */
		CurrentCook = 0,

		/** The current cook is a DLC cook and the data being provided came from the basegame cook. */
		PatchBase,

		/** The data was written to the ShaderLibrary artifacts in a previous incremental cook. */
		PreviousIncremental,
	};

} // namespace ShaderCodeArchive

/**
 * Bi-directional mapping from shadermap hashes to an array of asset names.
 * Assets are identified by LongPackageName.
 * ShaderMaps are identified by their SHAHash.
 * This structure is used to identify the ShaderMaps required in the projection of this library for a pakfile chunk
 * based on assets in the chunk, and to remove stale ShaderMaps and their Shaders from this library when the assets
 * are recooked or are pruned from the incremental cook oplog.
 */
class FShaderMapAssetAssociations
{
public:
	using ECookShaderLibrarySource = ShaderCodeArchive::ECookShaderLibrarySource;

	struct FAssociatedAssetData
	{
		TArray<FShaderHash> ShaderMaps;
		ECookShaderLibrarySource LatestSource = ECookShaderLibrarySource::PreviousIncremental;
		// These reference flags are transient. By default they are initialized to true, for shaderlibraries that do not use
		// reference tracking such as the global shader library. When tracking references, we initialize them to false.
		// These flags are read/written only during end of cook.
		/** Set to true if the Asset is needed either for incremental cook or for staging, and should therefore not be pruned. */
		bool bReferencedByOplog = true;
		/** Set to true if the Asset is needed for runtime and staging, and should therefore be added to the staging artifacts. */
		bool bReferencedByStaging = true;

		/** this->LatestSource = NewestOf(LatestSource, InCookSource). */
		FORCEINLINE void MergeSource(ECookShaderLibrarySource InCookSource)
		{
			LatestSource = (ECookShaderLibrarySource)FMath::Min((uint8)LatestSource, (uint8)InCookSource);
		}
	};

	/** Function signature to filter asset associated with its metadata. */
	using FAssetFilterFunctionRef = TFunctionRef<bool(FName AssetName, const FAssociatedAssetData& AssetData)>;

	/** Function signature to filter shadermaps. */
	using FShaderMapFilterFunction = TFunction<bool(const FShaderHash& ShaderMapHash)>;

	inline bool IsEmpty() const
	{
		return ShaderMapToAssets.IsEmpty();
	}

	inline SIZE_T GetAllocatedSize() const
	{
		return ShaderMapToAssets.GetAllocatedSize() + AssetToShaderMaps.GetAllocatedSize();
	}

	FORCEINLINE TMap<FName, FAssociatedAssetData>& ViewAssets()
	{
		return AssetToShaderMaps;
	}

	FORCEINLINE const TMap<FName, FAssociatedAssetData>& ViewAssets() const
	{
		return AssetToShaderMaps;
	}

	FORCEINLINE const TMap<FShaderHash, FShaderMapAssetPaths>& ViewShaderMaps() const
	{
		return ShaderMapToAssets;
	}

	FORCEINLINE FAssociatedAssetData* FindAsset(FName AssetName)
	{
		return AssetToShaderMaps.Find(AssetName);
	}

	FORCEINLINE const FAssociatedAssetData* FindAsset(FName AssetName) const
	{
		return AssetToShaderMaps.Find(AssetName);

	}

	FORCEINLINE FShaderMapAssetPaths* FindShaderMap(const FShaderHash& ShaderMap)
	{
		return ShaderMapToAssets.Find(ShaderMap);
	}

	FORCEINLINE const FShaderMapAssetPaths* FindShaderMap(const FShaderHash& ShaderMap) const
	{
		return ShaderMapToAssets.Find(ShaderMap);
	}

	FORCEINLINE void Empty()
	{
		ShaderMapToAssets.Empty();
		AssetToShaderMaps.Empty();
	}

	RENDERCORE_API void ReserveAssets(int32 NumAssets);
	RENDERCORE_API FAssociatedAssetData& FindOrAddAsset(FName Asset, const FShaderHash& ShaderMap);
	RENDERCORE_API void RemoveAsset(FName Asset);
	RENDERCORE_API void SortForSaving();
	RENDERCORE_API void Append(const FShaderMapAssetAssociations& InOtherAssociation);

private:
	TMap<FShaderHash, FShaderMapAssetPaths> ShaderMapToAssets;
	TMap<FName, FAssociatedAssetData> AssetToShaderMaps;
};

#endif // WITH_EDITORONLY_DATA
