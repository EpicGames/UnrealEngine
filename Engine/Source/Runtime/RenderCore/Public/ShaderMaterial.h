// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ShaderMaterial.h: Shader materials helper definitions.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "Templates/RefCounting.h"
#include "Serialization/MemoryLayout.h"

namespace ERHIFeatureLevel { enum Type : int; }
struct FShaderCompilerEnvironment;

struct FShaderGlobalDefines
{
	//void ModifyEnvironment(FShaderCompilerEnvironment& OutEnvironment) const;

	bool USES_BASE_PASS_VELOCITY;
	bool GBUFFER_HAS_VELOCITY;
	bool GBUFFER_HAS_TANGENT;
	bool ALLOW_STATIC_LIGHTING;
	bool CLEAR_COAT_BOTTOM_NORMAL;
	bool IRIS_NORMAL;
	bool DXT5_NORMALMAPS;
	bool SELECTIVE_BASEPASS_OUTPUTS;
	bool USE_DBUFFER;
	bool FORWARD_SHADING;
	bool PROJECT_VERTEX_FOGGING_FOR_OPAQUE;
	bool PROJECT_MOBILE_DISABLE_VERTEX_FOG;
	bool PROJECT_ALLOW_GLOBAL_CLIP_PLANE;
	bool EARLY_Z_PASS_ONLY_MATERIAL_MASKING;
	bool PROJECT_SUPPORT_SKY_ATMOSPHERE;
	bool PROJECT_SUPPORT_SKY_ATMOSPHERE_AFFECTS_HEIGHFOG;
	bool PROJECT_EXPFOG_MATCHES_VFOG;
	bool SUPPORT_CLOUD_SHADOW_ON_FORWARD_LIT_TRANSLUCENT;
	bool SUPPORT_CLOUD_SHADOW_ON_SINGLE_LAYER_WATER;
	bool SUPPORT_PRIMITIVE_ALPHA_HOLDOUT;
	bool PLATFORM_SUPPORTS_SHADER_ROOT_CONSTANTS;
	bool PLATFORM_SUPPORTS_SHADER_BUNDLE_DISPATCH;
	bool PLATFORM_SUPPORTS_RENDERTARGET_WRITE_MASK;
	bool PLATFORM_SUPPORTS_PER_PIXEL_DBUFFER_MASK;
	bool PLATFORM_SUPPORTS_DISTANCE_FIELDS;
	bool PLATFORM_ALLOW_SCENE_DATA_COMPRESSED_TRANSFORMS;

	// This is a sepcial one. The flag COMPILE_SHADERS_FOR_DEVELOPMENT is set by the cvar r.CompileShadersForDevelopment, but only
	// if bAllowDevelopmentShaderCompile is true in the call to GlobalBeginCompileShader(). So the flag COMPILE_SHADERS_FOR_DEVELOPMENT_ALLOWED
	// simply stores the result of r.CompileShadersForDevelopment. Then the logic later down the pipeline is:
	// COMPILE_SHADERS_FOR_DEVELOPMENT = COMPILE_SHADERS_FOR_DEVELOPMENT_ALLOWED && bAllowDevelopmentShaderCompile;
	bool COMPILE_SHADERS_FOR_DEVELOPMENT_ALLOWED;
	bool bSupportsDualBlending;
	UE_DEPRECATED(5.7, "LegacyGBufferFormat is no longer used in the engine")
	int LegacyGBufferFormat;
	UE_DEPRECATED(5.7, "bNeedVelocityDepth is no longer used in the engine")
	bool bNeedVelocityDepth;

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	// Explicitly-defaulted ctors & assignment operators are needed temporarily due to deprecation of
	// LegacyGBufferFormat/bNeedVelocityDepth fields. These can be removed along with the fields in 5.9
	FShaderGlobalDefines() = default;
	FShaderGlobalDefines(FShaderGlobalDefines&&) = default;
	FShaderGlobalDefines(const FShaderGlobalDefines&) = default;
	FShaderGlobalDefines& operator=(FShaderGlobalDefines&&) = default;
	FShaderGlobalDefines& operator=(const FShaderGlobalDefines&) = default;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
};


// maybe should rename this to VertexFactoryDefines?
struct FShaderLightmapPropertyDefines
{
	void ModifyEnvironment(FShaderCompilerEnvironment& OutEnvironment) const;

	bool LQ_TEXTURE_LIGHTMAP;
	bool HQ_TEXTURE_LIGHTMAP;
	bool CACHED_POINT_INDIRECT_LIGHTING;
	bool STATICLIGHTING_TEXTUREMASK;
	bool STATICLIGHTING_SIGNEDDISTANCEFIELD;
	bool TRANSLUCENT_SELF_SHADOWING;
	bool PRECOMPUTED_IRRADIANCE_VOLUME_LIGHTING;
	bool CACHED_VOLUME_INDIRECT_LIGHTING;

	bool WATER_MESH_FACTORY;
	bool NIAGARA_MESH_FACTORY;
	bool NIAGARA_MESH_INSTANCED;

	bool PARTICLE_MESH_FACTORY;
	bool PARTICLE_MESH_INSTANCED;

	bool MANUAL_VERTEX_FETCH;
};

struct FShaderMaterialPropertyDefines
{
	//DECLARE_TYPE_LAYOUT(FShaderMaterialPropertyDefines, NonVirtual);

	//void ModifyEnvironment(FShaderCompilerEnvironment& OutEnvironment) const;
	//void WriteFrozenVertexFactoryParameters(FMemoryImageWriter& Writer, const TMemoryImagePtr<FShaderMaterialPropertyDefines>& InPropDefines) const;

	uint8 MATERIAL_ENABLE_TRANSLUCENCY_FOGGING : 1;

	uint8 MATERIALBLENDING_ANY_TRANSLUCENT : 1;
	uint8 MATERIAL_USES_SCENE_COLOR_COPY : 1;
	uint8 MATERIALBLENDING_MASKED_USING_COVERAGE : 1;

	uint8 MATERIAL_COMPUTE_FOG_PER_PIXEL : 1;
	uint8 MATERIAL_SHADINGMODEL_UNLIT : 1;

	uint8 MATERIAL_SHADINGMODEL_DEFAULT_LIT : 1;
	uint8 MATERIAL_SHADINGMODEL_SUBSURFACE : 1;
	uint8 MATERIAL_SHADINGMODEL_PREINTEGRATED_SKIN : 1;
	uint8 MATERIAL_SHADINGMODEL_SUBSURFACE_PROFILE : 1;
	uint8 MATERIAL_SHADINGMODEL_CLEAR_COAT : 1;
	uint8 MATERIAL_SHADINGMODEL_TWOSIDED_FOLIAGE : 1;
	uint8 MATERIAL_SHADINGMODEL_HAIR : 1;
	uint8 MATERIAL_SHADINGMODEL_CLOTH : 1;
	uint8 MATERIAL_SHADINGMODEL_EYE : 1;
	uint8 MATERIAL_SHADINGMODEL_SINGLELAYERWATER : 1;
	uint8 SINGLE_LAYER_WATER_SEPARATED_MAIN_LIGHT : 1;
	uint8 MATERIAL_SHADINGMODEL_THIN_TRANSLUCENT : 1;
	uint8 MATERIAL_SHADINGMODEL_TOON : 1;

	uint8 TRANSLUCENCY_LIGHTING_VOLUMETRIC_NONDIRECTIONAL : 1;
	uint8 TRANSLUCENCY_LIGHTING_VOLUMETRIC_DIRECTIONAL : 1;
	uint8 TRANSLUCENCY_LIGHTING_VOLUMETRIC_PERVERTEX_NONDIRECTIONAL : 1;
	uint8 TRANSLUCENCY_LIGHTING_VOLUMETRIC_PERVERTEX_DIRECTIONAL : 1;
	uint8 TRANSLUCENCY_LIGHTING_SURFACE_LIGHTINGVOLUME : 1;
	uint8 TRANSLUCENCY_LIGHTING_SURFACE_FORWARDSHADING : 1;

	uint8 EDITOR_PRIMITIVE_MATERIAL : 1;

	uint8 MATERIAL_FULLY_ROUGH : 1;

	uint8 MATERIALBLENDING_SOLID : 1;
	uint8 MATERIALBLENDING_MASKED : 1;
	uint8 MATERIALBLENDING_ALPHACOMPOSITE : 1;
	uint8 MATERIALBLENDING_TRANSLUCENT : 1;
	uint8 MATERIALBLENDING_ADDITIVE : 1;
	uint8 MATERIALBLENDING_MODULATE : 1;
	uint8 MATERIALBLENDING_ALPHAHOLDOUT : 1;

	uint8 SUBSTRATE_BLENDING_OPAQUE : 1;
	uint8 SUBSTRATE_BLENDING_MASKED : 1;
	uint8 SUBSTRATE_BLENDING_TRANSLUCENT_GREYTRANSMITTANCE : 1;
	uint8 SUBSTRATE_BLENDING_TRANSLUCENT_COLOREDTRANSMITTANCE : 1;
	uint8 SUBSTRATE_BLENDING_COLOREDTRANSMITTANCEONLY : 1;
	uint8 SUBSTRATE_BLENDING_ALPHAHOLDOUT : 1;

	uint8 USES_EMISSIVE_COLOR : 1;
	
	uint8 REFRACTION_USE_INDEX_OF_REFRACTION : 1;
	uint8 REFRACTION_USE_PIXEL_NORMAL_OFFSET : 1;
	uint8 REFRACTION_USE_2D_OFFSET : 1;

	uint8 USE_DITHERED_LOD_TRANSITION_FROM_MATERIAL : 1;
	uint8 MATERIAL_TWOSIDED : 1;
	uint8 MATERIAL_ISTHINSURFACE : 1;
	uint8 MATERIAL_TANGENTSPACENORMAL : 1;
	uint8 GENERATE_SPHERICAL_PARTICLE_NORMALS : 1;
	uint8 MATERIAL_USE_PREINTEGRATED_GF : 1;
	uint8 MATERIAL_HQ_FORWARD_REFLECTIONS : 1;
	uint8 MATERIAL_PLANAR_FORWARD_REFLECTIONS : 1;
	uint8 MATERIAL_NONMETAL : 1;
	uint8 MATERIAL_USE_LM_DIRECTIONALITY : 1;
	uint8 MATERIAL_INJECT_EMISSIVE_INTO_LPV : 1;
	uint8 MATERIAL_SSR : 1;
	uint8 MATERIAL_CONTACT_SHADOWS : 1;
	uint8 MATERIAL_BLOCK_GI : 1;
	uint8 MATERIAL_DITHER_OPACITY_MASK : 1;
	uint8 MATERIAL_NORMAL_CURVATURE_TO_ROUGHNESS : 1;
	uint8 MATERIAL_ALLOW_NEGATIVE_EMISSIVECOLOR : 1;
	uint8 MATERIAL_OUTPUT_OPACITY_AS_ALPHA : 1;
	uint8 TRANSLUCENT_SHADOW_WITH_MASKED_OPACITY : 1;

	uint8 MATERIAL_DOMAIN_SURFACE : 1;
	uint8 MATERIAL_DOMAIN_DEFERREDDECAL : 1;
	uint8 MATERIAL_DOMAIN_LIGHTFUNCTION : 1;
	uint8 MATERIAL_DOMAIN_VOLUME : 1;
	uint8 MATERIAL_DOMAIN_POSTPROCESS : 1;
	uint8 MATERIAL_DOMAIN_UI : 1;
	uint8 MATERIAL_DOMAIN_VIRTUALTEXTURE : 1;

	uint8 USE_STENCIL_LOD_DITHER_DEFAULT : 1;

	uint8 MATERIALDOMAIN_SURFACE : 1;
	uint8 MATERIALDOMAIN_DEFERREDDECAL : 1;
	uint8 MATERIALDOMAIN_LIGHTFUNCTION : 1;
	uint8 MATERIALDOMAIN_POSTPROCESS : 1;
	uint8 MATERIALDOMAIN_UI : 1;

	uint8 OUT_BASECOLOR : 1;
	uint8 OUT_BASECOLOR_NORMAL_ROUGHNESS : 1;
	uint8 OUT_BASECOLOR_NORMAL_SPECULAR : 1;
	uint8 OUT_MASK4 : 1;
	uint8 OUT_WORLDHEIGHT : 1;
	uint8 OUT_DISPLACEMENT : 1;

	uint8 SUBSTRATE_ENABLED : 1;
	uint8 MATERIAL_IS_SUBSTRATE : 1;

	uint8 PROJECT_OIT : 1;

	uint8 DUAL_SOURCE_COLOR_BLENDING_ENABLED : 1;

	uint8 IS_MATERIAL_SHADER : 1;
	//// end

	uint8 SUBSTRATE_GBUFFER_FORMAT;

	uint32 MATERIALDECALRESPONSEMASK;

	// new ones
	uint8 IS_VIRTUAL_TEXTURE_MATERIAL : 1;
	uint8 IS_DECAL : 1;
	uint8 IS_BASE_PASS : 1;
	uint8 COMPUTE_SHADED : 1;
	uint8 USES_WORLD_POSITION_OFFSET : 1;
	uint8 USES_FIRST_PERSON_INTERPOLATION : 1;
	uint32 DECAL_RENDERTARGET_COUNT;
	uint32 GBUFFER_LAYOUT;

	uint8 bAllowDevelopmentShaderCompile : 1;// = Material->GetAllowDevelopmentShaderCompile();

};

struct FShaderMaterialDerivedDefines
{
	// Translucent materials need to compute fogging in the forward shading pass
	// Materials that read from scene color skip getting fogged, because the contents of the scene color lookup have already been fogged
	// This is not foolproof, as any additional color the material adds will then not be fogged correctly
	bool TRANSLUCENCY_NEEDS_BASEPASS_FOGGING;

	// With forward shading, fog always needs to be computed in the base pass to work correctly with MSAA
	bool OPAQUE_NEEDS_BASEPASS_FOGGING;

	bool NEEDS_BASEPASS_VERTEX_FOGGING;
	bool NEEDS_BASEPASS_PIXEL_FOGGING;

	// Volumetric fog interpolated per vertex gives very poor results, always sample the volumetric fog texture per-pixel
	// Opaque materials in the deferred renderer get volumetric fog applied in a deferred fog pass
	bool NEEDS_BASEPASS_PIXEL_VOLUMETRIC_FOGGING;

	bool NEEDS_LIGHTMAP_COORDINATE;
	bool NEEDS_LIGHTMAP;

	bool USES_GBUFFER;

	// Only some shader models actually need custom data.
	bool WRITES_CUSTOMDATA_TO_GBUFFER;

	// Based on GetPrecomputedShadowMasks()
	// Note: WRITES_PRECSHADOWFACTOR_TO_GBUFFER is currently disabled because we use the precomputed shadow factor GBuffer outside of STATICLIGHTING_TEXTUREMASK to store UseSingleSampleShadowFromStationaryLights
	bool GBUFFER_HAS_PRECSHADOWFACTOR;
	bool WRITES_PRECSHADOWFACTOR_ZERO;
	bool WRITES_PRECSHADOWFACTOR_TO_GBUFFER;

	// If a primitive has static lighting, we assume it is not moving. If it is, it will be rerendered in an extra renderpass.
	bool SUPPORTS_WRITING_VELOCITY_TO_BASE_PASS;
	bool WRITES_VELOCITY_TO_GBUFFER;

	bool FORCE_FULLY_ROUGH;
	bool EDITOR_ALPHA2COVERAGE;
	bool POST_PROCESS_SUBSURFACE;

	bool TRANSLUCENCY_ANY_PERVERTEX_LIGHTING;
	bool TRANSLUCENCY_ANY_VOLUMETRIC;
	bool TRANSLUCENCY_PERVERTEX_LIGHTING_VOLUME;
	bool TRANSLUCENCY_PERVERTEX_FORWARD_SHADING;

	bool PIXELSHADEROUTPUT_BASEPASS;
		
	bool PIXELSHADEROUTPUT_MRT0;
	bool PIXELSHADEROUTPUT_MRT1;

	bool PIXELSHADEROUTPUT_MRT2;
	bool PIXELSHADEROUTPUT_MRT3;
	bool PIXELSHADEROUTPUT_MRT4;
	bool PIXELSHADEROUTPUT_MRT5;
	bool PIXELSHADEROUTPUT_MRT6;
	bool PIXELSHADEROUTPUT_A2C;
	bool PIXELSHADEROUTPUT_COVERAGE;

	bool SUPPORTS_PIXEL_COVERAGE;

	bool COMPILE_SHADERS_FOR_DEVELOPMENT;
	bool USE_DEVELOPMENT_SHADERS;

	bool USE_EDITOR_SHADERS;
	bool USE_EDITOR_COMPOSITING;
	bool MATERIALBLENDING_ANY_TRANSLUCENT;
	bool IS_MESHPARTICLE_FACTORY;

	bool PLATFORM_SUPPORTS_EDITOR_SHADERS;

	bool SUBSTRATE_ENABLED;
	uint8 SUBSTRATE_GBUFFER_FORMAT;
	bool SHADER_SUBSTRATE_TRANSLUCENT_ENABLED;

	bool OIT_ENABLED;

	bool THIN_TRANSLUCENT_USE_DUAL_BLEND;
	bool MATERIAL_WORKS_WITH_DUAL_SOURCE_COLOR_BLENDING;
};

// These are also the platform defines
struct FShaderCompilerDefines
{
	bool COMPILER_GLSL_ES3_1;
	bool ES3_1_PROFILE;

	bool COMPILER_GLSL;
	bool COMPILER_GLSL_ES3_1_EXT;
	bool ESDEFERRED_PROFILE;
	bool GL4_PROFILE;

	bool METAL_PROFILE;
	bool VULKAN_PROFILE;
	bool MAC;

	bool PLATFORM_SUPPORTS_DEVELOPMENT_SHADERS;
};


FShaderMaterialDerivedDefines RENDERCORE_API CalculateDerivedMaterialParameters(
	const FShaderMaterialPropertyDefines& Mat,
	const FShaderLightmapPropertyDefines& Lightmap,
	const FShaderGlobalDefines& SrcGlobal,
	const FShaderCompilerDefines& Compiler,
	ERHIFeatureLevel::Type FEATURE_LEVEL);

