// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ShaderParameterMacros.h: Macros to builds shader parameter structures and
		their metadata.
=============================================================================*/

#pragma once

#include "ShaderParameterStructDeclaration.h"
#include "ShaderParameterMetadata.h"
#include "RenderGraphAllocator.h"
#include "Algo/Reverse.h"
#include "Templates/IsArrayOrRefOfTypeByPredicate.h"
#include "Traits/IsCharEncodingCompatibleWith.h"
#include "Misc/AlignedElement.h"
#include "Misc/AssertionMacros.h"
#include "RHICommandList.h"
#include "RenderGraphTextureSubresource.h"

PGO_LINK_DISABLE_WARNINGS

class FRDGTexture;
class FRDGTextureSRV;
class FRDGTextureUAV;
class FRDGBuffer;
class FRDGBufferSRV;
class FRDGBufferUAV;
class FRDGUniformBuffer;
enum class EShaderBindingLayoutFlags : uint8;
class FShaderBindingLayoutContainer;
template <typename TUniformStruct> class TRDGUniformBuffer;

/** Alignements tools because alignas() does not work on type in clang. */
template<typename T, int32 Alignment>
class TAlignedTypedef;

#define IMPLEMENT_ALIGNED_TYPE(Alignment) \
	template<typename T> \
	class TAlignedTypedef<T,Alignment> \
	{ \
	public: \
		typedef MS_ALIGN(Alignment) T Type GCC_ALIGN(Alignment); \
	};

IMPLEMENT_ALIGNED_TYPE(1);
IMPLEMENT_ALIGNED_TYPE(2);
IMPLEMENT_ALIGNED_TYPE(4);
IMPLEMENT_ALIGNED_TYPE(8);
IMPLEMENT_ALIGNED_TYPE(16);
#undef IMPLEMENT_ALIGNED_TYPE

/** Fixed 8bytes sized and aligned pointer for shader parameters. */
template<typename PtrType>
using TAlignedShaderParameterPtr = typename TAlignedTypedef<PtrType, SHADER_PARAMETER_POINTER_ALIGNMENT>::Type;

/** Retrieve the metadata of a UB type */
template<class UniformBufferStructType, typename = void>
struct TUniformBufferMetadataHelper
{
	static const FShaderParametersMetadata* GetStructMetadata()
	{
		// This uses ADL rather than templates, because template specializations can't be defined in a different namespace
		return GetForwardDeclaredShaderParametersStructMetadata((UniformBufferStructType*)nullptr);
	}
};

template<class UniformBufferStructType>
struct TUniformBufferMetadataHelper<UniformBufferStructType, typename std::enable_if<!std::is_same<typename UniformBufferStructType::FTypeInfo, void>::value>::type>
{
	static const FShaderParametersMetadata* GetStructMetadata()
	{
		return UniformBufferStructType::FTypeInfo::GetStructMetadata();
	}
};


/** A reference to a uniform buffer RHI resource with a specific structure. */
template<typename TBufferStruct>
class TUniformBufferRef : public FUniformBufferRHIRef
{
	static_assert(!TIsUECoreVariant<TBufferStruct, double>::Value, "UniformBufferRHIRef cannot be double core variants! Switch to float variant.");

public:
	/** Initializes the reference to null. */
	TUniformBufferRef() = default;

	/** Construct an instance from an existing RHI uniform buffer pointer. Validates at runtime that the type of the uniform buffer matches the template struct type. */
	explicit TUniformBufferRef(FRHIUniformBuffer* InRHIRef)
		: FUniformBufferRHIRef(InRHIRef)
	{
		checkf(!InRHIRef || TUniformBufferMetadataHelper<TBufferStruct>::GetStructMetadata()->GetLayoutPtr() == InRHIRef->GetLayoutPtr(),
			TEXT("Attempted to create a uniform buffer of type '%s' from uniform buffer pointer of type '%s'"),
			*TUniformBufferMetadataHelper<TBufferStruct>::GetStructMetadata()->GetLayout().Name,
			*InRHIRef->GetLayout().Name);
	}

	/** Creates a uniform buffer with the given value, and returns a structured reference to it. */
	static TUniformBufferRef<TBufferStruct> CreateUniformBufferImmediate(const TBufferStruct& Value, EUniformBufferUsage Usage, EUniformBufferValidation Validation = EUniformBufferValidation::ValidateResources)
	{
		return TUniformBufferRef<TBufferStruct>(RHICreateUniformBuffer(&Value, TUniformBufferMetadataHelper<TBufferStruct>::GetStructMetadata()->GetLayoutPtr(), Usage, Validation));
	}
	/** Creates a uniform buffer with the given value, and returns a structured reference to it. */
	static TUniformBufferRef<TBufferStruct> CreateEmptyUniformBufferImmediate(EUniformBufferUsage Usage)
	{
		return TUniformBufferRef<TBufferStruct>(RHICreateUniformBuffer(nullptr, TUniformBufferMetadataHelper<TBufferStruct>::GetStructMetadata()->GetLayoutPtr(), Usage, EUniformBufferValidation::ValidateResources));
	}

	void UpdateUniformBufferImmediate(FRHICommandListBase& RHICmdList, const TBufferStruct& Value)
	{
		RHICmdList.UpdateUniformBuffer(GetReference(), &Value);
	}

private:

	template<typename TBufferStruct2>
	friend class TUniformBuffer;

	friend class TRDGUniformBuffer<TBufferStruct>;
};

class alignas(SHADER_PARAMETER_STRUCT_ALIGNMENT) FUniformBufferBinding
{
public:
	FUniformBufferBinding() = default;

	inline FRHIUniformBuffer* GetUniformBuffer() const
	{
		return UniformBuffer;
	}

	inline EUniformBufferBindingFlags GetBindingFlags() const
	{
		return BindingFlags;
	}

	inline bool IsStatic() const
	{
		return EnumHasAnyFlags(BindingFlags, EUniformBufferBindingFlags::Static);
	}

	inline bool IsShader() const
	{
		return EnumHasAnyFlags(BindingFlags, EUniformBufferBindingFlags::Shader);
	}

	inline operator bool() const
	{
		return UniformBuffer != nullptr;
	}

protected:
	inline FUniformBufferBinding(const FUniformBufferRHIRef& InUniformBuffer, EUniformBufferBindingFlags InBindingFlags)
		: UniformBuffer(InUniformBuffer)
		, BindingFlags(InBindingFlags)
	{}

private:
	FRHIUniformBuffer* UniformBuffer = nullptr;
	EUniformBufferBindingFlags BindingFlags = EUniformBufferBindingFlags::Shader;
};

template<typename TBufferStruct>
class alignas(SHADER_PARAMETER_STRUCT_ALIGNMENT) TUniformBufferBinding
	: public FUniformBufferBinding
{
public:
	TUniformBufferBinding() = default;

	TUniformBufferBinding(const TUniformBufferRef<TBufferStruct>& InUniformBuffer)
		: FUniformBufferBinding(InUniformBuffer, TUniformBufferMetadataHelper<TBufferStruct>::GetStructMetadata()->GetPreferredBindingFlag())
	{}

	TUniformBufferBinding(const TUniformBufferRef<TBufferStruct>& InUniformBuffer, EUniformBufferBindingFlags InBindingFlags)
		: FUniformBufferBinding(InUniformBuffer, InBindingFlags)
	{
#if DO_CHECK
		const auto* StructMetadata = TUniformBufferMetadataHelper<TBufferStruct>::GetStructMetadata();
		checkf(
			EnumHasAllFlags(StructMetadata->GetBindingFlags(), GetBindingFlags()),
			TEXT("Uniform buffer binding flags don't match those supported by the uniform buffer layout '%s."),
			StructMetadata->GetStructTypeName());
#endif
	}

	inline TUniformBufferRef<TBufferStruct> GetUniformBufferRef() const
	{
		return TUniformBufferRef<TBufferStruct>(FUniformBufferBinding::GetUniformBuffer());
	}

	inline TRDGUniformBuffer<TBufferStruct>* operator->() const
	{
		return GetUniformBuffer();
	}
};

template <typename TBufferStruct>
inline TUniformBufferBinding<TBufferStruct> GetStaticBinding(const TUniformBufferRef<TBufferStruct>& InUniformBuffer)
{
	return TUniformBufferBinding<TBufferStruct>(InUniformBuffer, EUniformBufferBindingFlags::Static);
}

template <typename TBufferStruct>
inline TUniformBufferBinding<TBufferStruct> GetShaderBinding(const TUniformBufferRef<TBufferStruct>& InUniformBuffer)
{
	return TUniformBufferBinding<TBufferStruct>(InUniformBuffer, EUniformBufferBindingFlags::Shader);
}

class alignas(SHADER_PARAMETER_STRUCT_ALIGNMENT) FRDGUniformBufferBinding
{
public:
	FRDGUniformBufferBinding() = default;

	inline FRDGUniformBuffer* GetUniformBuffer() const
	{
		return UniformBuffer;
	}

	inline EUniformBufferBindingFlags GetBindingFlags() const
	{
		return BindingFlags;
	}

	inline bool IsStatic() const
	{
		return EnumHasAnyFlags(BindingFlags, EUniformBufferBindingFlags::Static);
	}

	inline bool IsShader() const
	{
		return EnumHasAnyFlags(BindingFlags, EUniformBufferBindingFlags::Shader);
	}

	inline operator bool() const
	{
		return UniformBuffer != nullptr;
	}

	inline FRDGUniformBuffer* operator->() const
	{
		return UniformBuffer;
	}

protected:
	FRDGUniformBufferBinding(FRDGUniformBuffer* InUniformBuffer, EUniformBufferBindingFlags InBindingFlags)
		: UniformBuffer(InUniformBuffer)
		, BindingFlags(InBindingFlags)
	{}

private:
	FRDGUniformBuffer* UniformBuffer = nullptr;
	EUniformBufferBindingFlags BindingFlags = EUniformBufferBindingFlags::Shader;
};

template <typename TBufferStruct>
class alignas(SHADER_PARAMETER_STRUCT_ALIGNMENT) TRDGUniformBufferBinding
	: public FRDGUniformBufferBinding
{
public:
	TRDGUniformBufferBinding() = default;

	TRDGUniformBufferBinding(TRDGUniformBuffer<TBufferStruct>* InUniformBuffer)
		: FRDGUniformBufferBinding(InUniformBuffer, TUniformBufferMetadataHelper<TBufferStruct>::GetStructMetadata()->GetPreferredBindingFlag())
	{}

	TRDGUniformBufferBinding(TRDGUniformBuffer<TBufferStruct>* InUniformBuffer, EUniformBufferBindingFlags InBindingFlags)
		: FRDGUniformBufferBinding(InUniformBuffer, InBindingFlags)
	{
#if DO_CHECK
		const auto* StructMetadata = TUniformBufferMetadataHelper<TBufferStruct>::GetStructMetadata();
		checkf(
			EnumHasAllFlags(StructMetadata->GetBindingFlags(), GetBindingFlags()),
			TEXT("RDG uniform buffer binding flags don't match those supported by the uniform buffer layout '%s."),
			StructMetadata->GetStructTypeName());
#endif
	}

	inline TRDGUniformBuffer<TBufferStruct>* GetUniformBuffer() const
	{
		return static_cast<TRDGUniformBuffer<TBufferStruct>*>(FRDGUniformBufferBinding::GetUniformBuffer());
	}

	inline TRDGUniformBuffer<TBufferStruct>* operator->() const
	{
		return GetUniformBuffer();
	}
};

/** Constructs a uniform buffer binding that will utilize the shader binding model. */
template <typename TBufferStruct>
inline TRDGUniformBufferBinding<TBufferStruct> GetShaderBinding(TRDGUniformBuffer<TBufferStruct>* InUniformBuffer)
{
	return TRDGUniformBufferBinding<TBufferStruct>(InUniformBuffer, EUniformBufferBindingFlags::Shader);
}

/** Constructs a uniform buffer binding that will utilize the static binding model. */
template <typename TBufferStruct>
inline TRDGUniformBufferBinding<TBufferStruct> GetStaticBinding(TRDGUniformBuffer<TBufferStruct>* InUniformBuffer)
{
	return TRDGUniformBufferBinding<TBufferStruct>(InUniformBuffer, EUniformBufferBindingFlags::Static);
}

class alignas(SHADER_PARAMETER_POINTER_ALIGNMENT) FRDGBufferAccess
{
public:
	FRDGBufferAccess() = default;
	FRDGBufferAccess(FRDGBuffer* InBuffer, ERHIAccess InAccess)
		: Buffer(InBuffer)
		, Access(InAccess)
	{
		check(IsValidAccess(InAccess));
	}

	inline FRDGBuffer* GetBuffer() const { return Buffer; }
	inline ERHIAccess  GetAccess() const { return Access; }

	inline operator bool() const
	{
		return Buffer != nullptr;
	}

	inline FRDGBuffer* operator->() const
	{
		check(Buffer);
		return Buffer;
	}

	inline operator FRDGBuffer*() const
	{
		return Buffer;
	}

	inline bool operator==(const FRDGBufferAccess& RHS) const
	{
		return Buffer == RHS.Buffer && Access == RHS.Access;
	}

	inline bool operator!=(const FRDGBufferAccess& RHS) const
	{
		return Buffer != RHS.Buffer || Access != RHS.Access;
	}

private:
	TAlignedShaderParameterPtr<FRDGBuffer*> Buffer = nullptr;
	ERHIAccess  Access = ERHIAccess::Unknown;
};

template <ERHIAccess InAccess>
class alignas(SHADER_PARAMETER_POINTER_ALIGNMENT) TRDGBufferAccess
	: public FRDGBufferAccess
{
public:
	static_assert(IsValidAccess(InAccess), "Buffer access is invalid.");

	TRDGBufferAccess()
		: FRDGBufferAccess(nullptr, InAccess)
	{
		check(IsValidAccess(InAccess));
	}

	TRDGBufferAccess(FRDGBuffer* InBuffer)
		: FRDGBufferAccess(InBuffer, InAccess)
	{}
};

class alignas(SHADER_PARAMETER_POINTER_ALIGNMENT) FRDGTextureAccess
{
public:
	FRDGTextureAccess() = default;

	FRDGTextureAccess(FRDGTexture* InTexture, FRDGTextureSubresourceRange InSubresourceRange, ERHIAccess InAccess)
		: Texture(InTexture)
		, SubresourceRange(InSubresourceRange)
		, Access(InAccess)
	{}

	RENDERCORE_API FRDGTextureAccess(FRDGTexture* InTexture, ERHIAccess InAccess);
	RENDERCORE_API FRDGTextureAccess(FRDGTextureSRV* InTextureSRV, ERHIAccess InAccess);
	RENDERCORE_API FRDGTextureAccess(FRDGTextureUAV* InTextureUAV, ERHIAccess InAccess);

	inline FRDGTexture* GetTexture() const { return Texture; }
	inline FRDGTextureSubresourceRange GetSubresourceRange() const { return SubresourceRange; }
	inline ERHIAccess GetAccess() const  { return Access; }

	inline operator bool() const
	{
		return Texture != nullptr;
	}

	inline FRDGTexture* operator->() const
	{
		check(Texture);
		return Texture;
	}

	inline operator FRDGTexture*() const
	{
		return Texture;
	}

	inline bool operator==(const FRDGTextureAccess& RHS) const
	{
		return Texture == RHS.Texture && Access == RHS.Access;
	}

	inline bool operator!=(const FRDGTextureAccess& RHS) const
	{
		return Texture != RHS.Texture || Access != RHS.Access;
	}

private:
	TAlignedShaderParameterPtr<FRDGTexture*> Texture = nullptr;
	FRDGTextureSubresourceRange SubresourceRange;
	ERHIAccess Access = ERHIAccess::Unknown;
};

template <ERHIAccess InAccess>
class alignas(SHADER_PARAMETER_POINTER_ALIGNMENT) TRDGTextureAccess
	: public FRDGTextureAccess
{
public:
	static_assert(IsValidAccess(InAccess), "Texture access is invalid.");

	TRDGTextureAccess() = default;

	TRDGTextureAccess(FRDGTexture* InTexture, FRDGTextureSubresourceRange InSubresourceRange)
		: FRDGTextureAccess(InTexture, InSubresourceRange, InAccess)
	{}

	TRDGTextureAccess(FRDGTexture* InTexture)
		: FRDGTextureAccess(InTexture, InAccess)
	{}

	TRDGTextureAccess(FRDGTextureSRV* InTextureSRV)
		: FRDGTextureAccess(InTextureSRV, InAccess)
	{}

	TRDGTextureAccess(FRDGTextureUAV* InTextureUAV)
		: FRDGTextureAccess(InTextureUAV, InAccess)
	{}
};

template <typename ResourceAccessType>
class alignas(SHADER_PARAMETER_POINTER_ALIGNMENT) TRDGResourceAccessArray
	: public TArray<ResourceAccessType, FRDGArrayAllocator>
{
	using Super = TArray<ResourceAccessType, FRDGArrayAllocator>;
public:
	using Super::Super;
};

using FRDGBufferAccessArray = TRDGResourceAccessArray<FRDGBufferAccess>;
using FRDGTextureAccessArray = TRDGResourceAccessArray<FRDGTextureAccess>;

/** Render graph information about how to bind a render target. */
struct FRenderTargetBinding
{
	FRenderTargetBinding() = default;

	FRenderTargetBinding(FRDGTexture* InTexture, ERenderTargetLoadAction InLoadAction, uint8 InMipIndex = 0, int16 InArraySlice = -1)
		: Texture(InTexture)
		, LoadAction(InLoadAction)
		, MipIndex(InMipIndex)
		, ArraySlice(InArraySlice)
	{
		check(Validate());
	}

	FRenderTargetBinding(FRDGTexture* InTexture, FRDGTexture* InResolveTexture, ERenderTargetLoadAction InLoadAction, uint8 InMipIndex = 0, int16 InArraySlice = -1)
		: Texture(InTexture)
		, ResolveTexture(InResolveTexture)
		, LoadAction(InLoadAction)
		, MipIndex(InMipIndex)
		, ArraySlice(InArraySlice)
	{
		check(Validate());
	}

	FRDGTexture* GetTexture() const
	{
		return Texture;
	}

	FRDGTexture* GetResolveTexture() const
	{
		return ResolveTexture;
	}

	ERenderTargetLoadAction GetLoadAction() const
	{
		return LoadAction;
	}

	uint8 GetMipIndex() const
	{
		return MipIndex;
	}

	int16 GetArraySlice() const
	{
		return ArraySlice;
	}

	/** Whether we can merge a render pass using Other into a render pass using this render target binding. */
	inline bool CanMergeBefore(const FRenderTargetBinding& Other) const
	{
		return
			Texture == Other.Texture &&
			ResolveTexture == Other.ResolveTexture &&
			Other.LoadAction != ERenderTargetLoadAction::EClear &&
			MipIndex == Other.MipIndex &&
			ArraySlice == Other.ArraySlice;
	}

	void SetTexture(FRDGTexture* InTexture)
	{
		Texture = InTexture;
		check(Validate());
	}

	void SetResolveTexture(FRDGTexture* InTexture)
	{
		ResolveTexture = InTexture;
		check(Validate());
	}

	void SetLoadAction(ERenderTargetLoadAction InLoadAction)
	{
		LoadAction = InLoadAction;
		check(Validate());
	}

	void SetMipIndex(uint8 InMipIndex)
	{
		MipIndex = InMipIndex;
		check(Validate());
	}

	void SetArraySlice(int16 InArraySlice)
	{
		ArraySlice = InArraySlice;
		check(Validate());
	}

private:
	/** Intentionally kept private to force setting the load action in the constructor. */
	TAlignedShaderParameterPtr<FRDGTexture*> Texture		= nullptr;
	TAlignedShaderParameterPtr<FRDGTexture*> ResolveTexture	= nullptr;
	ERenderTargetLoadAction		LoadAction		= ERenderTargetLoadAction::ENoAction;
	uint8						MipIndex		= 0;
	int16						ArraySlice		= -1;

	RENDERCORE_API bool Validate() const;
};


/** Render graph information about how to bind a depth-stencil render target. */
struct FDepthStencilBinding
{
	FDepthStencilBinding() = default;

	/**
	 * Creates a render target binding informations for a depth/stencil texture.
	 *
	 * Notes: Load and store action are on explicit without default values, to force the user to not forget one of these.
	 */
	inline FDepthStencilBinding(
		FRDGTexture* InTexture,
		ERenderTargetLoadAction InDepthLoadAction,
		ERenderTargetLoadAction InStencilLoadAction,
		FExclusiveDepthStencil InDepthStencilAccess)
		: Texture(InTexture)
		, ResolveTexture(nullptr)
		, DepthLoadAction(InDepthLoadAction)
		, StencilLoadAction(InStencilLoadAction)
		, DepthStencilAccess(InDepthStencilAccess)
	{
		check(Validate());
	}

	inline FDepthStencilBinding(
		FRDGTexture* InTexture,
		FRDGTexture* InResolveTexture,
		ERenderTargetLoadAction InDepthLoadAction,
		ERenderTargetLoadAction InStencilLoadAction,
		FExclusiveDepthStencil InDepthStencilAccess)
		: Texture(InTexture)
		, ResolveTexture(InResolveTexture)
		, DepthLoadAction(InDepthLoadAction)
		, StencilLoadAction(InStencilLoadAction)
		, DepthStencilAccess(InDepthStencilAccess)
	{
		check(Validate());
	}

	inline FDepthStencilBinding(
		FRDGTexture* InTexture,
		ERenderTargetLoadAction InDepthLoadAction,
		FExclusiveDepthStencil InDepthStencilAccess)
		: Texture(InTexture)
		, ResolveTexture(nullptr)
		, DepthLoadAction(InDepthLoadAction)
		, DepthStencilAccess(InDepthStencilAccess)
	{
		check(Validate());
	}

	inline FDepthStencilBinding(
		FRDGTexture* InTexture,
		FRDGTexture* InResolveTexture,
		ERenderTargetLoadAction InDepthLoadAction,
		FExclusiveDepthStencil InDepthStencilAccess)
		: Texture(InTexture)
		, ResolveTexture(InResolveTexture)
		, DepthLoadAction(InDepthLoadAction)
		, DepthStencilAccess(InDepthStencilAccess)
	{
		check(Validate());
	}

	inline FRDGTexture* GetTexture() const
	{
		return Texture;
	}
	inline FRDGTexture* GetResolveTexture() const
	{
		return ResolveTexture;
	}
	inline ERenderTargetLoadAction GetDepthLoadAction() const
	{
		return DepthLoadAction;
	}
	inline ERenderTargetLoadAction GetStencilLoadAction() const
	{
		return StencilLoadAction;
	}
	inline FExclusiveDepthStencil GetDepthStencilAccess() const
	{
		return DepthStencilAccess;
	}

	// Whether we can merge a render pass using @ref Other into a render pass using this depth stencil binding.
	inline bool CanMergeBefore(const FDepthStencilBinding& Other) const
	{
		return
			Texture == Other.Texture &&
			ResolveTexture == Other.ResolveTexture &&
			Other.DepthLoadAction != ERenderTargetLoadAction::EClear &&
			Other.StencilLoadAction != ERenderTargetLoadAction::EClear &&
			DepthStencilAccess == Other.DepthStencilAccess;
	}

	void SetTexture(FRDGTexture* InTexture)
	{
		Texture = InTexture;
		check(Validate());
	}

	void SetResolveTexture(FRDGTexture* InTexture)
	{
		ResolveTexture = InTexture;
		check(Validate());
	}

	void SetDepthStencilAccess(FExclusiveDepthStencil InAccess)
	{
		DepthStencilAccess = InAccess;
		check(Validate());
	}

	void SetDepthLoadAction(ERenderTargetLoadAction InAction)
	{
		DepthLoadAction = InAction;
		check(Validate());
	}

	void SetStencilLoadAction(ERenderTargetLoadAction InAction)
	{
		StencilLoadAction = InAction;
		check(Validate());
	}

private:
	/** 
	 * All parameters required to bind a depth render target deferred. This are purposefully private to
	 * force the user to call FDepthStencilBinding() constructors. No defaults allowed.
	 */
	TAlignedShaderParameterPtr<FRDGTexture*> Texture = nullptr;
	TAlignedShaderParameterPtr<FRDGTexture*> ResolveTexture = nullptr;
	ERenderTargetLoadAction		DepthLoadAction		= ERenderTargetLoadAction::ENoAction;
	ERenderTargetLoadAction		StencilLoadAction	= ERenderTargetLoadAction::ENoAction;
	FExclusiveDepthStencil		DepthStencilAccess	= FExclusiveDepthStencil::DepthNop_StencilNop;

	RENDERCORE_API bool Validate() const;
};

/** Special shader parameters type for a pass parameter to setup render targets. */
struct alignas(SHADER_PARAMETER_STRUCT_ALIGNMENT) FRenderTargetBindingSlots
{
	TStaticArray<FRenderTargetBinding, MaxSimultaneousRenderTargets> Output;
	FDepthStencilBinding DepthStencil;
	FResolveRect ResolveRect;
	uint32 NumOcclusionQueries = 0;
	ESubpassHint SubpassHint = ESubpassHint::None;
	uint8 MultiViewCount = 0;
	FRDGTexture* ShadingRateTexture = nullptr;

	/** Accessors for regular output to simplify the syntax to:
	 *
	 *	FRenderTargetParameters PassParameters;
	 *	PassParameters.RenderTargets.DepthStencil = ... ;
	 *	PassParameters.RenderTargets[0] = ... ;
	 */
	FRenderTargetBinding& operator[](uint32 Index)
	{
		return Output[Index];
	}

	const FRenderTargetBinding& operator[](uint32 Index) const
	{
		return Output[Index];
	}

	template <typename TFunction>
	void Enumerate(TFunction Function)
	{
		for (int32 Index = 0; Index < Output.Num(); Index++)
		{
			if (Output[Index].GetTexture())
			{
				Function(Output[Index]);
			}
			else
			{
				break;
			}
		}
	}

	template <typename TFunction>
	void Enumerate(TFunction Function) const
	{
		for (int32 Index = 0; Index < Output.Num(); Index++)
		{
			if (Output[Index].GetTexture())
			{
				Function(Output[Index]);
			}
			else
			{
				break;
			}
		}
	}

	uint32 GetActiveCount() const
	{
		uint32 Count = 0;
		for (; Output[Count].GetTexture() != nullptr; ++Count) {};
		return Count;
	}

	bool CanMergeBefore(const FRenderTargetBindingSlots& Other) const
	{
		for (int32 Index = 0; Index < MaxSimultaneousRenderTargets; ++Index)
		{
			if (!Output[Index].CanMergeBefore(Other.Output[Index]))
			{
				return false;
			}
		}

		if (!DepthStencil.CanMergeBefore(Other.DepthStencil))
		{
			return false;
		}

		if (ResolveRect != Other.ResolveRect ||
			(NumOcclusionQueries != Other.NumOcclusionQueries && Other.NumOcclusionQueries != 0) ||
			SubpassHint != Other.SubpassHint ||
			MultiViewCount != Other.MultiViewCount ||
			ShadingRateTexture != Other.ShadingRateTexture)
		{
			return false;
		}

		return true;
	}

	struct FTypeInfo
	{
		static constexpr int32 NumRows = 1;
		static constexpr int32 NumColumns = 1;
		static constexpr int32 NumElements = 0;
		static constexpr int32 Alignment = SHADER_PARAMETER_STRUCT_ALIGNMENT;
		static constexpr bool bIsStoredInConstantBuffer = false;

		using TAlignedType = FRenderTargetBindingSlots;

		static inline const FShaderParametersMetadata* GetStructMetadata() { return nullptr; }
	};
};

static_assert(sizeof(FRenderTargetBindingSlots) == 256, "FRenderTargetBindingSlots needs to be same size on all platforms.");

/** Static array of shader resource shader that is initialized to nullptr. */
template<typename TElement, uint32 NumElements>
class alignas(SHADER_PARAMETER_POINTER_ALIGNMENT) TShaderResourceParameterArray : public TStaticArray<std::conditional_t<SHADER_PARAMETER_POINTER_ALIGNMENT == alignof(TElement), TElement, UE::Core::TAlignedElement<TElement, SHADER_PARAMETER_POINTER_ALIGNMENT>>, NumElements>
{
public:
	inline TShaderResourceParameterArray()
	{
		for (uint32 i = 0; i < NumElements; i++)
		{
			(*this)[i] = nullptr;
		}
	}
};


/** Template to transcode some meta data information for a type <TypeParameter> not specific to shader parameters API. */
template<typename TypeParameter>
struct TShaderParameterTypeInfo
{
	/** Defines what the type actually is. */
	static constexpr EUniformBufferBaseType BaseType = UBMT_INVALID;

	/** Defines the number rows and columns for vector or matrix based data typed. */
	static constexpr int32 NumRows = 1;
	static constexpr int32 NumColumns = 1;

	/** Defines the number of elements in an array fashion. 0 means this is not a TStaticArray,
	 * which therefor means there is 1 element.
	 */
	static constexpr int32 NumElements = 0;

	/** Defines the alignment of the elements in bytes. */
	static constexpr int32 Alignment = alignof(TypeParameter);

	/** Defines whether this element is stored in constant buffer or not.
	 * This informations is usefull to ensure at compile time everything in the
	 * structure get defined at the end of the structure, to reduce as much as possible
	 * the size of the constant buffer.
	 */
	static constexpr bool bIsStoredInConstantBuffer = true;

	/** Type that is actually alligned. */
	using TAlignedType = TypeParameter;

	/** Type that has a multiple of 4 components. */
	using TInstancedType = TypeParameter;

	static const FShaderParametersMetadata* GetStructMetadata() { return TypeParameter::FTypeInfo::GetStructMetadata(); }
};

// Compile SHADER_PARAMETER(bool, MyBool), just to give good error message to programmer why they shouldn't do that.
template<>
struct TShaderParameterTypeInfo<bool>
{
	static constexpr EUniformBufferBaseType BaseType = UBMT_BOOL;
	static constexpr int32 NumRows = 1;
	static constexpr int32 NumColumns = 1;
	static constexpr int32 NumElements = 0;
	static constexpr int32 Alignment = 4;
	static constexpr bool bIsStoredInConstantBuffer = true;
	
	using TAlignedType = TAlignedTypedef<bool, Alignment>::Type;
	using TInstancedType = FUintVector4;

	static const FShaderParametersMetadata* GetStructMetadata() { return nullptr; }
};

template<>
struct TShaderParameterTypeInfo<uint32>
{
	static constexpr EUniformBufferBaseType BaseType = UBMT_UINT32;
	static constexpr int32 NumRows = 1;
	static constexpr int32 NumColumns = 1;
	static constexpr int32 NumElements = 0;
	static constexpr int32 Alignment = 4;
	static constexpr bool bIsStoredInConstantBuffer = true;

	using TAlignedType = TAlignedTypedef<uint32, Alignment>::Type;
	using TInstancedType = FUintVector4;

	static const FShaderParametersMetadata* GetStructMetadata() { return nullptr; }
};
	

template<>
struct TShaderParameterTypeInfo<int32>
{
	static constexpr EUniformBufferBaseType BaseType = UBMT_INT32;
	static constexpr int32 NumRows = 1;
	static constexpr int32 NumColumns = 1;
	static constexpr int32 NumElements = 0;
	static constexpr int32 Alignment = 4;
	static constexpr bool bIsStoredInConstantBuffer = true;

	using TAlignedType = TAlignedTypedef<int32, Alignment>::Type;
	using TInstancedType = FIntVector4;

	static const FShaderParametersMetadata* GetStructMetadata() { return nullptr; }
};

template<>
struct TShaderParameterTypeInfo<float>
{
	static constexpr EUniformBufferBaseType BaseType = UBMT_FLOAT32;
	static constexpr int32 NumRows = 1;
	static constexpr int32 NumColumns = 1;
	static constexpr int32 NumElements = 0;
	static constexpr int32 Alignment = 4;
	static constexpr bool bIsStoredInConstantBuffer = true;

	using TAlignedType = TAlignedTypedef<float, Alignment>::Type;
	using TInstancedType = FVector4f;

	static const FShaderParametersMetadata* GetStructMetadata() { return nullptr; }
};

template<>
struct TShaderParameterTypeInfo<FVector2f>
{
	static constexpr EUniformBufferBaseType BaseType = UBMT_FLOAT32;
	static constexpr int32 NumRows = 1;
	static constexpr int32 NumColumns = 2;
	static constexpr int32 NumElements = 0;
	static constexpr int32 Alignment = 8;
	static constexpr bool bIsStoredInConstantBuffer = true;

	using TAlignedType = TAlignedTypedef<FVector2f, Alignment>::Type;
	using TInstancedType = FVector4f;

	static const FShaderParametersMetadata* GetStructMetadata() { return nullptr; }
};

template<>
struct TShaderParameterTypeInfo<FVector3f>
{
	static constexpr EUniformBufferBaseType BaseType = UBMT_FLOAT32;
	static constexpr int32 NumRows = 1;
	static constexpr int32 NumColumns = 3;
	static constexpr int32 NumElements = 0;
	static constexpr int32 Alignment = 16;
	static constexpr bool bIsStoredInConstantBuffer = true;

	using TAlignedType = TAlignedTypedef<FVector3f, Alignment>::Type;
	using TInstancedType = FVector4f;

	static const FShaderParametersMetadata* GetStructMetadata() { return nullptr; }
};

template<>
struct TShaderParameterTypeInfo<FVector4f>
{
	static constexpr EUniformBufferBaseType BaseType = UBMT_FLOAT32;
	static constexpr int32 NumRows = 1;
	static constexpr int32 NumColumns = 4;
	static constexpr int32 NumElements = 0;
	static constexpr int32 Alignment = 16;
	static constexpr bool bIsStoredInConstantBuffer = true;

	using TAlignedType = TAlignedTypedef<FVector4f, Alignment>::Type;
	using TInstancedType = FVector4f;

	static const FShaderParametersMetadata* GetStructMetadata() { return nullptr; }
};
	
template<>
struct TShaderParameterTypeInfo<FLinearColor>
{
	static constexpr EUniformBufferBaseType BaseType = UBMT_FLOAT32;
	static constexpr int32 NumRows = 1;
	static constexpr int32 NumColumns = 4;
	static constexpr int32 NumElements = 0;
	static constexpr int32 Alignment = 16;
	static constexpr bool bIsStoredInConstantBuffer = true;

	using TAlignedType = TAlignedTypedef<FLinearColor, Alignment>::Type;
	using TInstancedType = FLinearColor;

	static const FShaderParametersMetadata* GetStructMetadata() { return nullptr; }
};
	
template<>
struct TShaderParameterTypeInfo<FIntPoint>
{
	static constexpr EUniformBufferBaseType BaseType = UBMT_INT32;
	static constexpr int32 NumRows = 1;
	static constexpr int32 NumColumns = 2;
	static constexpr int32 NumElements = 0;
	static constexpr int32 Alignment = 8;
	static constexpr bool bIsStoredInConstantBuffer = true;

	using TAlignedType = TAlignedTypedef<FIntPoint, Alignment>::Type;
	using TInstancedType = FIntVector4;

	static const FShaderParametersMetadata* GetStructMetadata() { return nullptr; }
};

template<>
struct TShaderParameterTypeInfo<FUintVector2>
{
	static constexpr EUniformBufferBaseType BaseType = UBMT_UINT32;
	static constexpr int32 NumRows = 1;
	static constexpr int32 NumColumns = 2;
	static constexpr int32 NumElements = 0;
	static constexpr int32 Alignment = 8;
	static constexpr bool bIsStoredInConstantBuffer = true;

	using TAlignedType = TAlignedTypedef<FUintVector2, Alignment>::Type;
	using TInstancedType = FUintVector4;

	static const FShaderParametersMetadata* GetStructMetadata() { return nullptr; }
};

template<>
struct TShaderParameterTypeInfo<FIntVector>
{
	static constexpr EUniformBufferBaseType BaseType = UBMT_INT32;
	static constexpr int32 NumRows = 1;
	static constexpr int32 NumColumns = 3;
	static constexpr int32 NumElements = 0;
	static constexpr int32 Alignment = 16;
	static constexpr bool bIsStoredInConstantBuffer = true;

	using TAlignedType = TAlignedTypedef<FIntVector, Alignment>::Type;
	using TInstancedType = FIntVector4;

	static const FShaderParametersMetadata* GetStructMetadata() { return nullptr; }
};

template<>
struct TShaderParameterTypeInfo<FUintVector3>
{
	static constexpr EUniformBufferBaseType BaseType = UBMT_UINT32;
	static constexpr int32 NumRows = 1;
	static constexpr int32 NumColumns = 3;
	static constexpr int32 NumElements = 0;
	static constexpr int32 Alignment = 16;
	static constexpr bool bIsStoredInConstantBuffer = true;

	using TAlignedType = TAlignedTypedef<FUintVector3, Alignment>::Type;
	using TInstancedType = FUintVector4;

	static const FShaderParametersMetadata* GetStructMetadata() { return nullptr; }
};

template<>
struct TShaderParameterTypeInfo<FIntVector4>
{
	static constexpr EUniformBufferBaseType BaseType = UBMT_INT32;
	static constexpr int32 NumRows = 1;
	static constexpr int32 NumColumns = 4;
	static constexpr int32 NumElements = 0;
	static constexpr int32 Alignment = 16;
	static constexpr bool bIsStoredInConstantBuffer = true;

	using TAlignedType = TAlignedTypedef<FIntVector4, Alignment>::Type;
	using TInstancedType = FIntVector4;

	static const FShaderParametersMetadata* GetStructMetadata() { return nullptr; }
};

template<>
struct TShaderParameterTypeInfo<FUintVector4>
{
	static constexpr EUniformBufferBaseType BaseType = UBMT_UINT32;
	static constexpr int32 NumRows = 1;
	static constexpr int32 NumColumns = 4;
	static constexpr int32 NumElements = 0;
	static constexpr int32 Alignment = 16;
	static constexpr bool bIsStoredInConstantBuffer = true;

	using TAlignedType = TAlignedTypedef<FUintVector4, Alignment>::Type;
	using TInstancedType = FUintVector4;

	static const FShaderParametersMetadata* GetStructMetadata() { return nullptr; }
};

template<>
struct TShaderParameterTypeInfo<FIntRect>
{
	static constexpr EUniformBufferBaseType BaseType = UBMT_INT32;
	static constexpr int32 NumRows = 1;
	static constexpr int32 NumColumns = 4;
	static constexpr int32 NumElements = 0;
	static constexpr int32 Alignment = 16;
	static constexpr bool bIsStoredInConstantBuffer = true;

	using TAlignedType = TAlignedTypedef<FIntRect, Alignment>::Type;
	using TInstancedType = FIntRect;

	static const FShaderParametersMetadata* GetStructMetadata() { return nullptr; }
};

template<>
struct TShaderParameterTypeInfo<FQuat4f>
{
	static constexpr EUniformBufferBaseType BaseType = UBMT_FLOAT32;
	static constexpr int32 NumRows = 1;
	static constexpr int32 NumColumns = 4;
	static constexpr int32 NumElements = 0;
	static constexpr int32 Alignment = 16;
	static constexpr bool bIsStoredInConstantBuffer = true;

	using TAlignedType = TAlignedTypedef<FQuat4f, Alignment>::Type;
	using TInstancedType = FQuat4f;

	static const FShaderParametersMetadata* GetStructMetadata() { return nullptr; }
};

template<>
struct TShaderParameterTypeInfo<FMatrix44f>
{
	static constexpr EUniformBufferBaseType BaseType = UBMT_FLOAT32;
	static constexpr int32 NumRows = 4;
	static constexpr int32 NumColumns = 4;
	static constexpr int32 NumElements = 0;
	static constexpr int32 Alignment = 16;
	static constexpr bool bIsStoredInConstantBuffer = true;

	using TAlignedType = TAlignedTypedef<FMatrix44f, Alignment>::Type;
	using TInstancedType = FMatrix44f;

	static const FShaderParametersMetadata* GetStructMetadata() { return nullptr; }
};

template <typename BufferAccessType>
struct TRDGResourceAccessTypeInfo
{
	static constexpr int32 NumRows = 1;
	static constexpr int32 NumColumns = 1;
	static constexpr int32 NumElements = 0;
	static constexpr int32 Alignment = SHADER_PARAMETER_POINTER_ALIGNMENT;
	static constexpr bool bIsStoredInConstantBuffer = false;

	using TAlignedType = BufferAccessType;

	static const FShaderParametersMetadata* GetStructMetadata() { return nullptr; }

	static_assert(sizeof(TAlignedType) == SHADER_PARAMETER_POINTER_ALIGNMENT * 2, "Uniform buffer layout must not be platform dependent.");
};

template <typename TextureAccessType>
struct TRDGTextureAccessTypeInfo
{
	static constexpr int32 NumRows = 1;
	static constexpr int32 NumColumns = 1;
	static constexpr int32 NumElements = 0;
	static constexpr int32 Alignment = SHADER_PARAMETER_POINTER_ALIGNMENT;
	static constexpr bool bIsStoredInConstantBuffer = false;

	using TAlignedType = TextureAccessType;

	static const FShaderParametersMetadata* GetStructMetadata() { return nullptr; }

	static_assert(sizeof(TAlignedType) == SHADER_PARAMETER_POINTER_ALIGNMENT * 3, "Uniform buffer layout must not be platform dependent.");
};


template<typename T, size_t InNumElements>
struct TShaderParameterTypeInfo<T[InNumElements]>
{
	static constexpr EUniformBufferBaseType BaseType = TShaderParameterTypeInfo<T>::BaseType;
	static constexpr int32 NumRows = TShaderParameterTypeInfo<T>::NumRows;
	static constexpr int32 NumColumns = TShaderParameterTypeInfo<T>::NumColumns;
	static constexpr int32 NumElements = InNumElements;
	static constexpr int32 Alignment = SHADER_PARAMETER_ARRAY_ELEMENT_ALIGNMENT;
	static constexpr bool bIsStoredInConstantBuffer = TShaderParameterTypeInfo<T>::bIsStoredInConstantBuffer;

	using TAlignedType = TStaticArray<std::conditional_t<Alignment == alignof(T), T, UE::Core::TAlignedElement<T, Alignment>>, InNumElements>;
	using TInstancedType = T;

	static const FShaderParametersMetadata* GetStructMetadata() { return TShaderParameterTypeInfo<T>::GetStructMetadata(); }

	static_assert(((sizeof(T) % 16) == 0), "Array element size does not satisfy 16 byte alignment requirements.  Try packing it with other values in a vector or using SHADER_PARAMETER_SCALAR_ARRAY.");
};
	
template<typename T,size_t InNumElements,uint32 IgnoredAlignment>
struct TShaderParameterTypeInfo<TStaticArray<T,InNumElements,IgnoredAlignment>>
{
	static constexpr EUniformBufferBaseType BaseType = TShaderParameterTypeInfo<T>::BaseType;
	static constexpr int32 NumRows = TShaderParameterTypeInfo<T>::NumRows;
	static constexpr int32 NumColumns = TShaderParameterTypeInfo<T>::NumColumns;
	static constexpr int32 NumElements = InNumElements;
	static constexpr int32 Alignment = SHADER_PARAMETER_ARRAY_ELEMENT_ALIGNMENT;
	static constexpr bool bIsStoredInConstantBuffer = TShaderParameterTypeInfo<T>::bIsStoredInConstantBuffer;

	using TAlignedType = TStaticArray<std::conditional_t<Alignment == alignof(T), T, UE::Core::TAlignedElement<T, Alignment>>, InNumElements>;
	using TInstancedType = T;

	static const FShaderParametersMetadata* GetStructMetadata() { return TShaderParameterTypeInfo<T>::GetStructMetadata(); }

	static_assert(((sizeof(T) % 16) == 0), "Array element size does not satisfy 16 byte alignment requirements.  Try packing it with other values in a vector or using SHADER_PARAMETER_SCALAR_ARRAY.");
};

template<typename ShaderResourceType>
struct TShaderResourceParameterTypeInfo
{
	static constexpr int32 NumRows = 1;
	static constexpr int32 NumColumns = 1;
	static constexpr int32 NumElements = 0;
	static constexpr int32 Alignment = SHADER_PARAMETER_POINTER_ALIGNMENT;
	static constexpr bool bIsStoredInConstantBuffer = false;

	using TAlignedType = TAlignedShaderParameterPtr<ShaderResourceType>;

	static const FShaderParametersMetadata* GetStructMetadata() { return nullptr; }

	static_assert(sizeof(TAlignedType) == SHADER_PARAMETER_POINTER_ALIGNMENT, "Uniform buffer layout must not be platform dependent.");
};

template<typename ShaderResourceType, size_t InNumElements>
struct TShaderResourceParameterTypeInfo<ShaderResourceType[InNumElements]>
{
	static constexpr int32 NumRows = 1;
	static constexpr int32 NumColumns = 1;
	static constexpr int32 NumElements = InNumElements;
	static constexpr int32 Alignment = SHADER_PARAMETER_POINTER_ALIGNMENT;
	static constexpr bool bIsStoredInConstantBuffer = false;

	using TAlignedType = TShaderResourceParameterArray<ShaderResourceType, InNumElements>;

	static const FShaderParametersMetadata* GetStructMetadata() { return nullptr; }
};

template<class UniformBufferStructType>
struct TShaderParameterTypeInfo<TUniformBufferRef<UniformBufferStructType>>
{
	static constexpr int32 NumRows = 1;
	static constexpr int32 NumColumns = 1;
	static constexpr int32 NumElements = 0;
	static constexpr int32 Alignment = SHADER_PARAMETER_POINTER_ALIGNMENT;
	static constexpr bool bIsStoredInConstantBuffer = false;
	
	using TAlignedType = TAlignedShaderParameterPtr<TUniformBufferRef<UniformBufferStructType>>;
	using TInstancedType = UniformBufferStructType;

	static const FShaderParametersMetadata* GetStructMetadata() { return TUniformBufferMetadataHelper<UniformBufferStructType>::GetStructMetadata(); }
};

template<class UniformBufferStructType>
struct TShaderParameterTypeInfo<TUniformBufferBinding<UniformBufferStructType>>
{
	static constexpr int32 NumRows = 1;
	static constexpr int32 NumColumns = 1;
	static constexpr int32 NumElements = 0;
	static constexpr int32 Alignment = SHADER_PARAMETER_STRUCT_ALIGNMENT;
	static constexpr bool bIsStoredInConstantBuffer = false;

	using TAlignedType = TUniformBufferBinding<UniformBufferStructType>;
	using TInstancedType = UniformBufferStructType;

	static const FShaderParametersMetadata* GetStructMetadata() { return TUniformBufferMetadataHelper<UniformBufferStructType>::GetStructMetadata(); }
};

template<class UniformBufferStructType>
struct TShaderParameterTypeInfo<TRDGUniformBufferBinding<UniformBufferStructType>>
{
	static constexpr int32 NumRows = 1;
	static constexpr int32 NumColumns = 1;
	static constexpr int32 NumElements = 0;
	static constexpr int32 Alignment = SHADER_PARAMETER_STRUCT_ALIGNMENT;
	static constexpr bool bIsStoredInConstantBuffer = false;

	using TAlignedType = TRDGUniformBufferBinding<UniformBufferStructType>;
	using TInstancedType = UniformBufferStructType;

	static const FShaderParametersMetadata* GetStructMetadata() { return TUniformBufferMetadataHelper<UniformBufferStructType>::GetStructMetadata(); }
};

template<typename StructType>
struct TShaderParameterStructTypeInfo
{
	static constexpr int32 NumRows = 1;
	static constexpr int32 NumColumns = 1;
	static constexpr int32 NumElements = 0;
	static constexpr int32 Alignment = SHADER_PARAMETER_STRUCT_ALIGNMENT;
	static constexpr bool bIsStoredInConstantBuffer = true;

	using TAlignedType = StructType;

	static const FShaderParametersMetadata* GetStructMetadata() { return StructType::FTypeInfo::GetStructMetadata(); }
};

template<typename StructType, size_t InNumElements>
struct TShaderParameterStructTypeInfo<StructType[InNumElements]>
{
	static constexpr int32 NumRows = 1;
	static constexpr int32 NumColumns = 1;
	static constexpr int32 NumElements = InNumElements;
	static constexpr int32 Alignment = SHADER_PARAMETER_STRUCT_ALIGNMENT;
	static constexpr bool bIsStoredInConstantBuffer = true;

	using TAlignedType = TStaticArray<StructType, InNumElements>;

	static const FShaderParametersMetadata* GetStructMetadata() { return StructType::FTypeInfo::GetStructMetadata(); }
};

#define INTERNAL_BEGIN_UNIFORM_BUFFER_STRUCT(DllStorage) \
	static DllStorage const FShaderParametersMetadata* GetStructMetadata(); 

#define INTERNAL_UNIFORM_BUFFER_STRUCT_GET_STRUCT_METADATA(StructTypeName) \
	{ return StructTypeName::GetStructMetadata(); }

#define INTERNAL_SHADER_PARAMETER_GET_STRUCT_METADATA(StructTypeName) \
	{ \
		static FShaderParametersMetadata StaticStructMetadata(\
			FShaderParametersMetadata::EUseCase::ShaderParameterStruct, \
			EUniformBufferBindingFlags::Shader, \
			TEXT(#StructTypeName), \
			TEXT(#StructTypeName), \
			nullptr, \
			nullptr, \
			FTypeInfo::FileName, \
			FTypeInfo::FileLine, \
			sizeof(StructTypeName), \
			StructTypeName::zzGetMembers()); \
		return &StaticStructMetadata; \
	}


#define INTERNAL_SHADER_PARAMETER_STRUCT_CREATE_UNIFORM_BUFFER return nullptr;

#define INTERNAL_UNIFORM_BUFFER_STRUCT_CREATE_UNIFORM_BUFFER return RHICreateUniformBuffer(&InContents, FTypeInfo::GetStructMetadata()->GetLayoutPtr(), InUsage);

/** Begins a uniform buffer struct declaration. */
#define INTERNAL_SHADER_PARAMETER_STRUCT_BEGIN(StructTypeName,DllStorage,ConstructorSuffix,GetStructMetadataScope,CreateUniformBufferImpl) \
	MS_ALIGN(SHADER_PARAMETER_STRUCT_ALIGNMENT) class StructTypeName \
	{ \
	public: \
		DllStorage StructTypeName () ConstructorSuffix \
		struct FTypeInfo { \
			static constexpr int32 NumRows = 1; \
			static constexpr int32 NumColumns = 1; \
			static constexpr int32 NumElements = 0; \
			static constexpr int32 Alignment = SHADER_PARAMETER_STRUCT_ALIGNMENT; \
			static constexpr bool bIsStoredInConstantBuffer = true; \
			static constexpr const ANSICHAR* const FileName = UE_LOG_SOURCE_FILE(__builtin_FILE()); \
			static constexpr int32 FileLine = __builtin_LINE(); \
			using TAlignedType = StructTypeName; \
			static const FShaderParametersMetadata* GetStructMetadata() GetStructMetadataScope \
		}; \
		static FUniformBufferRHIRef CreateUniformBuffer(const StructTypeName& InContents, EUniformBufferUsage InUsage) \
		{ \
			CreateUniformBufferImpl \
		} \
	private: \
		typedef StructTypeName zzTThisStruct; \
		struct zzFirstMemberId { enum { HasDeclaredResource = 0 }; }; \
		typedef void* zzFuncPtr; \
		typedef zzFuncPtr(*zzMemberFunc)(zzFirstMemberId, TArray<FShaderParametersMetadata::FMember>*); \
		static zzFuncPtr zzAppendMemberGetPrev(zzFirstMemberId, TArray<FShaderParametersMetadata::FMember>*) \
		{ \
			return nullptr; \
		} \
		typedef zzFirstMemberId

/** Declares a member of a uniform buffer struct. */
#define INTERNAL_SHADER_PARAMETER_EXPLICIT(BaseType,TypeInfo,MemberType,MemberName,ArrayDecl,DefaultValue,Precision,OptionalShaderType,IsMemberStruct) \
		zzMemberId##MemberName; \
	public: \
		alignas(TypeInfo::Alignment) TypeInfo::TAlignedType MemberName DefaultValue; \
		static_assert(BaseType != UBMT_INVALID, "Invalid type " #MemberType " of member " #MemberName "."); \
	private: \
		struct zzNextMemberId##MemberName { enum { HasDeclaredResource = zzMemberId##MemberName::HasDeclaredResource || !TypeInfo::bIsStoredInConstantBuffer }; }; \
		static zzFuncPtr zzAppendMemberGetPrev(zzNextMemberId##MemberName, TArray<FShaderParametersMetadata::FMember>* Members) \
		{ \
			static_assert(TypeInfo::bIsStoredInConstantBuffer || TIsArrayOrRefOfTypeByPredicate<decltype(OptionalShaderType), TIsCharEncodingCompatibleWithTCHAR>::Value, "No shader type for " #MemberName "."); \
			static_assert(\
				(STRUCT_OFFSET(zzTThisStruct, MemberName) & (TypeInfo::Alignment - 1)) == 0, \
				"Misaligned uniform buffer struct member " #MemberName "."); \
			Members->Add(FShaderParametersMetadata::FMember( \
				TEXT(#MemberName), \
				(const TCHAR*)OptionalShaderType, \
				__LINE__, \
				STRUCT_OFFSET(zzTThisStruct,MemberName), \
				EUniformBufferBaseType(BaseType), \
				Precision, \
				TypeInfo::NumRows, \
				TypeInfo::NumColumns, \
				TypeInfo::NumElements, \
				TypeInfo::GetStructMetadata() \
				)); \
			zzFuncPtr(*PrevFunc)(zzMemberId##MemberName, TArray<FShaderParametersMetadata::FMember>*); \
			PrevFunc = zzAppendMemberGetPrev; \
			return (zzFuncPtr)PrevFunc; \
		} \
		typedef zzNextMemberId##MemberName

/** Finds the FShaderParametersMetadata corresponding to the given name, or NULL if not found. */
extern RENDERCORE_API FShaderParametersMetadata* FindUniformBufferStructByName(const TCHAR* StructName);
extern RENDERCORE_API FShaderParametersMetadata* FindUniformBufferStructByFName(FName StructName);

/** Finds the FShaderParameterMetadata corresponding to the given uniform buffer layout hash, or null if not found. */
extern RENDERCORE_API FShaderParametersMetadata* FindUniformBufferStructByLayoutHash(uint32 Hash);

extern RENDERCORE_API FShaderParametersMetadata* FindUniformBufferStructByShaderVariableName(const FHashedName& Name);

/** Build the shader binding layout desc from array of used Uniform buffers which will be bound as static uniform buffers to the rhi */
extern RENDERCORE_API void BuildShaderBindingLayout(TConstArrayView<FShaderParametersMetadata*> UniformBuffers, EShaderBindingLayoutFlags BaseShaderBindingLayoutFlags, FShaderBindingLayoutContainer& OutShaderBindingLayoutContainer);

/** Begins & ends a shader parameter structure.
 *
 * Example:
 *	BEGIN_SHADER_PARAMETER_STRUCT(FMyParameterStruct, RENDERER_API)
 *	END_SHADER_PARAMETER_STRUCT()
 */
#define BEGIN_SHADER_PARAMETER_STRUCT(StructTypeName, DllStorage) \
	INTERNAL_SHADER_PARAMETER_STRUCT_BEGIN(StructTypeName,, {}, INTERNAL_SHADER_PARAMETER_GET_STRUCT_METADATA(StructTypeName), INTERNAL_SHADER_PARAMETER_STRUCT_CREATE_UNIFORM_BUFFER)

#define END_SHADER_PARAMETER_STRUCT() \
		zzLastMemberId; \
	public: \
		static TArray<FShaderParametersMetadata::FMember> zzGetMembers() { \
			TArray<FShaderParametersMetadata::FMember> Members; \
			zzFuncPtr(*LastFunc)(zzLastMemberId, TArray<FShaderParametersMetadata::FMember>*); \
			LastFunc = zzAppendMemberGetPrev; \
			zzFuncPtr Ptr = (zzFuncPtr)LastFunc; \
			do \
			{ \
				Ptr = reinterpret_cast<zzMemberFunc>(Ptr)(zzFirstMemberId(), &Members); \
			} while (Ptr); \
			Algo::Reverse(Members); \
			return Members; \
		} \
	} GCC_ALIGN(SHADER_PARAMETER_STRUCT_ALIGNMENT);

/** Begins & ends a shader global parameter structure.
 *
 * Example:
 *	// header
 *	BEGIN_UNIFORM_BUFFER_STRUCT(FMyParameterStruct, RENDERER_API)
 *	END_UNIFORM_BUFFER_STRUCT()
 *
 *	// C++ file
 *	IMPLEMENT_UNIFORM_BUFFER_STRUCT(FMyParameterStruct, "MyShaderBindingName");
 */
#define BEGIN_UNIFORM_BUFFER_STRUCT(StructTypeName, DllStorage) \
	DECLARE_UNIFORM_BUFFER_STRUCT(StructTypeName, DllStorage) \
	INTERNAL_SHADER_PARAMETER_STRUCT_BEGIN(StructTypeName,,{} INTERNAL_BEGIN_UNIFORM_BUFFER_STRUCT(DllStorage), INTERNAL_UNIFORM_BUFFER_STRUCT_GET_STRUCT_METADATA(StructTypeName), INTERNAL_UNIFORM_BUFFER_STRUCT_CREATE_UNIFORM_BUFFER)

#define BEGIN_UNIFORM_BUFFER_STRUCT_WITH_CONSTRUCTOR(StructTypeName, DllStorage) \
	DECLARE_UNIFORM_BUFFER_STRUCT(StructTypeName, DllStorage) \
	INTERNAL_SHADER_PARAMETER_STRUCT_BEGIN(StructTypeName,DllStorage,; INTERNAL_BEGIN_UNIFORM_BUFFER_STRUCT(DllStorage), INTERNAL_UNIFORM_BUFFER_STRUCT_GET_STRUCT_METADATA(StructTypeName), INTERNAL_UNIFORM_BUFFER_STRUCT_CREATE_UNIFORM_BUFFER)

#define END_UNIFORM_BUFFER_STRUCT() \
	END_SHADER_PARAMETER_STRUCT()


class FShaderParametersMetadataRegistration
{
public:
	FShaderParametersMetadataRegistration(TFunctionRef<const FShaderParametersMetadata* ()> LazyShaderParametersMetadataAccessor)
		: LazyShaderParametersMetadataAccessor(LazyShaderParametersMetadataAccessor)
	{
		GetInstances().Add(this);
	}

	static RENDERCORE_API TArray<const FShaderParametersMetadataRegistration*>& GetInstances();

	// Actually register all the instances and clear the array
	static RENDERCORE_API void CommitAll();
	static RENDERCORE_API bool IsReadyForRegistration();

private:
	TFunctionRef<const FShaderParametersMetadata* ()> LazyShaderParametersMetadataAccessor;
};

#define IMPLEMENT_UNIFORM_BUFFER_STRUCT_EX(StructTypeName,ShaderVariableName,UsageFlags) \
	const FShaderParametersMetadata* GetForwardDeclaredShaderParametersStructMetadata(const StructTypeName* DummyPtr) { return StructTypeName::FTypeInfo::GetStructMetadata(); } \
	const FShaderParametersMetadata* StructTypeName::GetStructMetadata() \
	{ \
		ensureMsgf(FShaderParametersMetadataRegistration::IsReadyForRegistration(), TEXT("GetStructMetadata should not be called from a static initializer")); \
		static const FShaderParametersMetadata StaticStructMetadata( \
			FShaderParametersMetadata::EUseCase::UniformBuffer, \
			EUniformBufferBindingFlags::Shader, \
			TEXT(#StructTypeName), \
			TEXT(#StructTypeName), \
			TEXT(ShaderVariableName), \
			nullptr, \
			StructTypeName::FTypeInfo::FileName, \
			StructTypeName::FTypeInfo::FileLine, \
			sizeof(StructTypeName), \
			StructTypeName::zzGetMembers(), \
			false, \
			nullptr, \
			UsageFlags); \
		return &StaticStructMetadata; \
	} \
	FShaderParametersMetadataRegistration StructTypeName##MetadataRegistration { TFunctionRef<const ::FShaderParametersMetadata* ()>{StructTypeName::GetStructMetadata} };

#define IMPLEMENT_UNIFORM_BUFFER_STRUCT(StructTypeName,ShaderVariableName) \
	IMPLEMENT_UNIFORM_BUFFER_STRUCT_EX(StructTypeName,ShaderVariableName,FShaderParametersMetadata::EUsageFlags::None)

#define IMPLEMENT_UNIFORM_BUFFER_ALIAS_STRUCT(StructTypeName, UniformBufferAlias) \
	static const FShaderParametersMetadata UniformBufferAlias( \
	FShaderParametersMetadata::EUseCase::UniformBuffer, \
	EUniformBufferBindingFlags::Shader, \
	TEXT(#StructTypeName), \
	TEXT(#StructTypeName), \
	TEXT(#UniformBufferAlias), \
	nullptr, \
	StructTypeName::FTypeInfo::FileName, \
	StructTypeName::FTypeInfo::FileLine, \
	sizeof(StructTypeName), \
	StructTypeName::zzGetMembers())
 
#define IMPLEMENT_STATIC_UNIFORM_BUFFER_STRUCT_EX(StructTypeName,ShaderVariableName,StaticSlotName,BindingFlagsEnum) \
	static_assert(EnumHasAnyFlags(EUniformBufferBindingFlags::BindingFlagsEnum, EUniformBufferBindingFlags::Static), "Shader bindings must include 'Static'."); \
	const FShaderParametersMetadata* GetForwardDeclaredShaderParametersStructMetadata(const StructTypeName* DummyPtr) { return StructTypeName::FTypeInfo::GetStructMetadata(); } \
	const FShaderParametersMetadata* StructTypeName::GetStructMetadata() \
	{ \
		ensureMsgf(FShaderParametersMetadataRegistration::IsReadyForRegistration(), TEXT("GetStructMetadata should not be called from a static initializer")); \
		static const FShaderParametersMetadata StaticStructMetadata( \
			FShaderParametersMetadata::EUseCase::UniformBuffer, \
			EUniformBufferBindingFlags::BindingFlagsEnum, \
			TEXT(#StructTypeName), \
			TEXT(#StructTypeName), \
			TEXT(ShaderVariableName), \
			TEXT(#StaticSlotName), \
			StructTypeName::FTypeInfo::FileName, \
			StructTypeName::FTypeInfo::FileLine, \
			sizeof(StructTypeName), \
			StructTypeName::zzGetMembers()); \
		return &StaticStructMetadata; \
	} \
	FShaderParametersMetadataRegistration StructTypeName##MetadataRegistration { TFunctionRef<const ::FShaderParametersMetadata* ()>{StructTypeName::GetStructMetadata} };

#define IMPLEMENT_STATIC_UNIFORM_BUFFER_STRUCT_EX2(StructTypeName,ShaderVariableName,StaticSlotName,BindingFlagsEnum,UsageFlags) \
	static_assert(EnumHasAnyFlags(EUniformBufferBindingFlags::BindingFlagsEnum, EUniformBufferBindingFlags::Static), "Shader bindings must include 'Static'."); \
	const FShaderParametersMetadata* GetForwardDeclaredShaderParametersStructMetadata(const StructTypeName* DummyPtr) {  return StructTypeName::FTypeInfo::GetStructMetadata(); } \
	const FShaderParametersMetadata* StructTypeName::GetStructMetadata() \
	{ \
		ensureMsgf(FShaderParametersMetadataRegistration::IsReadyForRegistration(), TEXT("GetStructMetadata should not be called from a static initializer")); \
		static const FShaderParametersMetadata StaticStructMetadata( \
			FShaderParametersMetadata::EUseCase::UniformBuffer, \
			EUniformBufferBindingFlags::BindingFlagsEnum, \
			TEXT(#StructTypeName), \
			TEXT(#StructTypeName), \
			TEXT(ShaderVariableName), \
			TEXT(#StaticSlotName), \
			StructTypeName::FTypeInfo::FileName, \
			StructTypeName::FTypeInfo::FileLine, \
			sizeof(StructTypeName), \
			StructTypeName::zzGetMembers(), \
			false, \
			nullptr, \
			UsageFlags); \
		return &StaticStructMetadata; \
	} \
	FShaderParametersMetadataRegistration StructTypeName##MetadataRegistration { TFunctionRef<const ::FShaderParametersMetadata* ()>{StructTypeName::GetStructMetadata} };

/** Implements the same contract as IMPLEMENT_STATIC_UNIFORM_BUFFER_STRUCT, with the addition of the 'StaticAndShader' binding
 *  flag. This means that the uniform buffer may be bound statically or through SetShaderParameters. The uniform buffer is NOT
 *  removed from the parameter map, which means it is possible to end up with both global and shader bindings. The RHI validation
 *  layer will catch and report these cases, as they are both inefficient and would produce confusing behavior if the two bindings
 *  differ. Only one binding model should be used at a time.
 */
#define IMPLEMENT_STATIC_AND_SHADER_UNIFORM_BUFFER_STRUCT(StructTypeName,ShaderVariableName,StaticSlotName) \
	IMPLEMENT_STATIC_UNIFORM_BUFFER_STRUCT_EX(StructTypeName, ShaderVariableName, StaticSlotName, StaticAndShader)

#define IMPLEMENT_STATIC_AND_SHADER_UNIFORM_BUFFER_STRUCT_EX(StructTypeName,ShaderVariableName,StaticSlotName,UsageFlags) \
	IMPLEMENT_STATIC_UNIFORM_BUFFER_STRUCT_EX2(StructTypeName, ShaderVariableName, StaticSlotName, StaticAndShader, UsageFlags)

 /** Implements a uniform buffer tied to a static binding slot. The third parameter is the name of the slot.
  *  Multiple uniform buffers can be associated to a slot; only one uniform buffer can be bound to a slot
  *  at one time.
  *
  * Example:
  *	BEGIN_UNIFORM_BUFFER_STRUCT(FMyParameterStruct, RENDERER_API)
  *	END_UNIFORM_BUFFER_STRUCT()
  *
  *	// C++ file
  *
  *	// Define uniform buffer slot.
  *	IMPLEMENT_STATIC_UNIFORM_BUFFER_SLOT(MySlot)
  *
  *	// Associate uniform buffer with slot.
  *	IMPLEMENT_STATIC_UNIFORM_BUFFER_STRUCT(FMyParameterStruct, "MyShaderBindingName", MySlot);
  */
#define IMPLEMENT_STATIC_UNIFORM_BUFFER_STRUCT(StructTypeName,ShaderVariableName,StaticSlotName) \
	IMPLEMENT_STATIC_UNIFORM_BUFFER_STRUCT_EX(StructTypeName, ShaderVariableName, StaticSlotName, Static)

/** Implements a uniform buffer static binding slot. */
#define IMPLEMENT_STATIC_UNIFORM_BUFFER_SLOT(SlotName) \
	static FUniformBufferStaticSlotRegistrar UniformBufferStaticSlot_##SlotName(TEXT(#SlotName));

 /** Legacy macro definitions. */
#define BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT \
	 BEGIN_UNIFORM_BUFFER_STRUCT

#define BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT_WITH_CONSTRUCTOR \
	BEGIN_UNIFORM_BUFFER_STRUCT_WITH_CONSTRUCTOR

#define END_GLOBAL_SHADER_PARAMETER_STRUCT \
	END_UNIFORM_BUFFER_STRUCT

#define IMPLEMENT_GLOBAL_SHADER_PARAMETER_STRUCT \
	IMPLEMENT_UNIFORM_BUFFER_STRUCT

#define IMPLEMENT_GLOBAL_SHADER_PARAMETER_ALIAS_STRUCT \
	IMPLEMENT_UNIFORM_BUFFER_ALIAS_STRUCT

/** Adds a constant-buffer stored value.
 *
 * Example:
 *	SHADER_PARAMETER(float, MyScalar)
 *	SHADER_PARAMETER(FMatrix44f, MyMatrix)
 */
#define SHADER_PARAMETER(MemberType,MemberName) \
	SHADER_PARAMETER_EX(MemberType,MemberName,EShaderPrecisionModifier::Float)

#define SHADER_PARAMETER_EX(MemberType,MemberName,Precision) \
	INTERNAL_SHADER_PARAMETER_EXPLICIT(TShaderParameterTypeInfo<MemberType>::BaseType, TShaderParameterTypeInfo<MemberType>, MemberType,MemberName,,,Precision,TEXT(""),false)

/** Adds a constant-buffer stored array of values.
 *  MemberType size must be a multiple of 16byte (see SHADER_PARAMETER_SCALAR_ARRAY for scalar arrays).
 *
 * Example:
 *	SHADER_PARAMETER_ARRAY(FMatrix44f, MyMatrixArray, [2])
 */
#define SHADER_PARAMETER_ARRAY(MemberType,MemberName,ArrayDecl) \
	SHADER_PARAMETER_ARRAY_EX(MemberType,MemberName,ArrayDecl,EShaderPrecisionModifier::Float)

#define SHADER_PARAMETER_ARRAY_EX(MemberType,MemberName,ArrayDecl,Precision) \
	INTERNAL_SHADER_PARAMETER_EXPLICIT(TShaderParameterTypeInfo<MemberType ArrayDecl>::BaseType, TShaderParameterTypeInfo<MemberType ArrayDecl>, MemberType,MemberName,ArrayDecl,,Precision,TEXT(""),false)

/** Adds a texture.
 *
 * Example:
 *	SHADER_PARAMETER_TEXTURE(Texture2D, MyTexture)
 *	SHADER_PARAMETER_TEXTURE_ARRAY(Texture2D, MyArrayOfTextures, [8])
 */
#define SHADER_PARAMETER_TEXTURE(ShaderType,MemberName) \
	INTERNAL_SHADER_PARAMETER_EXPLICIT(UBMT_TEXTURE, TShaderResourceParameterTypeInfo<FRHITexture*>, FRHITexture*,MemberName,, = nullptr,EShaderPrecisionModifier::Float,TEXT(#ShaderType),false)

#define SHADER_PARAMETER_TEXTURE_ARRAY(ShaderType,MemberName, ArrayDecl) \
	INTERNAL_SHADER_PARAMETER_EXPLICIT(UBMT_TEXTURE, TShaderResourceParameterTypeInfo<FRHITexture* ArrayDecl>, FRHITexture*,MemberName,ArrayDecl,,EShaderPrecisionModifier::Float,TEXT(#ShaderType),false)

/** Adds a shader resource view.
 *
 * Example:
 *	SHADER_PARAMETER_SRV(Texture2D, MySRV)
 *	SHADER_PARAMETER_SRV_ARRAY(Texture2D, MyArrayOfSRVs, [8])
 */
#define SHADER_PARAMETER_SRV(ShaderType,MemberName) \
	INTERNAL_SHADER_PARAMETER_EXPLICIT(UBMT_SRV, TShaderResourceParameterTypeInfo<FRHIShaderResourceView*>, FRHIShaderResourceView*,MemberName,, = nullptr,EShaderPrecisionModifier::Float,TEXT(#ShaderType),false)

#define SHADER_PARAMETER_SRV_ARRAY(ShaderType,MemberName, ArrayDecl) \
	INTERNAL_SHADER_PARAMETER_EXPLICIT(UBMT_SRV, TShaderResourceParameterTypeInfo<FRHIShaderResourceView* ArrayDecl>, FRHIShaderResourceView*,MemberName,ArrayDecl,,EShaderPrecisionModifier::Float,TEXT(#ShaderType),false)

/** Adds a shader resource collection.
 *
 * Example:
 *	SHADER_PARAMETER_RESOURCE_COLLECTION(FResourceCollection, MyResourceCollection)
 */
#define SHADER_PARAMETER_RESOURCE_COLLECTION(ShaderType,MemberName) \
	INTERNAL_SHADER_PARAMETER_EXPLICIT(UBMT_RESOURCE_COLLECTION, TShaderResourceParameterTypeInfo<FRHIResourceCollection*>, FRHIResourceCollection*,MemberName,, = nullptr,EShaderPrecisionModifier::Float,TEXT(#ShaderType),false)

/** Adds an unordered access view.
 *
 * Example:
 *	SHADER_PARAMETER_UAV(RWTexture2D, MyUAV)
 */
#define SHADER_PARAMETER_UAV(ShaderType,MemberName) \
	INTERNAL_SHADER_PARAMETER_EXPLICIT(UBMT_UAV, TShaderResourceParameterTypeInfo<FRHIUnorderedAccessView*>, FRHIUnorderedAccessView*,MemberName,, = nullptr,EShaderPrecisionModifier::Float,TEXT(#ShaderType),false)

/** Adds a sampler.
 *
 * Example:
 *	SHADER_PARAMETER_SAMPLER(SamplerState, MySampler)
 *	SHADER_PARAMETER_SAMPLER_ARRAY(SamplerState, MyArrayOfSamplers, [8])
 */
#define SHADER_PARAMETER_SAMPLER(ShaderType,MemberName) \
	INTERNAL_SHADER_PARAMETER_EXPLICIT(UBMT_SAMPLER, TShaderResourceParameterTypeInfo<FRHISamplerState*>, FRHISamplerState*,MemberName,, = nullptr,EShaderPrecisionModifier::Float,TEXT(#ShaderType),false)

#define SHADER_PARAMETER_SAMPLER_ARRAY(ShaderType,MemberName, ArrayDecl) \
	INTERNAL_SHADER_PARAMETER_EXPLICIT(UBMT_SAMPLER, TShaderResourceParameterTypeInfo<FRHISamplerState* ArrayDecl>, FRHISamplerState*,MemberName,ArrayDecl,,EShaderPrecisionModifier::Float,TEXT(#ShaderType),false)

/** Adds a descriptor range.
 *
 * Example:
 *	SHADER_PARAMETER_DESCRIPTOR_RANGE(FDescriptorRange, MyDescriptorRange)
 */
#define SHADER_PARAMETER_DESCRIPTOR_RANGE(ShaderType,MemberName) \
	INTERNAL_SHADER_PARAMETER_EXPLICIT(UBMT_DESCRIPTOR_RANGE,TShaderResourceParameterTypeInfo<FRHIDescriptorRange*>,FRHIDescriptorRange*,MemberName,,=nullptr,EShaderPrecisionModifier::Float,TEXT(#ShaderType),false)

/** Adds a render graph tracked texture.
 *
 * Example:
 *	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, MyTexture)
 *	SHADER_PARAMETER_RDG_TEXTURE_ARRAY(Texture2D, MyArrayOfTextures, [4])
 */
#define SHADER_PARAMETER_RDG_TEXTURE(ShaderType,MemberName) \
	INTERNAL_SHADER_PARAMETER_EXPLICIT(UBMT_RDG_TEXTURE, TShaderResourceParameterTypeInfo<FRDGTexture*>, FRDGTexture*,MemberName,, = nullptr,EShaderPrecisionModifier::Float,TEXT(#ShaderType),false)

#define SHADER_PARAMETER_RDG_TEXTURE_ARRAY(ShaderType,MemberName, ArrayDecl) \
	INTERNAL_SHADER_PARAMETER_EXPLICIT(UBMT_RDG_TEXTURE, TShaderResourceParameterTypeInfo<FRDGTexture* ArrayDecl>, FRDGTexture*,MemberName,ArrayDecl,,EShaderPrecisionModifier::Float,TEXT(#ShaderType),false)

/** Adds a shader resource view for a render graph tracked texture.
 *
 * Example:
 *	SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, MySRV)
 *	SHADER_PARAMETER_RDG_TEXTURE_SRV_ARRAY(Texture2D, MyArrayOfSRVs, [4])
 */
#define SHADER_PARAMETER_RDG_TEXTURE_SRV(ShaderType,MemberName) \
	INTERNAL_SHADER_PARAMETER_EXPLICIT(UBMT_RDG_TEXTURE_SRV, TShaderResourceParameterTypeInfo<FRDGTextureSRV*>, FRDGTextureSRV*,MemberName,, = nullptr,EShaderPrecisionModifier::Float,TEXT(#ShaderType),false)

#define SHADER_PARAMETER_RDG_TEXTURE_SRV_ARRAY(ShaderType,MemberName, ArrayDecl) \
	INTERNAL_SHADER_PARAMETER_EXPLICIT(UBMT_RDG_TEXTURE_SRV, TShaderResourceParameterTypeInfo<FRDGTextureSRV* ArrayDecl>, FRDGTextureSRV*,MemberName,ArrayDecl,,EShaderPrecisionModifier::Float,TEXT(#ShaderType),false)

/** Adds a non-pixel shader resource view for a render graph tracked texture.
 *
 * Example:
 *	SHADER_PARAMETER_RDG_TEXTURE_NON_PIXEL_SRV(Texture2D, MySRV)
 *	SHADER_PARAMETER_RDG_TEXTURE_NON_PIXEL_SRV_ARRAY(Texture2D, MyArrayOfSRVs, [4])
 */
#define SHADER_PARAMETER_RDG_TEXTURE_NON_PIXEL_SRV(ShaderType,MemberName) \
	INTERNAL_SHADER_PARAMETER_EXPLICIT(UBMT_RDG_TEXTURE_NON_PIXEL_SRV, TShaderResourceParameterTypeInfo<FRDGTextureSRV*>, FRDGTextureSRV*,MemberName,, = nullptr,EShaderPrecisionModifier::Float,TEXT(#ShaderType),false)

#define SHADER_PARAMETER_RDG_TEXTURE_NON_PIXEL_SRV_ARRAY(ShaderType,MemberName, ArrayDecl) \
	INTERNAL_SHADER_PARAMETER_EXPLICIT(UBMT_RDG_TEXTURE_NON_PIXEL_SRV, TShaderResourceParameterTypeInfo<FRDGTextureSRV* ArrayDecl>, FRDGTextureSRV*,MemberName,ArrayDecl,,EShaderPrecisionModifier::Float,TEXT(#ShaderType),false)

/** Adds a unordered access view for a render graph tracked texture.
 *
 * Example:
 *	SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, MyUAV)
 *	SHADER_PARAMETER_RDG_TEXTURE_UAV_ARRAY(Texture2D, MyArrayOfUAVs, [4])
 */
#define SHADER_PARAMETER_RDG_TEXTURE_UAV(ShaderType,MemberName) \
	INTERNAL_SHADER_PARAMETER_EXPLICIT(UBMT_RDG_TEXTURE_UAV, TShaderResourceParameterTypeInfo<FRDGTextureUAV*>, FRDGTextureUAV*,MemberName,, = nullptr,EShaderPrecisionModifier::Float,TEXT(#ShaderType),false)

#define SHADER_PARAMETER_RDG_TEXTURE_UAV_ARRAY(ShaderType,MemberName, ArrayDecl) \
	INTERNAL_SHADER_PARAMETER_EXPLICIT(UBMT_RDG_TEXTURE_UAV, TShaderResourceParameterTypeInfo<FRDGTextureUAV* ArrayDecl>, FRDGTextureUAV*,MemberName,ArrayDecl,,EShaderPrecisionModifier::Float,TEXT(#ShaderType),false)

/** Adds a shader resource view for a render graph tracked buffer.
 *
 * Example:
 *	SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float4>, MySRV)
 *	SHADER_PARAMETER_RDG_BUFFER_SRV_ARRAY(Buffer<float4>, MyArrayOfSRVs, [4])
 */
#define SHADER_PARAMETER_RDG_BUFFER_SRV(ShaderType,MemberName) \
	INTERNAL_SHADER_PARAMETER_EXPLICIT(UBMT_RDG_BUFFER_SRV, TShaderResourceParameterTypeInfo<FRDGBufferSRV*>, FRDGBufferSRV*,MemberName,, = nullptr,EShaderPrecisionModifier::Float,TEXT(#ShaderType),false)

#define SHADER_PARAMETER_RDG_BUFFER_SRV_ARRAY(ShaderType,MemberName, ArrayDecl) \
	INTERNAL_SHADER_PARAMETER_EXPLICIT(UBMT_RDG_BUFFER_SRV, TShaderResourceParameterTypeInfo<FRDGBufferSRV* ArrayDecl>, FRDGBufferSRV*,MemberName,ArrayDecl,,EShaderPrecisionModifier::Float,TEXT(#ShaderType),false)

/** Adds a unordered access view for a render graph tracked buffer.
 *
 * Example:
 *	SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<float4>, MyUAV)
 *	SHADER_PARAMETER_RDG_BUFFER_UAV_ARRAY(RWBuffer<float4>, MyArrayOfUAVs, [4])
 */
#define SHADER_PARAMETER_RDG_BUFFER_UAV(ShaderType,MemberName) \
	INTERNAL_SHADER_PARAMETER_EXPLICIT(UBMT_RDG_BUFFER_UAV, TShaderResourceParameterTypeInfo<FRDGBufferUAV*>, FRDGBufferUAV*,MemberName,, = nullptr,EShaderPrecisionModifier::Float,TEXT(#ShaderType),false)

#define SHADER_PARAMETER_RDG_BUFFER_UAV_ARRAY(ShaderType,MemberName, ArrayDecl) \
	INTERNAL_SHADER_PARAMETER_EXPLICIT(UBMT_RDG_BUFFER_UAV, TShaderResourceParameterTypeInfo<FRDGBufferUAV* ArrayDecl>, FRDGBufferUAV*,MemberName,ArrayDecl,,EShaderPrecisionModifier::Float,TEXT(#ShaderType),false)

 /** Adds a render graph tracked uniform buffer.
  *
  * Example:
  *	SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FMyStruct, MemberName)
  */
#define SHADER_PARAMETER_RDG_UNIFORM_BUFFER(StructType, MemberName) \
	INTERNAL_SHADER_PARAMETER_EXPLICIT(UBMT_RDG_UNIFORM_BUFFER, TShaderParameterTypeInfo<TRDGUniformBufferBinding<StructType>>, TRDGUniformBufferBinding<StructType>,MemberName,,,EShaderPrecisionModifier::Float,TEXT(#StructType),false)

/** Nests a shader parameter structure into another one, in C++ and shader code.
 *
 * Example:
 *	BEGIN_SHADER_PARAMETER_STRUCT(FMyNestedStruct,)
 *		SHADER_PARAMETER(float, MyScalar)
 *		// ...
 *	END_SHADER_PARAMETER_STRUCT()
 *
 *	BEGIN_SHADER_PARAMETER_STRUCT(FOtherStruct)
 *		SHADER_PARAMETER_STRUCT(FMyNestedStruct, MyStruct)
 *
 * C++ use case:
 *	FOtherStruct Parameters;
 *	Parameters.MyStruct.MyScalar = 1.0f;
 *
 * Shader code for a globally named shader parameter struct:
 *	float MyScalar = MyGlobalShaderBindingName.MyStruct.MyScalar;
 *
 * Shader code for a unnamed shader parameter struct:
 *	float MyScalar = MyStruct_MyScalar;
 */
#define SHADER_PARAMETER_STRUCT(StructType,MemberName) \
	INTERNAL_SHADER_PARAMETER_EXPLICIT(UBMT_NESTED_STRUCT, StructType::FTypeInfo, StructType, MemberName,,,EShaderPrecisionModifier::Float,TEXT(#StructType),true)

/** Nests an array of shader parameter structure into another one, in C++ and shader code.
 *
 * Example:
 *	BEGIN_SHADER_PARAMETER_STRUCT(FMyNestedStruct,)
 *		SHADER_PARAMETER(float, MyScalar)
 *		// ...
 *	END_SHADER_PARAMETER_STRUCT()
 *
 *	BEGIN_SHADER_PARAMETER_STRUCT(FOtherStruct)
 *		SHADER_PARAMETER_STRUCT_ARRAY(FMyNestedStruct, MyStructArray, [4])
 *
 * C++ use case:
 *	FOtherStruct Parameters;
 *	Parameters.MyStructArray[0].MyScalar = 1.0f;
 *
 * Shader code for a globally named shader parameter struct (UNSUPPORTED):
 *	float MyScalar = MyGlobalShaderBindingName.MyStructArray_0.MyScalar;
 *
 * Shader code for a unnamed shader parameter struct:
 *	float MyScalar = MyStructArray_0_MyScalar;
 */
#define SHADER_PARAMETER_STRUCT_ARRAY(StructType,MemberName, ArrayDecl) \
	INTERNAL_SHADER_PARAMETER_EXPLICIT(UBMT_NESTED_STRUCT, TShaderParameterStructTypeInfo<StructType ArrayDecl>, StructType, MemberName,ArrayDecl,,EShaderPrecisionModifier::Float,TEXT(#StructType),true)

/** Include a shader parameter structure into another one in shader code.
 *
 * Example:
 *	BEGIN_SHADER_PARAMETER_STRUCT(FMyNestedStruct,)
 *		SHADER_PARAMETER(float, MyScalar)
 *		// ...
 *	END_SHADER_PARAMETER_STRUCT()
 *
 *	BEGIN_SHADER_PARAMETER_STRUCT(FOtherStruct)
 *		SHADER_PARAMETER_STRUCT_INCLUDE(FMyNestedStruct, MyStruct)
 *
 * C++ use case:
 *	FOtherStruct Parameters;
 *	Parameters.MyStruct.MyScalar = 1.0f;
 *
 * Shader code for a globally named shader parameter struct:
 *	float MyScalar = MyGlobalShaderBindingName.MyScalar;
 *
 * Shader code for a unnamed shader parameter struct:
 *	float MyScalarValue = MyScalar;
 */
#define SHADER_PARAMETER_STRUCT_INCLUDE(StructType,MemberName) \
	INTERNAL_SHADER_PARAMETER_EXPLICIT(UBMT_INCLUDED_STRUCT, StructType::FTypeInfo, StructType, MemberName,,,EShaderPrecisionModifier::Float,TEXT(#StructType),true)

/** Include a binding slot for a globally named shader parameter structure
 *
 * Example:
 *	BEGIN_UNIFORM_BUFFER_STRUCT(FGlobalViewParameters,)
 *		SHADER_PARAMETER(FVector4f, ViewSizeAndInvSize)
 *		// ...
 *	END_UNIFORM_BUFFER_STRUCT()
 *
 *	BEGIN_SHADER_PARAMETER_STRUCT(FOtherStruct)
 *		SHADER_PARAMETER_STRUCT_REF(FMyNestedStruct, MyStruct)
 */
#define SHADER_PARAMETER_STRUCT_REF(StructType,MemberName) \
	INTERNAL_SHADER_PARAMETER_EXPLICIT(UBMT_REFERENCED_STRUCT, TShaderParameterTypeInfo<TUniformBufferBinding<StructType>>, TUniformBufferBinding<StructType>,MemberName,,,EShaderPrecisionModifier::Float,TEXT(#StructType),false)

/** Informs the RDG pass to transition the buffer into the requested state.
 *
 *  Example:
 *  BEGIN_SHADER_PARAMETER_STRUCT(FParameters,)
 *      RDG_BUFFER_ACCESS(MyBuffer)
 *      // ...
 *  END_SHADER_PARAMETER_STRUCT()
 *
 *  FParameters Parameters;
 *  Parameters.MyBuffer = FRDGBufferAccess(Buffer, ERHIAccess::CopyDest);
 **/
#define RDG_BUFFER_ACCESS(MemberName, Access) \
	INTERNAL_SHADER_PARAMETER_EXPLICIT(UBMT_RDG_BUFFER_ACCESS, TRDGResourceAccessTypeInfo<TRDGBufferAccess<Access>>, TRDGBufferAccess<Access>,MemberName,,,EShaderPrecisionModifier::Float,TEXT(""),false)

#define RDG_BUFFER_ACCESS_DYNAMIC(MemberName) \
	INTERNAL_SHADER_PARAMETER_EXPLICIT(UBMT_RDG_BUFFER_ACCESS, TRDGResourceAccessTypeInfo<FRDGBufferAccess>, FRDGBufferAccess,MemberName,,,EShaderPrecisionModifier::Float,TEXT(""),false)

#define RDG_BUFFER_ACCESS_ARRAY(MemberName) \
	INTERNAL_SHADER_PARAMETER_EXPLICIT(UBMT_RDG_BUFFER_ACCESS_ARRAY, TRDGResourceAccessTypeInfo<FRDGBufferAccessArray>, FRDGBufferAccessArray,MemberName,,,EShaderPrecisionModifier::Float,TEXT(""),false)

/** Informs the RDG pass to transition the texture into the requested state. */
#define RDG_TEXTURE_ACCESS(MemberName, Access) \
	INTERNAL_SHADER_PARAMETER_EXPLICIT(UBMT_RDG_TEXTURE_ACCESS, TRDGTextureAccessTypeInfo<TRDGTextureAccess<Access>>, TRDGTextureAccess<Access>,MemberName,,,EShaderPrecisionModifier::Float,TEXT(""),false)

#define RDG_TEXTURE_ACCESS_DYNAMIC(MemberName) \
	INTERNAL_SHADER_PARAMETER_EXPLICIT(UBMT_RDG_TEXTURE_ACCESS, TRDGTextureAccessTypeInfo<FRDGTextureAccess>, FRDGTextureAccess,MemberName,,,EShaderPrecisionModifier::Float,TEXT(""),false)

#define RDG_TEXTURE_ACCESS_ARRAY(MemberName) \
	INTERNAL_SHADER_PARAMETER_EXPLICIT(UBMT_RDG_TEXTURE_ACCESS_ARRAY, TRDGResourceAccessTypeInfo<FRDGTextureAccessArray>, FRDGTextureAccessArray,MemberName,,,EShaderPrecisionModifier::Float,TEXT(""),false)

/** Adds bindings slots for render targets on the structure. This is important for rasterizer based pass bind the
 * render target at the RHI pass creation. The name of the struct member will forced to RenderTargets, and
 * its type is a FRenderTargetBindingSlots.
 *
 * Example:
 *	BEGIN_SHADER_PARAMETER_STRUCT(FGlobalViewParameters,)
 *		RENDER_TARGET_BINDING_SLOTS()
 *		// ...
 *	END_SHADER_PARAMETER_STRUCT()
 *
 *	FGlobalViewParameters Parameters;
 *	Parameters.RenderTargets.DepthStencil = FDepthStencilBinding( //... );
 */
#define RENDER_TARGET_BINDING_SLOTS() \
	INTERNAL_SHADER_PARAMETER_EXPLICIT(UBMT_RENDER_TARGET_BINDING_SLOTS, FRenderTargetBindingSlots::FTypeInfo, FRenderTargetBindingSlots,RenderTargets,,,EShaderPrecisionModifier::Float,TEXT(""),false)

/** An empty shader parameter structure ready to be used anywhere. */
BEGIN_SHADER_PARAMETER_STRUCT(FEmptyShaderParameters, RENDERCORE_API)
END_SHADER_PARAMETER_STRUCT()

/** Useful parameter struct that only have render targets.
 *
 *	FRenderTargetParameters PassParameters;
 *	PassParameters.RenderTargets.DepthStencil = ... ;
 *	PassParameters.RenderTargets[0] = ... ;
 */
BEGIN_SHADER_PARAMETER_STRUCT(FRenderTargetParameters, RENDERCORE_API)
	RENDER_TARGET_BINDING_SLOTS()
END_SHADER_PARAMETER_STRUCT()

/** Returns the name of the macro that should be used for a given shader parameter base type. */
const TCHAR* GetShaderParameterMacroName(EUniformBufferBaseType ShaderParameterBaseType);

/** Deprecated buffer macros. */
#define SHADER_PARAMETER_RDG_BUFFER_UPLOAD(MemberName) \
	UE_DEPRECATED_MACRO(5.0, "SHADER_PARAMETER_RDG_BUFFER_UPLOAD has been deprecated. Use RDG_BUFFER_ACCESS(Buffer, ERHIAccess::CopyDest) instead.") \
	RDG_BUFFER_ACCESS(MemberName, ERHIAccess::CopyDest)

#define SHADER_PARAMETER_RDG_BUFFER(ShaderType,MemberName) \
	UE_DEPRECATED_MACRO(5.0, "SHADER_PARAMETER_RDG_BUFFER has been deprecated. Use RDG_BUFFER_ACCESS with an explicit RHI state instead.") \
	RDG_BUFFER_ACCESS(MemberName, ERHIAccess::SRVMask | ERHIAccess::IndirectArgs)

#define SHADER_PARAMETER_RDG_BUFFER_ARRAY(ShaderType,MemberName, ArrayDecl) \
	UE_DEPRECATED_MACRO(5.0, "SHADER_PARAMETER_RDG_BUFFER_ARRAY has been deprecated. Use RDG_BUFFER_ACCESS_ARRAY instead.")




/** Upgrade the scalar type to a vector for storage. */

template<typename TypeParameter>
struct TShaderParameterScalarArrayTypeInfo
{
	static_assert(sizeof(TypeParameter) == 0, "This type is not supported for automatic packing.");
	using PackedArrayType = void;
};

template<>
struct TShaderParameterScalarArrayTypeInfo<uint32>
{
	using PackedArrayType = FUintVector4;
};

template<>
struct TShaderParameterScalarArrayTypeInfo<int32>
{
	using PackedArrayType = FIntVector4;
};

template<>
struct TShaderParameterScalarArrayTypeInfo<float>
{
	using PackedArrayType = FVector4f;
};

constexpr uint32 CalcPackedArraySize(uint32 NumElements)		{ return (NumElements + 3) / 4; }
constexpr uint32 CalcPackedArrayIndex(uint32 ElementIndex)		{ return (ElementIndex >> 2);	}
constexpr uint32 CalcPackedComponentIndex(uint32 ElementIndex)	{ return (ElementIndex & 3);	}

/** Adds a packed constant-buffer stored array of values.
 *
 * Example:
 *	SHADER_PARAMETER_SCALAR_ARRAY(float, MyScalarArray, 8)
 */
#define SHADER_PARAMETER_SCALAR_ARRAY(MemberType,MemberName,ArrayDecl) \
	SHADER_PARAMETER_ARRAY(TShaderParameterScalarArrayTypeInfo<MemberType>::PackedArrayType, MemberName, [CalcPackedArraySize(TShaderParameterTypeInfo<TShaderParameterScalarArrayTypeInfo<MemberType>::PackedArrayType ArrayDecl>::NumElements)])

#define GET_SCALAR_ARRAY_ELEMENT(PackedArray, ElementIndex) \
	PackedArray[CalcPackedArrayIndex(ElementIndex)][CalcPackedComponentIndex(ElementIndex)]

PGO_LINK_ENABLE_WARNINGS
