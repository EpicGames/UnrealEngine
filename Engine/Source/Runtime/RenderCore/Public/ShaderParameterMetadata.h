// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ShaderParameterMetadata.h: Meta data about shader parameter structures
=============================================================================*/

#pragma once

#include "Containers/Array.h"
#include "Containers/List.h"
#include "Containers/Map.h"
#include "Containers/StaticArray.h"
#include "Containers/StringFwd.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "Misc/AssertionMacros.h"
#include "Misc/CString.h"
#include "Misc/StringBuilder.h"
#include "RHI.h"
#include "RHIDefinitions.h"
#include "Serialization/MemoryHasher.h"
#include "Serialization/MemoryImage.h"
#include "Serialization/MemoryLayout.h"
#include "Templates/AlignmentTemplates.h"
#include "UObject/NameTypes.h"
#include "Templates/SharedPointer.h"

using FThreadSafeSharedStringPtr = TSharedPtr<FString, ESPMode::ThreadSafe>;
using FThreadSafeSharedAnsiStringPtr = TSharedPtr<TArray<ANSICHAR>, ESPMode::ThreadSafe>;
using FThreadSafeNameBufferPtr = TSharedPtr<TArray<TCHAR>, ESPMode::ThreadSafe>;
class FShaderKeyGenerator;
struct FShaderResourceTableMap;

namespace EShaderPrecisionModifier
{
	enum Type : uint8
	{
		Float,
		Half,
		Fixed,
		Invalid
	};
};

/** Returns whether EShaderPrecisionModifier is supported. */
bool SupportShaderPrecisionModifier(EShaderPlatform Platform);

/** Each entry in a resource table is provided to the shader compiler for creating mappings. */
struct FUniformResourceEntry
{
	/** The name of the uniform buffer member for this resource. */
	const TCHAR* UniformBufferMemberName;
	/** The Uniform Buffer's name is a prefix of this length at the start of UniformBufferMemberName. */
	uint8 UniformBufferNameLength{};
	/** The type of the resource (EUniformBufferBaseType). */
	uint8 Type{};
	/** The index of the resource in the table. */
	uint16 ResourceIndex{};

	inline FStringView GetUniformBufferName() const { return FStringView(UniformBufferMemberName, UniformBufferNameLength); }
};

/** Minimal information about each uniform buffer entry fed to the shader compiler. */
struct FUniformBufferEntry
{
	/** The name of the uniform buffer static slot (if global). */
	FString StaticSlotName;

	/** Storage for member names for this uniform buffer (pointed to by FUniformResourceEntry::UniformBufferMemberName)  */
	FThreadSafeNameBufferPtr MemberNameBuffer;

	/** Hash of the resource table layout. */
	uint32 LayoutHash{};

	/** The binding flags used by this resource table. */
	EUniformBufferBindingFlags BindingFlags{ EUniformBufferBindingFlags::Shader };

	/** Whether to force a real uniform buffer when using emulated uniform buffers */
	ERHIUniformBufferFlags Flags{ ERHIUniformBufferFlags::None };
};

/** Parse the shader resource binding from the binding type used in shader code. */
EShaderCodeResourceBindingType ParseShaderResourceBindingType(const TCHAR* ShaderType);

/** Simple class that registers a uniform buffer static slot in the constructor. */
class FUniformBufferStaticSlotRegistrar
{
public:
	RENDERCORE_API FUniformBufferStaticSlotRegistrar(const TCHAR* InName);
};

/** Registry for uniform buffer static slots. */
class FUniformBufferStaticSlotRegistry
{
public:
	static RENDERCORE_API FUniformBufferStaticSlotRegistry& Get();

	RENDERCORE_API void RegisterSlot(FName SlotName);

	inline int32 GetSlotCount() const
	{
		return SlotNames.Num();
	}

	inline FString GetDebugDescription(FUniformBufferStaticSlot Slot) const
	{
		return FString::Printf(TEXT("[Name: %s, Slot: %u]"), *GetSlotName(Slot).ToString(), Slot);
	}

	inline FName GetSlotName(FUniformBufferStaticSlot Slot) const
	{
		checkf(Slot < SlotNames.Num(), TEXT("Requesting name for an invalid slot: %u."), Slot);
		return SlotNames[Slot];
	}

	inline FUniformBufferStaticSlot FindSlotByName(FName SlotName) const
	{
		// Brute force linear search. The search space is small and the find operation should not be critical path.
		for (int32 Index = 0; Index < SlotNames.Num(); ++Index)
		{
			if (SlotNames[Index] == SlotName)
			{
				return FUniformBufferStaticSlot(Index);
			}
		}
		return MAX_UNIFORM_BUFFER_STATIC_SLOTS;
	}

private:
	TArray<FName> SlotNames;
};

/** A uniform buffer struct. */
class FShaderParametersMetadata
{
public:
	/** The use case of the uniform buffer structures. */
	enum class EUseCase : uint8
	{
		/** Stand alone shader parameter struct used for render passes and shader parameters. */
		ShaderParameterStruct,

		/** Uniform buffer definition authored at compile-time. */
		UniformBuffer,

		/** Uniform buffer generated from assets, such as material parameter collection or Niagara. */
		DataDrivenUniformBuffer,
	};

	/** Additional flags that can be used to determine usage */
	enum class EUsageFlags : uint8
	{
		None = 0,

		/** On platforms that support emulated uniform buffers, disable them for this uniform buffer */
		NoEmulatedUniformBuffer = 1 << 0,
		
		/** This struct is a view into uniform buffer object, on platforms that support UBO */
		UniformView = 1 << 1,

		/** This struct needs its members reflected for binding information. */
		NeedsReflectedMembers = 1 << 2,

		/** Signals that the uniform buffer is manually bound by the pass and should be ignored by the mesh pass processor. */
		ManuallyBoundByPass = 1 << 3,

		/** When bindless is enabled, allow this uniform buffer to be accessed via a bindless constant buffer. */
		BindlessAccessible = 1 << 4,
	};

	/** Shader binding name of the uniform buffer that contains the root shader parameters. */
	static constexpr const TCHAR* kRootUniformBufferBindingName = TEXT("_RootShaderParameters");

	/** Shader binding name of the uniform buffer that contains the root shader parameters. */
	static constexpr int32 kRootCBufferBindingIndex = 0;

	/** A member of a shader parameter structure. */
	class FMember
	{
	public:

		/** Initialization constructor. */
		FMember(
			const TCHAR* InName,
			const TCHAR* InShaderType,
			int32 InFileLine,
			uint32 InOffset,
			EUniformBufferBaseType InBaseType,
			EShaderPrecisionModifier::Type InPrecision,
			uint32 InNumRows,
			uint32 InNumColumns,
			uint32 InNumElements,
			const FShaderParametersMetadata* InStruct
			)
		:	Name(InName)
		,	ShaderType(InShaderType)
		,	FileLine(InFileLine)
		,	Offset(InOffset)
		,	BaseType(InBaseType)
		,	Precision(InPrecision)
		,	NumRows(InNumRows)
		,	NumColumns(InNumColumns)
		,	NumElements(InNumElements)
		,	Struct(InStruct)
		{
			check(InShaderType);
		}

		/** Returns the string of the name of the element or name of the array of elements. */
		const TCHAR* GetName() const { return Name; }

		/** Returns the string of the type. */
		const TCHAR* GetShaderType() const { return ShaderType; }

		/** Returns the C++ line number where the parameter is declared. */
		int32 GetFileLine() const { return int32(FileLine); }

		/** Returns the offset of the element in the shader parameter struct in bytes. */
		uint32 GetOffset() const { return Offset; }

		/** Returns the type of the elements, int, UAV... */
		EUniformBufferBaseType GetBaseType() const { return BaseType; }

		/** Floating point the element is being stored. */
		EShaderPrecisionModifier::Type GetPrecision() const { return Precision; }

		/** Returns the number of row in the element. For instance FMatrix would return 4, or FVector would return 1. */
		uint32 GetNumRows() const { return NumRows; }

		/** Returns the number of column in the element. For instance FMatrix would return 4, or FVector would return 3. */
		uint32 GetNumColumns() const { return NumColumns; }

		/** Returns the number of elements in array, or 0 if this is not an array. */
		uint32 GetNumElements() const { return NumElements; }

		/** Returns the metadata of the struct. */
		const FShaderParametersMetadata* GetStructMetadata() const { return Struct; }

		inline bool IsVariableNativeType() const
		{
			return 
				BaseType == UBMT_INT32 ||
				BaseType == UBMT_UINT32 ||
				BaseType == UBMT_FLOAT32;
		}

		/** Returns the size of the member. */
		inline uint32 GetMemberSize() const
		{
			check(IsVariableNativeType());
			uint32 ElementSize = sizeof(uint32) * NumRows * NumColumns;

			/** If this an array, the alignment of the element are changed. */
			if (NumElements > 0)
			{
				return Align(ElementSize, SHADER_PARAMETER_ARRAY_ELEMENT_ALIGNMENT) * NumElements;
			}
			return ElementSize;
		}

		static RENDERCORE_API void GenerateShaderParameterType(
			FString& Result,
			bool bSupportsPrecisionModifier,
			EUniformBufferBaseType BaseType,
			EShaderPrecisionModifier::Type PrecisionModifier,
			uint32 NumRows,
			uint32 NumColumns
		);
		RENDERCORE_API void GenerateShaderParameterType(FString& Result, bool bSupportsPrecisionModifier) const;
		RENDERCORE_API void GenerateShaderParameterType(FString& Result, EShaderPlatform ShaderPlatform) const;

	private:
		friend class FShaderParametersMetadata;
		
		template<typename FHasherType, typename FHashType>
		void HashLayout(TMemoryHasher<FHasherType, FHashType>& Hasher)
		{
			Hasher << Offset;
			Hasher << reinterpret_cast<uint8&>(BaseType);

			Hasher.Serialize(const_cast<TCHAR*>(Name), FCString::Strlen(Name));
			Hasher << NumElements;

			const bool bIsRHIResource = IsShaderParameterTypeReadOnlyRHIResource(BaseType);
			const bool bIsRDGResource = IsRDGResourceReferenceShaderParameterType(BaseType);

			if (BaseType == UBMT_INT32 ||
				BaseType == UBMT_UINT32 ||
				BaseType == UBMT_FLOAT32)
			{
				Hasher << reinterpret_cast<uint8&>(Precision);
				Hasher << NumRows;
				Hasher << NumColumns;
			}
			else if (BaseType == UBMT_INCLUDED_STRUCT || BaseType == UBMT_NESTED_STRUCT)
			{
				const_cast<FShaderParametersMetadata*>(Struct)->HashLayout(Hasher);
			}
			else if (bIsRHIResource || bIsRDGResource)
			{
				Hasher.Serialize(const_cast<TCHAR*>(ShaderType), FCString::Strlen(ShaderType));
			}
		}

		const TCHAR* Name;
		const TCHAR* ShaderType;
		int32 FileLine;
		uint32 Offset;
		EUniformBufferBaseType BaseType;
		EShaderPrecisionModifier::Type Precision;
		uint32 NumRows;
		uint32 NumColumns;
		uint32 NumElements;
		const FShaderParametersMetadata* Struct;
	};

	/** Initialization constructor.
	 *
	 * EUseCase::UniformBuffer are listed in the global GetStructList() that will be visited at engine startup to know all the global uniform buffer
	 * that can generate code in /Engine/Generated/GeneratedUniformBuffers.ush. Their initialization will be finished during the this list
	 * traversal. bForceCompleteInitialization force to ignore the list for EUseCase::UniformBuffer and instead handle it like a standalone non
	 * globally listed EUseCase::ShaderParameterStruct. This is required for the ShaderCompileWorker to deserialize them without side global effects.
	 */
	RENDERCORE_API FShaderParametersMetadata(
		EUseCase UseCase,
		EUniformBufferBindingFlags InBindingFlags,
		const TCHAR* InLayoutName,
		const TCHAR* InStructTypeName,
		const TCHAR* InShaderVariableName,
		const TCHAR* InStaticSlotName,
		const ANSICHAR* InFileName,
		const int32 InFileLine,
		uint32 InSize,
		const TArray<FMember>& InMembers,
		bool bForceCompleteInitialization = false,
		FRHIUniformBufferLayoutInitializer* OutLayoutInitializer = nullptr,
		EUsageFlags InUsageFlags = EUsageFlags::None);

	RENDERCORE_API virtual ~FShaderParametersMetadata();

	RENDERCORE_API void GetNestedStructs(TArray<const FShaderParametersMetadata*>& OutNestedStructs) const;

#if WITH_EDITOR
	RENDERCORE_API void AddResourceTableEntries(FShaderResourceTableMap& ResourceTableMap, TMap<FString, FUniformBufferEntry>& UniformBufferMap) const;
#endif

	const TCHAR* GetStructTypeName() const { return StructTypeName; }
	const TCHAR* GetShaderVariableName() const { return ShaderVariableName; }
	const FHashedName& GetShaderVariableHashedName() const { return ShaderVariableHashedName; }
	const TCHAR* GetStaticSlotName() const { return StaticSlotName; }

	bool HasStaticSlot() const { return StaticSlotName != nullptr; }

	EUniformBufferBindingFlags GetBindingFlags() const { return BindingFlags; }

	EUniformBufferBindingFlags GetPreferredBindingFlag() const
	{
		// Decay to static when both binding flags are specified.
		return BindingFlags != EUniformBufferBindingFlags::StaticAndShader
			? BindingFlags
			: EUniformBufferBindingFlags::Static;
	}

	/** Returns the C++ file name where the parameter structure is declared. */
	const ANSICHAR* GetFileName() const { return FileName; }

	/** Returns the C++ line number where the parameter structure is declared. */
	const int32 GetFileLine() const { return FileLine; }

	uint32 GetSize() const { return Size; }
	EUseCase GetUseCase() const { return UseCase; }
	inline bool IsLayoutInitialized() const { return Layout != nullptr; }
	EUsageFlags GetUsageFlags() const { return UsageFlags; }

	const FRHIUniformBufferLayout& GetLayout() const
	{
		check(IsLayoutInitialized());
		return *Layout;
	}
	const FRHIUniformBufferLayout* GetLayoutPtr() const
	{
		check(IsLayoutInitialized());
		return Layout;
	}
	const TArray<FMember>& GetMembers() const { return Members; }

#if WITH_EDITOR
	inline bool IsUniformBufferDeclarationInitialized() const { return UniformBufferDeclaration.IsValid(); }
	FThreadSafeSharedStringPtr GetUniformBufferDeclarationPtr() const { return UniformBufferDeclaration; }
	FThreadSafeSharedAnsiStringPtr GetUniformBufferDeclarationAnsiPtr() const { return UniformBufferDeclarationAnsi; }
	const FString& GetUniformBufferDeclaration() const { return *UniformBufferDeclaration; }
	inline const FString& GetUniformBufferPath() const { return UniformBufferPath; }
	inline const FString& GetUniformBufferInclude() const { return UniformBufferInclude; }
	inline uint32 GetUniformBufferPathHash() const { return UniformBufferPathHash; }
#endif // WITH_EDITOR

	/** Find a member for a given offset. */
	RENDERCORE_API void FindMemberFromOffset(
		uint16 MemberOffset,
		const FShaderParametersMetadata** OutContainingStruct,
		const FShaderParametersMetadata::FMember** OutMember,
		int32* ArrayElementId, FString* NamePrefix) const;

	/** Returns the full C++ member name from it's byte offset in the structure. */
	RENDERCORE_API FString GetFullMemberCodeName(uint16 MemberOffset) const;

	static RENDERCORE_API TLinkedList<FShaderParametersMetadata*>*& GetStructList();
	/** Speed up finding the uniform buffer by its name */
	static RENDERCORE_API TMap<FHashedName, FShaderParametersMetadata*>& GetNameStructMap();
#if WITH_EDITOR
	static RENDERCORE_API TMap<FString, FShaderParametersMetadata*>& GetStringStructMap();
#endif // WITH_EDITOR

	/** Initialize all the global shader parameter structs. */
	static RENDERCORE_API void InitializeAllUniformBufferStructs();

	/** Returns a hash about the entire layout of the structure. */
	uint32 GetLayoutHash() const
	{
		check(UseCase == EUseCase::ShaderParameterStruct || UseCase == EUseCase::UniformBuffer);
		check(IsLayoutInitialized());
		return LayoutHash;	
	}

#if WITH_EDITOR
	RENDERCORE_API void AppendKeyString(FString& OutKeyString) const;
	RENDERCORE_API void Append(FShaderKeyGenerator& KeyGen) const;
#endif

	inline const FBlake3Hash& GetLayoutSignature() const
	{
#if WITH_EDITOR
		check(IsLayoutInitialized());
		return LayoutSignature;
#else
		// shader compilation types & WITH_EDITOR is a massive mess upstream; this should never actually be called outside of the editor
		// but actually compiling the function out is a headache, so we instead just assert if it's called 
		checkNoEntry();
		static FBlake3Hash Dummy;
		return Dummy;
#endif
	}


	/** Iterate recursively over all FShaderParametersMetadata. */
	template<typename TParameterFunction>
	void IterateStructureMetadataDependencies(TParameterFunction Lambda) const
	{
		for (const FShaderParametersMetadata::FMember& Member : Members)
		{
			const FShaderParametersMetadata* NewParametersMetadata = Member.GetStructMetadata();

			if (NewParametersMetadata)
			{
				NewParametersMetadata->IterateStructureMetadataDependencies(Lambda);
			}
		}

		Lambda(this);
	}

private:
	const TCHAR* const LayoutName;

	/** Name of the structure type in C++ and shader code. */
	const TCHAR* const StructTypeName;

	/** Name of the shader variable name for global shader parameter structs. */
	const TCHAR* const ShaderVariableName;

	/** Name of the static slot to use for the uniform buffer (or null). */
	const TCHAR* const StaticSlotName;

	FHashedName ShaderVariableHashedName;

	/** Name of the C++ file where the parameter structure is declared. */
	const ANSICHAR* const FileName;

	/** Line in the C++ file where the parameter structure is declared. */
	const int32 FileLine;

	/** Size of the entire struct in bytes. */
	const uint32 Size;

	/** The use case of this shader parameter struct. */
	const EUseCase UseCase;

	/** The binding model used by this parameter struct. */
	const EUniformBufferBindingFlags BindingFlags;

	/** Additional flags for how to use the buffer */
	const EUsageFlags UsageFlags;

	/** Layout of all the resources in the shader parameter struct. */
	FUniformBufferLayoutRHIRef Layout{};
	
	/** List of all members. */
	TArray<FMember> Members;

#if WITH_EDITOR
	/** Uniform buffer declaration, created once */
	FThreadSafeSharedStringPtr UniformBufferDeclaration;
	FThreadSafeSharedAnsiStringPtr UniformBufferDeclarationAnsi;

	/** Cache of uniform buffer resource table, and storage for member names used by the table, created once */
	TArray<FUniformResourceEntry> ResourceTableCache;
	FThreadSafeNameBufferPtr MemberNameBuffer;

	/** Strings for uniform buffer generated path and include, created once */
	FString UniformBufferPath;		// Format:  "/Engine/Generated/UniformBuffers/%s.ush"
	FString UniformBufferInclude;	// Format:  "#include \"/Engine/Generated/UniformBuffers/%s.ush\"" HLSL_LINE_TERMINATOR

	/** Hashes for frequently used strings */
	uint32 UniformBufferPathHash;
	uint32 ShaderVariableNameHash;
#endif

	/** Shackle elements in global link list of globally named shader parameters. */
	TLinkedList<FShaderParametersMetadata*> GlobalListLink;

	/** Hash about the entire memory layout of the structure. */
	uint32 LayoutHash = 0;

	template<typename FHasherType, typename FHashType>
	void HashLayout(TMemoryHasher<FHasherType, FHashType>& Hasher)
	{
		for (FMember& CurrentMember : Members)
		{
			CurrentMember.HashLayout(Hasher);
		}
	}

#if WITH_EDITOR
	
	/** Strong persistable hash representing the binary layout of the entire parameter structure */
	FBlake3Hash LayoutSignature;
#endif

	RENDERCORE_API void InitializeLayout(FRHIUniformBufferLayoutInitializer* OutLayoutInitializer = nullptr);

#if WITH_EDITOR
	RENDERCORE_API void InitializeUniformBufferDeclaration();
#endif
};

ENUM_CLASS_FLAGS(FShaderParametersMetadata::EUsageFlags);
