// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "RHI.h"
#include "RHIDefinitions.h"
#include "ShaderParameterMacros.h"
#include "ShaderParameterMetadata.h"
#include "Templates/AlignmentTemplates.h"

class FShaderParametersMetadataBuilder
{
public:
	FShaderParametersMetadataBuilder() {}

	explicit FShaderParametersMetadataBuilder(const FShaderParametersMetadata* RootParametersMetadata)
	{
		if (RootParametersMetadata)
		{
			Members = RootParametersMetadata->GetMembers();
			NextMemberOffset = RootParametersMetadata->GetSize();
		}
	}

	template<typename T>
	void AddParam(
		const TCHAR* Name,
		EShaderPrecisionModifier::Type Precision = EShaderPrecisionModifier::Float
		)
	{
		using TParamTypeInfo = TShaderParameterTypeInfo<T>;

		NextMemberOffset = Align(NextMemberOffset, TParamTypeInfo::Alignment);

		Members.Emplace(
			Name,
			TEXT(""),
			__LINE__,
			NextMemberOffset,
			TParamTypeInfo::BaseType,
			Precision,
			TParamTypeInfo::NumRows,
			TParamTypeInfo::NumColumns,
			TParamTypeInfo::NumElements,
			TParamTypeInfo::GetStructMetadata()
			);

		NextMemberOffset += sizeof(typename TParamTypeInfo::TAlignedType);
	}

	template<typename T>
	void AddParamArray(
		const TCHAR* Name,
		int32 NumElements,
		EShaderPrecisionModifier::Type Precision = EShaderPrecisionModifier::Float
		)
	{
		using TParamTypeInfo = TShaderParameterTypeInfo<T>;

		NextMemberOffset = Align(NextMemberOffset, SHADER_PARAMETER_ARRAY_ELEMENT_ALIGNMENT);

		Members.Emplace(
			Name,
			TEXT(""),
			__LINE__,
			NextMemberOffset,
			TParamTypeInfo::BaseType,
			Precision,
			TParamTypeInfo::NumRows,
			TParamTypeInfo::NumColumns,
			NumElements,
			TParamTypeInfo::GetStructMetadata()
			);

		NextMemberOffset += sizeof(typename TParamTypeInfo::TAlignedType) * NumElements;
	}

	template<typename T>
	void AddReferencedStruct(
		const TCHAR* Name,
		EShaderPrecisionModifier::Type Precision = EShaderPrecisionModifier::Float
		)
	{
		AddReferencedStruct(Name, TShaderParameterStructTypeInfo<T>::GetStructMetadata(), Precision);
	}

	RENDERCORE_API void AddReferencedStruct(
		const TCHAR* Name,
		const FShaderParametersMetadata* StructMetadata,
		EShaderPrecisionModifier::Type Precision = EShaderPrecisionModifier::Float
		);

	template<typename T>
	void AddIncludedStruct(
		EShaderPrecisionModifier::Type Precision = EShaderPrecisionModifier::Float
	)
	{
		AddIncludedStruct(TShaderParameterStructTypeInfo<T>::GetStructMetadata(), Precision);
	}

	RENDERCORE_API void AddIncludedStruct(
		const FShaderParametersMetadata* StructMetadata,
		EShaderPrecisionModifier::Type Precision = EShaderPrecisionModifier::Float
	);

	template<typename T>
	uint32 AddNestedStruct(
		const TCHAR* Name,
		EShaderPrecisionModifier::Type Precision = EShaderPrecisionModifier::Float
		)
	{
		using TParamTypeInfo = TShaderParameterStructTypeInfo<T>;

		NextMemberOffset = Align(NextMemberOffset, TParamTypeInfo::Alignment);
		const uint32 ThisMemberOffset = NextMemberOffset;

		Members.Emplace(
			Name,
			TEXT(""),
			__LINE__,
			NextMemberOffset,
			UBMT_NESTED_STRUCT,
			Precision,
			TParamTypeInfo::NumRows,
			TParamTypeInfo::NumColumns,
			TParamTypeInfo::NumElements,
			TParamTypeInfo::GetStructMetadata()
		);

		NextMemberOffset += sizeof(typename TParamTypeInfo::TAlignedType);
		return ThisMemberOffset;
	}

	RENDERCORE_API uint32 AddNestedStruct(
		const TCHAR* Name,
		const FShaderParametersMetadata* StructMetadata,
		EShaderPrecisionModifier::Type Precision = EShaderPrecisionModifier::Float
		);

	RENDERCORE_API void AddBufferSRV(
		const TCHAR* Name,
		const TCHAR* ShaderType,
		EShaderPrecisionModifier::Type Precision = EShaderPrecisionModifier::Float
		);

	RENDERCORE_API void AddBufferUAV(
		const TCHAR* Name,
		const TCHAR* ShaderType,
		EShaderPrecisionModifier::Type Precision = EShaderPrecisionModifier::Float
		);

	RENDERCORE_API void AddRDGBufferSRV(
		const TCHAR* Name,
		const TCHAR* ShaderType,
		EShaderPrecisionModifier::Type Precision = EShaderPrecisionModifier::Float
		);

	RENDERCORE_API void AddRDGBufferUAV(
		const TCHAR* Name,
		const TCHAR* ShaderType,
		EShaderPrecisionModifier::Type Precision = EShaderPrecisionModifier::Float
		);

	RENDERCORE_API void AddTexture(
		const TCHAR* Name,
		const TCHAR* ShaderType,
		EShaderPrecisionModifier::Type Precision = EShaderPrecisionModifier::Float
	);

	RENDERCORE_API void AddRDGTexture(
		const TCHAR* Name,
		const TCHAR* ShaderType,
		EShaderPrecisionModifier::Type Precision = EShaderPrecisionModifier::Float
	);

	RENDERCORE_API void AddSampler(
		const TCHAR* Name,
		const TCHAR* ShaderType,
		EShaderPrecisionModifier::Type Precision = EShaderPrecisionModifier::Float
	);

	RENDERCORE_API void AddRenderTargetBindingSlots(
		const TCHAR* Name = TEXT("RenderTargets"),
		const TCHAR* ShaderType = TEXT(""),
		EShaderPrecisionModifier::Type Precision = EShaderPrecisionModifier::Float
	);

	void AlignNextMemberToStruct()
	{
		NextMemberOffset = Align(NextMemberOffset, SHADER_PARAMETER_STRUCT_ALIGNMENT);
	}

	uint32 GetNextMemberOffset() const { return NextMemberOffset; }

	RENDERCORE_API FShaderParametersMetadata* Build(
		FShaderParametersMetadata::EUseCase UseCase,
		const TCHAR* ShaderParameterName
		);

	RENDERCORE_API FShaderParametersMetadata* Build(
		FShaderParametersMetadata::EUseCase InUseCase,
		EUniformBufferBindingFlags InBindingFlags,
		const TCHAR* InLayoutName,
		const TCHAR* InStructTypeName,
		const TCHAR* InShaderVariableName,
		const TCHAR* InStaticSlotName,
		const ANSICHAR* InFileName,
		const int32 InFileLine,
		bool bForceCompleteInitialization = false,
		FRHIUniformBufferLayoutInitializer* OutLayoutInitializer = nullptr,
		FShaderParametersMetadata::EUsageFlags InUsageFlags = FShaderParametersMetadata::EUsageFlags::None
		);

private:
	TArray<FShaderParametersMetadata::FMember> Members;
	uint32 NextMemberOffset = 0;
};
