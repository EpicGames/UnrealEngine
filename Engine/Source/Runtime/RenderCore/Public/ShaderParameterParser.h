// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ShaderParameterMetadata.h"
#include "ShaderCore.h"

class FShaderCompilerFlags;
struct FShaderCompilerInput;
struct FShaderCompilerOutput;
struct FShaderCompilerEnvironment;
struct FShaderCompilerError;

enum class EShaderParameterType : uint8;

enum class EBindlessConversionType : uint8
{
	None,
	SRV,
	UAV,
	Sampler,
};

enum class EShaderParameterParserConfigurationFlags
{
	None                    = 0,
	UseStableConstantBuffer = 1 << 0,
	SupportsBindless        = 1 << 1,
	// "Vulkan" style
	BindlessUsesArrays      = 1 << 2,
	// Some RHI Validation requires parameters to be parsed
	AlwaysParseParams		= 1 << 3,
	// Replace loose data globals in bindless shaders
	ReplaceGlobals			= 1 << 4,
};
ENUM_CLASS_FLAGS(EShaderParameterParserConfigurationFlags)

enum class EParsedShaderParameterFlags : uint8
{
	None                = 0,
	GloballyCoherent    = 1 << 0,
	WasInRootParameters = 1 << 1,
	IsGlobalParameter   = 1 << 2,
};
ENUM_CLASS_FLAGS(EParsedShaderParameterFlags)

/** Validates and moves all the shader loose data parameter defined in the root scope of the shader into the root uniform buffer. */
class FShaderParameterParser
{
public:
	struct FPlatformConfiguration
	{
		FPlatformConfiguration() = default;
		FPlatformConfiguration(FStringView InStableConstantBufferType, EShaderParameterParserConfigurationFlags InFlags = EShaderParameterParserConfigurationFlags::None)
			: StableConstantBufferType(InStableConstantBufferType)
			, Flags(InFlags)
		{
			if (InStableConstantBufferType.Len())
			{
				EnumAddFlags(Flags, EShaderParameterParserConfigurationFlags::UseStableConstantBuffer);
			}
		}

		virtual ~FPlatformConfiguration() = default;

		/** Generate shader code for accessing a bindless resource or sampler */
		virtual FString GenerateBindlessAccess(EBindlessConversionType BindlessType, FStringView FullTypeString, FStringView ArrayNameOverride, FStringView IndexString) const
		{
			checkf(false, TEXT("Platforms that support bindless must override GenerateBindlessAccess"));
			return FString();
		}

		/** Generate shader code for accessing a bindless resource or sampler */
		virtual FString ReplaceGlobal(FStringView FullDeclString, FStringView ParamName) const
		{
			checkf(false, TEXT("Platforms that support replacing globals must override ReplaceGlobal"));
			return FString();
		}

		/** Since the register/space for the global constant buffer can vary based on frequency, it's up to the shader format to communicate that to us */
		virtual FString GetStableConstantBufferRegisterString(EShaderFrequency ShaderFrequency) const
		{
			// Not required to be overridden
			return FString();
		}

		TConstArrayView<FStringView> ExtraSRVTypes;
		TConstArrayView<FStringView> ExtraUAVTypes;
		FStringView StableConstantBufferType;
		EShaderParameterParserConfigurationFlags Flags = EShaderParameterParserConfigurationFlags::None;
	};

	struct FParsedShaderParameter
	{
		/** Original information about the member. */
		EUniformBufferBaseType BaseType{ UBMT_INVALID };
		EShaderPrecisionModifier::Type PrecisionModifier{ EShaderPrecisionModifier::Invalid };
		uint32 NumRows = 0u;
		uint32 NumColumns = 0u;
		uint32 MemberSize = 0u;

		/** Offset the member should be in the constant buffer. */
		int32 ConstantBufferOffset = 0;

		/** Information found about the member when parsing the preprocessed code. */
		FStringView ParsedName; /** View into FShaderParameterParser::OriginalParsedShader */
		FStringView ParsedType; /** View into FShaderParameterParser::OriginalParsedShader */
		FStringView ParsedArraySize; /** View into FShaderParameterParser::OriginalParsedShader */

		int32 ParsedPragmaLineOffset = 0;
		int32 ParsedLineOffset = 0;

		/** Character position of the start and end of the parameter declaration in FParsedShaderParameter::OriginalParsedShader */
		int32 ParsedCharOffsetStart = INDEX_NONE;
		int32 ParsedCharOffsetEnd = INDEX_NONE;

		/** Only set when parameter needs to be moved to the root constant buffer. */
		EShaderParameterType RootCBParameterType{};

		EBindlessConversionType BindlessConversionType{};
		EShaderCodeResourceBindingType ParsedTypeDecl = EShaderCodeResourceBindingType::Invalid;
		EParsedShaderParameterFlags Flags{};

		/* Returns whether the shader parameter has been found when parsing. */
		bool IsFound() const
		{
			return !ParsedType.IsEmpty();
		}
	};

	FShaderParameterParser() = delete;
	RENDERCORE_API FShaderParameterParser(const FPlatformConfiguration& InPlatformConfiguration);

	RENDERCORE_API virtual ~FShaderParameterParser();

	static constexpr const TCHAR* kBindlessSRVPrefix = TEXT("BindlessSRV_");
	static constexpr const TCHAR* kBindlessUAVPrefix = TEXT("BindlessUAV_");
	static constexpr const TCHAR* kBindlessSamplerPrefix = TEXT("BindlessSampler_");

	// Prefix used to declare arrays of samplers/resources for bindless
	static constexpr const TCHAR* kBindlessSRVArrayPrefix = TEXT("SRVDescriptorHeap_");
	static constexpr const TCHAR* kBindlessUAVArrayPrefix = TEXT("UAVDescriptorHeap_");
	static constexpr const TCHAR* kBindlessCBVArrayPrefix = TEXT("CBVDescriptorHeap_");
	static constexpr const TCHAR* kBindlessSamplerArrayPrefix = TEXT("SamplerDescriptorHeap_");

	static RENDERCORE_API EShaderParameterType ParseAndRemoveBindlessParameterPrefix(FStringView& InName);
	static RENDERCORE_API EShaderParameterType ParseAndRemoveBindlessParameterPrefix(FString& InName);
	static RENDERCORE_API bool RemoveBindlessParameterPrefix(FString& InName);
	static RENDERCORE_API FStringView GetBindlessParameterPrefix(EShaderParameterType InShaderParameterType);

	RENDERCORE_API bool ParseAndModify(
		const FShaderCompilerInput& CompilerInput,
		TArray<FShaderCompilerError>& OutErrors,
		FString& PreprocessedShaderSource
	);

	/** Gets parsing information from a parameter binding name. */
	const FParsedShaderParameter& FindParameterInfos(FStringView ParameterName) const
	{
		const FParsedShaderParameter* ShaderParameter = FindParameterInfosUnsafe(ParameterName);
		checkf(ShaderParameter != nullptr, TEXT("Missing shader parameter '%.*s'"), ParameterName.Len(), ParameterName.GetData());
		return *ShaderParameter;
	}

	const FParsedShaderParameter* FindParameterInfosUnsafe(FStringView ParameterName) const
	{
		const uint32 KeyHash = decltype(ParsedParameters)::KeyFuncsType::GetKeyHash(ParameterName);
		return ParsedParameters.FindByHash(KeyHash, ParameterName);
	}

	/** Validates the shader parameter in code is compatible with the shader parameter structure. */
	RENDERCORE_API void ValidateShaderParameterType(
		const FShaderCompilerInput& CompilerInput,
		FStringView ShaderBindingName,
		int32 ReflectionOffset,
		int32 ReflectionSize,
		bool bPlatformSupportsPrecisionModifier,
		FShaderCompilerOutput& CompilerOutput) const;

	void ValidateShaderParameterType(
		const FShaderCompilerInput& CompilerInput,
		FStringView ShaderBindingName,
		int32 ReflectionOffset,
		int32 ReflectionSize,
		FShaderCompilerOutput& CompilerOutput) const
	{
		ValidateShaderParameterType(CompilerInput, ShaderBindingName, ReflectionOffset, ReflectionSize, false, CompilerOutput);
	}

	/** Validates shader parameter map is compatible with the shader parameter structure. */
	RENDERCORE_API void ValidateShaderParameterTypes(
		const FShaderCompilerInput& CompilerInput,
		bool bPlatformSupportsPrecisionModifier,
		FShaderCompilerOutput& CompilerOutput) const;

	void ValidateShaderParameterTypes(
		const FShaderCompilerInput& CompilerInput,
		FShaderCompilerOutput& CompilerOutput) const
	{
		ValidateShaderParameterTypes(CompilerInput, false, CompilerOutput);
	}

	/** Gets file and line of the parameter in the shader source code. */
	void GetParameterFileAndLine(const FParsedShaderParameter& ParsedParameter, FString& OutFile, FString& OutLine) const
	{
		return ExtractFileAndLine(ParsedParameter.ParsedPragmaLineOffset, ParsedParameter.ParsedLineOffset, OutFile, OutLine);
	}

	bool DidModifyShader() const { return bModifiedShader; }

protected:
	RENDERCORE_API EShaderParameterType ParseParameterType(FStringView InType);

	/** Parses the preprocessed shader code */
	RENDERCORE_API bool ParseParameters(
		const FShaderParametersMetadata* RootParametersStructure,
		TArray<FShaderCompilerError>& OutErrors
	);

	RENDERCORE_API void RemoveMovingParametersFromSource(
		FString& PreprocessedShaderSource
	);

	/** Converts parsed parameters into their bindless forms. */
	RENDERCORE_API void ApplyBindlessModifications(
		FString& PreprocessedShaderSource
	);

	/** Moves parsed parameters into the root constant buffer. */
	RENDERCORE_API bool MoveShaderParametersToRootConstantBuffer(
		const FShaderParametersMetadata* RootParametersStructure,
		EShaderFrequency ShaderFrequency,
		FString& PreprocessedShaderSource
	);

	RENDERCORE_API void ExtractFileAndLine(int32 PragamLineoffset, int32 LineOffset, FString& OutFile, FString& OutLine) const;

	/**
	* Generates shader source code to declare a bindless resource or sampler (for automatic bindless conversion).
	* May be overriden to allow custom implementations for different platforms.
	*/
	RENDERCORE_API virtual FString GenerateBindlessParameterDeclaration(const FParsedShaderParameter& ParsedParameter) const;

	const FPlatformConfiguration& PlatformConfiguration;

	FString OriginalParsedShader;

	TMap<FString, FParsedShaderParameter> ParsedParameters;

	bool bBindlessEnabled = false;

	/** Indicates that parameters should be moved to the root constant buffer. */
	bool bMoveGlobalsToRootCB = false;

	/** Indicates that parameters were actually moved to the root constant buffer. */
	bool bMovedLoosedParametersToRootConstantBuffer = false;

	/** Indicates that the shader source was actually modified. */
	bool bModifiedShader = false;
};
