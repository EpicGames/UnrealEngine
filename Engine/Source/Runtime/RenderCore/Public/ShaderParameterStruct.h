// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ShaderParameterStruct.h: API to submit all shader parameters in single function call.
=============================================================================*/

#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "Misc/AssertionMacros.h"
#include "RHI.h"
#include "RHIDefinitions.h"
#include "RenderGraphResources.h"
#include "Serialization/MemoryImage.h"
#include "Shader.h"

class FRHICommandList;
class FRHIComputeCommandList;
class FRHIComputeShader;
class FRHIGraphicsShader;
class FRHIResource;
class FShaderParameterMap;
class FShaderParametersMetadata;
struct FRHIUniformBufferLayout;

template <typename FParameterStruct>
void BindForLegacyShaderParameters(FShader* Shader, int32 PermutationId, const FShaderParameterMap& ParameterMap, bool bShouldBindEverything = false)
{
	Shader->Bindings.BindForLegacyShaderParameters(Shader, PermutationId, ParameterMap, *FParameterStruct::FTypeInfo::GetStructMetadata(), bShouldBindEverything);
}

/** Tag a shader class to use the structured shader parameters API.
 *
 * class FMyShaderClassCS : public FGlobalShader
 * {
 *		DECLARE_GLOBAL_SHADER(FMyShaderClassCS);
 *		SHADER_USE_PARAMETER_STRUCT(FMyShaderClassCS, FGlobalShader);
 *
 *		BEGIN_SHADER_PARAMETER_STRUCT(FParameters)
 *			SHADER_PARAMETER(FMatrix44f, ViewToClip)
 *			//...
 *		END_SHADER_PARAMETER_STRUCT()
 * };
 *
 * Notes: Long term, this macro will no longer be needed. Instead, parameter binding will become the default behavior for shader declarations.
 */

#define SHADER_USE_PARAMETER_STRUCT_INTERNAL(ShaderClass, ShaderParentClass, bShouldBindEverything) \
	ShaderClass() = default; \
	ShaderClass(const ShaderMetaType::CompiledShaderInitializerType& Initializer) \
		: ShaderParentClass(Initializer) \
	{ \
		BindForLegacyShaderParameters<FParameters>(this, Initializer.PermutationId, Initializer.ParameterMap, bShouldBindEverything); \
	}

#define SHADER_ROOT_PARAMETERS_SIGNATURE() \
	static inline const FShaderParametersMetadata* GetRootParametersMetadata() { return FParameters::FTypeInfo::GetStructMetadata(); }

// TODO(RDG): would not even need ShaderParentClass anymore. And in fact should not so Bindings.Bind() is not being called twice.
#define SHADER_USE_PARAMETER_STRUCT(ShaderClass, ShaderParentClass) \
	SHADER_USE_PARAMETER_STRUCT_INTERNAL(ShaderClass, ShaderParentClass, true) \
	SHADER_ROOT_PARAMETERS_SIGNATURE()

/** Use when sharing shader parameter binding with legacy parameters in the base class; i.e. FMaterialShader or FMeshMaterialShader.
 *  Note that this disables validation that the parameter struct contains all shader bindings.
 */
#define SHADER_USE_PARAMETER_STRUCT_WITH_LEGACY_BASE(ShaderClass, ShaderParentClass) \
	SHADER_USE_PARAMETER_STRUCT_INTERNAL(ShaderClass, ShaderParentClass, false)

 /** Use when sharing shader parameter binding with legacy parameters in the base class; i.e. FMaterialShader or FMeshMaterialShader.
  *  This forces the shader to be compiled with the Parameters.
  */
#define SHADER_USE_PARAMETER_STRUCT_MIXED(ShaderClass, ShaderParentClass) \
	SHADER_USE_PARAMETER_STRUCT_INTERNAL(ShaderClass, ShaderParentClass, false) \
	SHADER_ROOT_PARAMETERS_SIGNATURE()

#define SHADER_USE_ROOT_PARAMETER_STRUCT(ShaderClass, ShaderParentClass) \
	SHADER_ROOT_PARAMETERS_SIGNATURE() \
	ShaderClass() = default; \
	ShaderClass(const ShaderMetaType::CompiledShaderInitializerType& Initializer) \
		: ShaderParentClass(Initializer) \
	{ \
		this->Bindings.BindForRootShaderParameters(this, Initializer.PermutationId, Initializer.ParameterMap); \
	}

 /** Dereferences the RHI resource from a shader parameter struct. */
inline FRHIResource* GetShaderParameterResourceRHI(const void* Contents, uint16 MemberOffset, EUniformBufferBaseType MemberType)
{
	checkSlow(Contents);
	if (IsShaderParameterTypeIgnoredByRHI(MemberType))
	{
		return nullptr;
	}

	const uint8* MemberPtr = (const uint8*)Contents + MemberOffset;

	if (IsRDGResourceReferenceShaderParameterType(MemberType))
	{
		const FRDGResource* ResourcePtr = *reinterpret_cast<const FRDGResource* const*>(MemberPtr);
		return ResourcePtr ? ResourcePtr->GetRHI() : nullptr;
	}
	else
	{
		return *reinterpret_cast<FRHIResource* const*>(MemberPtr);
	}
}

/** Validates that all resource parameters of a uniform buffer are set. */
#if DO_CHECK
extern RENDERCORE_API void ValidateShaderParameterResourcesRHI(const void* Contents, const FRHIUniformBufferLayout& Layout);
#else
inline void ValidateShaderParameterResourcesRHI(const void* Contents, const FRHIUniformBufferLayout& Layout) {}
#endif


/** Raise fatal error when a required shader parameter has not been set. */
extern RENDERCORE_API void EmitNullShaderParameterFatalError(const TShaderRef<FShader>& Shader, const FShaderParametersMetadata* ParametersMetadata, uint16 MemberOffset);

/** Generate a list of excluded resources from the specialization constants. */
using FResourceExclusionArray = TArray<uint16>;
using FResourceExclusionArrayView = TArrayView<uint16>;
extern RENDERCORE_API FResourceExclusionArray GetSpecializationResourceExclusions(const TShaderRef<FShader>& Shader);

/** Validates that all resource parameters of a shader are set. */
#if DO_CHECK
extern RENDERCORE_API void ValidateShaderParameters(const TShaderRef<FShader>& Shader, const FShaderParametersMetadata* ParametersMetadata, const void* ParametersData, const FResourceExclusionArrayView ExclusionList);
#else
inline void ValidateShaderParameters(const TShaderRef<FShader>& Shader, const FShaderParametersMetadata* ParametersMetadata, const void* ParametersData, const FResourceExclusionArrayView ExclusionList) {}
#endif

inline void ValidateShaderParameters(const TShaderRef<FShader>& Shader, const FShaderParametersMetadata* ParametersMetadata, const void* ParametersData)
{
	ValidateShaderParameters(Shader, ParametersMetadata, ParametersData, GetSpecializationResourceExclusions(Shader));
}


template<typename TShaderClass>
inline void ValidateShaderParameters(const TShaderRef<TShaderClass>& Shader, const typename TShaderClass::FParameters& Parameters)
{
	ValidateShaderParameters(Shader, TShaderClass::FParameters::FTypeInfo::GetStructMetadata(), &Parameters);
}

template<typename TParameterType>
inline void CollectUAVsToUnset(FRHIBatchedShaderUnbinds& BatchedUnbinds, const TMemoryImageArray<TParameterType>& InParameters)
{
	for (const TParameterType& Parameter : InParameters)
	{
		if (Parameter.BaseType == UBMT_UAV ||
			Parameter.BaseType == UBMT_RDG_TEXTURE_UAV ||
			Parameter.BaseType == UBMT_RDG_BUFFER_UAV)
		{
			BatchedUnbinds.UnsetUAV(GetParameterIndex(Parameter));
		}
	}
}

template<typename TParameterType>
inline void CollectSRVsToUnset(FRHIBatchedShaderUnbinds& BatchedUnbinds, const TMemoryImageArray<TParameterType>& InParameters)
{
	for (const TParameterType& Parameter : InParameters)
	{
		if (Parameter.BaseType == UBMT_SRV ||
			Parameter.BaseType == UBMT_RDG_TEXTURE_SRV ||
			Parameter.BaseType == UBMT_RDG_TEXTURE_NON_PIXEL_SRV ||
			Parameter.BaseType == UBMT_RDG_BUFFER_SRV)
		{
			BatchedUnbinds.UnsetSRV(GetParameterIndex(Parameter));
		}
	}
}

inline void UnsetShaderUAVs(FRHIBatchedShaderUnbinds& BatchedUnbinds, const FShaderParameterBindings& Bindings)
{
	CollectUAVsToUnset(BatchedUnbinds, Bindings.ResourceParameters);
}

inline void UnsetShaderSRVs(FRHIBatchedShaderUnbinds& BatchedUnbinds, const FShaderParameterBindings& Bindings)
{
	CollectSRVsToUnset(BatchedUnbinds, Bindings.ResourceParameters);
}

template<typename TShaderClass>
inline void UnsetShaderUAVs(FRHIBatchedShaderUnbinds& BatchedUnbinds, const TShaderRef<TShaderClass>& Shader)
{
	// TODO(RDG): Once all shader sets their parameter through this, can refactor RHI so all UAVs of a shader get unset through a single RHI function call.
	const FShaderParameterBindings& Bindings = Shader->Bindings;

	checkf(Bindings.RootParameterBufferIndex == FShaderParameterBindings::kInvalidBufferIndex, TEXT("Can't use UnsetShaderUAVs() for root parameter buffer index."));

	UnsetShaderUAVs(BatchedUnbinds, Bindings);
}

template<typename TShaderClass>
inline void UnsetShaderSRVs(FRHIBatchedShaderUnbinds& BatchedUnbinds, const TShaderRef<TShaderClass>& Shader)
{
	// TODO(RDG): Once all shader sets their parameter through this, can refactor RHI so all SRVs of a shader get unset through a single RHI function call.
	const FShaderParameterBindings& Bindings = Shader->Bindings;

	checkf(Bindings.RootParameterBufferIndex == FShaderParameterBindings::kInvalidBufferIndex, TEXT("Can't use UnsetShaderSRVs() for root parameter buffer index."));

	UnsetShaderSRVs(BatchedUnbinds, Bindings);
}

/** Unset compute shader UAVs. */
template<typename TRHICmdList, typename TShaderClass>
inline void UnsetShaderUAVs(TRHICmdList& RHICmdList, const TShaderRef<TShaderClass>& Shader, FRHIComputeShader* ShaderRHI)
{
	if (RHICmdList.NeedsShaderUnbinds())
	{
		FRHIBatchedShaderUnbinds& BatchedUnbinds = RHICmdList.GetScratchShaderUnbinds();
		UnsetShaderUAVs(BatchedUnbinds, Shader);
		RHICmdList.SetBatchedShaderUnbinds(ShaderRHI, BatchedUnbinds);
	}
}

/** Unset compute shader SRVs. */
template<typename TRHICmdList, typename TShaderClass>
inline void UnsetShaderSRVs(TRHICmdList& RHICmdList, const TShaderRef<TShaderClass>& Shader, FRHIComputeShader* ShaderRHI)
{
	if (RHICmdList.NeedsShaderUnbinds())
	{
		FRHIBatchedShaderUnbinds& BatchedUnbinds = RHICmdList.GetScratchShaderUnbinds();
		UnsetShaderSRVs(BatchedUnbinds, Shader);
		RHICmdList.SetBatchedShaderUnbinds(ShaderRHI, BatchedUnbinds);
	}
}

/** Unset compute shader SRVs. */
template<typename TRHICmdList, typename TShaderClass>
inline void UnsetShaderSRVs(TRHICmdList& RHICmdList, const TShaderRef<TShaderClass>& Shader, FRHIGraphicsShader* ShaderRHI)
{
	if (RHICmdList.NeedsShaderUnbinds())
	{
		FRHIBatchedShaderUnbinds& BatchedUnbinds = RHICmdList.GetScratchShaderUnbinds();
		UnsetShaderSRVs(BatchedUnbinds, Shader);
		RHICmdList.SetBatchedShaderUnbinds(ShaderRHI, BatchedUnbinds);
	}
}

/** Set all resources described by ParametersMetadata using data from ParametersData as BindlessParameters */
RENDERCORE_API void SetAllShaderParametersAsBindless(
	FRHIBatchedShaderParameters& BatchedParameters,
	const FShaderParametersMetadata* ParametersMetadata,
	const void* ParametersData
);

template<typename TParameters>
inline void SetAllShaderParametersAsBindless(
	FRHIBatchedShaderParameters& BatchedParameters,
	const TParameters& InParameters
)
{
	SetAllShaderParametersAsBindless(BatchedParameters, TParameters::FTypeInfo::GetStructMetadata(), &InParameters);
}

RENDERCORE_API void SetShaderParameters(
	FRHIBatchedShaderParameters& BatchedParameters,
	const FShaderParameterBindings& Bindings,
	const FShaderParametersMetadata* ParametersMetadata,
	const void* ParametersData,
	const FResourceExclusionArrayView ResourceExclusions);

inline void SetShaderParameters(
	FRHIBatchedShaderParameters& BatchedParameters,
	const FShaderParameterBindings& Bindings,
	const FShaderParametersMetadata* ParametersMetadata,
	const void* ParametersData)
{
	const FResourceExclusionArrayView NoResourceExclusions;
	SetShaderParameters(BatchedParameters, Bindings, ParametersMetadata, ParametersData, NoResourceExclusions);
}

template<typename TShaderClass>
inline void SetShaderParameters(
	FRHIBatchedShaderParameters& BatchedParameters,
	const TShaderRef<TShaderClass>& Shader,
	const FShaderParametersMetadata* ParametersMetadata,
	const typename TShaderClass::FParameters& Parameters)
{
	ValidateShaderParameters(Shader, ParametersMetadata, &Parameters);

#if RHI_RAYTRACING
	if (IsRayTracingShaderFrequency(Shader->GetFrequency()))
	{
		SetRayTracingShaderParameters(BatchedParameters, Shader->Bindings, ParametersMetadata, &Parameters);
	}
	else
#endif // RHI_RAYTRACING
	{
		SetShaderParameters(BatchedParameters, Shader->Bindings, ParametersMetadata, &Parameters);
	}
}

template<typename TShaderClass>
inline void SetShaderParameters(FRHIBatchedShaderParameters& BatchedParameters, const TShaderRef<TShaderClass>& Shader, const typename TShaderClass::FParameters& Parameters)
{
	const FShaderParametersMetadata* ParametersMetadata = TShaderClass::FParameters::FTypeInfo::GetStructMetadata();
	SetShaderParameters(BatchedParameters, Shader, ParametersMetadata, Parameters);
}

RENDERCORE_API void SetShaderParameters(
	FRHIComputeCommandList& RHICmdList,
	FRHIComputeShader* ShaderRHI,
	const FShaderParameterBindings& Bindings,
	const FShaderParametersMetadata* ParametersMetadata,
	const void* ParametersData,
	const FResourceExclusionArrayView ResourceExclusions);

//UE_DEPRECATED(5.8, "Calls to SetShaderParameters now require ResourceExclusions to be specified.")
inline void SetShaderParameters(FRHIComputeCommandList& RHICmdList, FRHIComputeShader* ShaderRHI, const FShaderParameterBindings& Bindings, const FShaderParametersMetadata* ParametersMetadata, const void* ParametersData)
{
	checkf(!ShaderRHI->HasSpecializationConstants(), TEXT("SetShaderParameters called with a shader using specialization constants, a ResourceExclusion array needs to be specified."));
	SetShaderParameters(RHICmdList, ShaderRHI, Bindings, ParametersMetadata, ParametersData, FResourceExclusionArrayView());
}

RENDERCORE_API void SetShaderParameters(
	FRHICommandList& RHICmdList,
	FRHIGraphicsShader* ShaderRHI,
	const FShaderParameterBindings& Bindings,
	const FShaderParametersMetadata* ParametersMetadata,
	const void* ParametersData,
	const FResourceExclusionArrayView ResourceExclusions);

//UE_DEPRECATED(5.8, "Calls to SetShaderParameters now require ResourceExclusions to be specified.")
inline void SetShaderParameters(FRHICommandList& RHICmdList, FRHIGraphicsShader* ShaderRHI, const FShaderParameterBindings& Bindings, const FShaderParametersMetadata* ParametersMetadata, const void* ParametersData)
{
	checkf(!ShaderRHI->HasSpecializationConstants(), TEXT("SetShaderParameters called with a shader using specialization constants, a ResourceExclusion array needs to be specified."));
	SetShaderParameters(RHICmdList, ShaderRHI, Bindings, ParametersMetadata, ParametersData, FResourceExclusionArrayView());
}

RENDERCORE_API void SetShaderParameters(
	FRHICommandList& RHICmdList,
	FRHIComputeShader* ShaderRHI,
	const FShaderParameterBindings& Bindings,
	const FShaderParametersMetadata* ParametersMetadata,
	const void* ParametersData,
	const FResourceExclusionArrayView ResourceExclusions);

//UE_DEPRECATED(5.8, "Calls to SetShaderParameters now require ResourceExclusions to be specified.")
inline void SetShaderParameters(FRHICommandList& RHICmdList, FRHIComputeShader* ShaderRHI, const FShaderParameterBindings& Bindings, const FShaderParametersMetadata* ParametersMetadata, const void* ParametersData)
{
	checkf(!ShaderRHI->HasSpecializationConstants(), TEXT("SetShaderParameters called with a shader using specialization constants, a ResourceExclusion array needs to be specified."));
	SetShaderParameters(RHICmdList, ShaderRHI, Bindings, ParametersMetadata, ParametersData, FResourceExclusionArrayView());
}

template<typename TRHICmdList, typename TShaderClass, typename TShaderRHI>
inline void SetShaderParameters(
	TRHICmdList& RHICmdList,
	const TShaderRef<TShaderClass>& Shader,
	TShaderRHI* ShaderRHI,
	const FShaderParametersMetadata* ParametersMetadata,
	const typename TShaderClass::FParameters& Parameters)
{
	FResourceExclusionArray ResourceExclusions = GetSpecializationResourceExclusions(Shader);

	ValidateShaderParameters(Shader, ParametersMetadata, &Parameters, ResourceExclusions);

	// TODO(RDG): Once all shader sets their parameter through this, can refactor RHI so all shader parameters get sets through a single RHI function call.

	SetShaderParameters(RHICmdList, ShaderRHI, Shader->Bindings, ParametersMetadata, &Parameters, ResourceExclusions);
}

template<typename TRHICmdList, typename TShaderClass, typename TShaderRHI>
inline void SetShaderParameters(TRHICmdList& RHICmdList, const TShaderRef<TShaderClass>& Shader, TShaderRHI* ShaderRHI, const typename TShaderClass::FParameters& Parameters)
{
	const FShaderParametersMetadata* ParametersMetadata = TShaderClass::FParameters::FTypeInfo::GetStructMetadata();
	SetShaderParameters(RHICmdList, Shader, ShaderRHI, ParametersMetadata, Parameters);
}

#if RHI_RAYTRACING

/**
* Similar to SetShaderParameters(), but also binds static uniform buffers that are skipped otherwise.
* This helper function exists in UE 5.5 to aid with compatibility between FRHIBatchedShaderParameters and legacy FRayTracingShaderBindings[Writer].
* It will be deprecated in a future release, once legacy FRayTracingShaderBindings[Writer] is removed.
*/
RENDERCORE_API void SetRayTracingShaderParameters(
	FRHIBatchedShaderParameters& BatchedParameters,
	const FShaderParameterBindings& Bindings,
	const FShaderParametersMetadata* ParametersMetadata,
	const void* ParametersData);

PRAGMA_DISABLE_DEPRECATION_WARNINGS // Allow FRayTracingShaderBindingsWriter

RENDERCORE_API void SetShaderParameters(
	FRayTracingShaderBindingsWriter& RTBindingsWriter,
	const FShaderParameterBindings& Bindings,
	const FShaderParametersMetadata* ParametersMetadata,
	const void* ParametersData);

/** Set shader's parameters from its parameters struct. */
template<typename TShaderClass>
void SetShaderParameters(FRayTracingShaderBindingsWriter& RTBindingsWriter, const TShaderRef<TShaderClass>& Shader, const typename TShaderClass::FParameters& Parameters)
{
	const FShaderParametersMetadata* ParametersMetadata = TShaderClass::FParameters::FTypeInfo::GetStructMetadata();

	ValidateShaderParameters(Shader, ParametersMetadata, &Parameters);

	checkf(Shader->Bindings.Parameters.Num() == 0, TEXT("Ray tracing shader should use SHADER_USE_ROOT_PARAMETER_STRUCT() to passdown the cbuffer layout to the shader compiler."));

	SetShaderParameters(RTBindingsWriter, Shader->Bindings, ParametersMetadata, &Parameters);
}

PRAGMA_ENABLE_DEPRECATION_WARNINGS

#endif // RHI_RAYTRACING
