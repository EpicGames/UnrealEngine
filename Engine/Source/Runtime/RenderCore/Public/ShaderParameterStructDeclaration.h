// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <type_traits>
#include "HAL/Platform.h"

class FShaderParametersMetadata;

#define DECLARE_UNIFORM_BUFFER_STRUCT(StructTypeName, PrefixKeywords) \
	class StructTypeName; \
	PrefixKeywords const FShaderParametersMetadata* GetForwardDeclaredShaderParametersStructMetadata(const StructTypeName* DummyPtr);


/** Retrieves the metadata for a shader parameter struct type without having a definition of the type.
 *  This is the default overload in case there is no exact non-template overload (prevents implicit casting).
 */
template<typename T>
const FShaderParametersMetadata* GetForwardDeclaredShaderParametersStructMetadata(const T*)
{
	static_assert(!std::is_same<T, T>::value /* true */,
		"Partial uniform buffer struct declaration. Use `DECLARE_UNIFORM_BUFFER_STRUCT()` instead of `class T;`."
	);
	return nullptr;
}
