// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ShaderParameters.h: Shader parameter inline definitions.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "RHI.h"
#include "RHIUtilities.h"
#include "ShaderParameters.h"
#include "ShaderCore.h"
#include "Misc/App.h"

class FShaderMapPointerTable;

template<typename TBufferStruct> class TUniformBuffer;
template<typename TBufferStruct> class TUniformBufferRef;
template<typename ShaderType, typename PointerTableType> class TShaderRefBase;
template<typename ShaderType> using TShaderRef = TShaderRefBase<ShaderType, FShaderMapPointerTable>;

template<class ParameterType>
void SetShaderValue(
	FRHIBatchedShaderParameters& BatchedParameters
	, const FShaderParameter& Parameter
	, const ParameterType& Value
	, uint32 ElementIndex = 0
)
{
	// This will trigger if the parameter was not serialized
	checkSlow(Parameter.IsInitialized());

	static_assert(!TIsPointer<ParameterType>::Value, "Passing by value is not valid.");

	const uint32 AlignedTypeSize = Align<uint32>(sizeof(ParameterType), SHADER_PARAMETER_ARRAY_ELEMENT_ALIGNMENT);
	const uint32 ElementByteOffset = ElementIndex * AlignedTypeSize;
	const int32 NumBytesToSet = FMath::Min<int32>(sizeof(ParameterType), static_cast<int32>(Parameter.GetNumBytes()) - ElementByteOffset);

	if (NumBytesToSet > 0)
	{
		BatchedParameters.SetShaderParameter(
			Parameter.GetBufferIndex(),
			Parameter.GetBaseIndex() + ElementByteOffset,
			(uint32)NumBytesToSet,
			&Value);
	}
}

template<class ParameterType>
void SetShaderValueArray(
	FRHIBatchedShaderParameters& BatchedParameters
	, const FShaderParameter& Parameter
	, const ParameterType* Values
	, uint32 NumElements
	, uint32 ElementIndex = 0
)
{
	const uint32 AlignedTypeSize = Align<uint32>(sizeof(ParameterType), SHADER_PARAMETER_ARRAY_ELEMENT_ALIGNMENT);
	const uint32 ElementByteOffset = ElementIndex * AlignedTypeSize;
	const int32 NumBytesToSet = FMath::Min<int32>(NumElements * AlignedTypeSize, Parameter.GetNumBytes() - ElementByteOffset);

	// This will trigger if the parameter was not serialized
	checkSlow(Parameter.IsInitialized());

	if (NumBytesToSet > 0)
	{
		BatchedParameters.SetShaderParameter(
			Parameter.GetBufferIndex(),
			Parameter.GetBaseIndex() + ElementByteOffset,
			(uint32)NumBytesToSet,
			Values
		);
	}
}

inline void SetTextureParameter(FRHIBatchedShaderParameters& BatchedParameters, const FShaderResourceParameter& Parameter, FRHITexture* TextureRHI)
{
	if (Parameter.IsBound())
	{
#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
		if (Parameter.GetType() == EShaderParameterType::BindlessSRV)
		{
			BatchedParameters.SetBindlessTexture(Parameter.GetBaseIndex(), TextureRHI);
		}
		else
#endif
		{
			BatchedParameters.SetShaderTexture(Parameter.GetBaseIndex(), TextureRHI);
		}
	}
}

inline void SetSamplerParameter(FRHIBatchedShaderParameters& BatchedParameters, const FShaderResourceParameter& Parameter, FRHISamplerState* SamplerStateRHI)
{
	if (Parameter.IsBound())
	{
#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
		if (Parameter.GetType() == EShaderParameterType::BindlessSampler)
		{
			BatchedParameters.SetBindlessSampler(Parameter.GetBaseIndex(), SamplerStateRHI);
		}
		else
#endif
		{
			BatchedParameters.SetShaderSampler(Parameter.GetBaseIndex(), SamplerStateRHI);
		}
	}
}

inline void SetTextureParameter(
	FRHIBatchedShaderParameters& BatchedParameters,
	const FShaderResourceParameter& TextureParameter,
	const FShaderResourceParameter& SamplerParameter,
	FRHISamplerState* SamplerStateRHI,
	FRHITexture* TextureRHI
)
{
	SetTextureParameter(BatchedParameters, TextureParameter, TextureRHI);
	SetSamplerParameter(BatchedParameters, SamplerParameter, SamplerStateRHI);
}

inline void SetTextureParameter(
	FRHIBatchedShaderParameters& BatchedParameters,
	const FShaderResourceParameter& TextureParameter,
	const FShaderResourceParameter& SamplerParameter,
	const FTexture* Texture
)
{
	if (TextureParameter.IsBound())
	{
		Texture->LastRenderTime = FAppTime::GetApproximateCurrentTime();
	}

	SetTextureParameter(BatchedParameters, TextureParameter, Texture->TextureRHI);
	SetSamplerParameter(BatchedParameters, SamplerParameter, Texture->SamplerStateRHI);
}

inline void SetSRVParameter(FRHIBatchedShaderParameters& BatchedParameters, const FShaderResourceParameter& Parameter, FRHIShaderResourceView* SRV)
{
	if (Parameter.IsBound())
	{
#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
		if (Parameter.GetType() == EShaderParameterType::BindlessSRV)
		{
			BatchedParameters.SetBindlessResourceView(Parameter.GetBaseIndex(), SRV);
		}
		else
#endif
		{
			BatchedParameters.SetShaderResourceViewParameter(Parameter.GetBaseIndex(), SRV);
		}
	}
}

inline void SetUAVParameter(FRHIBatchedShaderParameters& BatchedParameters, const FShaderResourceParameter& Parameter, FRHIUnorderedAccessView* UAV)
{
	if (Parameter.IsBound())
	{
#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
		if (Parameter.GetType() == EShaderParameterType::BindlessUAV)
		{
			BatchedParameters.SetBindlessUAV(Parameter.GetBaseIndex(), UAV);
		}
		else
#endif
		{
			BatchedParameters.SetUAVParameter(Parameter.GetBaseIndex(), UAV);
		}
	}
}

inline void UnsetSRVParameter(FRHIBatchedShaderUnbinds& BatchedUnbinds, const FShaderResourceParameter& Parameter)
{
	if (Parameter.IsBound())
	{
#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
		if (Parameter.GetType() == EShaderParameterType::BindlessSRV)
		{
			// We don't need to clear Bindless views
		}
		else
#endif
		{
			BatchedUnbinds.UnsetSRV(Parameter.GetBaseIndex());
		}
	}
}

inline void UnsetUAVParameter(FRHIBatchedShaderUnbinds& BatchedUnbinds, const FShaderResourceParameter& Parameter)
{
	if (Parameter.IsBound())
	{
#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
		if (Parameter.GetType() == EShaderParameterType::BindlessSRV)
		{
			// We don't need to clear Bindless views
		}
		else
#endif
		{
			BatchedUnbinds.UnsetUAV(Parameter.GetBaseIndex());
		}
	}
}

inline void SetUniformBufferParameter(FRHIBatchedShaderParameters& BatchedParameters, const FShaderUniformBufferParameter& Parameter, FRHIUniformBuffer* UniformBufferRHI)
{
	// This will trigger if the parameter was not serialized
	checkSlow(Parameter.IsInitialized());
	// If it is bound, we must set it so something valid
	checkSlow(!Parameter.IsBound() || UniformBufferRHI);
	if (Parameter.IsBound())
	{
		BatchedParameters.SetShaderUniformBuffer(Parameter.GetBaseIndex(), UniformBufferRHI);
	}
}

template<typename TBufferStruct>
inline void SetUniformBufferParameter(FRHIBatchedShaderParameters& BatchedParameters, const TShaderUniformBufferParameter<TBufferStruct>& Parameter, const TUniformBufferRef<TBufferStruct>& UniformBufferRef)
{
	// This will trigger if the parameter was not serialized
	checkSlow(Parameter.IsInitialized());
	// If it is bound, we must set it so something valid
	checkSlow(!Parameter.IsBound() || IsValidRef(UniformBufferRef));
	if (Parameter.IsBound())
	{
		SetUniformBufferParameter(BatchedParameters, Parameter, UniformBufferRef.GetReference());
	}
}

template<typename TBufferStruct>
inline void SetUniformBufferParameter(FRHIBatchedShaderParameters& BatchedParameters, const TShaderUniformBufferParameter<TBufferStruct>& Parameter, const TUniformBuffer<TBufferStruct>& UniformBuffer)
{
	// This will trigger if the parameter was not serialized
	checkSlow(Parameter.IsInitialized());
	// If it is bound, we must set it so something valid
	checkSlow(!Parameter.IsBound() || UniformBuffer.GetUniformBufferRHI());
	if (Parameter.IsBound())
	{
		SetUniformBufferParameter(BatchedParameters, Parameter, UniformBuffer.GetUniformBufferRHI());
	}
}

template<typename TBufferStruct>
inline void SetUniformBufferParameterImmediate(FRHIBatchedShaderParameters& BatchedParameters, const TShaderUniformBufferParameter<TBufferStruct>& Parameter, const TBufferStruct& UniformBufferValue)
{
	// This will trigger if the parameter was not serialized
	checkSlow(Parameter.IsInitialized());
	if (Parameter.IsBound())
	{
		FUniformBufferRHIRef UniformBufferRef = RHICreateUniformBuffer(&UniformBufferValue, &TBufferStruct::FTypeInfo::GetStructMetadata()->GetLayout(), UniformBuffer_SingleDraw);
		SetUniformBufferParameter(BatchedParameters, Parameter, UniformBufferRef.GetReference());
	}
}

// Utility to set a single shader value on a shader. Should only be used if a shader requires only a single value.
template<typename TRHICmdList, typename TShaderTypeRHI, class ParameterType>
void SetSingleShaderValue(
	TRHICmdList& RHICmdList
	, TShaderTypeRHI* InShaderRHI
	, const FShaderParameter& Parameter
	, const ParameterType& Value
)
{
	FRHIBatchedShaderParameters& BatchedParameters = RHICmdList.GetScratchShaderParameters();
	SetShaderValue(BatchedParameters, Parameter, Value);
	RHICmdList.SetBatchedShaderParameters(InShaderRHI, BatchedParameters);
}

// Mixed mode binding utilities

/// Utility to set all legacy and non-legacy parameters for a shader. Requires the shader type to implement SetParameters(FRHIBatchedShaderParameters& BatchedParameters, ...)
template<typename TShaderType, typename... TArguments>
inline void SetBatchedShaderParametersMixed(
	FRHIBatchedShaderParameters& BatchedParameters,
	const TShaderRef<TShaderType>& InShader,
	const typename TShaderType::FParameters& Parameters,
	TArguments&&... InArguments)
{
	// New Style first
	SetShaderParameters(BatchedParameters, InShader, Parameters);

	// Legacy second
	InShader->SetParameters(BatchedParameters, Forward<TArguments>(InArguments)...);
}

/// Utility to set all legacy and non-legacy parameters for a shader. Requires the shader type to implement SetParameters(FRHIBatchedShaderParameters& BatchedParameters, ...)
template<typename TRHICmdList, typename TShaderType, typename TShaderTypeRHI, typename... TArguments>
inline void SetShaderParametersMixed(
	TRHICmdList& RHICmdList,
	const TShaderRef<TShaderType>& InShader,
	TShaderTypeRHI* InShaderRHI,
	const typename TShaderType::FParameters& Parameters,
	TArguments&&... InArguments)
{
	FRHIBatchedShaderParameters& BatchedParameters = RHICmdList.GetScratchShaderParameters();
	SetBatchedShaderParametersMixed(BatchedParameters, InShader, Parameters, Forward<TArguments>(InArguments)...);
	RHICmdList.SetBatchedShaderParameters(InShaderRHI, BatchedParameters);
}

/// Utility to set all legacy and non-legacy parameters for a Vertex shader. Requires the shader type to implement SetParameters(FRHIBatchedShaderParameters& BatchedParameters, ...)
template<typename TRHICmdList, typename TShaderType, typename... TArguments>
inline void SetShaderParametersMixedVS(TRHICmdList& RHICmdList, const TShaderRef<TShaderType>& InShader, const typename TShaderType::FParameters& Parameters, TArguments&&... InArguments)
{
	SetShaderParametersMixed(RHICmdList, InShader, InShader.GetVertexShader(), Parameters, Forward<TArguments>(InArguments)...);
}

/// Utility to set all legacy and non-legacy parameters for a Mesh shader. Requires the shader type to implement SetParameters(FRHIBatchedShaderParameters& BatchedParameters, ...)
template<typename TRHICmdList, typename TShaderType, typename... TArguments>
inline void SetShaderParametersMixedMS(TRHICmdList& RHICmdList, const TShaderRef<TShaderType>& InShader, const typename TShaderType::FParameters& Parameters, TArguments&&... InArguments)
{
	SetShaderParametersMixed(RHICmdList, InShader, InShader.GetMeshShader(), Parameters, Forward<TArguments>(InArguments)...);
}

/// Utility to set all legacy and non-legacy parameters for an Amplification shader. Requires the shader type to implement SetParameters(FRHIBatchedShaderParameters& BatchedParameters, ...)
template<typename TRHICmdList, typename TShaderType, typename... TArguments>
inline void SetShaderParametersMixedAS(TRHICmdList& RHICmdList, const TShaderRef<TShaderType>& InShader, const typename TShaderType::FParameters& Parameters, TArguments&&... InArguments)
{
	SetShaderParametersMixed(RHICmdList, InShader, InShader.GetAmplificationShader(), Parameters, Forward<TArguments>(InArguments)...);
}

/// Utility to set all legacy and non-legacy parameters for a Pixel shader. Requires the shader type to implement SetParameters(FRHIBatchedShaderParameters& BatchedParameters, ...)
template<typename TRHICmdList, typename TShaderType, typename... TArguments>
inline void SetShaderParametersMixedPS(TRHICmdList& RHICmdList, const TShaderRef<TShaderType>& InShader, const typename TShaderType::FParameters& Parameters, TArguments&&... InArguments)
{
	SetShaderParametersMixed(RHICmdList, InShader, InShader.GetPixelShader(), Parameters, Forward<TArguments>(InArguments)...);
}

/// Utility to set all legacy and non-legacy parameters for a Geometry shader. Requires the shader type to implement SetParameters(FRHIBatchedShaderParameters& BatchedParameters, ...)
template<typename TRHICmdList, typename TShaderType, typename... TArguments>
inline void SetShaderParametersMixedGS(TRHICmdList& RHICmdList, const TShaderRef<TShaderType>& InShader, const typename TShaderType::FParameters& Parameters, TArguments&&... InArguments)
{
	SetShaderParametersMixed(RHICmdList, InShader, InShader.GetGeometryShader(), Parameters, Forward<TArguments>(InArguments)...);
}

/// Utility to set all legacy and non-legacy parameters for a Compute shader. Requires the shader type to implement SetParameters(FRHIBatchedShaderParameters& BatchedParameters, ...)
template<typename TRHICmdList, typename TShaderType, typename... TArguments>
inline void SetShaderParametersMixedCS(TRHICmdList& RHICmdList, const TShaderRef<TShaderType>& InShader, const typename TShaderType::FParameters& Parameters, TArguments&&... InArguments)
{
	SetShaderParametersMixed(RHICmdList, InShader, InShader.GetComputeShader(), Parameters, Forward<TArguments>(InArguments)...);
}

// Legacy binding utilities

/// Utility to set all legacy parameters for a shader. Requires the shader type to implement SetParameters(FRHIBatchedShaderParameters& BatchedParameters, ...)
template<typename TRHICmdList, typename TShaderType, typename TShaderTypeRHI, typename... TArguments>
inline void SetShaderParametersLegacy(TRHICmdList& RHICmdList, const TShaderRef<TShaderType>& InShader, TShaderTypeRHI* InShaderRHI, TArguments&&... InArguments)
{
	FRHIBatchedShaderParameters& BatchedParameters = RHICmdList.GetScratchShaderParameters();
	InShader->SetParameters(BatchedParameters, Forward<TArguments>(InArguments)...);
	RHICmdList.SetBatchedShaderParameters(InShaderRHI, BatchedParameters);
}

/// Utility to set all legacy parameters for a Vertex shader. Requires the shader type to implement SetParameters(FRHIBatchedShaderParameters& BatchedParameters, ...)
template<typename TRHICmdList, typename TShaderType, typename... TArguments>
inline void SetShaderParametersLegacyVS(TRHICmdList& RHICmdList, const TShaderRef<TShaderType>& InShader, TArguments&&... InArguments)
{
	SetShaderParametersLegacy(RHICmdList, InShader, InShader.GetVertexShader(), Forward<TArguments>(InArguments)...);
}

/// Utility to set all legacy parameters for a Mesh shader. Requires the shader type to implement SetParameters(FRHIBatchedShaderParameters& BatchedParameters, ...)
template<typename TRHICmdList, typename TShaderType, typename... TArguments>
inline void SetShaderParametersLegacyMS(TRHICmdList& RHICmdList, const TShaderRef<TShaderType>& InShader, TArguments&&... InArguments)
{
	SetShaderParametersLegacy(RHICmdList, InShader, InShader.GetMeshShader(), Forward<TArguments>(InArguments)...);
}

/// Utility to set all legacy parameters for an Amplification shader. Requires the shader type to implement SetParameters(FRHIBatchedShaderParameters& BatchedParameters, ...)
template<typename TRHICmdList, typename TShaderType, typename... TArguments>
inline void SetShaderParametersLegacyAS(TRHICmdList& RHICmdList, const TShaderRef<TShaderType>& InShader, TArguments&&... InArguments)
{
	SetShaderParametersLegacy(RHICmdList, InShader, InShader.GetAmplificationShader(), Forward<TArguments>(InArguments)...);
}

/// Utility to set all legacy parameters for a Pixel shader. Requires the shader type to implement SetParameters(FRHIBatchedShaderParameters& BatchedParameters, ...)
template<typename TRHICmdList, typename TShaderType, typename... TArguments>
inline void SetShaderParametersLegacyPS(TRHICmdList& RHICmdList, const TShaderRef<TShaderType>& InShader, TArguments&&... InArguments)
{
	SetShaderParametersLegacy(RHICmdList, InShader, InShader.GetPixelShader(), Forward<TArguments>(InArguments)...);
}

/// Utility to set all legacy parameters for a Geometry shader. Requires the shader type to implement SetParameters(FRHIBatchedShaderParameters& BatchedParameters, ...)
template<typename TRHICmdList, typename TShaderType, typename... TArguments>
inline void SetShaderParametersLegacyGS(TRHICmdList& RHICmdList, const TShaderRef<TShaderType>& InShader, TArguments&&... InArguments)
{
	SetShaderParametersLegacy(RHICmdList, InShader, InShader.GetGeometryShader(), Forward<TArguments>(InArguments)...);
}

/// Utility to set all legacy parameters for a Compute shader. Requires the shader type to implement SetParameters(FRHIBatchedShaderParameters& BatchedParameters, ...)
template<typename TRHICmdList, typename TShaderType, typename... TArguments>
inline void SetShaderParametersLegacyCS(TRHICmdList& RHICmdList, const TShaderRef<TShaderType>& InShader, TArguments&&... InArguments)
{
	SetShaderParametersLegacy(RHICmdList, InShader, InShader.GetComputeShader(), Forward<TArguments>(InArguments)...);
}

/// Utility to unset all legacy parameters for a Pixel shader. Requires the shader type to implement UnsetParameters(FRHIBatchedShaderUnbinds& BatchedUnbinds)
template<typename TRHICmdList, typename TShaderType>
inline void UnsetShaderParametersLegacyPS(TRHICmdList& RHICmdList, const TShaderRef<TShaderType>& InShader)
{
	if (RHICmdList.NeedsShaderUnbinds())
	{
		FRHIBatchedShaderUnbinds& BatchedUnbinds = RHICmdList.GetScratchShaderUnbinds();
		InShader->UnsetParameters(BatchedUnbinds);
		RHICmdList.SetBatchedShaderUnbinds(InShader.GetPixelShader(), BatchedUnbinds);
	}
}

/// Utility to unset all legacy parameters for a Compute shader. Requires the shader type to implement UnsetParameters(FRHIBatchedShaderUnbinds& BatchedUnbinds)
template<typename TRHICmdList, typename TShaderType>
inline void UnsetShaderParametersLegacyCS(TRHICmdList& RHICmdList, const TShaderRef<TShaderType>& InShader)
{
	if (RHICmdList.NeedsShaderUnbinds())
	{
		FRHIBatchedShaderUnbinds& BatchedUnbinds = RHICmdList.GetScratchShaderUnbinds();
		InShader->UnsetParameters(BatchedUnbinds);
		RHICmdList.SetBatchedShaderUnbinds(InShader.GetComputeShader(), BatchedUnbinds);
	}
}

// LWC_TODO: Setting guards to catch attempts to pass a type with double components. Could just convert these to the correct type internally, but would prefer to catch potential issues + optimize where possible.
#define GUARD_SETSHADERVALUE(_TYPE)																																						\
template<typename ShaderRHIParamRef, typename TRHICmdList>																																\
void SetShaderValue(  TRHICmdList& RHICmdList,	const ShaderRHIParamRef& Shader, const FShaderParameter& Parameter,																		\
	const _TYPE##d& Value, uint32 ElementIndex = 0) { static_assert(sizeof(ShaderRHIParamRef) == 0, "Passing unsupported "#_TYPE"d. Requires "#_TYPE"f"); }								\
template<typename ShaderRHIParamRef>																																					\
void SetShaderValueOnContext(IRHICommandContext& RHICmdListContext,	const ShaderRHIParamRef& Shader, const FShaderParameter& Parameter,													\
	const _TYPE##d& Value, uint32 ElementIndex = 0) { static_assert(sizeof(ShaderRHIParamRef) == 0, "Passing unsupported "#_TYPE"d. Requires "#_TYPE"f"); }								\
template<typename ShaderRHIParamRef, typename TRHICmdList>																																\
void SetShaderValueArray(TRHICmdList& RHICmdList, const ShaderRHIParamRef& Shader, const FShaderParameter& Parameter,																	\
	const _TYPE##d* Values, uint32 NumElements, uint32 BaseElementIndex = 0) { static_assert(sizeof(ShaderRHIParamRef) == 0, "Passing unsupported "#_TYPE"d*. Requires "#_TYPE"f*"); }	\

// Primary
GUARD_SETSHADERVALUE(FMatrix44)
GUARD_SETSHADERVALUE(FVector2)
GUARD_SETSHADERVALUE(FVector3)
GUARD_SETSHADERVALUE(FVector4)
GUARD_SETSHADERVALUE(FPlane4)
GUARD_SETSHADERVALUE(FQuat4)
// Secondary
GUARD_SETSHADERVALUE(::FSphere3)
GUARD_SETSHADERVALUE(FBox3)

/**
 * Sets the value of a shader texture parameter. Template'd on shader type.
 */
template<typename TRHIShader, typename TRHICmdList>
inline void SetTextureParameter(TRHICmdList& RHICmdList, TRHIShader* Shader, const FShaderResourceParameter& TextureParameter, const FShaderResourceParameter& SamplerParameter, FRHISamplerState* SamplerStateRHI, FRHITexture* TextureRHI)
{
	FRHIBatchedShaderParameters& BatchedParameters = RHICmdList.GetScratchShaderParameters();
	SetTextureParameter(BatchedParameters, TextureParameter, SamplerParameter, SamplerStateRHI, TextureRHI);
	RHICmdList.SetBatchedShaderParameters(Shader, BatchedParameters);
}
