// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ShaderParameters.h: Shader parameter definitions.
=============================================================================*/

#pragma once

#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "Misc/AssertionMacros.h"
#include "RHI.h"
#include "RHICommandList.h"
#include "RHIDefinitions.h"
#include "Serialization/Archive.h"
#include "Serialization/MemoryLayout.h"

class FPointerTableBase;
class FRHIComputeShader;
class FRHITexture;
class FRHIUnorderedAccessView;
class FShaderParameterMap;
class FShaderParametersMetadata;
struct FRWBuffer;
struct FRWBufferStructured;
struct FShaderCompilerEnvironment;
struct FRHIUniformBufferShaderBindingLayout;

enum class EShaderParameterType : uint8;
DECLARE_INTRINSIC_TYPE_LAYOUT(EShaderParameterType);

enum class EShaderCodeResourceBindingType : uint8;
DECLARE_INTRINSIC_TYPE_LAYOUT(EShaderCodeResourceBindingType);

#if WITH_EDITOR
namespace UE::ShaderParameters
{
	/** Creates a shader code declaration of this struct for the given shader platform. */
	RENDERCORE_API FString CreateUniformBufferShaderDeclaration(const TCHAR* Name, const FShaderParametersMetadata& UniformBufferStruct, const FRHIUniformBufferShaderBindingLayout* UniformBufferSBLayout);
	RENDERCORE_API void AddUniformBufferIncludesToEnvironment(FShaderCompilerEnvironment& OutEnvironment, const TSet<const FShaderParametersMetadata*>& InUniformBuffers);
}
#endif

enum EShaderParameterFlags
{
	// no shader error if the parameter is not used
	SPF_Optional,
	// shader error if the parameter is not used
	SPF_Mandatory
};

/** A shader parameter's register binding. e.g. float1/2/3/4, can be an array, UAV */
class FShaderParameter
{
	DECLARE_EXPORTED_TYPE_LAYOUT(FShaderParameter, RENDERCORE_API, NonVirtual);
public:
	FShaderParameter()
	:	BufferIndex(0)
	,	BaseIndex(0)
	,	NumBytes(0)
	{}

	RENDERCORE_API void Bind(const FShaderParameterMap& ParameterMap,const TCHAR* ParameterName, EShaderParameterFlags Flags = SPF_Optional);
	friend RENDERCORE_API FArchive& operator<<(FArchive& Ar,FShaderParameter& P);
	inline bool IsBound() const { return NumBytes > 0; }
	
	inline bool IsInitialized() const 
	{ 
		return true;
	}

	uint32 GetBufferIndex() const { return BufferIndex; }
	uint32 GetBaseIndex() const { return BaseIndex; }
	uint32 GetNumBytes() const { return NumBytes; }

private:
	LAYOUT_FIELD(uint16, BufferIndex);
	LAYOUT_FIELD(uint16, BaseIndex);
	// 0 if the parameter wasn't bound
	LAYOUT_FIELD(uint16, NumBytes);
};

/** A shader resource binding (textures or samplerstates). */
class FShaderResourceParameter
{
	DECLARE_EXPORTED_TYPE_LAYOUT(FShaderResourceParameter, RENDERCORE_API, NonVirtual);
public:
	FShaderResourceParameter() = default;

	RENDERCORE_API void Bind(const FShaderParameterMap& ParameterMap,const TCHAR* ParameterName,EShaderParameterFlags Flags = SPF_Optional);
	friend RENDERCORE_API FArchive& operator<<(FArchive& Ar,FShaderResourceParameter& P);

	inline bool IsBound() const { return NumResources > 0; }
	inline bool IsInitialized() const { return true; }

	inline uint32 GetBaseIndex() const { return BaseIndex; }
	inline uint32 GetNumResources() const { return NumResources; }
	inline EShaderParameterType GetType() const { return Type; }

private:
	LAYOUT_FIELD_INITIALIZED(uint16, BaseIndex, 0);
	LAYOUT_FIELD_INITIALIZED(uint8, NumResources, 0);
	LAYOUT_FIELD_INITIALIZED(EShaderParameterType, Type, {});
};

class FShaderUniformBufferParameter
{
	DECLARE_EXPORTED_TYPE_LAYOUT(FShaderUniformBufferParameter, RENDERCORE_API, NonVirtual);
public:
	FShaderUniformBufferParameter()
	:	BaseIndex(0xffff)
	{}

#if WITH_EDITOR
	static RENDERCORE_API void ModifyCompilationEnvironment(const TCHAR* ParameterName,const FShaderParametersMetadata& Struct,EShaderPlatform Platform,FShaderCompilerEnvironment& OutEnvironment);
#endif // WITH_EDITOR

	RENDERCORE_API void Bind(const FShaderParameterMap& ParameterMap,const TCHAR* ParameterName,EShaderParameterFlags Flags = SPF_Optional);

	friend FArchive& operator<<(FArchive& Ar,FShaderUniformBufferParameter& P)
	{
		P.Serialize(Ar);
		return Ar;
	}

	inline bool IsBound() const { return BaseIndex != 0xffff; }

	void Serialize(FArchive& Ar)
	{
		Ar << BaseIndex;
	}

	inline bool IsInitialized() const 
	{ 
		return true;
	}
	uint32 GetBaseIndex() const { ensure(IsBound()); return BaseIndex; }

private:
	LAYOUT_FIELD(uint16, BaseIndex);
};

/** A shader uniform buffer binding with a specific structure. */
template<typename TBufferStruct>
class TShaderUniformBufferParameter : public FShaderUniformBufferParameter
{
public:
#if WITH_EDITOR
	static void ModifyCompilationEnvironment(const TCHAR* ParameterName,EShaderPlatform Platform, FShaderCompilerEnvironment& OutEnvironment)
	{
		FShaderUniformBufferParameter::ModifyCompilationEnvironment(ParameterName,TBufferStruct::StaticStruct,Platform,OutEnvironment);
	}
#endif // WITH_EDITOR

	friend FArchive& operator<<(FArchive& Ar,TShaderUniformBufferParameter& P)
	{
		P.Serialize(Ar);
		return Ar;
	}
};

/** A shader uniform buffer member binding, this is only used to determine if the member is used in the compiled shader. */
class FShaderUniformBufferMemberParameter
{
	DECLARE_EXPORTED_TYPE_LAYOUT(FShaderUniformBufferMemberParameter, RENDERCORE_API, NonVirtual);
public:
	FShaderUniformBufferMemberParameter() = default;

	RENDERCORE_API void Bind(const FShaderParameterMap& ParameterMap, const TCHAR* ParameterName);
	friend RENDERCORE_API FArchive& operator<<(FArchive& Ar, FShaderUniformBufferMemberParameter& P);

	inline bool IsBound() const { return bIsBound != 0; }
	inline bool IsInitialized() const { return true; }

private:
	LAYOUT_FIELD_INITIALIZED(uint8, bIsBound, 0);
};

#if RHI_RAYTRACING

struct UE_DEPRECATED(5.5, "Use FRHIBatchedShaderParameters and SetShaderParameters() instead.") FRayTracingShaderBindingsWriter : FRayTracingShaderBindings
{
	FUniformBufferRHIRef RootUniformBuffer;

	void AddBindlessParameter(const FRHIShaderParameterResource& Parameter)
	{
		BindlessParameters.Add(Parameter);
	}

	void Set(const FShaderResourceParameter& Param, FRHITexture* Value)
	{
		if (Param.IsBound())
		{
			checkf(Param.GetNumResources() == 1, TEXT("Resource array binding is not implemented"));
			Textures[Param.GetBaseIndex()] = Value;
		}
	}

	void Set(const FShaderResourceParameter& Param, FRHIShaderResourceView* Value)
	{
		if (Param.IsBound())
		{
			checkf(Param.GetNumResources() == 1, TEXT("Resource array binding is not implemented"));
			SRVs[Param.GetBaseIndex()] = Value;
		}
	}

	void Set(const FShaderUniformBufferParameter& Param, FRHIUniformBuffer* Value)
	{
		if (Param.IsBound())
		{
			UniformBuffers[Param.GetBaseIndex()] = Value;
		}
	}

	void Set(const FShaderResourceParameter& Param, FRHIUnorderedAccessView* Value)
	{
		if (Param.IsBound())
		{
			checkf(Param.GetNumResources() == 1, TEXT("Resource array binding is not implemented"));
			UAVs[Param.GetBaseIndex()] = Value;
		}
	}

	void Set(const FShaderResourceParameter& Param, FRHISamplerState* Value)
	{
		if (Param.IsBound())
		{
			checkf(Param.GetNumResources() == 1, TEXT("Resource array binding is not implemented"));
			Samplers[Param.GetBaseIndex()] = Value;
		}
	}

	void SetTexture(uint16 BaseIndex, FRHITexture* Value)
	{
		checkSlow(BaseIndex < UE_ARRAY_COUNT(Textures));
		Textures[BaseIndex] = Value;
	}

	void SetSRV(uint16 BaseIndex, FRHIShaderResourceView* Value)
	{
		checkSlow(BaseIndex < UE_ARRAY_COUNT(SRVs));
		SRVs[BaseIndex] = Value;
	}

	void SetSampler(uint16 BaseIndex, FRHISamplerState* Value)
	{
		checkSlow(BaseIndex < UE_ARRAY_COUNT(Samplers));
		Samplers[BaseIndex] = Value;
	}

	void SetUAV(uint16 BaseIndex, FRHIUnorderedAccessView* Value)
	{
		checkSlow(BaseIndex < UE_ARRAY_COUNT(UAVs));
		UAVs[BaseIndex] = Value;
	}

	void SetUniformBuffer(uint16 BaseIndex, FRHIUniformBuffer* Value)
	{
		checkSlow(BaseIndex < UE_ARRAY_COUNT(UniformBuffers));
		UniformBuffers[BaseIndex] = Value;
	}
};
#endif //RHI_RAYTRACING
