// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ShaderPermutation.h: All shader permutation's compile time API.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "ShaderCore.h"

// Enable this to log the parameters of each compiled permutation
inline constexpr bool bLogPermutations = false;

struct FShaderCompilerEnvironment;



// Flags that can specialize shader permutations compiled for specific platforms
enum class EShaderPermutationFlags : uint32
{
	None = 0u,
	HasEditorOnlyData = (1u << 0),
	IsODSCOnly = (1u << 1),
};
ENUM_CLASS_FLAGS(EShaderPermutationFlags);

struct FShaderPermutationParameters
{
	// Shader platform to compile to.
	const EShaderPlatform Platform;

	// Unique permutation identifier of the material shader type.
	const int32 PermutationId;

	// Flags that describe the permutation
	const EShaderPermutationFlags Flags;

	// Default to include editor-only shaders, to maintain backwards-compatibility
	explicit FShaderPermutationParameters(EShaderPlatform InPlatform, int32 InPermutationId = 0, EShaderPermutationFlags InFlags = EShaderPermutationFlags::HasEditorOnlyData)
		: Platform(InPlatform)
		, PermutationId(InPermutationId)
		, Flags(InFlags)
	{
	}
};

/** Defines at compile time a boolean permutation dimension. */
template <bool TIsSpecialization>
struct FShaderPermutationBool
{
	/** Setup the dimension's type in permutation domain as boolean. */
	using Type = bool;

	/** Setup the dimension's number of permutation. */
	static constexpr int32 PermutationCount = 2;

	/** Setup the dimension as non multi-dimensional, so that the ModifyCompilationEnvironement's
	 * define can conventily be set up in SHADER_PERMUTATION_BOOL.
	 */
	static constexpr bool IsMultiDimensional = false;

	/** Specialization constant data. */
	static constexpr bool IsSpecialization = TIsSpecialization;


	/** Converts dimension boolean value to dimension's value id. */
	static int32 ToDimensionValueId(Type E)
	{
		return E ? 1 : 0;
	}

	/** Pass down a boolean to FShaderCompilerEnvironment::SetDefine(). */
	static bool ToDefineValue(Type E)
	{
		return E;
	}

	/** Converts dimension's value id to dimension boolean value (exact reciprocal of ToDimensionValueId). */
	static Type FromDimensionValueId(int32 PermutationId)
	{
		checkf(PermutationId == 0 || PermutationId == 1, TEXT("Invalid shader permutation dimension id %i."), PermutationId);
		return PermutationId == 1;
	}

	/** Returns all the possible values of a specialization. */
	static void GetPossibleValues(TArray<int32>& OutValues)
	{
		checkf(TIsSpecialization, TEXT("Getting possible values should only be called for specializations!"));
		OutValues.Add(0);
		OutValues.Add(1);
	}
};


/** Defines at compile time a permutation dimension made of int32 from 0 to N -1. */
template <bool TIsSpecialization, typename TType, int32 TDimensionSize, int32 TFirstValue=0>
struct TShaderPermutationInt
{
	/** Setup the dimension's type in permutation domain as integer. */
	using Type = TType;

	/** Setup the dimension's number of permutation. */
	static constexpr int32 PermutationCount = TDimensionSize;
	
	/** Setup the dimension as non multi-dimensional, so that the ModifyCompilationEnvironement's
	 * define can conventily be set up in SHADER_PERMUTATION_INT.
	 */
	static constexpr bool IsMultiDimensional = false;
	
	/** Specialization constant data. */
	static constexpr bool IsSpecialization = TIsSpecialization;

	/** Min and max values. */
	static constexpr Type MinValue = static_cast<Type>(TFirstValue);
	static constexpr Type MaxValue = static_cast<Type>(TFirstValue + TDimensionSize - 1);


	/** Converts dimension's integer value to dimension's value id. */
	static int32 ToDimensionValueId(Type E)
	{
		const int32 PermutationId = static_cast<int32>(E) - TFirstValue;
		checkf(PermutationId < PermutationCount && PermutationId >= 0, TEXT("Unknown shader permutation dimension value id %i."), PermutationId);
		return PermutationId;
	}

	/** Pass down a int32 to FShaderCompilerEnvironment::SetDefine() even for contiguous enum classes. */
	static int32 ToDefineValue(Type E)
	{
		return ToDimensionValueId(E) + TFirstValue;
	}

	/** Converts dimension's value id to dimension's integer value (exact reciprocal of ToDimensionValueId). */
	static Type FromDimensionValueId(int32 PermutationId)
	{
		checkf(PermutationId < PermutationCount && PermutationId >= 0, TEXT("Invalid shader permutation dimension value id %i."), PermutationId);
		return static_cast<Type>(PermutationId + TFirstValue);
	}

	/** Returns all the possible values of a specialization. */
	static void GetPossibleValues(TArray<int32>& OutValues)
	{
		checkf(TIsSpecialization, TEXT("Getting possible values should only be called for specializations!"));
		for (int32 Value = MinValue; Value <= MaxValue; ++Value)
		{
			OutValues.Add(Value);
		}
	}
};


/** Defines at compile time a permutation dimension made of specific int32. */
template <bool TIsSpecialization, int32... Ts>
struct TShaderPermutationSparseInt
{
	/** Setup the dimension's type in permutation domain as integer. */
	using Type = int32;

	/** Setup the dimension's number of permutation. */
	static constexpr int32 PermutationCount = 0;
	
	/** Setup the dimension as non multi-dimensional, so that the ModifyCompilationEnvironement's
	 * define can conventily be set up in SHADER_PERMUTATION_SPARSE_INT.
	 */
	static constexpr bool IsMultiDimensional = false;

	/** Specialization constant data. */
	static constexpr bool IsSpecialization = TIsSpecialization;

	/** Converts dimension's integer value to dimension's value id, bu in this case fail because the dimension value was wrong. */
	static int32 ToDimensionValueId(Type E)
	{
		checkf(false, TEXT("Unknown shader permutation dimension value %i."), E);
		return int32(0);
	}

	/** Converts dimension's value id to dimension's integer value (exact reciprocal of ToDimensionValueId). */
	static Type FromDimensionValueId(int32 PermutationId)
	{
		checkf(false, TEXT("Invalid shader permutation dimension id %i."), PermutationId);
		return Type(0);
	}

	/** Returns all the possible values of a specialization. */
	static void GetPossibleValues(TArray<int32>& OutValues)
	{
	}
};

template <bool TIsSpecialization, int32 TUniqueValue, int32... Ts>
struct TShaderPermutationSparseInt<TIsSpecialization, TUniqueValue, Ts...>
{
	/** Setup the dimension's type in permutation domain as integer. */
	using Type = int32;

	/** Setup the dimension's number of permutation. */
	static constexpr int32 PermutationCount = TShaderPermutationSparseInt<TIsSpecialization, Ts...>::PermutationCount + 1;
	
	/** Setup the dimension as non multi-dimensional, so that the ModifyCompilationEnvironement's
	 * define can conventily be set up in SHADER_PERMUTATION_SPARSE_INT.
	 */
	static constexpr bool IsMultiDimensional = false;

	/** Specialization constant data. */
	static constexpr bool IsSpecialization = TIsSpecialization;

	/** Converts dimension's integer value to dimension's value id. */
	static int32 ToDimensionValueId(Type E)
	{
		if (E == TUniqueValue)
		{
			return PermutationCount - 1;
		}
		return TShaderPermutationSparseInt<TIsSpecialization, Ts...>::ToDimensionValueId(E);
	}

	/** Pass down a int32 to FShaderCompilerEnvironment::SetDefine(). */
	static int32 ToDefineValue(Type E)
	{
		return int32(E);
	}

	/** Converts dimension's value id to dimension's integer value (exact reciprocal of ToDimensionValueId). */
	static Type FromDimensionValueId(int32 PermutationId)
	{
		if (PermutationId == PermutationCount - 1)
		{
			return TUniqueValue;
		}
		return TShaderPermutationSparseInt<TIsSpecialization, Ts...>::FromDimensionValueId(PermutationId);
	}

	/** Returns all the possible values of a specialization. */
	static void GetPossibleValues(TArray<int32>& OutValues)
	{
		checkf(TIsSpecialization, TEXT("Getting possible values should only be called for specializations!"));
		OutValues.Add(TUniqueValue);
		return TShaderPermutationSparseInt<TIsSpecialization, Ts...>::GetPossibleValues(OutValues);
	}
};


/** Variadic template that defines an arbitrary multi-dimensional permutation domain, that can be instantiated to represent
 * a vector within the domain.
 *
 * // Defines a permutation domain with arbitrary number of dimensions. Dimensions can themselves be domains.
 * // It is totally legal to have a domain with no dimensions.
 * class FMyPermutationDomain = TShaderPermutationDomain<FMyDimensionA, FMyDimensionB, FMyDimensionC>;
 *
 * // ...
 *
 * // Create a permutation vector to be initialized. By default a vector is set at the origin of the domain.
 * // The origin of the domain is the ShaderPermutationId == 0.
 * FMyPermutationDomain PermutationVector;
 *
 * // Set the permutation vector's dimensions.
 * PermutationVector.Set<FMyDimensionA>(MyDimensionValueA);
 * PermutationVector.Set<FMyDimensionB>(MyDimensionValueB);
 * PermutationVector.Set<FMyDimensionC>(MyDimensionValueC);
 *
 * // Get the permutation id from the permutation vector for shader compiler.
 * int32 ShaderPermutationId = PermutationVector.ToDimensionValueId();
 *
 * // Reconstruct the permutation vector from shader permutation id.
 * FMyPermutationDomain PermutationVector2(ShaderPermutationId);
 *
 * // Get permutation vector's dimension.
 * if (PermutationVector2.Get<FMyDimensionA>())
 * { }
 */
template <typename... Ts>
struct TShaderPermutationDomain
{
	/** Setup the dimension's type in permutation domain as itself so that a permutation domain can be
	 * used as a dimension of another domain.
	 */
	using Type = TShaderPermutationDomain<Ts...>;

	/** Define a domain as a multidimensional dimension so that ModifyCompilationEnvironment() is getting used. */
	static constexpr bool IsMultiDimensional = true;

	/** Total number of permutation within the domain is one if no dimension at all. */
	static constexpr int32 PermutationCount = 1;

	/** Total number of specialization within the domain is one if no dimension at all. */
	static constexpr int32 SpecializationCount = 0;

	/** Constructors. */
	TShaderPermutationDomain<Ts...>() {}
	explicit TShaderPermutationDomain<Ts...>(int32 PermutationId)
	{
		checkf(PermutationId == 0, TEXT("Invalid shader permutation id %i."), PermutationId);
	}


	/** Set dimension's value, but in this case emit compile time error if could not find the dimension to set. */
	template<class DimensionToSet>
	void Set(typename DimensionToSet::Type)
	{
		// On clang, we can't do static_assert(false), because is evaluated even when method is not used. So
		// we test sizeof(DimensionToSet::Type) == 0 to make the static assert depend on the DimensionToSet
		// template parameter.
		static_assert(sizeof(typename DimensionToSet::Type) == 0, "Unknown shader permutation dimension.");
	}

	/** get dimension's value, but in this case emit compile time error if could not find the dimension to get. */
	template<class DimensionToGet>
	const typename DimensionToGet::Type Get() const
	{
		// On clang, we can't do static_assert(false), because is evaluated even when method is not used. So
		// we test sizeof(DimensionToSet::Type) == 0 to make the static assert depend on the DimensionToGet
		// template parameter.
		static_assert(sizeof(typename DimensionToGet::Type) == 0, "Unknown shader permutation dimension.");
		return DimensionToGet::Type();
	}


	/** Modify compilation environment. */
	void ModifyCompilationEnvironment(FShaderCompilerEnvironment& OutEnvironment) const {}

	/** Returns if the permutation is a specialization (requries a specialization constant array to be set on the shader). */
	bool IsSpecialization() const { return false; }

	/** Generate a specialization constant array. */
	void ToSpecializationConstantArray(FSpecializationContainerType& OutArray) const {}

	/** Converts domain permutation vector to domain's value id. */
	static int32 ToDimensionValueId(const Type& PermutationVector)
	{
		return 0;
	}

	int32 ToDimensionValueId() const
	{
		return ToDimensionValueId(*this);
	}

	static int32 ToDimensionUnspecializedId(const Type& PermutationVector)
	{
		return 0;
	}

	int32 ToDimensionUnspecializedId() const
	{
		return ToDimensionUnspecializedId(*this);
	}

	/** Returns the permutation domain from the unique ID. */
	static Type FromDimensionValueId(const int32 PermutationId)
	{
		return Type(PermutationId);
	}


	/** Test if equal. */
	bool operator==(const Type& Other) const
	{
		return true;
	}
};


// C++11 doesn't allow partial specialization of templates method or function. So we spetialise class that have
// non spetialised static method, but leave templated static function.
template<bool BooleanSpecialization>
class TShaderPermutationDomainSpetialization
{
public:

	template<typename TPermutationVector, typename TDimension>
	static void ModifyCompilationEnvironment(const TPermutationVector& PermutationVector, FShaderCompilerEnvironment& OutEnvironment)
	{
		if constexpr (bLogPermutations)
		{
			UE_LOGF(LogShaders, Verbose, "		%ls = %d", TDimension::DefineName, TDimension::ToDefineValue(PermutationVector.DimensionValue));
		}

		OutEnvironment.SetDefine(TDimension::DefineName, TDimension::ToDefineValue(PermutationVector.DimensionValue));

		if constexpr (TDimension::IsSpecialization)
		{
			const int32 SpecializationConstantIndex = OutEnvironment.SpecializationConstantValues.Num();
			OutEnvironment.SetDefine(TDimension::SpecializationIndexName, SpecializationConstantIndex);

			TArray<int32>& Values = OutEnvironment.SpecializationConstantValues.AddDefaulted_GetRef();
			Values.Reserve(TDimension::PermutationCount);
			TDimension::GetPossibleValues(Values);
		}

		return PermutationVector.Tail.ModifyCompilationEnvironment(OutEnvironment);
	}


	template<typename TPermutationVector, typename TDimension>
	static void ToSpecializationConstantArray(const TPermutationVector& PermutationVector, FSpecializationContainerType& OutArray)
	{
		if constexpr (TDimension::IsSpecialization)
		{
			OutArray.Add(TDimension::ToDefineValue(PermutationVector.DimensionValue));
		}

		return PermutationVector.Tail.ToSpecializationConstantArray(OutArray);
	}

	template<typename TPermutationVector, typename TDimensionToGet>
	static const typename TDimensionToGet::Type& GetDimension(const TPermutationVector& PermutationVector)
	{
		return PermutationVector.Tail.template Get<TDimensionToGet>();
	}

	template<typename TPermutationVector, typename TDimensionToSet>
	static void SetDimension(TPermutationVector& PermutationVector, const typename TDimensionToSet::Type& Value)
	{
		return PermutationVector.Tail.template Set<TDimensionToSet>(Value);
	}
};

template<>
class TShaderPermutationDomainSpetialization<true>
{
public:

	template<typename TPermutationVector, typename TDimension>
	static void ModifyCompilationEnvironment(const TPermutationVector& PermutationVector, FShaderCompilerEnvironment& OutEnvironment)
	{
		PermutationVector.DimensionValue.ModifyCompilationEnvironment(OutEnvironment);
		return PermutationVector.Tail.ModifyCompilationEnvironment(OutEnvironment);
	}

	template<typename TPermutationVector, typename TDimension>
	static void ToSpecializationConstantArray(const TPermutationVector& PermutationVector, FSpecializationContainerType& OutArray)
	{
		PermutationVector.DimensionValue.ToSpecializationConstantArray(OutArray);
		return PermutationVector.Tail.ToSpecializationConstantArray(OutArray);
	}

	template<typename TPermutationVector, typename TDimensionToGet>
	static const typename TDimensionToGet::Type& GetDimension(const TPermutationVector& PermutationVector)
	{
		return PermutationVector.DimensionValue;
	}

	template<typename TPermutationVector, typename TDimensionToSet>
	static void SetDimension(TPermutationVector& PermutationVector, const typename TDimensionToSet::Type& Value)
	{
		PermutationVector.DimensionValue = Value;
	}
};


template <typename TDimension, typename... Ts>
struct TShaderPermutationDomain<TDimension, Ts...>
{
	/** Setup the dimension's type in permutation domain as itself so that a permutation domain can be
	 * used as a dimension of another domain.
	 */
	using Type = TShaderPermutationDomain<TDimension, Ts...>;

	/** Define a domain as a multidimensional dimension so that ModifyCompilationEnvironment() is used. */
	static constexpr bool IsMultiDimensional = true;

	/** Parent type in the variadic template to reduce code. */
	using Super = TShaderPermutationDomain<Ts...>;

	/** Total number of permutation within the domain. */
	static constexpr int32 PermutationCount = Super::PermutationCount * TDimension::PermutationCount;

	/** Total number of specializations within the domain. */
	static constexpr int32 GetDimensionSpecCount()
	{
		if constexpr (TDimension::IsMultiDimensional)
		{
			return TDimension::SpecializationCount;
		}
		else
		{
			return TDimension::IsSpecialization ? 1 : 0;
		}
	}
	static constexpr int32 SpecializationCount = Super::SpecializationCount + GetDimensionSpecCount();

	/** Constructors. */
	TShaderPermutationDomain<TDimension, Ts...>()
		: DimensionValue(TDimension::FromDimensionValueId(0))
	{
	}

	explicit TShaderPermutationDomain<TDimension, Ts...>(int32 PermutationId)
		: DimensionValue(TDimension::FromDimensionValueId(PermutationId % TDimension::PermutationCount))
		, Tail(PermutationId / TDimension::PermutationCount)
	{
		checkf(PermutationId >= 0 && PermutationId < PermutationCount, TEXT("Invalid shader permutation id %i."), PermutationId);
	}


	/** Set dimension's value. */
	template<class DimensionToSet>
	void Set(typename DimensionToSet::Type Value)
	{
		return TShaderPermutationDomainSpetialization<std::is_same_v<TDimension, DimensionToSet>>::template SetDimension<Type, DimensionToSet>(*this, Value);
	}


	/** Get dimension's value. */
	template<class DimensionToGet>
	const typename DimensionToGet::Type& Get() const
	{
		return TShaderPermutationDomainSpetialization<std::is_same_v<TDimension, DimensionToGet>>::template GetDimension<Type, DimensionToGet>(*this);
	}


	/** Get the tail of the dimensions. */
	inline const typename Super::Type& GetTail() const
	{
		return Tail;
	}


	/** Modify the shader's compilation environment. */
	void ModifyCompilationEnvironment(FShaderCompilerEnvironment& OutEnvironment) const
	{
		TShaderPermutationDomainSpetialization<TDimension::IsMultiDimensional>::template ModifyCompilationEnvironment<Type, TDimension>(*this, OutEnvironment);
	}

	/** Returns true if any of the specialization values are in use (different than 0).
	* All shaders created are created with the default value of 0 for every specialization constant,
	* as soon as a value differs from 0 the permutation is considered to be a specialization.
	*/
	bool IsSpecialization() const
	{
		return ToDimensionValueId(*this) != ToDimensionUnspecializedId(*this);
	}

	/** Fill an array with all the specialization constant values. */
	void ToSpecializationConstantArray(FSpecializationContainerType& OutArray) const
	{
		TShaderPermutationDomainSpetialization<TDimension::IsMultiDimensional>::template ToSpecializationConstantArray<Type, TDimension>(*this, OutArray);
	}

	/** Converts domain permutation vector to domain's value id. */
	static int32 ToDimensionValueId(const Type& PermutationVector)
	{
		return PermutationVector.ToDimensionValueId();
	}

	int32 ToDimensionValueId() const
	{
		return TDimension::ToDimensionValueId(DimensionValue) + TDimension::PermutationCount * Tail.ToDimensionValueId();
	}

	/** Same as ToDimensionValueId but only collects values for Dimensions that are not specializations. 
	* Used to map a specialization to an actual shader permutation.
	*/
	static int32 ToDimensionUnspecializedId(const Type& PermutationVector)
	{
		return PermutationVector.ToDimensionUnspecializedId();
	}

	int32 ToDimensionUnspecializedId() const
	{
		if constexpr (TDimension::IsMultiDimensional)
		{
			return TDimension::ToDimensionUnspecializedId(DimensionValue) + TDimension::PermutationCount * Tail.ToDimensionUnspecializedId();
		}
		else
		{
			if constexpr (TDimension::IsSpecialization)
			{
				return TDimension::PermutationCount * Tail.ToDimensionUnspecializedId();
			}
			else
			{
				return TDimension::ToDimensionValueId(DimensionValue) + TDimension::PermutationCount * Tail.ToDimensionUnspecializedId();
			}
		}
	}

	/** Returns the permutation domain from the unique ID. */
	static Type FromDimensionValueId(const int32 PermutationId)
	{
		return Type(PermutationId);
	}


	/** Test if equal. */
	bool operator==(const Type& Other) const
	{
		return DimensionValue == Other.DimensionValue && Tail == Other.Tail;
	}

	/** Test if not equal. */
	bool operator!=(const Type& Other) const
	{
		return !(*this == Other);
	}

private:
	template<bool BooleanSpecialization>
	friend class TShaderPermutationDomainSpetialization;

	typename TDimension::Type DimensionValue;
	Super Tail;
};


/** Global shader permutation domain with no dimension. */
using FShaderPermutationNone = TShaderPermutationDomain<>;


// Internal implementation of non multi-dimensional shader permutation dimension.
#define DECLARE_SHADER_PERMUTATION_IMPL(IsSpecialization,InDefineName,PermutationMetaType,...) \
	public PermutationMetaType<IsSpecialization,__VA_ARGS__> { \
	public: \
		static constexpr const TCHAR* DefineName = IsSpecialization ? TEXT(InDefineName "_DEFAULT") : TEXT(InDefineName); \
		static constexpr const TCHAR* SpecializationIndexName = IsSpecialization ? TEXT(InDefineName "_INDEX") : nullptr; \
	}



// Make things clearer in macro below
#define SHADER_SPECIALIZATION	true
#define SHADER_PERMUTATION		false


/** Implements a boolean shader permutation dimensions. Meant to be used like so:
 *
 * class FMyShaderDim : SHADER_PERMUTATION_BOOL("MY_SHADER_DEFINE_NAME");
 */
#define SHADER_PERMUTATION_BOOL(InDefineName) \
	public FShaderPermutationBool<SHADER_PERMUTATION> { \
	public: \
		static constexpr const TCHAR* DefineName = TEXT(InDefineName); \
		static constexpr const TCHAR* SpecializationIndexName = nullptr; \
	}

#define SHADER_SPECIALIZATION_BOOL(InDefineName) \
	public FShaderPermutationBool<SHADER_SPECIALIZATION> { \
	public: \
		static constexpr const TCHAR* DefineName = TEXT(InDefineName "_DEFAULT"); \
		static constexpr const TCHAR* SpecializationIndexName = TEXT(InDefineName "_INDEX"); \
	}


/** Implements an integer shader permutation dimensions with N permutation values from [[0; N[[. Meant to be used like so:
 *
 * class FMyShaderDim : SHADER_PERMUTATION_INT("MY_SHADER_DEFINE_NAME", N);
 */
#define SHADER_PERMUTATION_INT(InDefineName, Count) \
	DECLARE_SHADER_PERMUTATION_IMPL(SHADER_PERMUTATION, InDefineName, TShaderPermutationInt, int32, Count)

#define SHADER_SPECIALIZATION_INT(InDefineName, Count) \
	DECLARE_SHADER_PERMUTATION_IMPL(SHADER_SPECIALIZATION, InDefineName, TShaderPermutationInt, int32, Count)

/** Implements an integer shader permutation dimensions with N permutation values from [[X; X+N[[. Meant to be used like so:
 *
 * class FMyShaderDim : SHADER_PERMUTATION_RANGE_INT("MY_SHADER_DEFINE_NAME", X, N);
 */
#define SHADER_PERMUTATION_RANGE_INT(InDefineName, Start, Count) \
	DECLARE_SHADER_PERMUTATION_IMPL(SHADER_PERMUTATION, InDefineName, TShaderPermutationInt, int32, Count, Start)

#define SHADER_SPECIALIZATION_RANGE_INT(InDefineName, Start, Count) \
	DECLARE_SHADER_PERMUTATION_IMPL(SHADER_SPECIALIZATION, InDefineName, TShaderPermutationInt, int32, Count, Start)

/** Implements an integer shader permutation dimensions with non contiguous permutation values. Meant to be used like so:
 *
 * class FMyShaderDim : SHADER_PERMUTATION_SPARSE_INT("MY_SHADER_DEFINE_NAME", 1, 2, 4, 8);
 */
#define SHADER_PERMUTATION_SPARSE_INT(InDefineName,...) \
	DECLARE_SHADER_PERMUTATION_IMPL(SHADER_PERMUTATION, InDefineName, TShaderPermutationSparseInt, __VA_ARGS__)

#define SHADER_SPECIALIZATION_SPARSE_INT(InDefineName,...) \
	DECLARE_SHADER_PERMUTATION_IMPL(SHADER_SPECIALIZATION, InDefineName, TShaderPermutationSparseInt, __VA_ARGS__)

/** Implements an shader permutation dimensions with an enum class assumed to have contiguous integer values. Meant to be used like so:
 *
 * enum class EMyEnum
 * {
 *		Hello,
 *		World,
 *		// [...]
 *		MAX
 * };
 *
 * class FMyShaderDim : SHADER_PERMUTATION_ENUM_CLASS("MY_SHADER_DEFINE_NAME", EMyEnum);
 */
#define SHADER_PERMUTATION_ENUM_CLASS(InDefineName, EnumName) \
	DECLARE_SHADER_PERMUTATION_IMPL(SHADER_PERMUTATION, InDefineName, TShaderPermutationInt, EnumName, static_cast<int32>(EnumName::MAX))

#define SHADER_SPECIALIZATION_ENUM_CLASS(InDefineName, EnumName) \
	DECLARE_SHADER_PERMUTATION_IMPL(SHADER_SPECIALIZATION, InDefineName, TShaderPermutationInt, EnumName, static_cast<int32>(EnumName::MAX))




