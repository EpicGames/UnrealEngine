// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataDrivenShaderPlatformInfo.h"
#include "RHIGlobals.h"
#include "ShaderPermutation.h"

namespace UE::ShaderPermutationUtils
{
	inline bool ShouldCompileWithWaveSize(const FShaderPermutationParameters& Parameters, int32 WaveSize)
	{
		if (WaveSize)
		{
			if (!RHISupportsWaveOperations(Parameters.Platform))
			{
				return false;
			}

			if (WaveSize < int32(FDataDrivenShaderPlatformInfo::GetMinimumWaveSize(Parameters.Platform)) ||
				WaveSize > int32(FDataDrivenShaderPlatformInfo::GetMaximumWaveSize(Parameters.Platform)))
			{
				return false;
			}
		}

		return true;
	}

	inline bool ShouldPrecacheWithWaveSize(const FShaderPermutationParameters& Parameters, int32 WaveSize)
	{
		if (WaveSize)
		{
			if (WaveSize < GRHIGlobals.MinimumWaveSize || WaveSize > GRHIGlobals.MaximumWaveSize)
			{
				return false;
			}
		}

		return true;
	}

	template <typename TDimension>
	void FormatPermutationParameter(const typename TDimension::Type& E, FString& OutString, bool bFullNames, const TCHAR* Prefix);
	
	inline void FormatPermutationDomain(const TShaderPermutationDomain<>& InShaderPermutationDomain, FString& OutString, bool bFullNames, const TCHAR* Prefix)
	{
		// Dummy
	}

	template <typename TDimension, typename... Ts>
	void FormatPermutationDomain(const TShaderPermutationDomain<TDimension, Ts...>& InShaderPermutationDomain, FString& OutString, bool bFullNames, const TCHAR* Prefix)
	{
		const int32 Value = TDimension::ToDimensionValueId(InShaderPermutationDomain.template Get<TDimension>());
		if (bFullNames)
		{
			if (!OutString.IsEmpty())
			{
				OutString.AppendChar(TEXT('\n'));
			}

			if (Prefix != nullptr)
			{
				OutString.Append(Prefix);
			}

			if (TDimension::IsMultiDimensional)
			{
				OutString.Appendf(TEXT("TShaderPermutationDomain[%d]"), TDimension::PermutationCount);
			}

			if (Prefix != nullptr)
			{
				const FString ExtendedPrefix = FString::Printf(TEXT("%s -> "), Prefix);
				FormatPermutationParameter<TDimension>(InShaderPermutationDomain.template Get<TDimension>(), OutString, bFullNames, *ExtendedPrefix);
			}
			else
			{
				FormatPermutationParameter<TDimension>(InShaderPermutationDomain.template Get<TDimension>(), OutString, bFullNames, TEXT(" -> "));
			}
		}
		else
		{
			if (!OutString.IsEmpty())
			{
				OutString.AppendChar(TEXT(','));
			}
			FormatPermutationParameter<TDimension>(InShaderPermutationDomain.template Get<TDimension>(), OutString, bFullNames, TEXT(" -> "));
		}

		FormatPermutationDomain(InShaderPermutationDomain.GetTail(), OutString, bFullNames, Prefix);
	}

	template <typename TDimension>
	void FormatPermutationParameter(const typename TDimension::Type& E, FString& OutString, bool bFullNames, const TCHAR* Prefix)
	{
		if constexpr (TDimension::IsMultiDimensional)
		{
			FormatPermutationDomain(E, OutString, bFullNames, Prefix);
		}
		else if constexpr (std::is_base_of_v<FShaderPermutationBool<SHADER_SPECIALIZATION>, TDimension>)
		{
			if (bFullNames)
			{
				OutString.Appendf(TEXT("%s (%s) *spec*"), TDimension::DefineName, !FShaderPermutationBool<SHADER_SPECIALIZATION>::ToDefineValue(E) ? TEXT("false") : TEXT("true"));
			}
			else
			{
				OutString.Appendf(TEXT("%d"), static_cast<int32>(FShaderPermutationBool<SHADER_SPECIALIZATION>::ToDefineValue(E)));
			}
		}
		else if constexpr (std::is_base_of_v<FShaderPermutationBool<SHADER_PERMUTATION>, TDimension>)
		{
			if (bFullNames)
			{
				OutString.Appendf(TEXT("%s (%s)"), TDimension::DefineName, !FShaderPermutationBool<SHADER_PERMUTATION>::ToDefineValue(E) ? TEXT("false") : TEXT("true"));
			}
			else
			{
				OutString.Appendf(TEXT("%d"), static_cast<int32>(FShaderPermutationBool<SHADER_PERMUTATION>::ToDefineValue(E)));
			}
		}
		else
		{
			if (bFullNames)
			{
				OutString.Appendf(TEXT("%s (%d)%s"), TDimension::DefineName, static_cast<int32>(TDimension::ToDefineValue(E)), TDimension::IsSpecialization ? TEXT(" *spec*") : TEXT(""));
			}
			else
			{
				OutString.Appendf(TEXT("%d"), static_cast<int32>(TDimension::ToDefineValue(E)));
			}
		}
	}

	template <typename DimensionType>
	bool DoesDimensionContainValue(const typename DimensionType::Type& Value)
	{
		for (int32 PermutationId = 0; PermutationId < DimensionType::PermutationCount; ++PermutationId)
		{
			if (Value == DimensionType::FromDimensionValueId(PermutationId))
			{
				return true;
			}
		}
		return false;
	}

	template <typename DomainType>
	int32 ConvertToUnspecializedId(const EShaderPlatform Platform, const DomainType& PermutationVector)
	{
		if (FDataDrivenShaderPlatformInfo::GetSupportsSpecializationConstants(Platform))
		{
			return DomainType::ToDimensionUnspecializedId(PermutationVector);
		}
		else
		{
			return DomainType::ToDimensionValueId(PermutationVector);
		}
	}
}
