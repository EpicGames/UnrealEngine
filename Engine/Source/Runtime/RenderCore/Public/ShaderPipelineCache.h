// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * Shader Pipeline Precompilation Cache
 * Precompilation half of the shader pipeline cache, which builds on the runtime RHI pipeline cache.
 */
 
#pragma once

#include "CoreMinimal.h"
#include "Containers/List.h"
#include "Stats/Stats.h"
#include "RHI.h"
#include "TickableObjectRenderThread.h"
#include "PipelineFileCache.h"
#include "Delegates/DelegateCombinations.h"
#include "ShaderStableKeyDebugInfoReader.h"

class FBlendStateInitializerRHI;
class FRHIBlendState;
class FRHICommandListImmediate;
class FRHIDepthStencilState;
class FRHIRasterizerState;
class FShaderPipelineCacheArchive;
struct FDepthStencilStateInitializerRHI;
struct FRasterizerStateInitializerRHI;

/**
 * FShaderPipelineCache:
 * The FShaderPipelineCache provides the new Pipeline State Object (PSO) logging, serialisation & precompilation mechanism that replaces FShaderCache.
 * Caching Pipeline State Objects and serialising the initialisers to disk allows for precompilation of these states the next time the game is run, which reduces hitching.
 * To achieve this FShaderPipelineCache relies upon FShaderCodeLibrary & "Share Material Shader Code" and the RHI-level backend FPipelineFileCacheManager.
 *
 * Basic Runtime Usage:
 * - Enable the cache with r.ShaderPipelineCache.Enabled = 1, which allows the pipeline cache to load existing data from disk and precompile it.
 * - Set the default batch size with r.ShaderPipelineCache.BatchSize = X, where X is the maximum number of PSOs to compile in a single batch when precompiling in the default Fast BatchMode.
 * - Set the background batch size with r.ShaderPipelineCache.BackgroundBatchSize = X, where X is the maximum number of PSOs to compile when in the Background BatchMode.
 * - Instrument the game code to call FShaderPipelineCache::SetBatchMode to switch the batch mode between Fast & Background modes.
 * - BatchMode::Fast should be used when a loading screen or movie is being displayed to allow more PSOs to be compiled whereas Background should be used behind interactive menus.
 * - If required call NumPrecompilesRemaining to determine the total number of outstanding PSOs to compile and keep the loading screen or movie visible until complete.
 * - Depending on the game & target platform performance it may also be required to call PauseBatching to suspend precompilation during gameplay and then ResumeBatching when behind a loading screen, menu or movie to restart precompilation.
 *
 * Other Runtime Options:
 * - In the GGameIni (and thus also GGameUserSettingsIni) the Shader Pipeline Cache uses the [ShaderPipelineCache.CacheFile] section to store some settings.
 * - The LastOpened setting stores the name of the last opened cache file as specified with Open, which if present will be used within FShaderPipelineCache::Initialize to open the existing cache.
 *   If not specified this will default to the AppName.
 * - The SortMode settings stores the desired sort mode for the PSOs, which is one of:
 *		+ Default: Loaded in the order specified in the file.
 *		+ FirstToLatestUsed: Start with the PSOs with the lowest first-frame used and work toward those with the highest.
 *		+ MostToLeastUsed: Start with the most often used PSOs working toward the least.
 *   Will use "Default" within FShaderPipelineCache::Initialize & OpenPipelineFileCache if nothing is specified.
 * - The GameVersionKey is a read-only integer specified in the GGameIni that specifies the game content version to disambiguate incompatible versions of the game content. By default this is taken from the FEngineVersion changlist.
 *
 * Logging Usage:
 * - Enable the cache with r.ShaderPipelineCache.Enabled = 1 and also turn on runtime logging with r.ShaderPipelineCache.LogPSO = 1.
 * - Ensure that you have configured the game to open the appropriate cache on startup (see above) and allow the game to play.
 * - PSOs are logged as they are encountered as the Unreal Engine does not provide facility to cook them offline, so this system will only collect PSOs which are actually used to render.
 * - As such you must either manually play through the game to collect logs or automate the process which is beyond the scope of this code.
 * - The data can be saved at any time by calling FShaderPipelineCache::SavePipelineFileCache and this can happen automatically after a given number of PSOs by setting r.ShaderPipelineCache.SaveAfterPSOsLogged = X where X is the desired number of PSOs to log before saving (0 will disable auto-save).
 * - Log files are shader platform specific to reduce overheads.
 *
 * Notes:
 * - The open cache file can be changed by closing the existing file with ClosePipelineFileCache (which implicitly Fast saves) and then opening a new one with OpenPipelineFileCache.
 * - Different files can be used to minimise PSO compilation by having a file per-scalability bucket (i.e. one file for Low, one for Med, one for High).
 * - When logging if you switch files only new entries from after the switch will be logged, which means you will miss any PSOs that should have been logged prior to switching. This prevents polluting the cache with unexpected entries.
 * - The UnrealEd.MergeShaderPipelineCaches command-let can be used to merge cache files with the same file-version, shader platform and game-version.
 *
 * File Locations & Packaging:
 * - The writable cache file is always stored in the User Saved directory.
 * - The game can also provide an immutable copy within its Game Content directory which will be used as the initial or seed data.
 * - Having generated logs in development and merged them with UnrealEd.MergeShaderPipelineCaches they should be packaged inside the Gane Content directory for the relevant platform.
 *
 * Requirements:
 * - FShaderCodeLibrary must be enabled via Project Settings > Packaging > "Share Material Shader Code".
 * - Enabling "Native Shader Libraries" is optional, but strongly preferred for Metal.
 */
class FShaderPipelineCache : public FTickableObjectRenderThread
{
	// the FShaderPipelineCacheTask class represents a pipeline precompile of a single PSO file cache.
	friend class FShaderPipelineCacheTask;
public:
	/**
	 * Initializes the shader pipeline cache for the desired platform, called by the engine.
	 * The shader platform is used only to load the initial pipeline cache and can be changed by closing & reopening the cache if necessary.
	 */
	static RENDERCORE_API void Initialize(EShaderPlatform Platform);
	/** Terminates the shader pipeline cache, called by the engine. */
	static RENDERCORE_API void Shutdown();

	/** Pauses precompilation. */
	static RENDERCORE_API void PauseBatching();
	
	enum class BatchMode
	{
		Background, // The maximum batch size is defined by r.ShaderPipelineCache.BackgroundBatchSize
		Fast, // The maximum batch size is defined by r.ShaderPipelineCache.BatchSize
		Precompile // The maximum batch size is defined by r.ShaderPipelineCache.PrecompileBatchSize
	};
	
	/** Sets the precompilation batching mode. */
	static RENDERCORE_API void SetBatchMode(BatchMode Mode);
	
	/** Resumes precompilation batching. */
	static RENDERCORE_API void ResumeBatching();
	
	/** true if pipeline cache(s) are precompiling. */
	static RENDERCORE_API bool IsPrecompiling();

	/** Returns the number of pipelines waiting for precompilation of the current PSOFC task. 
		if there are multiple PSOFCs pending this will return a minimum value of 1. It may increase as subsequent caches are processed.*/
	static RENDERCORE_API uint32 NumPrecompilesRemaining();
    
	/** Opens the shader pipeline cache file with either the LastOpened setting if available, or the project name otherwise */
	static RENDERCORE_API bool OpenPipelineFileCache(EShaderPlatform Platform);

 	/** Opens the user pipeline cache file with either the LastOpened setting if available, or the project name otherwise */
 	static RENDERCORE_API bool OpenUserPipelineFileCache(EShaderPlatform Platform);
	
	/** Saves the current shader pipeline cache to disk using one of the defined save modes, Fast uses an incremental approach whereas Slow will consolidate all data into the file. */
	static RENDERCORE_API bool SavePipelineFileCache(FPipelineFileCacheManager::SaveMode Mode);
	
	/** Closes the existing pipeline cache, allowing it to be reopened with a different file and/or shader platform. Will implicitly invoke a Fast Save. */
	static RENDERCORE_API void CloseUserPipelineFileCache();

	static RENDERCORE_API int32 GetGameVersionForPSOFileCache();
	
	/**
	 * Set the current PSO Game Usage Mask and comparsion function .  Returns true if this mask is different from the old mask or false if not or the cache system is disabled or if the mask feature is disabled
	 * Any new PSO's found will be logged with this value or existing PSO should have their masks updated.  See FPipelineFileCacheManager for more details.
	 */
	static RENDERCORE_API bool SetGameUsageMaskWithComparison(uint64 Mask, FPSOMaskComparisonFn InComparisonFnPtr);

    static RENDERCORE_API bool IsBatchingPaused();
    
	RENDERCORE_API FShaderPipelineCache(EShaderPlatform Platform);
	RENDERCORE_API virtual ~FShaderPipelineCache();

	RENDERCORE_API bool IsTickable() const;

	RENDERCORE_API void Tick( FRHICommandListImmediate& RHICmdList, float DeltaTime );

	UE_DEPRECATED(5.7, "Tick needs a command list")
	RENDERCORE_API void Tick( float DeltaTime );

	RENDERCORE_API bool NeedsRenderingResumedForRenderingThreadTick() const;
	
	RENDERCORE_API TStatId GetStatId() const;
	
	enum ELibraryState
	{
		Opened,
		OpenedComponent,
		Closed
	};
	
    /** Called by FShaderCodeLibrary to notify us that the shader code library state changed and shader availability will need to be re-evaluated */
    static RENDERCORE_API void ShaderLibraryStateChanged(ELibraryState State, EShaderPlatform Platform, FString const& Name, int32 ComponentID);

	/** Allows handlers of FShaderCacheOpenedDelegate to send state to begin and complete pre-compilation delegates */
	class FShaderCachePrecompileContext
	{
		bool bSlowPrecompileTask = false;
		FString CacheName;
	public:
		FShaderCachePrecompileContext() = default;
		FShaderCachePrecompileContext(const FString& CacheNameIn) : CacheName(CacheNameIn) {}

		const FString& GetCacheName() const { return CacheName; }
		void SetPrecompilationIsSlowTask() { bSlowPrecompileTask = true; }
		bool IsPrecompilationSlowTask() const { return bSlowPrecompileTask; }
	};

	/**
	 * Delegate signature for being notified when we are about to open the pipeline cache
	 */
	DECLARE_MULTICAST_DELEGATE_ThreeParams(FShaderCachePreOpenDelegate, FString const& /* Name */, EShaderPlatform /* Platform*/, bool& /* Ready */);
	
	/**
	 * Gets the event delegate to register to to be notified when we are about to open the pipeline cache.
	 */
	static FShaderCachePreOpenDelegate& GetCachePreOpenDelegate() { return OnCachePreOpen; }
	
    /**
     * Delegate signature for being notified when a pipeline cache is opened
     */
    DECLARE_MULTICAST_DELEGATE_FiveParams(FShaderCacheOpenedDelegate, FString const& /* Name */, EShaderPlatform /* Platform*/, uint32 /* Count */, const FGuid& /* CacheFileGuid */, FShaderCachePrecompileContext& /*ShaderCachePrecompileContext*/);
    
    /**
     * Gets the event delegate to register to to be notified when a pipeline cache is opened.
     */
    static FShaderCacheOpenedDelegate& GetCacheOpenedDelegate() { return OnCachedOpened; }
    
    /**
     * Delegate signature for being notified when a pipeline cache is closed
     */
    DECLARE_MULTICAST_DELEGATE_TwoParams(FShaderCacheClosedDelegate, FString const& /* Name */, EShaderPlatform /* Platform*/ );
    
    /**
     * Gets the event delegate to register to to be notified when a pipeline cache is closed.
     */
    static FShaderCacheClosedDelegate& GetCacheClosedDelegate() { return OnCachedClosed; }

	/**
	* Delegate signature for being notified when currently viable PSOs have started to be precompiled from the cache, and how many will be precompiled
	*/
	DECLARE_MULTICAST_DELEGATE_TwoParams(FShaderPrecompilationBeginDelegate, uint32 /** Count */, const FShaderCachePrecompileContext& /*ShaderCachePrecompileContext*/);

	/**
     * Delegate signature for being notified that all currently viable PSOs have been precompiled from the cache, how many that was and how long spent precompiling (in sec.)
     */
    DECLARE_MULTICAST_DELEGATE_ThreeParams(FShaderPrecompilationCompleteDelegate, uint32 /** Count */, double /** Seconds */, const FShaderCachePrecompileContext& /*ShaderCachePrecompileContext*/);

	/**
	* Gets the event delegate to register to to be notified when currently viable PSOs have started to be precompiled from the cache.
	*/
	static FShaderPrecompilationBeginDelegate& GetPrecompilationBeginDelegate() { return OnPrecompilationBegin; }

	/**
     * Gets the event delegate to register to to be notified when all currently viable PSOs have been precompiled from the cache.
     */
    static FShaderPrecompilationCompleteDelegate& GetPrecompilationCompleteDelegate() { return OnPrecompilationComplete; }

private:
	static inline FShaderPipelineCache* Get() { return ShaderPipelineCache;	}

	RENDERCORE_API bool OpenPipelineFileCacheInternal(bool bUserCache, const FString& PSOCacheKey, const FString& CacheName, EShaderPlatform Platform);

	RENDERCORE_API void BeginNextPrecompileCacheTask();
	RENDERCORE_API void BeginNextPrecompileCacheTaskInternal();
	RENDERCORE_API bool ReadyForAutoSave() const;

	RENDERCORE_API void ClearPendingPrecompileTaskQueue();
	RENDERCORE_API void EnqueuePendingPrecompileTask(const FString& PSOCacheKey);
	RENDERCORE_API int ApplyNewUsageMaskToAllTasks(uint64 Mask);

	RENDERCORE_API FRHIBlendState* GetOrCreateBlendState(const FBlendStateInitializerRHI& Initializer);
	RENDERCORE_API FRHIRasterizerState* GetOrCreateRasterizerState(const FRasterizerStateInitializerRHI& Initializer);
	RENDERCORE_API FRHIDepthStencilState* GetOrCreateDepthStencilState(const FDepthStencilStateInitializerRHI& Initializer);
	
private:
	static RENDERCORE_API FShaderPipelineCache* ShaderPipelineCache;
	static RENDERCORE_API uint32 BatchSize;
	static RENDERCORE_API float BatchTime;
	bool bPaused;
    int32 PausedCount;
	double LastAutoSaveTime = 0.0;
	double LastAutoSaveTimeLogBoundPSO = 0.0;
	int32 LastAutoSaveNum = 0;
	TSet<uint32> CompiledHashes;

	/** Development tool that can describe PSOs in non-Shipping builds. */
	UE::ShaderUtils::FShaderStableKeyDebugInfoReader ShaderHashToStableKey;

	static RENDERCORE_API FString UserCacheTaskKey;

	struct FPipelineCachePSOContext
	{
		FString FileName;
		EShaderPlatform CurrentPlatform;

		FShaderCachePrecompileContext PrecompileContext;
	};
	
	TMap<FString, TUniquePtr<class FShaderPipelineCacheTask>> ShaderCacheTasks;

	class FShaderPipelineCacheTask* GetTask(const FString& Name) {
		TUniquePtr<class FShaderPipelineCacheTask>* CacheTask = FShaderPipelineCache::Get()->ShaderCacheTasks.Find(Name);
		return CacheTask ? CacheTask->Get() : nullptr;
	}

	// The list of pending PSO cache names to precompile.
	TArray<FString> PendingPrecompilePSOCacheKeys;

	// The name of the current precompiling PSO cache.
	FString CurrentPrecompilingPSOFCKey;

	// Used for threadsafe query of pending pipeline compiles.
	std::atomic_bool bHasShaderPipelineTask = false;

	mutable FCriticalSection Mutex;
	FGraphEventRef NextPrecompileCacheTask;

	static RENDERCORE_API FShaderCachePreOpenDelegate OnCachePreOpen;
    static RENDERCORE_API FShaderCacheOpenedDelegate OnCachedOpened;
    static RENDERCORE_API FShaderCacheClosedDelegate OnCachedClosed;
	static RENDERCORE_API FShaderPrecompilationBeginDelegate OnPrecompilationBegin;
	static RENDERCORE_API FShaderPrecompilationCompleteDelegate OnPrecompilationComplete;

	TMap<FBlendStateInitializerRHI, FRHIBlendState*> BlendStateCache;
	TMap<FRasterizerStateInitializerRHI, FRHIRasterizerState*> RasterizerStateCache;
	TMap<FDepthStencilStateInitializerRHI, FRHIDepthStencilState*> DepthStencilStateCache;
};
