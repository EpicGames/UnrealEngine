// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HAL/IConsoleManager.h"
#include "RHIShaderPlatform.h"
#include "RHIStrings.h"

RENDERCORE_API EShaderPlatform GetEditorShaderPlatform(EShaderPlatform ShaderPlatform);

template<typename Type>
struct FShaderPlatformCachedIniValue
{
	FShaderPlatformCachedIniValue(const TCHAR* InCVarName)
		: CVarName(InCVarName)
		, CVar(nullptr)
	{
	}

	FShaderPlatformCachedIniValue(IConsoleVariable* InCVar)
		: CVar(InCVar)
	{
	}

	Type Get(EShaderPlatform ShaderPlatform)
	{
		Type Value{};

		const EShaderPlatform ActualShaderPlatform = GetEditorShaderPlatform(ShaderPlatform);

		FName IniPlatformName = ShaderPlatformToPlatformName(ActualShaderPlatform);
		// find the cvar if needed
		if (CVar == nullptr)
		{
			CVar = IConsoleManager::Get().FindConsoleVariable(*CVarName);
		}

		// if we are looking up our own platform, just use the current value, however
		// ShaderPlatformToPlatformName can return the wrong platform than expected - for instance, Linux Vulkan will return Windows
		// so instead of hitting an asser below, we detect that the request SP is the current SP, and use the CVar value that is set currently
		if (IniPlatformName == FPlatformProperties::IniPlatformName() || ActualShaderPlatform == GMaxRHIShaderPlatform)
		{
			checkf(CVar != nullptr, TEXT("Failed to find CVar %s when getting current value for FShaderPlatformCachedIniValue"), *CVarName);

			CVar->GetValue(Value);
			return Value;
		}

#if ALLOW_OTHER_PLATFORM_CONFIG
		// create a dummy cvar if needed
		if (CVar == nullptr)
		{
			// this could be a cvar that only exists on the target platform so create a dummy one
			CVar = IConsoleManager::Get().RegisterConsoleVariable(*CVarName, Type(), TEXT(""), ECVF_ReadOnly);
		}

		// now get the value from the platform that makes sense for this shader platform
		TSharedPtr<IConsoleVariable> OtherPlatformVar = CVar->GetPlatformValueVariable(IniPlatformName);
		ensureMsgf(OtherPlatformVar.IsValid(), TEXT("Failed to get another platform's version of a cvar (possible name: '%s'). It is probably an esoteric subclass that needs to implement GetPlatformValueVariable."), *CVarName);
		if (OtherPlatformVar.IsValid())
		{
			OtherPlatformVar->GetValue(Value);
		}
		else
		{
			// get this platform's value, even tho it could be wrong
			CVar->GetValue(Value);
		}
#else
		checkf(IniPlatformName == FName(FPlatformProperties::IniPlatformName()), TEXT("FShaderPlatformCachedIniValue can only look up the current platform when ALLOW_OTHER_PLATFORM_CONFIG is false"));
#endif
		return Value;
	}

private:
	FString CVarName;
	IConsoleVariable* CVar;
};

/**
 * Caches a CVar value per target platform.
 * Use this instead of FShaderPlatformCachedIniValue when a setting must be identical across all shader platforms that share a target platform.
 * For example, a feature that affects both shader compilation AND mesh or texture format.
 * Under the hood this is almost the same as FShaderPlatformCachedIniValue but exposes the extra Get() by platform name.
 *
 * Usage from shader code:
 *   static FTargetPlatformCachedIniValue<bool> MyFeature(TEXT("r.MyFeature"));
 *   bool bEnabled = MyFeature.Get(ShaderPlatform);
 *
 * Usage from mesh cook code:
 *   bool bEnabled = MyFeature.Get(FName(TargetPlatform->IniPlatformName()));
 */
template<typename Type>
struct FTargetPlatformCachedIniValue
{
	FTargetPlatformCachedIniValue(TCHAR const* InCVarName)
		: CVarName(InCVarName)
		, CVar(nullptr)
	{
	}

	FTargetPlatformCachedIniValue(IConsoleVariable* InCVar)
		: CVar(InCVar)
	{
	}

	/** Look up by shader platform. Shader platforms on the same target should return the same value. */
	Type Get(EShaderPlatform ShaderPlatform)
	{
		const EShaderPlatform EditorShaderPlatform = GetEditorShaderPlatform(ShaderPlatform);
		const FName IniPlatformName = ShaderPlatformToPlatformName(EditorShaderPlatform);
		// ShaderPlatformToPlatformName can return the wrong name for GMaxRHIShaderPlatform (e.g. Linux Vulkan → Windows), so treat that SP as current.
		const bool bIsCurrentPlatform = IniPlatformName == FName(FPlatformProperties::IniPlatformName()) || EditorShaderPlatform == GMaxRHIShaderPlatform;
		return GetByIniPlatformName(IniPlatformName, bIsCurrentPlatform);
	}

	/** Look up by IniPlatformName directly. Can use FName(TargetPlatform->IniPlatformName()). */
	Type Get(FName IniPlatformName)
	{
		const bool bIsCurrentPlatform = IniPlatformName == FName(FPlatformProperties::IniPlatformName());
		return GetByIniPlatformName(IniPlatformName, bIsCurrentPlatform);
	}

private:
	Type GetByIniPlatformName(FName IniPlatformName, bool bIsCurrentPlatform)
	{
		Type Value{};

		if (CVar == nullptr)
		{
			CVar = IConsoleManager::Get().FindConsoleVariable(*CVarName);
		}

		if (bIsCurrentPlatform)
		{
			checkf(CVar != nullptr, TEXT("Failed to find CVar %s when getting current value for FTargetPlatformCachedIniValue"), *CVarName);
			CVar->GetValue(Value);
			return Value;
		}

#if ALLOW_OTHER_PLATFORM_CONFIG
		if (CVar == nullptr)
		{
			// CVar may only exist on the target platform. Create a dummy to hang the platform value on
			CVar = IConsoleManager::Get().RegisterConsoleVariable(*CVarName, Type(), TEXT(""), ECVF_ReadOnly);
		}

		TSharedPtr<IConsoleVariable> OtherPlatformVar = CVar->GetPlatformValueVariable(IniPlatformName);
		ensureMsgf(OtherPlatformVar.IsValid(), TEXT("Failed to get another platform's version of CVar '%s' for platform '%s'. The CVar subclass may need to implement GetPlatformValueVariable."), *CVarName, *IniPlatformName.ToString());
		if (OtherPlatformVar.IsValid())
		{
			OtherPlatformVar->GetValue(Value);
		}
		else
		{
			CVar->GetValue(Value);
		}
#else
		checkf(IniPlatformName == FName(FPlatformProperties::IniPlatformName()), TEXT("FTargetPlatformCachedIniValue can only look up the current platform when ALLOW_OTHER_PLATFORM_CONFIG is false"));
#endif
		return Value;
	}

	FString CVarName;
	IConsoleVariable* CVar;
};
