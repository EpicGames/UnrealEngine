// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/UnrealMemory.h"
#include "RHIShaderPlatform.h"

class FConfigCacheIni;

// Shader Platform Configuration
// 
// Similar to DataDrivenShaderPlatformInfo (DDSPI), but settings come from Engine config hierarchy. DDSPI is specifically for immutable shader platform
// capabilities, while SPC is for shader platform configuration.
// 
// Configs are stored in one section per shader platform with the same names that DDSPI uses.
// ex: [ShaderPlatform PCD3D_SM6] -> [ShaderPlatformConfig PCD3D_SM6]
//
// Interface initially built with static getters but those use per-platform structs with getters. While the static getters suffice for initial adoption, the
// ideal usage would be passing the per-platform struct around to code that needs them. This usage would make it easier to adopt "custom shader platforms" in
// the future.
//
// New fields need the following:
//   1. Value inside FPlatformConfig
//   2. Member getter inside FPlatformConfig
//   3. Static getter inside FShaderPlatformConfig
//   4. Default value inside FShaderPlatformConfig::FPlatformConfig::SetDefaultValues
//   5. Initialization inside FShaderPlatformConfig::FPlatformConfig::InitializeProperties
//   6. (optional) Engine defaults inside Base{Platform}Engine.ini
//   7. (optional) Preview override inside FShaderPlatformConfig::FPlatformConfig::InitializePropertiesForPreview
//

class FShaderPlatformConfig
{
public:
	struct FPlatformConfig
	{
		uint32 GetSubstrateMaxClosuresPerPixel() const
		{
			return SubstrateMaxClosuresPerPixel;
		}

		ERHIBindlessConfiguration GetBindlessConfiguration() const
		{
			return BindlessConfiguration;
		}

		/** Whether Nanite bindless shading is enabled for this platform. See UseNaniteBindlessShading(). */
		bool GetEnableNaniteBindlessShading() const
		{
			return bEnableNaniteBindlessShading;
		}

		/** Whether Nanite bindless rasterization is enabled for this platform. See UseNaniteBindlessRasterization(). */
		bool GetEnableNaniteBindlessRasterization() const
		{
			return bEnableNaniteBindlessRasterization;
		}

		/** Whether Nanite bindless pixel programmable materials are enabled for this platform. Requires bindless rasterization. See NaniteBindlessPixelProgrammableSupported(). */
		bool GetEnableNaniteBindlessPixelProgrammable() const
		{
			return bEnableNaniteBindlessPixelProgrammable;
		}

		bool IsValid() const
		{
			return bValidShaderPlatform;
		}

	protected:
		// Storing config section name for easy lookups
		FString ConfigSectionName;
		EShaderPlatform ShaderPlatform = SP_NumPlatforms;

		uint32 SubstrateMaxClosuresPerPixel{};
		ERHIBindlessConfiguration BindlessConfiguration{};
		bool bEnableNaniteBindlessShading = false;
		bool bEnableNaniteBindlessRasterization = false;
		bool bEnableNaniteBindlessPixelProgrammable = false;

	protected:
		bool bValidShaderPlatform = false;
		bool bLoadedFromConfigFiles = false;

		FPlatformConfig();
		FPlatformConfig(const FPlatformConfig&) = default;

		void SetDefaultValues();

		void Initialize(EShaderPlatform ShaderPlatform);
		void InitializeProperties(const FConfigSection& InConfigSection);

#if WITH_EDITOR
		void InitializeForPreview(EShaderPlatform ShaderPlatform, const FPlatformConfig& RuntimeConfig);
		void InitializePropertiesForPreview(const FPlatformConfig& RuntimeConfig);
#endif

		friend class FShaderPlatformConfig;
	};

public:
	static FORCEINLINE_DEBUGGABLE const uint32 GetSubstrateMaxClosuresPerPixel(const FStaticShaderPlatform Platform)
	{
		check(IsValid(Platform));
		return PlatformConfigs[Platform].GetSubstrateMaxClosuresPerPixel();
	}

	static FORCEINLINE_DEBUGGABLE const ERHIBindlessConfiguration GetBindlessConfiguration(const FStaticShaderPlatform Platform)
	{
		check(IsValid(Platform));
		return PlatformConfigs[Platform].GetBindlessConfiguration();
	}

	/** Returns whether Nanite bindless-aware shading is enabled for the given platform. Requires BindlessConfiguration to be Minimal or higher. */
	static FORCEINLINE_DEBUGGABLE const bool GetEnableNaniteBindlessShading(const FStaticShaderPlatform Platform)
	{
		check(IsValid(Platform));
		return PlatformConfigs[Platform].GetEnableNaniteBindlessShading();
	}

	/** Returns whether Nanite bindless-aware rasterization is enabled for the given platform. Requires BindlessConfiguration to be Minimal or higher. */
	static FORCEINLINE_DEBUGGABLE const bool GetEnableNaniteBindlessRasterization(const FStaticShaderPlatform Platform)
	{
		check(IsValid(Platform));
		return PlatformConfigs[Platform].GetEnableNaniteBindlessRasterization();
	}

	/** Returns whether Nanite bindless-aware pixel programmable materials are enabled for the given platform. Requires BindlessConfiguration to be Minimal or higher. */
	static FORCEINLINE_DEBUGGABLE const bool GetEnableNaniteBindlessPixelProgrammable(const FStaticShaderPlatform Platform)
	{
		check(IsValid(Platform));
		return PlatformConfigs[Platform].GetEnableNaniteBindlessPixelProgrammable();
	}

	static FORCEINLINE_DEBUGGABLE const bool IsValid(const FStaticShaderPlatform Platform)
	{
		return PlatformConfigs[Platform].IsValid();
	}

	RENDERCORE_API static void Initialize();

private:
	RENDERCORE_API static FPlatformConfig PlatformConfigs[SP_NumPlatforms];
};
