// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/AnsiString.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "ShaderCompilerCore.h"
#include "ShaderCore.h"
#include "ShaderSource.h"

struct FShaderCompilerEnvironment;
struct FShaderCompilerError;
struct FShaderCompilerInput;

RENDERCORE_API FShaderSource::FViewType GetShaderSourceDebugHashPrefix();
RENDERCORE_API FStringView GetShaderSourceDebugHashPrefixWide();

/*
* Helper class used to remap compiler diagnostic messages from stripped preprocessed source (i.e. source with all whitespace normalized
* and comments and line directives removed) back to line numbers/locations from the original source. 
*/
struct FShaderDiagnosticRemapper
{
private:
	void Remap(FShaderCompilerError& Diagnostic) const;

	friend class FShaderPreprocessOutput;
	void AddSourceBlock(int32 OriginalLineNum, int32 StrippedLineNum)
	{
		AddSourceBlock(OriginalLineNum, StrippedLineNum, FString());
	}
	void AddSourceBlock(int32 OriginalLineNum, int32 StrippedLineNum, FString&& OriginalPath);
	void AddStrippedLine(int32 StrippedLineNum, int32 Offset);
	void Shrink();

	struct FRemapData
	{
		const FString& Filename;
		int32 LineNumber = INDEX_NONE;

		bool IsValid() { return LineNumber != INDEX_NONE; }
	};
	FRemapData GetRemapData(int32 StrippedLineNum) const;

	struct FSourceBlock
	{
		// Line number of the first line of code in the stripped preprocessed source for this block
		int32 StrippedLineNum;
		
		// Associated line number of where the first line of code in the block occurred in the unstripped source
		int32 OriginalLineNum;

		// Full path associated with this block in the original unstripped source (as given by line directive)
		FString OriginalPath;
	};
	TArray<FSourceBlock> Blocks;
	TArray<int32> StrippedLineOffsets;
};

struct FShaderCompilerOutput;

class FShaderPreprocessOutput
{
public:
	FShaderPreprocessOutput()
	{
	}
	
	FShaderSource::FViewType GetSourceView() const
	{
		return PreprocessedSource.GetView();
	}

	FAnsiStringView GetSourceViewAnsi() const
	{
		return PreprocessedSource.GetView();
	}

	FStringView GetSourceViewWide() const
	{
		if (WideSource.IsEmpty())
		{
			WideSource = FString(PreprocessedSource.GetView());
		}
		return FStringView(WideSource);
	}

	FStringView GetUnstrippedSourceView() const
	{
		// if the unstripped source is requested, check if the "original source" field has been populated
		// if not then stripping hasn't occurred so there's only one preprocessed source; return it

		// convert and store wide versions of requested source if view is requested. 
		// this is only used in debug paths (debug output and viewing source in-editor)
		// and the job should be freed shortly after, so the memory overhead is not a concern
		if (OriginalPreprocessedSource.IsEmpty())
		{
			if (WideSource.IsEmpty())
			{
				WideSource = FString(PreprocessedSource.GetView());
			}
			return FStringView(WideSource);
		}
		else
		{
			if (WideSourceUnstripped.IsEmpty())
			{
				WideSourceUnstripped = FString(OriginalPreprocessedSource.GetView());
			}
			return FStringView(WideSourceUnstripped);
		}
	}

	FShaderSource& EditSource()
	{
		return PreprocessedSource;
	}

	inline void ForEachLine(TFunction<void(FAnsiStringView Line, int32 LineIndex)> Callback) const
	{
		FAnsiStringView Source = GetSourceViewAnsi();
		int32 EndIndex = 0, LineIndex = 0, StartIndex = 0;
		while (StartIndex < Source.Len())
		{
			EndIndex = Source.Find(ANSITEXTVIEW("\n"), StartIndex);
			if (EndIndex == INDEX_NONE)
			{
				EndIndex = Source.Len();
			}
			FAnsiStringView Line(Source.GetData() + StartIndex, EndIndex - StartIndex);
			StartIndex = EndIndex + 1;
			Callback(Line, LineIndex++);
		}
	}

	inline bool HasDirective(const FString& Directive) const
	{
		const int32 NumberOfDirectives = PragmaDirectives.Num();
		for (int32 i = 0; i < NumberOfDirectives; i++)
		{
			const FString& CurrentDirective = PragmaDirectives[i];
			if (CurrentDirective.Equals(Directive))
			{
				return true;
			}
		}
		return false;
	}

	inline void VisitDirectives(TFunction<void(const FString*)> Action) const
	{
		const int32 NumberOfDirectives = PragmaDirectives.Num();
		for (int32 i = 0; i < NumberOfDirectives; i++)
		{
			const FString& CurrentDirective = PragmaDirectives[i];
			Action(&CurrentDirective);
		}
	}

	inline void VisitDirectivesWithPrefix(const TCHAR* Prefix, TFunction<void(const FString*)> Action) const
	{
		const int32 NumberOfDirectives = PragmaDirectives.Num();
		for (int32 i = 0; i < NumberOfDirectives; i++)
		{
			const FString& CurrentDirective = PragmaDirectives[i];
			if (CurrentDirective.StartsWith(Prefix))
			{
				Action(&CurrentDirective);
			}
		}
	}

	inline void AddDirective(FString&& Directive)
	{
		PragmaDirectives.Add(Directive);
	}

	// Temporary helper for preprocessor wrapper function. Can be deprecated when
	// all backends move to independent preprocessing.
	inline void MoveDirectives(TArray<FString>& OutDirectives)
	{
		for (FString& Directive : PragmaDirectives)
		{
			OutDirectives.Add(MoveTemp(Directive));
		}
	}

	inline bool IsSecondary() const
	{
		return bIsSecondary;
	}

	inline bool GetSucceeded() const
	{
		return bSucceeded;
	}

	inline void LogError(FString&& Message)
	{
		FShaderCompilerError& CompilerError = Errors.AddDefaulted_GetRef();;
		CompilerError.StrippedErrorMessage = Message;
	}

	inline void LogError(FString&& FilePath, FString&& Message, FString&& LineNumberStr)
	{
		FShaderCompilerError& CompilerError = Errors.AddDefaulted_GetRef();
		CompilerError.ErrorVirtualFilePath = FilePath;
		CompilerError.ErrorLineString = LineNumberStr;
		CompilerError.StrippedErrorMessage = Message;
	}

	inline void LogError(FString&& FilePath, FString&& Message, int32 LineNumber)
	{
		FString LineNumberStr = LexToString(LineNumber);
		LogError(MoveTemp(FilePath), MoveTemp(Message), MoveTemp(LineNumberStr));
	}

	inline TArray<FShaderCompilerError>& EditErrors()
	{
		return Errors;
	}

	inline TConstArrayView<FShaderCompilerError> GetErrors() const
	{
		return MakeArrayView(Errors);
	}

	double GetPreprocessTime() const
	{
		return PreprocessTime;
	}

	TArray<FShaderDiagnosticData>& EditDiagnosticDatas()
	{
		return ShaderDiagnosticDatas;
	}

	const TArray<FShaderDiagnosticData>& GetDiagnosticDatas() const
	{
		return ShaderDiagnosticDatas;
	}

	FMemoryView GetCompressedSource() const
	{
		return PreprocessedSource.GetCompressed();
	}

	friend FArchive& operator<<(FArchive& Ar, FShaderPreprocessOutput& PreprocessOutput);

private:

	friend class FInternalShaderCompilerFunctions;
	friend class FShaderCompileJob;

	// Strips comments/whitespace/line directives from the preprocessed source, replacing the contents of PreprocessedSource
	// and saving the original source in the OriginalPreprocessedSource member
	void StripCode(bool bCopyOriginalPreprocessdSource);

	void CompressCode() 
	{
		PreprocessedSource.Compress();
		OriginalPreprocessedSource.Compress();
	}

	void DecompressCode()
	{
		PreprocessedSource.Decompress();
		OriginalPreprocessedSource.Decompress();
	}

	void RemapErrors(FShaderCompilerOutput& Output) const;

	// Output of preprocessing; should be set by IShaderFormat::PreprocessShader
	FShaderSource PreprocessedSource;

	// Set by Finalize; original preprocessed source as set by IShaderFormat::PreprocessShader
	FShaderSource OriginalPreprocessedSource;

	// Set when GetSourceViewWide or GetUnstrippedSourceView accessors are called
	// Mutable so we can maintain const correctness in the API otherwise.
	mutable FString WideSource;
	mutable FString WideSourceUnstripped;

	// Array of errors encountered in preprocessing; should be populated by IShaderFormat::PreprocessShader
	TArray<FShaderCompilerError> Errors;

	// Array of "UESHADERMETADATA" pragmas encountered by preprocessing; set automatically by core preprocessing
	// and expected to be queried by IShaderFormat 
	TArray<FString> PragmaDirectives;
	FShaderDiagnosticRemapper Remapper;

	double PreprocessTime = 0.0;
	bool bSucceeded = false;
	bool bIsSecondary = false;

	TArray<FShaderDiagnosticData> ShaderDiagnosticDatas;
};
