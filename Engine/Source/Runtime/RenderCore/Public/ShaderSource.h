// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/AnsiString.h"
#include "Containers/Array.h"
#include "Containers/StringView.h"
#include "Containers/UnrealString.h"
#include "Memory/MemoryView.h"
#include "Misc/StringBuilder.h"

#define SHADER_SOURCE_ANSI 1 UE_DEPRECATED_MACRO(5.6, "SHADER_SOURCE_ANSI has been deprecated and should be assumed to always be 1")
#define SHADER_SOURCE_LITERAL(S) S UE_DEPRECATED_MACRO(5.6, "SHADER_SOURCE_LITERAL has been deprecated and should be replaced with just plain string literal")
#define SHADER_SOURCE_VIEWLITERAL(S) ANSITEXTVIEW(S) UE_DEPRECATED_MACRO(5.6, "SHADER_SOURCE_VIEWLITERAL has been deprecated and should be replaced with ANSITEXTVIEW")

/* Class used in shader compilation pipelines which wraps source code and ensures sufficient padding such that 16-byte wide SIMD
 * operations on the source are guaranteed to read valid memory even if starting from the last character.
 */
class FShaderSource
{
private:
	inline void SetLen(int32 Num, EAllowShrinking AllowShrinking = EAllowShrinking::Default)
	{
		Source.SetNumUninitialized(Num + ShaderSourceSimdPadding, AllowShrinking);
		FMemory::Memzero(Source.GetData() + Num, sizeof(CharType) * ShaderSourceSimdPadding);

		SourceCompressed.Empty();
		DecompressedCharCount = 0;
	}

public:
	typedef ANSICHAR CharType;
	typedef FAnsiStringView FViewType;
	typedef FAnsiString FStringType;
	typedef FCStringAnsi FCStringType;

	template <int NumChars>
	using TStringBuilder = TAnsiStringBuilder<NumChars>;

	/* Construct an empty shader source object; will still contain padding */
	FShaderSource() { SetLen(0); };

	/* Given a string view construct a shader source object containing the contents of that string.
	 * Note that this will incur a memcpy of the string contents.
	 * @param InSrc The source string to be copied
	 * @param AdditionalSlack optional additional space to allocate; this is on top of the automatic padding.
	 */
	RENDERCORE_API explicit FShaderSource(FViewType InSrc, int32 AdditionalSlack = 0);

	/* Set the given string as the contents of this shader source object. The inner allocation will grow to fit
	 * the string contents as needed.
	 * Note that this will incur a memcpy of the string contents.
	 * @param InSrc The source string to be copied
	 * @param AdditionalSlack optional additional space to allocate; this is on top of the automatic padding.
	 */
	RENDERCORE_API void Set(FViewType InSrc, int32 AdditionalSlack = 0);

	/* Returns the array which contains the compressed data for external population.
	 * Note that this is only intended to be used in a very specific situation (loading UBA-virtualized files
	 * in SCW) and will only work properly if DecompressedCharCount is correctly set.
	 */
	inline TArray<uint8>& EditCompressed() { return SourceCompressed; }

	/* Move assignment operator accepting a string object. This will append padding bytes to the existing string, as such it's
	 * best if there's sufficient extra capacity in the string storage to avoid incurring a realloc-and-copy here.
	 * @param InSrc The source string whose data this object will take ownership of.
	 */
	RENDERCORE_API FShaderSource& operator=(FStringType&& InSrc);

	/* Reduces the set size of the stored string length, optionally shrinking the allocation.
	 * @param Num the desired allocation size (padding bytes will be added on top of this)
	 * @param AllowShrinking whether to reallocate or keep the existing larger size allocation
	 */
	inline void ShrinkToLen(int32 Num, EAllowShrinking AllowShrinking = EAllowShrinking::Default)
	{
		checkf(Num <= Len(), TEXT("Trying to shrink to %d characters but existing allocation is smaller (%d characters)"), Num, Len());
		SetLen(Num, AllowShrinking);
	}

	/* String view accessor. Will decompress data if it was previously compressed.
	 * @return a string view pointing to the source contents, excluding padding.
	 */
	inline FViewType GetView() const
	{ 
		checkf(!IsCompressed(), TEXT("FShaderSource is compressed; must decompress prior to calling GetView"));
		return { Source.GetData(), Len() }; 
	}

	/* Direct data pointer accessor. Will decompress data if it was previously compressed.
	 * @return a pointer to the source data; will be null terminated by the SIMD padding.
	 */
	inline CharType* GetData()
	{
		checkf(!IsCompressed(), TEXT("FShaderSource is compressed; must decompress prior to calling GetData"));
		return Source.GetData();
	}

	/* IsEmpty predicate
	 * @return true if this source object is empty excluding the SIMD padding, false otherwise.
	 */
	inline bool IsEmpty() const { return Len() == 0; }

	/* Length accessor.
	 * @return the non-padded length of the source (also excluding null terminator)
	 */
	inline int32 Len() const
	{ 
		checkf(!IsCompressed(), TEXT("Len should not be called on compressed FShaderSource."))
		return Source.Num() - ShaderSourceSimdPadding;
	};

	inline bool IsCompressed() const
	{
		return DecompressedCharCount != 0; // DecompressedCharCount member is only set when compression occurs
	}

	inline int32 GetDecompressedSize() const
	{
		return DecompressedCharCount * sizeof(FShaderSource::CharType);
	}

	/* FArchive serialization operator. Note this currently serializes padding for simplicity's sake.
	 * @param Ar The archive to serialize from/to
	 * @param ShaderSource the source object to serialize
	 */
	friend FArchive& operator<<(FArchive& Ar, FShaderSource& ShaderSource);

	RENDERCORE_API void Compress();
	RENDERCORE_API void Decompress();

	inline FMemoryView GetCompressed() const
	{
		checkf(IsCompressed(), TEXT("Requesting compressed shader source but source has not been compressed"));
		return FMemoryView(SourceCompressed.GetData(), (uint64)SourceCompressed.Num());
	}
	
private:
	friend class FShaderCompileJob;

	static_assert(sizeof(CharType) == 1);
	static constexpr int32 ShaderSourceSimdPadding = 15;
	TArray<CharType> Source;
	TArray<uint8> SourceCompressed;
	int32 DecompressedCharCount = 0;
};

