// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * Code for reading .shk files and using that information to map shader hashes back to human-readable identifies.
 */

#pragma once

#include "ShaderCodeLibrary.h"

namespace UE
{

namespace ShaderUtils
{

/** Class that uses build metadata (*.shk files storing mapping of stable shader keys to their hashes) to provide high level info on shaders and PSOs. */
class FShaderStableKeyDebugInfoReader
{
	/** Maps shader hashes to their sources. */
	TMap<FShaderHash, TSet<FStableShaderKeyAndValue>>  ShaderHashesToSource;

public:

	/** Whether the info reader is usable */
	bool IsInitialized() const { return !ShaderHashesToSource.IsEmpty(); }

	/** Initializes the class with a pointer to directory holding SHK files */
	bool Initialize(const FString& ShaderStableKeyFile);

	/** Returns a string describing a shader. Because of shader deduplication there can be several multiple (thousands) possible options */
	FString GetShaderStableNameOptions(const FShaderHash& ShaderHash, int32 MaxOptionsToInclude = 16);

	/** Returns a string describing a PSO (shaders only). Because of shader deduplication there can be several multiple (thousands) possible options. */
	FString GetPSOStableNameOptions(const FGraphicsPipelineStateInitializer& Initializer, int32 MaxOptionsToInclude = 16);

	/** Prints a pipeline info to log if configured to do so. */
	void DumpPSOToLogIfConfigured(const FGraphicsPipelineStateInitializer& Initializer);
};

}

}
