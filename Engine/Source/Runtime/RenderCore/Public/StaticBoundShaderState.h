// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	StaticBoundShaderState.h: Static bound shader state definitions.
=============================================================================*/

#pragma once

#include "Containers/List.h"
#include "CoreMinimal.h"
#include "Misc/Build.h"
#include "RHI.h"
#include "RHIDefinitions.h"
#include "RenderResource.h"

class FShader;

/**
 * FGlobalBoundShaderState
 * 
 * Encapsulates a global bound shader state resource.
 */
class FGlobalBoundShaderStateResource : public FRenderResource
{
public:

	/** @return The list of global bound shader states. */
	RENDERCORE_API static TLinkedList<FGlobalBoundShaderStateResource*>*& GetGlobalBoundShaderStateList();

	/** Initialization constructor. */
	RENDERCORE_API FGlobalBoundShaderStateResource();

	/** Destructor. */
	RENDERCORE_API virtual ~FGlobalBoundShaderStateResource();

private:

	/** The cached bound shader state. */
	FBoundShaderStateRHIRef BoundShaderState;

	/** This resource's link in the list of global bound shader states. */
	TLinkedList<FGlobalBoundShaderStateResource*> GlobalListLink;

	// FRenderResource interface.
	RENDERCORE_API virtual void ReleaseRHI();

#if DO_CHECK
	FRHIVertexDeclaration* BoundVertexDeclaration;
	FRHIVertexShader* BoundVertexShader;
	FRHIPixelShader* BoundPixelShader;
	FRHIGeometryShader* BoundGeometryShader;
#endif 
};

typedef TGlobalResource<FGlobalBoundShaderStateResource> FGlobalBoundShaderState_Internal;


struct FGlobalBoundShaderStateArgs
{
	FRHIVertexDeclaration* VertexDeclarationRHI;
	FShader* VertexShader;
	FShader* PixelShader;
	FShader* GeometryShader;
};

struct FGlobalBoundShaderStateWorkArea
{
	FGlobalBoundShaderStateArgs Args;
	FGlobalBoundShaderState_Internal* BSS; //ideally this would be part of this memory block and not a separate allocation...that is doable, if a little tedious. The point is we need to delay the construction until we get back to the render thread.

	FGlobalBoundShaderStateWorkArea()
		: BSS(nullptr)
	{
	}
};

struct FGlobalBoundShaderState
{
public:

	FGlobalBoundShaderStateWorkArea* Get(ERHIFeatureLevel::Type InFeatureLevel)  { return WorkAreas[InFeatureLevel]; }
	FGlobalBoundShaderStateWorkArea** GetPtr(ERHIFeatureLevel::Type InFeatureLevel)  { return &WorkAreas[InFeatureLevel]; }

private:

	FGlobalBoundShaderStateWorkArea* WorkAreas[ERHIFeatureLevel::Num];
};
