// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

enum EShaderPlatform : uint16;

namespace UE::StereoRenderUtils
{
	
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	
/*
* Detect the single-draw stereo shader variant, in order to support usage across different platforms
*/
class FStereoShaderAspects
{
public:
	/**
	* Determines the stereo aspects of the shader pipeline based on the input shader platform
	* @param Platform	Target shader platform used to determine stereo shader variant
	*/
	RENDERCORE_API FStereoShaderAspects(EShaderPlatform Platform);

	/**  
	* Default empty constructor for object in FSceneView. Do not use! 
	*/
	RENDERCORE_API FStereoShaderAspects();
	
	/**  
	* Default copy constructor. Do not use! 
	*/
	RENDERCORE_API FStereoShaderAspects(const FStereoShaderAspects& Copy);
	
	/**
	* Whether stereo rendering is enabled - i.e. using a single drawcall to render multiple views.
	*/
	inline bool IsSinglePassStereoEnabled() const { return bInstancedStereoEnabled || bMobileMultiViewEnabled; }

	/**
	* Whether instanced stereo rendering is enabled - i.e. using a single instanced drawcall to render to both stereo views.
	* The output is redirected via the viewport index.
	*/
	inline bool IsInstancedStereoEnabled() const { return bInstancedStereoEnabled; }

	/**
	* Whether mobile multiview is enabled - i.e. using VK_KHR_multiview. Another drawcall reduction technique, independent of instanced stereo.
	* Mobile multiview generates view indices to index into texture arrays.
	* Can be internally emulated using instanced stereo if native support is unavailable, by using ISR-generated view indices to index into texture arrays.
	*/
	inline bool IsMobileMultiViewEnabled() const { return bMobileMultiViewEnabled; };

	/**
	* Whether multiviewport rendering is enabled - i.e. using ViewportIndex to index into viewport.
	* Relies on instanced stereo rendering being enabled.
	*/
	UE_DEPRECATED(5.8, "IsInstancedMultiViewportEnabled() now represents this since there is no other ISR path other than multi viewport, if both are still required short term set r.InstancedStereoIsMultiViewport to false")
	[[nodiscard]] RENDERCORE_API bool IsInstancedMultiViewportEnabled() const;

private:
	bool bInstancedStereoEnabled : 1;
	bool bMobileMultiViewEnabled : 1;
	
	// DEPRECATED
	UE_DEPRECATED(5.8, "bInstancedStereoEnabled now represents this since there is no other ISR path other than multi viewport, if both are still required short term set r.InstancedStereoIsMultiViewport to false")
	bool bInstancedMultiViewportEnabled : 1;

	bool bInstancedStereoNative : 1;
	bool bMobileMultiViewNative : 1;
};

PRAGMA_ENABLE_DEPRECATION_WARNINGS
	
RENDERCORE_API void LogISRInit(const UE::StereoRenderUtils::FStereoShaderAspects& Aspects);

} // namespace UE::StereoRenderUtils
