// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Stats/Stats.h"

class FRHICommandListImmediate;

/**
 * This class provides common registration for render thread tickable objects. It is an
 * abstract base class requiring you to implement the Tick() method.
 */
class FTickableObjectRenderThread
{
public:

	/** Static array of tickable objects that are ticked from rendering thread*/
	struct FRenderingThreadTickableObjectsArray : public TArray<FTickableObjectRenderThread*>
	{
		~FRenderingThreadTickableObjectsArray()
		{
			// if there are any Tickable objects left registered at this point, force them to unregister
			int32 MaxIterations = Num();	// prevents runaway loop (extra safety)
			while(Num() > 0 && MaxIterations-- > 0)
			{
				FTickableObjectRenderThread* Object = Top();
				check(Object);
				Object->Unregister();
			}
			// if we exited uncleanly from a runaway loop, crash explicitly in Dev
			check(Num() == 0);
		}
	};

	static RENDERCORE_API FRenderingThreadTickableObjectsArray RenderingThreadTickableObjects;
	static RENDERCORE_API FRenderingThreadTickableObjectsArray RenderingThreadHighFrequencyTickableObjects;

	/**
	 * Registers this instance with the static array of tickable objects.	
	 *
	 * @param bRegisterImmediately true if the object should be registered immediately.
	 */
	FTickableObjectRenderThread(bool bRegisterImmediately=true, bool bInHighFrequency=false)
		: bHighFrequency(bInHighFrequency)
	{
		if(bRegisterImmediately)
		{
			Register();
		}
	}

	/**
	 * Removes this instance from the static array of tickable objects.
	 */
	virtual ~FTickableObjectRenderThread()
	{
		Unregister();
	}

	void Unregister()
	{
		if (bRegistered)
		{
			// make sure this tickable object was registered from the rendering thread
			checkf(IsInRenderingThread(), TEXT("Game thread attempted to unregister an object in the RenderingThreadTickableObjects array."));

			FRenderingThreadTickableObjectsArray& TickableObjectArray = bHighFrequency ? RenderingThreadHighFrequencyTickableObjects : RenderingThreadTickableObjects;
			const int32 Pos = TickableObjectArray.Find(this);
			check(Pos!=INDEX_NONE);
			TickableObjectArray.RemoveAt(Pos);
			bRegistered = false;
		}
	}

	/**
	 * Registers the object for ticking.
	 */
	void Register()
	{
		// make sure that only the rendering thread is attempting to add items to the RenderingThreadTickableObjects list
		checkf(IsInRenderingThread(), TEXT("Game thread attempted to register an object in the RenderingThreadTickableObjects array."));
		check(!RenderingThreadTickableObjects.Contains(this));
		check(!RenderingThreadHighFrequencyTickableObjects.Contains(this));
		check(!bRegistered);
		if (bHighFrequency)
		{
			RenderingThreadHighFrequencyTickableObjects.Add(this);
		}
		else
		{
			RenderingThreadTickableObjects.Add(this);
		}		
		bRegistered = true;
	}

	UE_DEPRECATED(5.7, "bIsRenderingThreadObject argument is no longer needed")
	void Register(bool bIsRenderingThreadObject)
	{
		Register();
	}

	/**
	 * Pure virtual that must be overloaded by the inheriting class. It will
	 * be called from the rendering thread.
	 *
	 * @param DeltaTime	Game time passed since the last call.
	 */
	virtual void Tick(FRHICommandListImmediate& RHICmdList, float DeltaTime) = 0;

	/** return the stat id to use for this tickable **/
	virtual TStatId GetStatId() const = 0;

	/**
	 * Pure virtual that must be overloaded by the inheriting class. It is
	 * used to determine whether an object is ready to be ticked. This is 
	 * required for example for all UObject derived classes as they might be
	 * loaded async and therefore won't be ready immediately.
	 *
	 * @return	true if class is ready to be ticked, false otherwise.
	 */
	virtual bool IsTickable() const = 0;

	/**
	 * Used to determine if a rendering thread tickable object must have rendering in a non-suspended
	 * state during it's Tick function.
	 *
	 * @return true if the RHIResumeRendering should be called before tick if rendering has been suspended
	 */
	virtual bool NeedsRenderingResumedForRenderingThreadTick() const
	{
		return false;
	}

private:
	bool bRegistered = false;
	const bool bHighFrequency;
};

extern RENDERCORE_API void TickRenderingTickables(FRHICommandListImmediate& RHICmdList);

UE_DEPRECATED(5.7, "TickRenderingTickables needs a command list")
extern RENDERCORE_API void TickRenderingTickables();
