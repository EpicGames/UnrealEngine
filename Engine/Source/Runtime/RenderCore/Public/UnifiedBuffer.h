// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"
#include "Containers/DynamicRHIResourceArray.h"
#include "CoreMinimal.h"
#include "GenericPlatform/GenericPlatformMemory.h"
#include "HAL/Platform.h"
#include "HAL/UnrealMemory.h"
#include "Misc/AssertionMacros.h"
#include "PixelFormat.h"
#include "RHI.h"
#include "RHIUtilities.h"
#include "RenderGraphDefinitions.h"
#include "RenderGraphResources.h"
#include "Templates/RefCounting.h"
#include "SceneRenderingAllocator.h"

class FRDGBuilder;
class FRHICommandList;

/*
 * Can store arbitrary data so long as it follows alignment restrictions. Intended mostly for read only data uploaded from CPU.
 * Allows sparse allocations and updates from CPU.
 * Float4 versions exist for platforms that don't yet support byte address buffers.
 */

struct FMemsetResourceParams
{
	uint32 Value;
	uint32 Count;
	uint32 DstOffset;
};

struct FMemcpyResourceParams
{
	uint32 Count;
	uint32 SrcOffset;
	uint32 DstOffset;
};

struct FResizeResourceSOAParams
{
	uint32 NumBytes;
	uint32 NumArrays;
};

template<typename ResourceType>
extern RENDERCORE_API void MemsetResource(FRHICommandList& RHICmdList, const ResourceType& DstBuffer, const FMemsetResourceParams& Params);

template<typename ResourceType>
extern RENDERCORE_API void MemcpyResource(FRHICommandList& RHICmdList, const ResourceType& DstBuffer, const ResourceType& SrcBuffer, const FMemcpyResourceParams& Params, bool bAlreadyInUAVOverlap = false);

RENDERCORE_API bool ResizeResourceSOAIfNeeded(FRHICommandList& RHICmdList, FRWBufferStructured& Texture, const FResizeResourceSOAParams& Params, const TCHAR* DebugName);

template<typename ResourceType>
extern RENDERCORE_API bool ResizeResourceIfNeeded(FRHICommandList& RHICmdList, ResourceType& Buffer, uint32 NumBytes, const TCHAR* DebugName);

RENDERCORE_API bool ResizeResourceIfNeeded(FRHICommandList& RHICmdList, FRWBuffer& Buffer, EPixelFormat Format, uint32 NumElements, const TCHAR* DebugName);

/**
 * This version will resize/allocate the buffer at once and add a RDG pass to perform the copy on the RDG time-line if there was previous data).
 */
RENDERCORE_API bool ResizeResourceSOAIfNeeded(FRDGBuilder& GraphBuilder, FRWBufferStructured& Buffer, const FResizeResourceSOAParams& Params, const TCHAR* DebugName);
RENDERCORE_API bool ResizeResourceIfNeeded(FRDGBuilder& GraphBuilder, FRWBufferStructured& Buffer, uint32 NumBytes, const TCHAR* DebugName);
RENDERCORE_API bool ResizeResourceIfNeeded(FRDGBuilder& GraphBuilder, FRWByteAddressBuffer& Buffer, uint32 NumBytes, const TCHAR* DebugName);
RENDERCORE_API bool ResizeResourceIfNeeded(FRDGBuilder& GraphBuilder, FRWBuffer& Buffer, EPixelFormat Format, uint32 NumElements, const TCHAR* DebugName);

class FScatterUploadBuffer
{
public:
	FByteAddressBuffer ScatterBuffer;
	FByteAddressBuffer UploadBuffer;

	uint32*	ScatterData	= nullptr;
	uint8*	UploadData	= nullptr;

	uint32	ScatterDataSize = 0;
	uint32	UploadDataSize = 0;
	uint32 	NumScatters = 0;
	uint32 	MaxScatters = 0;
	uint32	NumBytesPerElement = 0;

	bool	bFloat4Buffer = false;
	bool    bUploadViaCreate = false;

	~FScatterUploadBuffer()
	{
		Release();
	}

	RENDERCORE_API void Init( uint32 NumElements, uint32 InNumBytesPerElement, bool bInFloat4Buffer, const TCHAR* DebugName );

	template<typename ResourceType>
	void ResourceUploadTo(FRHICommandList& RHICmdList, const ResourceType& DstBuffer);

	template<typename ResourceType>
	UE_DEPRECATED(5.8, "ResourceUploadTo with the optional bFlush argument is deprecated. If submission of GPU work is required immediately afterwards, the caller should do this via the immediate command list.")
	void ResourceUploadTo(FRHICommandList& RHICmdList, const ResourceType& DstBuffer, bool /*unused bFlush*/)
	{
		ResourceUploadTo(RHICmdList, DstBuffer);
	}

	void Add( uint32 Index, const void* Data, uint32 Num = 1 )
	{
		void* Dst = Add_GetRef( Index, Num );
		FMemory::ParallelMemcpy(Dst, Data, Num * NumBytesPerElement, EMemcpyCachePolicy::StoreUncached);
	}

	void* Add_GetRef( uint32 Index, uint32 Num = 1 )
	{
		checkSlow( NumScatters + Num <= MaxScatters );
		checkSlow( ScatterData != nullptr );
		checkSlow( UploadData != nullptr );

		uint32* ScatterWriteData = ScatterData + NumScatters;

		for( uint32 i = 0; i < Num; i++ )
		{
			ScatterWriteData[ i ] = Index + i;
		}

		void* Result = UploadData + NumScatters * NumBytesPerElement;
		NumScatters += Num;
		return Result;
	}

	void* Set_GetRef(uint32 ElementIndex, uint32 ElementScatterOffset, uint32 Num = 1)
	{
		checkSlow(ElementIndex + Num <= MaxScatters );
		checkSlow( ScatterData != nullptr );
		checkSlow( UploadData != nullptr );
		for (uint32 i = 0; i < Num; i++)
		{
			ScatterData[ElementIndex + i] = ElementScatterOffset + i;
		}
		return UploadData + ElementIndex * NumBytesPerElement;
	}

	void Release()
	{
		ScatterBuffer.Release();
		UploadBuffer.Release();

		if (bUploadViaCreate)
		{
			if (ScatterData)
			{
				FMemory::Free(ScatterData);
				ScatterData = nullptr;
			}
			if (UploadData)
			{
				FMemory::Free(UploadData);
				UploadData = nullptr;
			}
			ScatterDataSize = 0;
			UploadDataSize = 0;
		}
	}

	uint32 GetNumBytes() const
	{
		return ScatterBuffer.NumBytes + UploadBuffer.NumBytes;
	}

	/**
	 * Init with presized num scatters, expecting each to be set at a later point. Requires the user to keep track of the offsets to use.
	 */
	RENDERCORE_API void InitPreSized(uint32 NumElements, uint32 InNumBytesPerElement, bool bInFloat4Buffer, const TCHAR* DebugName);

	/**
	 * Init with pre-existing destination index data, performs a bulk-copy.
	 */
	RENDERCORE_API void Init(TArrayView<const uint32> ElementScatterOffsets, uint32 InNumBytesPerElement, bool bInFloat4Buffer, const TCHAR* DebugName);
	/**
	 * Get pointer to an element data area, given the index of the element (not the destination scatter offset).
	 */
	inline void* GetRef(uint32 ElementIndex)
	{
		checkSlow(ScatterData != nullptr);
		checkSlow(UploadData != nullptr);

		return UploadData + ElementIndex * NumBytesPerElement;
	}

	void SetUploadViaCreate(bool bInUploadViaCreate)
	{
		if (bInUploadViaCreate != bUploadViaCreate)
		{
			// When switching the upload path, just free everything.
			Release();

			bUploadViaCreate = bInUploadViaCreate;
		}
	}
};

extern RENDERCORE_API void MemsetResource(FRDGBuilder& GraphBuilder, FRDGBuffer* DstResource, const FMemsetResourceParams& Params);
extern RENDERCORE_API void MemcpyResource(FRDGBuilder& GraphBuilder, FRDGBuffer* DstResource, FRDGBuffer* SrcResource, const FMemcpyResourceParams& Params);

extern RENDERCORE_API void MemsetResource(FRDGBuilder& GraphBuilder, FRDGBufferUAV* DstResource, const FMemsetResourceParams& Params);
extern RENDERCORE_API void MemcpyResource(FRDGBuilder& GraphBuilder, FRDGBufferUAV* DstResource, FRDGBufferSRV* SrcResource, const FMemcpyResourceParams& Params);

struct FScatterCopyParams
{
	uint32 NumScatters = 0u;
	uint32 NumBytesPerElement = 0u;
	int32 NumElementsPerScatter = INDEX_NONE; // INDEX_NONE lets the setup figure it out, otherwise it will run NumScatters * NumElementsPerScatter threads to copy the source data.
};

void RENDERCORE_API ScatterCopyResource(FRDGBuilder& GraphBuilder, FRDGBuffer* DstResource, FRDGBufferSRV* ScatterBufferSRV, FRDGBufferSRV* UploadBufferSRV, const FScatterCopyParams &Params);

struct FAsyncScatterCopyParams
{
	TFunction<uint64()> GetNumScatters;
	uint32 NumBytesPerElement = 0u;
	int32 NumElementsPerScatter = INDEX_NONE; // INDEX_NONE lets the setup figure it out, otherwise it will run NumScatters * NumElementsPerScatter threads to copy the source data.
};

void RENDERCORE_API ScatterCopyResource(FRDGBuilder& GraphBuilder, FRDGBuffer* DstResource, FRDGBufferSRV* ScatterBufferSRV, FRDGBufferSRV* UploadBufferSRV, const FAsyncScatterCopyParams& Params);

extern RENDERCORE_API FRDGBuffer* ResizeBufferIfNeeded(FRDGBuilder& GraphBuilder, TRefCountPtr<FRDGPooledBuffer>& ExternalBuffer, const FRDGBufferDesc& BufferDesc, const TCHAR* Name);
extern RENDERCORE_API FRDGBuffer* ResizeBufferIfNeeded(FRDGBuilder& GraphBuilder, TRefCountPtr<FRDGPooledBuffer>& ExternalBuffer, EPixelFormat Format, uint32 NumElements, const TCHAR* Name);
extern RENDERCORE_API FRDGBuffer* ResizeStructuredBufferIfNeeded(FRDGBuilder& GraphBuilder, TRefCountPtr<FRDGPooledBuffer>& ExternalBuffer, uint32 NumBytes, const TCHAR* Name);
extern RENDERCORE_API FRDGBuffer* ResizeStructuredBufferSOAIfNeeded(FRDGBuilder& GraphBuilder, TRefCountPtr<FRDGPooledBuffer>& ExternalBuffer, const FResizeResourceSOAParams& Params, const TCHAR* DebugName);
extern RENDERCORE_API FRDGBuffer* ResizeByteAddressBufferIfNeeded(FRDGBuilder& GraphBuilder, TRefCountPtr<FRDGPooledBuffer>& ExternalBuffer, uint32 NumBytes, const TCHAR* Name);
// Same as ResizeByteAddressBufferIfNeeded but will rebase the allocated buffer under the current LLM tag.
#if ENABLE_LOW_LEVEL_MEM_TRACKER || UE_MEMORY_TRACE_ENABLED
extern RENDERCORE_API FRDGBuffer* ResizeByteAddressBufferIfNeededWithCurrentLLMTag(FRDGBuilder& GraphBuilder, TRefCountPtr<FRDGPooledBuffer>& ExternalBuffer, uint32 NumBytes, const TCHAR* Name);
#else
inline FRDGBuffer* ResizeByteAddressBufferIfNeededWithCurrentLLMTag(FRDGBuilder& GraphBuilder, TRefCountPtr<FRDGPooledBuffer>& ExternalBuffer, uint32 NumBytes, const TCHAR* Name)
{
	return ResizeByteAddressBufferIfNeeded(GraphBuilder, ExternalBuffer, NumBytes, Name);
}
#endif
class FRDGAsyncScatterUploadBuffer;

class FRDGScatterUploadBase
{
public:
	void Add(TArrayView<const uint32> ElementScatterOffsets)
	{
		check(NumScatters + ElementScatterOffsets.Num() <= MaxScatters);
		check(ScatterData != nullptr);
		check(UploadData != nullptr);

		uint32* ScatterWriteData = ScatterData + NumScatters;
		FMemory::ParallelMemcpy(ScatterWriteData, ElementScatterOffsets.GetData(), ElementScatterOffsets.Num() * ElementScatterOffsets.GetTypeSize(), EMemcpyCachePolicy::StoreUncached);
		NumScatters += ElementScatterOffsets.Num();
	}

	void Add(uint32 Index, const void* Data, uint32 Num = 1)
	{
		void* Dst = Add_GetRef(Index, Num);
		FMemory::ParallelMemcpy(Dst, Data, Num * NumBytesPerElement, EMemcpyCachePolicy::StoreUncached);
	}

	void* Add_GetRef(uint32 Index, uint32 Num = 1)
	{
		check(NumScatters + Num <= MaxScatters);
		check(ScatterData != nullptr);
		check(UploadData != nullptr);

		uint32* ScatterWriteData = ScatterData + NumScatters;

		for (uint32 i = 0; i < Num; i++)
		{
			ScatterWriteData[i] = Index + i;
		}

		void* Result = UploadData + NumScatters * NumBytesPerElement;
		NumScatters += Num;
		return Result;
	}

	template <typename T>
	TArrayView<T> Add_GetRef(uint32 Index, uint32 Num = 1)
	{
		return MakeArrayView(reinterpret_cast<T*>(Add_GetRef(Index, Num)), Num);
	}

	void* Set_GetRef(uint32 ElementIndex, uint32 ElementScatterOffset, uint32 Num = 1)
	{
		check(ElementIndex + Num <= MaxScatters);
		check(ScatterData != nullptr);
		check(UploadData != nullptr);

		for (uint32 i = 0; i < Num; i++)
		{
			ScatterData[ElementIndex + i] = ElementScatterOffset + i;
		}
		return UploadData + ElementIndex * NumBytesPerElement;
	}

	template <typename T>
	TArrayView<T> Set_GetRef(uint32 Index, uint32 ElementScatterOffset, uint32 Num = 1)
	{
		return MakeArrayView(reinterpret_cast<T*>(Set_GetRef(Index, ElementScatterOffset, Num)), Num);
	}

	/**
	 * Get pointer to an element data area, given the index of the element (not the destination scatter offset).
	 */
	inline void* GetRef(uint32 ElementIndex)
	{
		check(ScatterData != nullptr);
		check(UploadData != nullptr);

		return UploadData + ElementIndex * NumBytesPerElement;
	}

protected:
	uint32* ScatterData = nullptr;
	uint8* UploadData = nullptr;
	uint32 NumScatters = 0;
	uint32 MaxScatters = 0;
	uint32 NumBytesPerElement = 0;

	friend FRDGAsyncScatterUploadBuffer;
};

class FRDGScatterUploader
	: public FRDGScatterUploadBase
{
public:
	RENDERCORE_API void Lock(FRHICommandListBase& RHICmdList, uint32 ActualMaxScatters);

	void Lock(FRHICommandListBase& RHICmdList)
	{
		Lock(RHICmdList, MaxScatters);
	}

	RENDERCORE_API void Unlock(FRHICommandListBase& RHICmdList);

	FRDGViewableResource* GetDstResource() const
	{
		return DstResource;
	}

	uint32 GetMaxScatters() const
	{
		return MaxScatters;
	}

private:
	uint32 GetFinalNumScatters() const
	{
		check(bNumScattersPreSized || State == EState::Unlocked);
		return NumScatters;
	}

	FRDGViewableResource* DstResource = nullptr;
	FRHIBuffer* ScatterBuffer = nullptr;
	FRHIBuffer* UploadBuffer = nullptr;
	uint32 ScatterBytes = 0;
	uint32 UploadBytes = 0;
	bool bNumScattersPreSized = false;

	enum class EState : uint8
	{
		Empty,
		Locked,
		Unlocked
	};

	std::atomic<EState> State{ EState::Empty };

	friend FRDGAsyncScatterUploadBuffer;
};

inline void LockIfValid(FRHICommandListBase& RHICmdList, FRDGScatterUploader* Uploader)
{
	if (Uploader)
	{
		Uploader->Lock(RHICmdList);
	}
}

inline void UnlockIfValid(FRHICommandListBase& RHICmdList, FRDGScatterUploader* Uploader)
{
	if (Uploader)
	{
		Uploader->Unlock(RHICmdList);
	}
}

class FRDGAsyncScatterUploadBuffer
{
public:
	/** Init with pre-existing destination index data, performs a bulk-copy. */
	RENDERCORE_API FRDGScatterUploader* Begin(FRDGBuilder& GraphBuilder, FRDGViewableResource* DstResource, uint32 NumElements, uint32 NumBytesPerElement, const TCHAR* Name);

	/** Init with presized num scatters, expecting each to be set at a later point. Requires the user to keep track of the offsets to use. */
	RENDERCORE_API FRDGScatterUploader* BeginPreSized(FRDGBuilder& GraphBuilder, FRDGViewableResource* DstResource, uint32 NumElements, uint32 NumBytesPerElement, const TCHAR* Name);

	RENDERCORE_API void End(FRDGBuilder& GraphBuilder, FRDGScatterUploader* Uploader);

	RENDERCORE_API void Release();

	RENDERCORE_API uint32 GetNumBytes() const;

private:
	TRefCountPtr<FRDGPooledBuffer> ScatterBuffer;
	TRefCountPtr<FRDGPooledBuffer> UploadBuffer;
};

class FRDGScatterUploadBuilder
{
public:
	using FPassFunction = TFunction<void(FRDGScatterUploader&)>;
	using FCommandListFunction = TFunction<void(FRHICommandList&)>;

	static UE::Tasks::FTask Process(
		FRDGBuilder& GraphBuilder,
		FRDGAsyncScatterUploadBuffer& UploadBuffer,
		FRDGViewableResource* DstResource,
		uint32 NumElements,
		uint32 NumBytesPerElement,
		const TCHAR* Name,
		FPassFunction&& Function)
	{
		FRDGScatterUploadBuilder* Builder = Create(GraphBuilder);
		Builder->AddPass(GraphBuilder, UploadBuffer, DstResource, NumElements, NumBytesPerElement, Name, MoveTemp(Function));
		return Builder->Execute(GraphBuilder);
	}

	static UE::Tasks::FTask Process_PreSized(
		FRDGBuilder& GraphBuilder,
		FRDGAsyncScatterUploadBuffer& UploadBuffer,
		FRDGViewableResource* DstResource,
		uint32 NumElements,
		uint32 NumBytesPerElement,
		const TCHAR* Name,
		FPassFunction&& Function)
	{
		FRDGScatterUploadBuilder* Builder = Create(GraphBuilder);
		Builder->AddPass_PreSized(GraphBuilder, UploadBuffer, DstResource, NumElements, NumBytesPerElement, Name, MoveTemp(Function));
		return Builder->Execute(GraphBuilder);
	}

	static RENDERCORE_API FRDGScatterUploadBuilder* Create(FRDGBuilder& GraphBuilder);

	/** Init with pre-existing destination index data, performs a bulk-copy. */
	void AddPass(
		FRDGBuilder& GraphBuilder,
		FRDGAsyncScatterUploadBuffer& UploadBuffer,
		FRDGViewableResource* DstResource,
		uint32 NumElements,
		uint32 NumBytesPerElement,
		const TCHAR* Name,
		FPassFunction&& Function)
	{
		Passes.Emplace(UploadBuffer, UploadBuffer.Begin(GraphBuilder, DstResource, NumElements, NumBytesPerElement, Name), MoveTemp(Function));
		MaxBytes += NumElements * NumBytesPerElement;
	}

	/** Init with presized num scatters, expecting each to be set at a later point. Requires the user to keep track of the offsets to use. */
	void AddPass_PreSized(
		FRDGBuilder& GraphBuilder,
		FRDGAsyncScatterUploadBuffer& UploadBuffer,
		FRDGViewableResource* DstResource,
		uint32 NumElements,
		uint32 NumBytesPerElement,
		const TCHAR* Name,
		FPassFunction&& Function)
	{
		Passes.Emplace(UploadBuffer, UploadBuffer.BeginPreSized(GraphBuilder, DstResource, NumElements, NumBytesPerElement, Name), MoveTemp(Function));
		MaxBytes += NumElements * NumBytesPerElement;
	}

	/** AddPass overload with deferred scatter count. GetMaxScattersFunction runs after prerequisite tasks
	  * but before Lock, and returns the actual maximum number of scatters. Must be <= MaxNumElements. */
	void AddPass(
		FRDGBuilder& GraphBuilder,
		FRDGAsyncScatterUploadBuffer& UploadBuffer,
		FRDGViewableResource* DstResource,
		uint32 MaxNumElements,
		uint32 NumBytesPerElement,
		const TCHAR* Name,
		TFunction<uint32()>&& GetMaxScattersFunction,
		FPassFunction&& Function)
	{
		Passes.Emplace(UploadBuffer, UploadBuffer.Begin(GraphBuilder, DstResource, MaxNumElements, NumBytesPerElement, Name), MoveTemp(Function));
		Passes.Last().GetMaxScattersFunction = MoveTemp(GetMaxScattersFunction);
		MaxBytes += MaxNumElements * NumBytesPerElement;
	}

	void AddFunction(FCommandListFunction&& InFunction)
	{
		CommandListFunctions.Emplace(MoveTemp(InFunction));
	}

	void AddPrerequisiteTask(const UE::Tasks::FTask& Task)
	{
		PrerequisiteTasks.Emplace(Task);
	}

	RENDERCORE_API UE::Tasks::FTask Execute(FRDGBuilder& GraphBuilder);

private:
	FRDGScatterUploadBuilder() = default;

	struct FPass
	{
		FPass(FRDGAsyncScatterUploadBuffer& InUploadBuffer, FRDGScatterUploader* InUploader, FPassFunction&& InFunction)
			: UploadBuffer(InUploadBuffer)
			, Uploader(InUploader)
			, Function(InFunction)
		{}

		FRDGAsyncScatterUploadBuffer& UploadBuffer;
		FRDGScatterUploader* Uploader = nullptr;
		FPassFunction Function;
		TFunction<uint32()> GetMaxScattersFunction;
	};
	TArray<FPass, SceneRenderingAllocator> Passes;
	TArray<FCommandListFunction, SceneRenderingAllocator> CommandListFunctions;
	TArray<UE::Tasks::FTask, SceneRenderingAllocator> PrerequisiteTasks;
	uint32 MaxBytes = 0;

	RDG_FRIEND_ALLOCATOR_FRIEND(FRDGScatterUploadBuilder);
};

class FRDGScatterUploadBuffer
	: public FRDGScatterUploadBase
{
public:
	/**
	 * Init with presized num scatters, expecting each to be set at a later point. Requires the user to keep track of the offsets to use.
	 */
	RENDERCORE_API void InitPreSized(FRDGBuilder& GraphBuilder, uint32 NumElements, uint32 InNumBytesPerElement, bool bInFloat4Buffer, const TCHAR* DebugName);

	/**
	 * Init with pre-existing destination index data, performs a bulk-copy.
	 */
	RENDERCORE_API void Init(FRDGBuilder& GraphBuilder, TArrayView<const uint32> ElementScatterOffsets, uint32 InNumBytesPerElement, bool bInFloat4Buffer, const TCHAR* DebugName);

	RENDERCORE_API void Init(FRDGBuilder& GraphBuilder, uint32 NumElements, uint32 NumBytesPerElement, bool bInFloat4Buffer, const TCHAR* Name);

	inline void ResourceUploadTo(FRDGBuilder& GraphBuilder, FRDGBuffer* DstResource)
	{
		ResourceUploadToInternal(GraphBuilder, DstResource);
	}

	RENDERCORE_API void Release();

	RENDERCORE_API uint32 GetNumBytes() const;

private:
	RENDERCORE_API void ResourceUploadToInternal(FRDGBuilder& GraphBuilder, FRDGViewableResource* DstResource);
	RENDERCORE_API void Reset();

	TRefCountPtr<FRDGPooledBuffer> ScatterBuffer;
	TRefCountPtr<FRDGPooledBuffer> UploadBuffer;

	bool bFloat4Buffer = false;
};
