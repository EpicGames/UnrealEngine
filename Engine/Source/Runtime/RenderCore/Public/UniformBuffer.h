// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	UniformBuffer.h: Uniform buffer declarations.
=============================================================================*/

#pragma once

#include "ShaderParameterMacros.h"
#include "RenderingThread.h"
#include "RenderResource.h"
#include "Templates/IsArrayOrRefOfType.h"

class FShaderUniformBufferParameter;
template<typename TBufferStruct> class TShaderUniformBufferParameter;

/** Creates a
uniform buffer with the given value, and returns a structured reference to it. */
template<typename TBufferStruct>
TUniformBufferRef<TBufferStruct> CreateUniformBufferImmediate(const TBufferStruct& Value, EUniformBufferUsage Usage, EUniformBufferValidation Validation = EUniformBufferValidation::ValidateResources)
{
	return TUniformBufferRef<TBufferStruct>::CreateUniformBufferImmediate(Value, Usage, Validation);
}


/** A uniform buffer resource. */
template<typename TBufferStruct>
class TUniformBuffer : public FRenderResource
{
	static_assert(!TIsUECoreVariant<TBufferStruct, double>::Value, "UniformBufferRHIRef cannot be double core variants! Switch to float variant.");

public:

	TUniformBuffer()
		: BufferUsage(UniformBuffer_MultiFrame)
		, Contents(nullptr)
	{
	}

	~TUniformBuffer()
	{
		if (Contents)
		{
			FMemory::Free(Contents);
		}
	}

	/** Sets the contents of the uniform buffer. */
	void SetContents(FRHICommandListBase& RHICmdList, const TBufferStruct& NewContents)
	{
		SetContentsNoUpdate(NewContents);
		UpdateRHI(RHICmdList);
	}

	/** Sets the contents of the uniform buffer to all zeros. */
	void SetContentsToZero(FRHICommandListBase& RHICmdList)
	{
		if (!Contents)
		{
			Contents = (uint8*)FMemory::Malloc(sizeof(TBufferStruct), SHADER_PARAMETER_STRUCT_ALIGNMENT);
		}
		FMemory::Memzero(Contents, sizeof(TBufferStruct));
		UpdateRHI(RHICmdList);
	}

	const uint8* GetContents() const 
	{
		return Contents;
	}

	// FRenderResource interface.
	virtual void InitRHI(FRHICommandListBase& RHICmdList) override
	{
		UniformBufferRHI.SafeRelease();
		if (Contents)
		{
			UniformBufferRHI = CreateUniformBufferImmediate<TBufferStruct>(*((const TBufferStruct*)Contents), BufferUsage);
		}
	}
	virtual void ReleaseRHI() override
	{
		UniformBufferRHI.SafeRelease();
	}

	// Accessors.
	FRHIUniformBuffer* GetUniformBufferRHI() const
	{ 
		checkSlow(IsInParallelRenderingThread() || IsInRenderingThread());
		checkf(UniformBufferRHI.GetReference(), TEXT("Attempted to access UniformBufferRHI on a TUniformBuffer that was never filled in with anything")); 
		check(UniformBufferRHI.GetReference()); // you are trying to use a UB that was never filled with anything
		return UniformBufferRHI; 
	}

	const TUniformBufferRef<TBufferStruct>& GetUniformBufferRef() const
	{
		check(UniformBufferRHI.GetReference()); // you are trying to use a UB that was never filled with anything
		return UniformBufferRHI;
	}

	EUniformBufferUsage BufferUsage;

protected:

	/** Sets the contents of the uniform buffer. Used within calls to InitRHI */
	void SetContentsNoUpdate(const TBufferStruct& NewContents)
	{
		if (!Contents)
		{
			Contents = (uint8*)FMemory::Malloc(sizeof(TBufferStruct), SHADER_PARAMETER_STRUCT_ALIGNMENT);
		}
		FMemory::Memcpy(Contents,&NewContents,sizeof(TBufferStruct));
	}

private:

	TUniformBufferRef<TBufferStruct> UniformBufferRHI;
	uint8* Contents;
};


/** Sends a message to the rendering thread to set the contents of a uniform buffer.  Called by the game thread. */
template<typename TBufferStruct>
void BeginSetUniformBufferContents(
	TUniformBuffer<TBufferStruct>& UniformBuffer,
	const TBufferStruct& Struct
	)
{
	TUniformBuffer<TBufferStruct>* UniformBufferPtr = &UniformBuffer;
	ENQUEUE_RENDER_COMMAND(SetUniformBufferContents)(
		[UniformBufferPtr, Struct](FRHICommandListImmediate& RHICmdList)
		{
			UniformBufferPtr->SetContents(RHICmdList, Struct);
		});
}
