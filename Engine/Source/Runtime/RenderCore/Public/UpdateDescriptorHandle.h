// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "GlobalShader.h"
#include "Shader.h"
#include "ShaderCompilerCore.h"
#include "ShaderCore.h"
#include "ShaderParameterUtils.h"
#include "ShaderParameters.h"
#include "ShaderPlatformConfig.h"
#include "DataDrivenShaderPlatformInfo.h"

struct FUpdateDescriptorHandleCS : public FGlobalShader
{
public:
    DECLARE_EXPORTED_SHADER_TYPE(FUpdateDescriptorHandleCS, Global, RENDERCORE_API);
    
    FUpdateDescriptorHandleCS() {}
    FUpdateDescriptorHandleCS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
        : FGlobalShader(Initializer)
    {
        NumUpdates.Bind(Initializer.ParameterMap, TEXT("NumUpdates"), SPF_Mandatory);
        DescriptorIndices.Bind(Initializer.ParameterMap, TEXT("DescriptorIndices"), SPF_Mandatory);
        DescriptorEntries.Bind(Initializer.ParameterMap, TEXT("DescriptorEntries"), SPF_Mandatory);
        OutputData.Bind(Initializer.ParameterMap, TEXT("OutputData"), SPF_Mandatory);
    }
    
    static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
    {
		return IsMetalPlatform(Parameters.Platform) && 
				FDataDrivenShaderPlatformInfo::GetSupportsBindless(Parameters.Platform) &&
				FShaderPlatformConfig::GetBindlessConfiguration(Parameters.Platform) != ERHIBindlessConfiguration::Disabled;
    }
    
    static inline void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
    {
        FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
    }
    
    LAYOUT_FIELD(FShaderParameter, NumUpdates);
    LAYOUT_FIELD(FShaderResourceParameter, DescriptorIndices);
    LAYOUT_FIELD(FShaderResourceParameter, DescriptorEntries);
    LAYOUT_FIELD(FShaderResourceParameter, OutputData);
};
