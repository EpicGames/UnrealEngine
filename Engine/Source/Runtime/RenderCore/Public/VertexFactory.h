// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	VertexFactory.h: Vertex factory definitions.
=============================================================================*/

#pragma once

#include "Containers/Array.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "Containers/EnumAsByte.h"
#include "Containers/List.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "HAL/PlatformCrt.h"
#include "Math/NumericLimits.h"
#include "Misc/AssertionMacros.h"
#include "Misc/CString.h"
#include "Misc/EnumClassFlags.h"
#include "Misc/SecureHash.h"
#include "RHI.h"
#include "RHIDefinitions.h"
#include "RenderResource.h"
#include "Serialization/Archive.h"
#include "Serialization/MemoryImage.h"
#include "Serialization/MemoryLayout.h"
#include "Shader.h"
#include "ShaderCore.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/NameTypes.h"
#include "VertexStreamComponent.h"

#include <atomic>

class FCbFieldView;
class FCbWriter;
class FMaterial;
class FMeshDrawSingleShaderBindings;
class FPrimitiveSceneProxy;
struct FVertexFactoryShaderPermutationParameters;

#if PLATFORM_SUPPORTS_PRAGMA_PACK
	#pragma pack (push,4)
#endif

struct FVertexInputStream
{
	uint32 StreamIndex : 4;
	uint32 Offset : 27;
	uint32 bStreamSourceSlot : 1;

	union
	{
		// Contains the direct vertex buffer pointer
		FRHIBuffer* VertexBuffer;

		// Contains the stream source slot
		FRHIStreamSourceSlot* StreamSourceSlot;

		// Represents either ptr type as an opaque int for comparisons.
		void* Pointer;
	};

	RENDERCORE_API void SetOnRHICommandList(FRHICommandList& RHICmdList) const;

	FVertexInputStream() :
		StreamIndex(0),
		Offset(0),
		bStreamSourceSlot(0),
		VertexBuffer(nullptr)
	{}
	
	FVertexInputStream(uint32 InStreamIndex, uint32 InOffset, TYPE_OF_NULLPTR)
		: StreamIndex(InStreamIndex), Offset(InOffset), bStreamSourceSlot(0), VertexBuffer(nullptr)
	{
		// Verify no overflow
		checkSlow(InStreamIndex == StreamIndex && InOffset == Offset);
	}

	FVertexInputStream(uint32 InStreamIndex, uint32 InOffset, FRHIStreamSourceSlot* InStreamSourceSlot)
		: StreamIndex(InStreamIndex), Offset(InOffset), bStreamSourceSlot(1), StreamSourceSlot(InStreamSourceSlot)
	{
		// Verify no overflow
		checkSlow(InStreamIndex == StreamIndex && InOffset == Offset);
	}

	FVertexInputStream(uint32 InStreamIndex, uint32 InOffset, FRHIBuffer* InVertexBuffer)
		: StreamIndex(InStreamIndex), Offset(InOffset), bStreamSourceSlot(0), VertexBuffer(InVertexBuffer)
	{
		// Verify no overflow
		checkSlow(InStreamIndex == StreamIndex && InOffset == Offset);
	}

	inline bool operator==(const FVertexInputStream& rhs) const
	{
		if (StreamIndex != rhs.StreamIndex ||
			Offset != rhs.Offset ||
			Pointer != rhs.Pointer)
		{
			return false;
		}

		return true;
	}

	inline bool operator!=(const FVertexInputStream& rhs) const
	{
		return !(*this == rhs);
	}
};

#if PLATFORM_SUPPORTS_PRAGMA_PACK
	#pragma pack (pop)
#endif

/** 
 * Number of vertex input bindings to allocate inline within a FMeshDrawCommand.
 * This is tweaked so that the bindings for commonly cached factories (FLocalVertexFactory, FGPUBaseSkinVertexFactory) fit into the inline storage.
 * Overflow of the inline storage will cause a heap allocation per draw (and corresponding cache miss on traversal)
 */
typedef TArray<FVertexInputStream, TInlineAllocator<7>> FVertexInputStreamArray;

ENUM_CLASS_FLAGS(EVertexStreamUsage);

enum class EVertexInputStreamType : uint8
{
	Default = 0,
	PositionOnly,
	PositionAndNormalOnly,
	Count
};

enum class EVertexFactoryFlags : uint32
{
	None                                  = 0u,
	UsedWithMaterials                     = 1u << 1,
	SupportsStaticLighting                = 1u << 2,
	SupportsDynamicLighting               = 1u << 3,
	SupportsPrecisePrevWorldPos           = 1u << 4,
	SupportsPositionOnly                  = 1u << 5,
	SupportsCachingMeshDrawCommands       = 1u << 6,
	SupportsPrimitiveIdStream             = 1u << 7,
	SupportsNaniteRendering               = 1u << 8,
	SupportsRayTracing                    = 1u << 9,
	SupportsRayTracingDynamicGeometry     = 1u << 10,
	SupportsRayTracingProceduralPrimitive = 1u << 11,
	SupportsLightmapBaking                = 1u << 12,
	SupportsPSOPrecaching                 = 1u << 13,
	SupportsManualVertexFetch             = 1u << 14,
	DoesNotSupportNullPixelShader         = 1u << 15,
	SupportsGPUSkinPassThrough            = 1u << 16,
	SupportsComputeShading                = 1u << 17,
	SupportsMeshShading                   = 1u << 18,
	SupportsLumenMeshCards                = 1u << 19,
	SupportsLandscape                     = 1u << 20,
	IncludesFwdShader                     = 1u << 21,
	SupportsTriangleSorting               = 1u << 22,
};
ENUM_CLASS_FLAGS(EVertexFactoryFlags);

/**
 * A macro which initializes a FVertexStreamComponent to read a member from a struct.
 */
#define STRUCTMEMBER_VERTEXSTREAMCOMPONENT(VertexBuffer,VertexType,Member,MemberType) \
	FVertexStreamComponent(VertexBuffer,STRUCT_OFFSET(VertexType,Member),sizeof(VertexType),MemberType)

/**
 * An interface to the parameter bindings for the vertex factory used by a shader.
 */
class FVertexFactoryShaderParameters
{
	DECLARE_EXPORTED_TYPE_LAYOUT(FVertexFactoryShaderParameters, RENDERCORE_API, NonVirtual);
public:
	void Bind(const class FShaderParameterMap& ParameterMap) {}

	/** 
	 * Gets the vertex factory's shader bindings and vertex streams.
	 * View can be null when caching mesh draw commands (only for supported vertex factories)
	 */
	void GetElementShaderBindings(
		const class FSceneInterface* Scene,
		const class FSceneView* View,
		const class FMeshMaterialShader* Shader,
		const EVertexInputStreamType InputStreamType,
		ERHIFeatureLevel::Type FeatureLevel,
		const class FVertexFactory* VertexFactory,
		const struct FMeshBatchElement& BatchElement,
		class FMeshDrawSingleShaderBindings& ShaderBindings,
		FVertexInputStreamArray& VertexStreams) const {}

private:
	// Should remove, but first need to fix some logic surrounding empty base classes
	LAYOUT_FIELD_INITIALIZED(uint32, Size_DEPRECATED, 0u);
};

template<EShaderFrequency ShaderFrequency, typename VertexFactoryType>
struct TVertexFactoryParameterTraits
{
	static const FTypeLayoutDesc* GetLayout() { return nullptr; }
	static FVertexFactoryShaderParameters* Create(const class FShaderParameterMap& ParameterMap) { return nullptr; }
	
	static void GetElementShaderBindings(
		const FVertexFactoryShaderParameters* Parameters,
		const class FSceneInterface* Scene,
		const class FSceneView* View,
		const class FMeshMaterialShader* Shader,
		const EVertexInputStreamType InputStreamType,
		ERHIFeatureLevel::Type FeatureLevel,
		const class FVertexFactory* VertexFactory,
		const struct FMeshBatchElement& BatchElement,
		class FMeshDrawSingleShaderBindings& ShaderBindings,
		FVertexInputStreamArray& VertexStreams) {}
};

#define IMPLEMENT_VERTEX_FACTORY_PARAMETER_TYPE(FactoryClass, ShaderFrequency, ParameterClass) \
	template<> struct TVertexFactoryParameterTraits<ShaderFrequency, FactoryClass> \
	{ \
		static const FTypeLayoutDesc* GetLayout() { return &StaticGetTypeLayoutDesc<ParameterClass>(); } \
		static FVertexFactoryShaderParameters* Create(const class FShaderParameterMap& ParameterMap) { ParameterClass* Result = new ParameterClass(); Result->Bind(ParameterMap); return Result; } \
		static void GetElementShaderBindings( \
			const FVertexFactoryShaderParameters* Parameters, \
			const class FSceneInterface* Scene, \
			const class FSceneView* View, \
			const class FMeshMaterialShader* Shader, \
			const EVertexInputStreamType InputStreamType, \
			ERHIFeatureLevel::Type FeatureLevel, \
			const class FVertexFactory* VertexFactory, \
			const struct FMeshBatchElement& BatchElement, \
			class FMeshDrawSingleShaderBindings& ShaderBindings, \
			FVertexInputStreamArray& VertexStreams) \
		{ \
			static_cast<const ParameterClass*>(Parameters)->GetElementShaderBindings(Scene, View, Shader, InputStreamType, FeatureLevel, VertexFactory, BatchElement, ShaderBindings, VertexStreams); \
		} \
	}

template<typename VertexFactoryType>
static const FTypeLayoutDesc* GetVertexFactoryParametersLayout(EShaderFrequency ShaderFrequency)
{
	switch (ShaderFrequency)
	{
	case SF_Vertex: return TVertexFactoryParameterTraits<SF_Vertex, VertexFactoryType>::GetLayout();
	case SF_Mesh: return TVertexFactoryParameterTraits<SF_Mesh, VertexFactoryType>::GetLayout();
	case SF_Amplification: return TVertexFactoryParameterTraits<SF_Amplification, VertexFactoryType>::GetLayout();
	case SF_Pixel: return TVertexFactoryParameterTraits<SF_Pixel, VertexFactoryType>::GetLayout();
	case SF_Geometry: return TVertexFactoryParameterTraits<SF_Geometry, VertexFactoryType>::GetLayout();
	case SF_Compute: return TVertexFactoryParameterTraits<SF_Compute, VertexFactoryType>::GetLayout();
	case SF_RayGen: return TVertexFactoryParameterTraits<SF_RayGen, VertexFactoryType>::GetLayout();
	case SF_RayMiss: return TVertexFactoryParameterTraits<SF_RayMiss, VertexFactoryType>::GetLayout();
	case SF_RayHitGroup: return TVertexFactoryParameterTraits<SF_RayHitGroup, VertexFactoryType>::GetLayout();
	case SF_RayCallable: return TVertexFactoryParameterTraits<SF_RayCallable, VertexFactoryType>::GetLayout();
	case SF_WorkGraphRoot: return TVertexFactoryParameterTraits<SF_WorkGraphRoot, VertexFactoryType>::GetLayout();
	case SF_WorkGraphComputeNode: return TVertexFactoryParameterTraits<SF_WorkGraphComputeNode, VertexFactoryType>::GetLayout();
	default: checkNoEntry(); return nullptr;
	}
}

template<typename VertexFactoryType>
static FVertexFactoryShaderParameters* ConstructVertexFactoryParameters(EShaderFrequency ShaderFrequency, const class FShaderParameterMap& ParameterMap)
{
	switch (ShaderFrequency)
	{
	case SF_Vertex: return TVertexFactoryParameterTraits<SF_Vertex, VertexFactoryType>::Create(ParameterMap);
	case SF_Mesh: return TVertexFactoryParameterTraits<SF_Mesh, VertexFactoryType>::Create(ParameterMap);
	case SF_Amplification: return TVertexFactoryParameterTraits<SF_Amplification, VertexFactoryType>::Create(ParameterMap);
	case SF_Pixel: return TVertexFactoryParameterTraits<SF_Pixel, VertexFactoryType>::Create(ParameterMap);
	case SF_Geometry: return TVertexFactoryParameterTraits<SF_Geometry, VertexFactoryType>::Create(ParameterMap);
	case SF_Compute: return TVertexFactoryParameterTraits<SF_Compute, VertexFactoryType>::Create(ParameterMap);
	case SF_RayGen: return TVertexFactoryParameterTraits<SF_RayGen, VertexFactoryType>::Create(ParameterMap);
	case SF_RayMiss: return TVertexFactoryParameterTraits<SF_RayMiss, VertexFactoryType>::Create(ParameterMap);
	case SF_RayHitGroup: return TVertexFactoryParameterTraits<SF_RayHitGroup, VertexFactoryType>::Create(ParameterMap);
	case SF_RayCallable: return TVertexFactoryParameterTraits<SF_RayCallable, VertexFactoryType>::Create(ParameterMap);
	case SF_WorkGraphRoot: return TVertexFactoryParameterTraits<SF_WorkGraphRoot, VertexFactoryType>::Create(ParameterMap);
	case SF_WorkGraphComputeNode: return TVertexFactoryParameterTraits<SF_WorkGraphComputeNode, VertexFactoryType>::Create(ParameterMap);
	default: checkNoEntry(); return nullptr;
	}
}


template<typename VertexFactoryType>
static void GetVertexFactoryParametersElementShaderBindings(EShaderFrequency ShaderFrequency, const FVertexFactoryShaderParameters* Parameters,
	const class FSceneInterface* Scene,
	const class FSceneView* View,
	const class FMeshMaterialShader* Shader,
	const EVertexInputStreamType InputStreamType,
	ERHIFeatureLevel::Type FeatureLevel,
	const class FVertexFactory* VertexFactory,
	const struct FMeshBatchElement& BatchElement,
	class FMeshDrawSingleShaderBindings& ShaderBindings,
	FVertexInputStreamArray& VertexStreams)
{
	switch (ShaderFrequency)
	{
	case SF_Vertex: TVertexFactoryParameterTraits<SF_Vertex, VertexFactoryType>::GetElementShaderBindings(Parameters, Scene, View, Shader, InputStreamType, FeatureLevel, VertexFactory, BatchElement, ShaderBindings, VertexStreams); break;
	case SF_Mesh: TVertexFactoryParameterTraits<SF_Mesh, VertexFactoryType>::GetElementShaderBindings(Parameters, Scene, View, Shader, InputStreamType, FeatureLevel, VertexFactory, BatchElement, ShaderBindings, VertexStreams); break;
	case SF_Amplification: TVertexFactoryParameterTraits<SF_Amplification, VertexFactoryType>::GetElementShaderBindings(Parameters, Scene, View, Shader, InputStreamType, FeatureLevel, VertexFactory, BatchElement, ShaderBindings, VertexStreams); break;
	case SF_Pixel: TVertexFactoryParameterTraits<SF_Pixel, VertexFactoryType>::GetElementShaderBindings(Parameters, Scene, View, Shader, InputStreamType, FeatureLevel, VertexFactory, BatchElement, ShaderBindings, VertexStreams); break;
	case SF_Geometry: TVertexFactoryParameterTraits<SF_Geometry, VertexFactoryType>::GetElementShaderBindings(Parameters, Scene, View, Shader, InputStreamType, FeatureLevel, VertexFactory, BatchElement, ShaderBindings, VertexStreams); break;
	case SF_Compute: TVertexFactoryParameterTraits<SF_Compute, VertexFactoryType>::GetElementShaderBindings(Parameters, Scene, View, Shader, InputStreamType, FeatureLevel, VertexFactory, BatchElement, ShaderBindings, VertexStreams); break;
	case SF_RayGen: TVertexFactoryParameterTraits<SF_RayGen, VertexFactoryType>::GetElementShaderBindings(Parameters, Scene, View, Shader, InputStreamType, FeatureLevel, VertexFactory, BatchElement, ShaderBindings, VertexStreams); break;
	case SF_RayMiss: TVertexFactoryParameterTraits<SF_RayMiss, VertexFactoryType>::GetElementShaderBindings(Parameters, Scene, View, Shader, InputStreamType, FeatureLevel, VertexFactory, BatchElement, ShaderBindings, VertexStreams); break;
	case SF_RayHitGroup: TVertexFactoryParameterTraits<SF_RayHitGroup, VertexFactoryType>::GetElementShaderBindings(Parameters, Scene, View, Shader, InputStreamType, FeatureLevel, VertexFactory, BatchElement, ShaderBindings, VertexStreams); break;
	case SF_RayCallable: TVertexFactoryParameterTraits<SF_RayCallable, VertexFactoryType>::GetElementShaderBindings(Parameters, Scene, View, Shader, InputStreamType, FeatureLevel, VertexFactory, BatchElement, ShaderBindings, VertexStreams); break;
	case SF_WorkGraphRoot: TVertexFactoryParameterTraits<SF_WorkGraphRoot, VertexFactoryType>::GetElementShaderBindings(Parameters, Scene, View, Shader, InputStreamType, FeatureLevel, VertexFactory, BatchElement, ShaderBindings, VertexStreams); break;
	case SF_WorkGraphComputeNode: TVertexFactoryParameterTraits<SF_WorkGraphComputeNode, VertexFactoryType>::GetElementShaderBindings(Parameters, Scene, View, Shader, InputStreamType, FeatureLevel, VertexFactory, BatchElement, ShaderBindings, VertexStreams); break;
	default: checkNoEntry(); break;
	}
}

/**
 * An object used to represent the type of a vertex factory.
 */
class FVertexFactoryType
{
public:

	typedef FVertexFactoryShaderParameters* (*ConstructParametersType)(EShaderFrequency ShaderFrequency, const class FShaderParameterMap& ParameterMap);
	typedef const FTypeLayoutDesc* (*GetParameterTypeLayoutType)(EShaderFrequency ShaderFrequency);
	typedef void (*GetParameterTypeElementShaderBindingsType)(EShaderFrequency ShaderFrequency,
		const FVertexFactoryShaderParameters* Parameters,
		const class FSceneInterface* Scene,
		const class FSceneView* View,
		const class FMeshMaterialShader* Shader,
		const EVertexInputStreamType InputStreamType,
		ERHIFeatureLevel::Type FeatureLevel,
		const class FVertexFactory* VertexFactory,
		const struct FMeshBatchElement& BatchElement,
		class FMeshDrawSingleShaderBindings& ShaderBindings,
		FVertexInputStreamArray& VertexStreams);
	typedef void (*GetPSOPrecacheVertexFetchElementsType)(EVertexInputStreamType VertexInputStreamType, FVertexDeclarationElementList& Elements);
	typedef bool (*ShouldCacheType)(const FVertexFactoryShaderPermutationParameters&);

#if WITH_EDITOR
	typedef void (*ModifyCompilationEnvironmentType)(const FVertexFactoryShaderPermutationParameters&, FShaderCompilerEnvironment&);
	typedef void (*ValidateCompiledResultType)(const FVertexFactoryType*, EShaderPlatform, const FShaderParameterMap& ParameterMap, TArray<FString>& OutErrors);
#endif // WITH_EDITOR

	static int32 GetNumVertexFactoryTypes() { return NumVertexFactories; }

	/**
	 * @return The global shader factory list.
	 */
	static RENDERCORE_API TLinkedList<FVertexFactoryType*>*& GetTypeList();

	static RENDERCORE_API const TArray<FVertexFactoryType*>& GetSortedMaterialTypes();

	/**
	 * Finds a FVertexFactoryType by name.
	 */
	static RENDERCORE_API FVertexFactoryType* GetVFByName(const FHashedName& VFName);

	/** Initialize FVertexFactoryType static members, this must be called before any VF types are created. */
	static void Initialize(const TMap<FString, TArray<const TCHAR*> >& ShaderFileToUniformBufferVariables);

	RENDERCORE_API FVertexFactoryType(
		const TCHAR* InName,
		const TCHAR* InShaderFilename,
		EVertexFactoryFlags InFlags,
		ConstructParametersType InConstructParameters,
		GetParameterTypeLayoutType InGetParameterTypeLayout,
		GetParameterTypeElementShaderBindingsType InGetParameterTypeElementShaderBindings,
		GetPSOPrecacheVertexFetchElementsType InGetPSOPrecacheVertexFetchElements,
		ShouldCacheType InShouldCache
#if WITH_EDITOR
		, ModifyCompilationEnvironmentType InModifyCompilationEnvironment
		, ValidateCompiledResultType InValidateCompiledResult
#endif // WITH_EDITOR
		);

	RENDERCORE_API virtual ~FVertexFactoryType();

	// Accessors.
	const TCHAR* GetName() const { return Name; }
	FName GetFName() const { return TypeName; }
	const FHashedName& GetHashedName() const { return HashedName; }
	const TCHAR* GetShaderFilename() const { return ShaderFilename; }
	const TCHAR* GetShaderFwdFilename() const { return *ShaderFwdFilename; }

	FVertexFactoryShaderParameters* CreateShaderParameters(EShaderFrequency ShaderFrequency, const class FShaderParameterMap& ParameterMap) const { return (*ConstructParameters)(ShaderFrequency, ParameterMap); }
	const FTypeLayoutDesc* GetShaderParameterLayout(EShaderFrequency ShaderFrequency) const { return (*GetParameterTypeLayout)(ShaderFrequency); }
	void GetShaderParameterElementShaderBindings(EShaderFrequency ShaderFrequency,
		const FVertexFactoryShaderParameters* Parameters,
		const class FSceneInterface* Scene,
		const class FSceneView* View,
		const class FMeshMaterialShader* Shader,
		const EVertexInputStreamType InputStreamType,
		ERHIFeatureLevel::Type FeatureLevel,
		const class FVertexFactory* VertexFactory,
		const struct FMeshBatchElement& BatchElement,
		class FMeshDrawSingleShaderBindings& ShaderBindings,
		FVertexInputStreamArray& VertexStreams) const { (*GetParameterTypeElementShaderBindings)(ShaderFrequency, Parameters, Scene, View, Shader, InputStreamType, FeatureLevel, VertexFactory, BatchElement, ShaderBindings, VertexStreams); }
	void GetShaderPSOPrecacheVertexFetchElements(EVertexInputStreamType VertexInputStreamType, FVertexDeclarationElementList& Elements) const { return (*GetPSOPrecacheVertexFetchElements)(VertexInputStreamType, Elements); }

	EVertexFactoryFlags GetFlags() const
	{
		return Flags;
	}

	/** Returns true if this vertex factory supports ALL feature flags in FlagsToCheck */
	bool HasFlags(EVertexFactoryFlags FlagsToCheck) const
	{
		return EnumHasAllFlags(Flags, FlagsToCheck);
	}

	bool IsUsedWithMaterials() const                   { return HasFlags(EVertexFactoryFlags::UsedWithMaterials); }
	bool SupportsStaticLighting() const                { return HasFlags(EVertexFactoryFlags::SupportsStaticLighting); }
	bool SupportsDynamicLighting() const               { return HasFlags(EVertexFactoryFlags::SupportsDynamicLighting); }
	bool SupportsPrecisePrevWorldPos() const           { return HasFlags(EVertexFactoryFlags::SupportsPrecisePrevWorldPos); }
	bool SupportsPositionOnly() const                  { return HasFlags(EVertexFactoryFlags::SupportsPositionOnly); }
	bool SupportsCachingMeshDrawCommands() const       { return HasFlags(EVertexFactoryFlags::SupportsCachingMeshDrawCommands); }
	bool SupportsPrimitiveIdStream() const             { return HasFlags(EVertexFactoryFlags::SupportsPrimitiveIdStream); }
	bool SupportsNaniteRendering() const               { return HasFlags(EVertexFactoryFlags::SupportsNaniteRendering); }
	bool SupportsRayTracing() const                    { return HasFlags(EVertexFactoryFlags::SupportsRayTracing); }
	bool SupportsRayTracingDynamicGeometry() const     { return HasFlags(EVertexFactoryFlags::SupportsRayTracingDynamicGeometry); }
	bool SupportsRayTracingProceduralPrimitive() const { return HasFlags(EVertexFactoryFlags::SupportsRayTracingProceduralPrimitive); }
	bool SupportsLightmapBaking() const                { return HasFlags(EVertexFactoryFlags::SupportsLightmapBaking); }
	bool SupportsPSOPrecaching() const                 { return HasFlags(EVertexFactoryFlags::SupportsPSOPrecaching); }
	bool SupportsNullPixelShader() const               { return !HasFlags(EVertexFactoryFlags::DoesNotSupportNullPixelShader); }
	bool SupportsGPUSkinPassThrough() const            { return HasFlags(EVertexFactoryFlags::SupportsGPUSkinPassThrough); }
	bool SupportsComputeShading() const                { return HasFlags(EVertexFactoryFlags::SupportsComputeShading); }
	bool SupportsMeshShading() const                   { return HasFlags(EVertexFactoryFlags::SupportsMeshShading); }
	bool SupportsLumenMeshCards() const                { return HasFlags(EVertexFactoryFlags::SupportsLumenMeshCards); }
	bool SupportsLandscape() const                     { return HasFlags(EVertexFactoryFlags::SupportsLandscape); }
	bool IncludesFwdShaderFile() const                 { return HasFlags(EVertexFactoryFlags::IncludesFwdShader); }
	bool SupportsTriangleSorting() const               { return HasFlags(EVertexFactoryFlags::SupportsTriangleSorting); }

	bool SupportsManualVertexFetch(ERHIFeatureLevel::Type InFeatureLevel) const 
	{
		return HasFlags(EVertexFactoryFlags::SupportsManualVertexFetch) && CheckManualVertexFetchSupport(InFeatureLevel);
	}

	static RENDERCORE_API bool CheckManualVertexFetchSupport(ERHIFeatureLevel::Type InFeatureLevel);

	// Hash function.
	friend uint32 GetTypeHash(const FVertexFactoryType* Type)
	{ 
		return Type ? GetTypeHash(Type->HashedName) : 0u;
	}

	/** Calculates a Hash based on this vertex factory type's source code and includes */
	RENDERCORE_API const FSHAHash& GetSourceHash(EShaderPlatform ShaderPlatform) const;

	/**
	 * Should we cache the material's shadertype on this platform with this vertex factory? 
	 */
	bool ShouldCache(const FVertexFactoryShaderPermutationParameters& Parameters) const
	{
		return (*ShouldCacheRef)(Parameters);
	}

#if WITH_EDITOR
	/**
	* Calls the function ptr for the shader type on the given environment
	* @param Environment - shader compile environment to modify
	*/
	void ModifyCompilationEnvironment(const FVertexFactoryShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment) const
	{
		// Set up the mapping from VertexFactory.usf to the vertex factory type's source code.
		FString VertexFactoryIncludeString = FString::Printf( TEXT("#include \"%s\""), GetShaderFilename() );
		OutEnvironment.IncludeVirtualPathToContentsMap.Add(TEXT("/Engine/Generated/VertexFactory.ush"), VertexFactoryIncludeString);

		if (IncludesFwdShaderFile())
		{
			FString VertexFactoryFwdIncludeString = FString::Printf(TEXT("#include \"%s\""), GetShaderFwdFilename());
			OutEnvironment.IncludeVirtualPathToContentsMap.Add(TEXT("/Engine/Generated/VertexFactoryFwd.ush"), VertexFactoryFwdIncludeString);
			OutEnvironment.SetDefine(TEXT("USE_VERTEX_FACTORY_FWD"), 1);
		}
		else
		{
			OutEnvironment.IncludeVirtualPathToContentsMap.Add(TEXT("/Engine/Generated/VertexFactoryFwd.ush"), TEXT("#include \"/Engine/Private/VertexFactoryDefaultFwd.ush\""));
		}

		OutEnvironment.SetDefine(TEXT("HAS_PRIMITIVE_UNIFORM_BUFFER"), 1);

		(*ModifyCompilationEnvironmentRef)(Parameters, OutEnvironment);
	}

	void ValidateCompiledResult(EShaderPlatform Platform, const FShaderParameterMap& ParameterMap, TArray<FString>& OutErrors) const
	{
		(*ValidateCompiledResultRef)(this, Platform, ParameterMap, OutErrors);
	}

	/** Adds include statements for uniform buffers that this shader type references, and builds a prefix for the shader file with the include statements. */
	RENDERCORE_API void AddUniformBufferIncludesToEnvironment(FShaderCompilerEnvironment& OutEnvironment, EShaderPlatform Platform) const;

	const TSet<const FShaderParametersMetadata*>& GetReferencedUniformBuffers() const
	{
		return ReferencedUniformBuffers;
	}

	RENDERCORE_API void UpdateReferencedUniformBufferNames(const TMap<FString, TArray<const TCHAR*>>& ShaderFileToUniformBufferVariables);
#endif // WITH_EDITOR

private:
	static RENDERCORE_API uint32 NumVertexFactories;

	/** Tracks whether engine vertex factory initialization has occurred */
	static bool bVertexFactoriesInitialized;

	const TCHAR* Name;
	const TCHAR* ShaderFilename;
	FString ShaderFwdFilename;
	FName TypeName;
	FHashedName HashedName;
	EVertexFactoryFlags Flags;
	ConstructParametersType ConstructParameters;
	GetParameterTypeLayoutType GetParameterTypeLayout;
	GetParameterTypeElementShaderBindingsType GetParameterTypeElementShaderBindings;
	GetPSOPrecacheVertexFetchElementsType GetPSOPrecacheVertexFetchElements;
	ShouldCacheType ShouldCacheRef;
#if WITH_EDITOR
	ModifyCompilationEnvironmentType ModifyCompilationEnvironmentRef;
	ValidateCompiledResultType ValidateCompiledResultRef;
#endif // WITH_EDITOR

	TLinkedList<FVertexFactoryType*> GlobalListLink;

#if WITH_EDITOR
	/** 
	 * Cache of referenced uniform buffer structs.
	 * These are derived from source files so they need to be flushed when editing and recompiling shaders on the fly. 
	 * FShaderType::Initialize will add the referenced uniform buffers, but this set may be updated by FlushShaderFileCache
	 */
	TSet<const FShaderParametersMetadata*> ReferencedUniformBuffers;
#endif // WITH_EDITOR
};

/**
 * Serializes a reference to a vertex factory type.
 */
extern RENDERCORE_API FArchive& operator<<(FArchive& Ar,FVertexFactoryType*& TypeRef);

/**
 * Find the vertex factory type with the given name.
 * @return NULL if no vertex factory type matched, otherwise a vertex factory type with a matching name.
 */
extern RENDERCORE_API FVertexFactoryType* FindVertexFactoryType(const FHashedName& TypeName);

/**
 * A macro for declaring a new vertex factory type, for use in the vertex factory class's definition body.
 */
#define DECLARE_VERTEX_FACTORY_TYPE(FactoryClass) \
	public: \
	static FVertexFactoryType StaticType; \
	virtual FVertexFactoryType* GetType() const override;

#define DECLARE_VERTEX_FACTORY_TYPE_API(FactoryClass, ModuleApi) \
	public: \
	ModuleApi static FVertexFactoryType StaticType; \
	ModuleApi virtual FVertexFactoryType* GetType() const override;

#if WITH_EDITOR
	#define IMPLEMENT_VERTEX_FACTORY_EDITOR_VTABLE(FactoryClass) \
		, FactoryClass::ModifyCompilationEnvironment \
		, FactoryClass::ValidateCompiledResult
#else
	#define IMPLEMENT_VERTEX_FACTORY_EDITOR_VTABLE(FactoryClass)
#endif // WITH_EDITOR

#define IMPLEMENT_VERTEX_FACTORY_VTABLE(FactoryClass) \
	&ConstructVertexFactoryParameters<FactoryClass>, \
	&GetVertexFactoryParametersLayout<FactoryClass>, \
	&GetVertexFactoryParametersElementShaderBindings<FactoryClass>, \
	FactoryClass::GetPSOPrecacheVertexFetchElements, \
	FactoryClass::ShouldCompilePermutation \
	IMPLEMENT_VERTEX_FACTORY_EDITOR_VTABLE(FactoryClass)

/**
 * A macro for implementing the static vertex factory type object, and specifying parameters used by the type.
 */
#define IMPLEMENT_VERTEX_FACTORY_TYPE(FactoryClass, ShaderFilename, Flags) \
	FVertexFactoryType FactoryClass::StaticType( \
		TEXT(#FactoryClass), \
		TEXT(ShaderFilename), \
		Flags, \
		IMPLEMENT_VERTEX_FACTORY_VTABLE(FactoryClass) \
		); \
		FVertexFactoryType* FactoryClass::GetType() const { return &StaticType; }
/**
* Templated macro to define a FVertexFactoryType. Can be used for templated Vertex Factories.
*/
#define IMPLEMENT_TEMPLATE_VERTEX_FACTORY_TYPE(TemplatePrefix, FactoryClass, ShaderFilename, Flags) \
	UE_REMOVE_OPTIONAL_PARENS(TemplatePrefix) FVertexFactoryType FactoryClass::StaticType( \
		TEXT(#FactoryClass), \
		TEXT(ShaderFilename), \
		Flags, \
		IMPLEMENT_VERTEX_FACTORY_VTABLE(FactoryClass) \
		); \
		UE_REMOVE_OPTIONAL_PARENS(TemplatePrefix) FVertexFactoryType* FactoryClass::GetType() const { return &StaticType; }

/** Encapsulates a dependency on a vertex factory type and saved state from that vertex factory type. */
class FVertexFactoryTypeDependency
{
public:
	FVertexFactoryTypeDependency() {}

	FHashedName VertexFactoryTypeName;

	/** Used to detect changes to the vertex factory source files. */
	FSHAHash VFSourceHash;

	friend FArchive& operator<<(FArchive& Ar,class FVertexFactoryTypeDependency& Ref)
	{
		Ar << Ref.VertexFactoryTypeName << Ref.VFSourceHash;
		return Ar;
	}

	bool operator==(const FVertexFactoryTypeDependency& Reference) const
	{
		return VertexFactoryTypeName == Reference.VertexFactoryTypeName && VFSourceHash == Reference.VFSourceHash;
	}

	bool operator!=(const FVertexFactoryTypeDependency& Reference) const
	{
		return !(*this == Reference);
	}

	/** Call GetShaderFileHash to get the cached value for the filename's hash in the current process. */
	RENDERCORE_API void RefreshCachedSourceHash(EShaderPlatform ShaderPlatform);

private:
#if WITH_EDITOR
	// Compact binary API with hidden friend operator<<
	RENDERCORE_API void Save(FCbWriter& Writer) const;
	bool TryLoad(FCbFieldView Field);
	friend inline FCbWriter& operator<<(FCbWriter& Writer, const FVertexFactoryTypeDependency& Value)
	{
		Value.Save(Writer);
		return Writer;
	}
	friend RENDERCORE_API bool LoadFromCompactBinary(FCbFieldView Field, FVertexFactoryTypeDependency& OutValue);
#endif
};

/** Used to compare two Vertex Factory types by name. */
class FCompareVertexFactoryTypes											
{																				
public:		
	inline bool operator()(const FVertexFactoryType& A, const FVertexFactoryType& B ) const
	{
		int32 AL = FCString::Strlen(A.GetName());
		int32 BL = FCString::Strlen(B.GetName());
		if ( AL == BL )
		{
			return FCString::Strncmp(A.GetName(), B.GetName(), AL) > 0;
		}
		return AL > BL;
	}
};

/**
 * Encapsulates a vertex data source which can be linked into a vertex shader.
 */
class FVertexFactory : public FRenderResource
{
public:
	FVertexFactory(ERHIFeatureLevel::Type InFeatureLevel) 
		: FRenderResource(InFeatureLevel)
	{
		for (int32 i = 0; i < UE_ARRAY_COUNT(PrimitiveIdStreamIndex); ++i)
		{
			PrimitiveIdStreamIndex[i] = INDEX_NONE;
		}
	}

	RENDERCORE_API virtual ~FVertexFactory();

	virtual FVertexFactoryType* GetType() const { return nullptr; }

	RENDERCORE_API void GetStreams(ERHIFeatureLevel::Type InFeatureLevel, EVertexInputStreamType VertexStreamType, FVertexInputStreamArray& OutVertexStreams) const;

	RENDERCORE_API void OffsetInstanceStreams(uint32 InstanceOffset, EVertexInputStreamType VertexStreamType, FVertexInputStreamArray& VertexStreams) const;

	/**
	* Can be overridden by FVertexFactory subclasses to modify their compile environment just before compilation occurs.
	*/
	static void ModifyCompilationEnvironment(const FVertexFactoryShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment ) {}

	/**
	* Can be overridden by FVertexFactory subclasses to fail a compile based on compilation output.
	*/
	static void ValidateCompiledResult(const FVertexFactoryType* Type, EShaderPlatform Platform, const FShaderParameterMap& ParameterMap, TArray<FString>& OutErrors) {}

	/**
	* Can be overridden by FVertexFactory subclasses which have EVertexFactoryFlags::SupportsPSOPrecaching set - this function will only be called when that flag is set
	*/
	static void GetPSOPrecacheVertexFetchElements(EVertexInputStreamType VertexInputStreamType, FVertexDeclarationElementList& Elements) { checkNoEntry(); }

	// FRenderResource interface.
	RENDERCORE_API virtual void ReleaseRHI();

	// Accessors.
	FVertexDeclarationRHIRef& GetDeclaration() { return Declaration; }
	void SetDeclaration(FVertexDeclarationRHIRef& NewDeclaration) { Declaration = NewDeclaration; }

	const FVertexDeclarationRHIRef& GetDeclaration(EVertexInputStreamType InputStreamType/* = FVertexInputStreamType::Default*/) const 
	{
		switch (InputStreamType)
		{
			case EVertexInputStreamType::Default:				return Declaration;
			case EVertexInputStreamType::PositionOnly:			return PositionDeclaration;
			case EVertexInputStreamType::PositionAndNormalOnly:	return PositionAndNormalDeclaration;
		}

		return Declaration;
	}

	/** Indicates whether the vertex factory supports a position-only stream. */
	virtual bool SupportsPositionOnlyStream() const { return !!PositionStream.Num(); }

	/** Indicates whether the vertex factory supports a position-and-normal-only stream. */
	virtual bool SupportsPositionAndNormalOnlyStream() const { return !!PositionAndNormalStream.Num(); }

	/** Indicates whether the vertex factory supports a null pixel shader. */
	bool SupportsNullPixelShader() const
	{
		return GetType()->SupportsNullPixelShader();
	}

	bool SupportsTriangleSorting() const
	{
		return GetType()->SupportsTriangleSorting();
	}

	virtual FShaderResourceViewRHIRef GetTriangleSortingPositionSRV() const
	{
		return nullptr;
	}

#if WITH_EDITORONLY_DATA
	virtual bool IsCoarseProxyMesh() const { return false; }
#endif

	virtual bool RendersPrimitivesAsCameraFacingSprites() const { return false; }

	bool NeedsDeclaration() const { return bNeedsDeclaration; }

	inline bool SupportsManualVertexFetch(const FStaticFeatureLevel InFeatureLevel) const
	{ 
		return GetType()->SupportsManualVertexFetch(InFeatureLevel);
	}

	inline int32 GetPrimitiveIdStreamIndex(const FStaticFeatureLevel InFeatureLevel, EVertexInputStreamType InputStreamType) const
	{
		return PrimitiveIdStreamIndex[TranslatePrimitiveIdStreamIndex(InFeatureLevel, InputStreamType)];
	}

	inline bool SupportsGPUScene(const FStaticFeatureLevel InFeatureLevel) const
	{
		return GetType()->SupportsPrimitiveIdStream() &&  GetPrimitiveIdStreamIndex(InFeatureLevel, EVertexInputStreamType::Default) != INDEX_NONE;
	}

protected:
	void SetPrimitiveIdStreamIndex(const FStaticFeatureLevel InFeatureLevel, EVertexInputStreamType InputStreamType, int32 StreamIndex)
	{
		check(StreamIndex >= TNumericLimits<int8>::Min() && StreamIndex <= TNumericLimits<int8>::Max());
		PrimitiveIdStreamIndex[TranslatePrimitiveIdStreamIndex(InFeatureLevel, InputStreamType)] = static_cast<int8>(StreamIndex);
	}
	
	RENDERCORE_API bool AddPrimitiveIdStreamElement(EVertexInputStreamType InputStreamType, FVertexDeclarationElementList& Elements, uint8 AttributeIndex, uint8 AttributeIndex_Mobile);

	/**
	 * Creates a vertex element for a vertex stream components.  Adds a unique stream index for the vertex buffer used by the component.
	 * @param Component - The vertex stream component.
	 * @param AttributeIndex - The attribute index to which the stream component is bound.
	 * @return The vertex element which corresponds to Component.
	 */
	FVertexElement AccessStreamComponent(const FVertexStreamComponent& Component, uint8 AttributeIndex)
	{
		return AccessStreamComponent(Component, AttributeIndex, Streams);
	}

	/**
	 * Creates a vertex element for a vertex stream component.  Adds a unique position stream index for the vertex buffer used by the component.
	 * @param Component - The vertex stream component.
	 * @param Usage - The vertex element usage semantics.
	 * @param AttributeIndex - The attribute index to which the stream component is bound.
	 * @return The vertex element which corresponds to Component.
	 */
	RENDERCORE_API FVertexElement AccessStreamComponent(const FVertexStreamComponent& Component, uint8 AttributeIndex, EVertexInputStreamType InputStreamType);
	
	/**
	* Initializes the vertex declaration.
	* @param Elements - The elements of the vertex declaration.
		*/
	RENDERCORE_API void InitDeclaration(const FVertexDeclarationElementList& Elements, EVertexInputStreamType StreamType = EVertexInputStreamType::Default);

public:

	/**
	 * Creates a vertex element for a vertex stream components.  Adds a unique stream index for the vertex buffer used by the component.
	 * @param Component - The vertex stream component.
	 * @param AttributeIndex - The attribute index to which the stream component is bound.
	 * @param AttributeIndex - Stream array where to add the new streams.
	 * @return The vertex element which corresponds to Component.
	 */
	template<typename VertexStreamListType>
	static FVertexElement AccessStreamComponent(const FVertexStreamComponent& Component, uint8 AttributeIndex, VertexStreamListType& InOutStreams)
	{
		uint8 StreamIndex = AddVertexStream(Component, InOutStreams);
		return FVertexElement(StreamIndex, Component.Offset, Component.Type, AttributeIndex, Component.Stride, EnumHasAnyFlags(EVertexStreamUsage::Instancing, Component.VertexStreamUsage));
	}

	template<typename VertexStreamListType>
	static uint8 AddVertexStream(const FVertexStreamComponent& Component, VertexStreamListType& InOutStreams)
	{
		FVertexStream VertexStream;
		VertexStream.VertexBuffer = Component.VertexBuffer;
		VertexStream.Stride = Component.Stride;
		VertexStream.Offset = Component.StreamOffset;
		VertexStream.VertexStreamUsage = Component.VertexStreamUsage;
		return (uint8)InOutStreams.AddUnique(VertexStream);
	}

	/**
	 * Information needed to set a vertex stream.
	 */
	struct FVertexStream
	{
		const FVertexBuffer* VertexBuffer = nullptr;
		uint32 Offset = 0;
		uint16 Stride = 0;
		EVertexStreamUsage VertexStreamUsage = EVertexStreamUsage::Default;
		uint8 Padding = 0;

		friend bool operator==(const FVertexStream& A,const FVertexStream& B)
		{
			return A.VertexBuffer == B.VertexBuffer && A.Stride == B.Stride && A.Offset == B.Offset && A.VertexStreamUsage == B.VertexStreamUsage;
		}

		FVertexStream()
		{
		}
	};

	typedef TArray<FVertexStream, TInlineAllocator<8> > FVertexStreamList;
	
protected:

	/** The vertex streams used to render the factory. */
	FVertexStreamList Streams;
	
private:

	/** The position only vertex stream used to render the factory during depth only passes. */
	TArray<FVertexStream,TInlineAllocator<2> > PositionStream;
	TArray<FVertexStream, TInlineAllocator<3> > PositionAndNormalStream;

	/** The RHI vertex declaration used to render the factory normally. */
	FVertexDeclarationRHIRef Declaration;

	/** The RHI vertex declaration used to render the factory during depth only passes. */
	FVertexDeclarationRHIRef PositionDeclaration;
	FVertexDeclarationRHIRef PositionAndNormalDeclaration;

#if WITH_EDITOR
	// In the editor PrimtiveId streams may differ between mobile and desktop feature levels, so we store them separately
	int8 PrimitiveIdStreamIndex[(int32)EVertexInputStreamType::Count * 2];
#else
	int8 PrimitiveIdStreamIndex[(int32)EVertexInputStreamType::Count];
#endif

protected:
	/* VF can explicitly set this to false to avoid errors without decls; this is for VFs that fetch from buffers directly (e.g. Niagara) */
	bool bNeedsDeclaration = true;

private:
	static int32 TranslatePrimitiveIdStreamIndex(const FStaticFeatureLevel InFeatureLevel, EVertexInputStreamType InputStreamType)
	{
	#if WITH_EDITOR
		return static_cast<int32>(InputStreamType) + (InFeatureLevel <= ERHIFeatureLevel::ES3_1 ? static_cast<int32>(EVertexInputStreamType::Count) : 0); 
	#else
		return static_cast<int32>(InputStreamType); 
	#endif
	}
};

/**
* Default PrimitiveId vertex buffer.  Contains a single index of 0.
* This is used when the VF is used for rendering outside normal mesh passes, where there is no valid scene.
*/
class FPrimitiveIdDummyBuffer : public FVertexBuffer
{
public:

	virtual void InitRHI(FRHICommandListBase& RHICmdList) override;

	virtual void ReleaseRHI() override
	{
		VertexBufferSRV.SafeRelease();
		FVertexBuffer::ReleaseRHI();
	}

	FShaderResourceViewRHIRef VertexBufferSRV;
};

extern RENDERCORE_API TGlobalResource<FPrimitiveIdDummyBuffer> GPrimitiveIdDummy;
