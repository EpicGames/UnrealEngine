// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RHIDefinitions.h"
#include "Containers/EnumAsByte.h"

class FVertexBuffer;

enum class EVertexStreamUsage : uint8
{
	Default = 0 << 0,
	Instancing = 1 << 0,
	Overridden = 1 << 1,
	ManualFetch = 1 << 2
};

/**
 * A typed data source for a vertex factory which streams data from a vertex buffer.
 */
struct FVertexStreamComponent
{
	/** The vertex buffer to stream data from.  If null, no data can be read from this stream. */
	const FVertexBuffer* VertexBuffer = nullptr;

	/** The offset to the start of the vertex buffer fetch. */
	uint32 StreamOffset = 0;

	/** The offset of the data, relative to the beginning of each element in the vertex buffer. */
	uint8 Offset = 0;

	/** The stride of the data. */
	uint8 Stride = 0;

	/** The type of the data read from this stream. */
	TEnumAsByte<EVertexElementType> Type = VET_None;

	EVertexStreamUsage VertexStreamUsage = EVertexStreamUsage::Default;

	/**
	 * Initializes the data stream to null.
	 */
	FVertexStreamComponent()
	{}

	/**
	 * Minimal initialization constructor.
	 */
	FVertexStreamComponent(const FVertexBuffer* InVertexBuffer, uint32 InOffset, uint32 InStride, EVertexElementType InType, EVertexStreamUsage Usage = EVertexStreamUsage::Default) :
		VertexBuffer(InVertexBuffer),
		StreamOffset(0),
		Offset((uint8)InOffset),
		Stride((uint8)InStride),
		Type(InType),
		VertexStreamUsage(Usage)
	{
		check(InStride <= 0xFF);
		check(InOffset <= 0xFF);
	}

	FVertexStreamComponent(const FVertexBuffer* InVertexBuffer, uint32 InStreamOffset, uint32 InOffset, uint32 InStride, EVertexElementType InType, EVertexStreamUsage Usage = EVertexStreamUsage::Default) :
		VertexBuffer(InVertexBuffer),
		StreamOffset(InStreamOffset),
		Offset((uint8)InOffset),
		Stride((uint8)InStride),
		Type(InType),
		VertexStreamUsage(Usage)
	{
		check(InStride <= 0xFF);
		check(InOffset <= 0xFF);
	}
};
