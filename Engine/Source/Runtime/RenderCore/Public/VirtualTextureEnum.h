// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/EnumRange.h"
#include "VirtualTextureEnum.generated.h"

/** Enumeration of the priority assigned to a given virtual texture producer. Must match EVTProducerPriority in RenderCore (they are duplicate because RenderCore cannot declare UENUMs) */
UENUM()
enum class EVTProducerPriority : uint8
{
	Lowest = 0,
	Lower,
	Low,
	BelowNormal,
	Normal,
	AboveNormal,
	High,
	Highest, 

	Count UMETA(Hidden)
};
ENUM_RANGE_BY_COUNT(EVTProducerPriority, EVTProducerPriority::Count);


/** Describes whether an invalidated VT area should be prioritized against others. Allows to improve reactiveness when invalidating a large number of pages */
UENUM()
enum class EVTInvalidatePriority : uint8
{
	Normal = 0,
	High,

	Count UMETA(Hidden)
};
ENUM_RANGE_BY_COUNT(EVTInvalidatePriority, EVTInvalidatePriority::Count);
