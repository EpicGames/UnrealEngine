// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/TaskGraphInterfaces.h"
#include "Containers/EnumAsByte.h"
#include "Containers/HashTable.h"
#include "CoreMinimal.h"
#include "Logging/LogMacros.h"
#include "Math/Color.h"
#include "Math/IntVector.h"
#include "Math/UnrealMathSSE.h"
#include "Misc/AssertionMacros.h"
#include "Misc/EnumClassFlags.h"
#include "PixelFormat.h"
#include "Stats/Stats.h"
#include "Templates/RefCounting.h"
#include "Templates/TypeHash.h"
#include "UObject/NameTypes.h"
#include "VirtualTexturingFwd.h"

class FRDGBuilder;
class FRHICommandListBase;
class FRHIShaderResourceView;
class FRHITexture;
class FRHIUnorderedAccessView;
struct IPooledRenderTarget;
class ISceneRenderer;
namespace ERHIFeatureLevel { enum Type : int; }
enum class EVTProducerPriority : uint8;
enum class EVTInvalidatePriority : uint8;

union FVirtualTextureProducerHandle
{
	FVirtualTextureProducerHandle() : PackedValue(0u) {}
	explicit FVirtualTextureProducerHandle(uint32 InPackedValue) : PackedValue(InPackedValue) {}
	FVirtualTextureProducerHandle(uint32 InIndex, uint32 InMagic) : Index(InIndex), Magic(InMagic) {}

	inline bool IsValid() const { return PackedValue != 0u; }
	inline bool IsNull() const { return PackedValue == 0u; }

	uint32 PackedValue;
	struct
	{
		uint32 Index : 22;
		uint32 Magic : 10;
	};

	friend inline bool operator==(const FVirtualTextureProducerHandle& Lhs, const FVirtualTextureProducerHandle& Rhs) { return Lhs.PackedValue == Rhs.PackedValue; }
	friend inline bool operator!=(const FVirtualTextureProducerHandle& Lhs, const FVirtualTextureProducerHandle& Rhs) { return Lhs.PackedValue != Rhs.PackedValue; }
};
static_assert(sizeof(FVirtualTextureProducerHandle) == sizeof(uint32), "Bad packing");

#define VIRTUALTEXTURE_INVALID_FORCE_SPACEID 0xFF

/**
 * Parameters needed to create an IAllocatedVirtualTexture
 * Describes both page table and physical texture size, format, and layout
 */
struct FAllocatedVTDescription
{
	FName Name;

	uint32 TileSize = 0u;
	uint32 TileBorderSize = 0u;
	uint32 MaxSpaceSize = 0u;
	uint32 IndirectionTextureSize = 0u;
	uint8 Dimensions = 0u;
	uint8 NumTextureLayers = 0u;
	uint8 ForceSpaceID = VIRTUALTEXTURE_INVALID_FORCE_SPACEID;
	uint8 AdaptiveLevelBias = 0u;
	uint8 AllocatedAdaptiveBias = 0u;

	/** Producer for each texture layer. */
	FVirtualTextureProducerHandle ProducerHandle[VIRTUALTEXTURE_SPACE_MAXLAYERS];
	/** Local layer inside producer for each texture layer. */
	uint8 ProducerLayerIndex[VIRTUALTEXTURE_SPACE_MAXLAYERS] = { 0u };

	union
	{
		uint8 PackedFlags = 0u;
		struct
		{
			/**
			 * Should the AllocatedVT create its own dedicated page table allocation? Can be useful to control total allocation.
			 * The system only supports a limited number of unique page tables, so this must be used carefully
			 */
			uint8 bPrivateSpace : 1;

			/**
			 * If the AllocatedVT has the same producer mapped to multiple layers, should those be merged into a single page table layer?
			 * This can make for more efficient page tables when enabled, but certain code may make assumption that number of layers
			 * specified when allocating VT exactly matches the resulting page page
			 */
			uint8 bShareDuplicateLayers : 1;
		};
	};

	friend inline bool operator==(const FAllocatedVTDescription& Lhs, const FAllocatedVTDescription& Rhs)
	{
		if (Lhs.TileSize != Rhs.TileSize ||
			Lhs.TileBorderSize != Rhs.TileBorderSize ||
			Lhs.Dimensions != Rhs.Dimensions ||
			Lhs.NumTextureLayers != Rhs.NumTextureLayers ||
			Lhs.AllocatedAdaptiveBias != Rhs.AllocatedAdaptiveBias ||
			Lhs.PackedFlags != Rhs.PackedFlags)
		{
			return false;
		}
		for (uint32 LayerIndex = 0u; LayerIndex < Lhs.NumTextureLayers; ++LayerIndex)
		{
			if (Lhs.ProducerHandle[LayerIndex] != Rhs.ProducerHandle[LayerIndex] ||
				Lhs.ProducerLayerIndex[LayerIndex] != Rhs.ProducerLayerIndex[LayerIndex])
			{
				return false;
			}
		}
		return true;
	}
	friend inline bool operator!=(const FAllocatedVTDescription& Lhs, const FAllocatedVTDescription& Rhs)
	{
		return !operator==(Lhs, Rhs);
	}

	friend inline uint32 GetTypeHash(const FAllocatedVTDescription& Description)
	{
		uint32 Hash = GetTypeHash(Description.TileSize);
		Hash = HashCombine(Hash, GetTypeHash(Description.TileBorderSize));
		Hash = HashCombine(Hash, GetTypeHash(Description.Dimensions));
		Hash = HashCombine(Hash, GetTypeHash(Description.AllocatedAdaptiveBias));
		Hash = HashCombine(Hash, GetTypeHash(Description.NumTextureLayers));
		Hash = HashCombine(Hash, GetTypeHash(Description.PackedFlags));
		for (uint32 LayerIndex = 0u; LayerIndex < Description.NumTextureLayers; ++LayerIndex)
		{
			Hash = HashCombine(Hash, GetTypeHash(Description.ProducerHandle[LayerIndex].PackedValue));
			Hash = HashCombine(Hash, GetTypeHash(Description.ProducerLayerIndex[LayerIndex]));
		}
		return Hash;
	}
};

struct FVTProducerDescription
{
	// Disable bNotifyCompleted deprecation warnings here.
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	RENDERCORE_API FVTProducerDescription();
	FVTProducerDescription(const FVTProducerDescription&) = default;
	FVTProducerDescription& operator=(const FVTProducerDescription&) = default;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	FName Name; /** Will be name of UTexture for streaming VTs, mostly here for debugging */
	uint32 FullNameHash;
	
	bool bPersistentHighestMip = true;
	bool bContinuousUpdate = false;
	bool bRequiresSinglePhysicalPool = false;
	bool bTrackSamplingHistory = false;
	
	UE_DEPRECATED(5.8, "OnRequestsCompleted() callbacks are deprecated for future removal. They are currently disabled by default with r.VT.EnableNotifyCompleted.")
	bool bNotifyCompleted = false;
	
	uint32 TileSize = 0u;
	uint32 TileBorderSize = 0u;

	/**
	 * Producers are made up of a number of block, each block has uniform size, and blocks are arranged in a larger grid
	 * 'Normal' VTs will typically be a single block, for UDIM textures, blocks will map to individual UDIM texture sheets
	 * When multiple producers are allocated together, they will be aligned such that blocks of each layer overlay on top of each other
	 * Number of blocks for each layer may be different in this case, this is handled by wrapping blocks for layers with fewer blocks
	 */
	uint32 BlockWidthInTiles = 0u;
	uint32 BlockHeightInTiles = 0u;
	uint32 DepthInTiles = 0u;
	uint16 WidthInBlocks = 1u;
	uint16 HeightInBlocks = 1u;
	uint8 Dimensions = 0u;
	uint8 MaxLevel = 0u;

	/**
	 * Producers will fill a number of texture layers.
	 * These texture layers can be distributed across one or more physical groups.
	 * Each physical group can contain one or more of the texture layers. 
	 * Within a physical group the texture layers share the same UV allocation/mapping and can be referenced by a single page table lookup.
	 */
	uint8 NumTextureLayers = 0u;
	TEnumAsByte<EPixelFormat> LayerFormat[VIRTUALTEXTURE_SPACE_MAXLAYERS] = { PF_Unknown };
	FLinearColor LayerFallbackColor[VIRTUALTEXTURE_SPACE_MAXLAYERS] = { FLinearColor::Black };
	bool bIsLayerSRGB[VIRTUALTEXTURE_SPACE_MAXLAYERS] = { false };
	
	uint8 NumPhysicalGroups = 0u;
	uint8 PhysicalGroupIndex[VIRTUALTEXTURE_SPACE_MAXLAYERS] = { 0 };

	/** Priority allows to process a producer before another which increases responsiveness in high contention situations */
	EVTProducerPriority Priority;
};



/** 
 * A Finalizer is an object that does the final work of filling the physical textures. 
 * The work for all finalizers is scheduled at a specific point in the frame where we can write to the physical texture without hazards, and
 * where the virtual texture page tables are also updated.
 * The finalizer work may be split into two parts.
 * RenderFinalize() does not write to the physical texture, but can read from virtual textures. This is a phase that allows any 
 * page rendering which may need to sample virtual textures. Runtime virtual textures and material systems require this.
 * Finalise() must write to the physical textures, but cannot sample from them.
 * All finalizers need to implement Finalize() but only ones that need to sample virtual textures need to implement RenderFinalize().
 */
class IVirtualTextureFinalizer
{
public:
	/** Finalize work that has read only access to the virtual texture physical pools. */
	virtual void RenderFinalize(FRDGBuilder& GraphBuilder, ISceneRenderer* SceneRenderingContext) {}
	/** Finalize work that has write only access to the virtual texture physical pools. */
	virtual void Finalize(FRDGBuilder& GraphBuilder) = 0;
};

enum class EVTRequestPageStatus
{
	/** The request is invalid and no data will ever be available */
	Invalid,

	/**
	* Requested data is not being produced, and a request can't be started as some part of the system is at capacity.
	* Requesting the same data at a later time should succeed.
	*/
	Saturated,

	/**
	* Requested data is currently being produced, but is not yet ready.
	* It's valid to produce this data, but doing so may block until data is ready.
	*/
	Pending,

	/** Requested data is available */
	Available,
};

/** Check to see there is data available (possibly requiring waiting) given the current status */
inline bool VTRequestPageStatus_HasData(EVTRequestPageStatus InStatus) { return InStatus == EVTRequestPageStatus::Pending || InStatus == EVTRequestPageStatus::Available; }

enum class EVTRequestPagePriority
{
	Normal,
	High,
};

enum class EVTProducePageFlags : uint8
{
	None = 0u,
	SkipPageBorders = (1u << 0),
	ContinuousUpdate = (1u << 1),
};
ENUM_CLASS_FLAGS(EVTProducePageFlags);

struct FVTRequestPageResult
{
	FVTRequestPageResult(EVTRequestPageStatus InStatus = EVTRequestPageStatus::Invalid, uint64 InHandle = 0u) : Handle(InHandle), Status(InStatus) {}

	/** Opaque handle to the request, must be passed to 'ProducePageData'.  Only valid if status is Pending/Available */
	uint64 Handle;

	/** Status of the request */
	EVTRequestPageStatus Status;
};

/** Describes a location to write a single layer of a VT tile */
struct FVTProduceTargetLayer
{
	/** Pooled render target to write to. */
	IPooledRenderTarget* PooledRenderTarget = nullptr;
	/** Location within the target texture to write. */
	FIntVector pPageLocation = FIntVector::ZeroValue;

	UE_DEPRECATED(5.6, "Use PooledRenderTarget instead.")
	FRHITexture* TextureRHI = nullptr;

	// Disable some TextureRHI deprecation warnings here.
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FVTProduceTargetLayer() = default;
	FVTProduceTargetLayer(const FVTProduceTargetLayer&) = default;
	FVTProduceTargetLayer& operator=(const FVTProduceTargetLayer&) = default;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
};

/**
* This is the interface that can produce tiles of virtual texture data
* This can be extended to represent different ways of generating VT, such as disk streaming, runtime compositing, or whatever
* It's provided to the renderer module
*/
class IVirtualTexture
{
public:
	inline IVirtualTexture() {}
	virtual	~IVirtualTexture() = default;

	/**
	 * Gives a localized mip bias for the given local vAddress.
	 * This is used to implement sparse VTs, the bias is number of mip levels to add to reach a resident page
	 * Must be thread-safe, may be called from any thread
	 * @param vLevel The mipmap level to check
	 * @param vAddress Virtual address to check
	 * @return Mip bias to be added to vLevel to reach a resident page at the given address
	 */
	virtual uint32 GetLocalMipBias(uint8 vLevel, uint32 vAddress) const { return 0u; }

	/**
	 * Whether data for the given page is streamed (e.g. loading from disk).
	 */
	virtual bool IsPageStreamed(uint8 vLevel, uint32 vAddress) const = 0;

	/**
	* Makes a request for the given page data.
	* For data sources that can generate data immediately, it's reasonable for this method to do nothing, and simply return 'Available'
	* Only called from render thread
	* @param ProducerHandle Handle to this producer, can be used as a UID for this producer for any internal caching mechanisms
	* @param LayerMask Mask of requested layers
	* @param vLevel The mipmap level of the data
	* @param vAddress Bit-interleaved x,y page indexes
	* @param Priority Priority of the request, used to drive async IO/task priority needed to generate data for request
	* @return FVTRequestPageResult describing the availability of the request
	*/
	virtual FVTRequestPageResult RequestPageData(FRHICommandListBase& RHICmdList, const FVirtualTextureProducerHandle& ProducerHandle, uint8 LayerMask, uint8 vLevel, uint64 vAddress, EVTRequestPagePriority Priority) = 0;

	/**
	* Upload page data to the cache, data must have been previously requested, and reported either 'Available' or 'Pending'
	* The system will attempt to call RequestPageData/ProducePageData only once for a given vLevel/vAddress, with all the requested layers set in LayerMask,
	* this is important for certain types of procedural producers that may generate multiple layers of VT data at the same time
	* It's valid to produce 'Pending' page data, but in this case ProducePageData may block until data is ready
	* Only called from render thread
	* @param RHICmdList Used to write any commands required to generate the VT page data
	* @param FeatureLevel The current RHI feature level
	* @param ProducerHandle Handle to this producer
	* @param LayerMask Mask of requested layers; can be used to only produce data for these layers as an optimization, or ignored if all layers are logically produced together
	* @param vLevel The mipmap level of the data
	* @param vAddress Bit-interleaved x,y page indexes
	* @param RequestHandle opaque handle returned from 'RequestPageData'
	* @param TargetLayers Array of 'FVTProduceTargetLayer' structs, gives location where each layer should write data
	* @return a 'IVirtualTextureFinalizer' which must be finalized to complete the operation
	*/
	virtual IVirtualTextureFinalizer* ProducePageData(
		FRHICommandListBase& RHICmdList,
		ERHIFeatureLevel::Type FeatureLevel,
		EVTProducePageFlags Flags,
		const FVirtualTextureProducerHandle& ProducerHandle, uint8 LayerMask, uint8 vLevel, uint64 vAddress,
		uint64 RequestHandle,
		const FVTProduceTargetLayer* TargetLayers) = 0;

	/** Collect all task graph events. */
	virtual void GatherProducePageDataTasks(FVirtualTextureProducerHandle const& ProducerHandle, FGraphEventArray& InOutTasks) const {}

	/** Collect all task graph events related to a request. */
	virtual void GatherProducePageDataTasks(uint64 RequestHandle, FGraphEventArray& InOutTasks) const {};

	/** Dump any type specific debug info. */
	virtual void DumpToConsole(bool verbose) {}

	/** Called on every virtual texture system update once all requests are completed, if bNotifyCompleted is enabled and r.VT.EnableNotifyCompleted=1. */
	UE_DEPRECATED(5.8, "OnRequestsCompleted() callbacks are deprecated for future removal.")
	virtual void OnRequestsCompleted() {}
};

enum class EVTPageTableFormat : uint8
{
	UInt16,
	UInt32,
};

/**
* This interface represents a chunk of VT data allocated and owned by the renderer module, backed by both a page table texture, and a physical texture cache for each layer.
* Both page table and physical texture may be shared amongst many different allocated virtual textures.
* Any method that deals with physical texture requires an explicit LayerIndex parameter to identify the physical texture in question,
* methods that don't have LayerIndex parameter refer to properties shared by all textures using the given page table
* These are created with IRendererModule::AllocateVirtualTexture, and destroyed with IRendererModule::DestroyVirtualTexture
* They must be allocated from render thread, but may be destroyed from any thread
*/
class IAllocatedVirtualTexture
{
public:
	static const uint32 LayersPerPageTableTexture = 4u;

	inline IAllocatedVirtualTexture(const FAllocatedVTDescription& InDesc,
		uint32 InBlockWidthInTiles,
		uint32 InBlockHeightInTiles,
		uint32 InWidthInBlocks,
		uint32 InHeightInBlocks,
		uint32 InDepthInTiles)
		: Description(InDesc)
		, BlockWidthInTiles(InBlockWidthInTiles)
		, BlockHeightInTiles(InBlockHeightInTiles)
		, WidthInBlocks(InWidthInBlocks)
		, HeightInBlocks(InHeightInBlocks)
		, DepthInTiles(InDepthInTiles)
		, FrameDeleted(0u)
		, NumRefs(0)
		, bIsWaitingToMap(false)
		, PageTableFormat(EVTPageTableFormat::UInt32)
		, SpaceID(~0u)
		, MaxLevel(0u)
		, VirtualAddress(~0u)
		, VirtualPageX(~0u)
		, VirtualPageY(~0u)
	{}

	virtual uint32 GetPersistentHash() const = 0;
	virtual uint32 GetNumPageTableTextures() const = 0;
	virtual FRHITexture* GetPageTableTexture(uint32 InPageTableIndex) const = 0;
	virtual FRHITexture* GetPageTableIndirectionTexture() const = 0;
	virtual uint32 GetPhysicalTextureSize(uint32 InLayerIndex) const = 0;
	virtual FRHITexture* GetPhysicalTexture(uint32 InLayerIndex) const = 0;
	virtual FRHIShaderResourceView* GetPhysicalTextureSRV(uint32 InLayerIndex, bool bSRGB) const = 0;

	/** Writes 2x FUintVector4 */
	virtual void GetPackedPageTableUniform(FUintVector4* OutUniform) const = 0;
	/** Writes 1x FUintVector4 */
	virtual void GetPackedUniform(FUintVector4* OutUniform, uint32 LayerIndex) const = 0;

	virtual void DumpToConsole(bool bVerbose) const {}

	inline const FAllocatedVTDescription& GetDescription() const { return Description; }
	inline const FVirtualTextureProducerHandle& GetProducerHandle(uint32 InLayerIndex) const { check(InLayerIndex < Description.NumTextureLayers); return Description.ProducerHandle[InLayerIndex]; }
	
	inline uint32 GetVirtualTileSize() const { return Description.TileSize; }
	inline uint32 GetTileBorderSize() const { return Description.TileBorderSize; }
	inline uint32 GetPhysicalTileSize() const { return Description.TileSize + Description.TileBorderSize * 2u; }
	inline uint32 GetNumTextureLayers() const { return Description.NumTextureLayers; }
	inline uint8 GetDimensions() const { return Description.Dimensions; }
	inline uint32 GetWidthInBlocks() const { return WidthInBlocks; }
	inline uint32 GetHeightInBlocks() const { return HeightInBlocks; }
	inline uint32 GetBlockWidthInTiles() const { return BlockWidthInTiles; }
	inline uint32 GetBlockHeightInTiles() const { return BlockHeightInTiles; }
	inline uint32 GetWidthInTiles() const { return BlockWidthInTiles * WidthInBlocks; }
	inline uint32 GetHeightInTiles() const { return BlockHeightInTiles * HeightInBlocks; }
	inline uint32 GetDepthInTiles() const { return DepthInTiles; }
	inline uint32 GetWidthInPixels() const { return GetWidthInTiles() * Description.TileSize; }
	inline uint32 GetHeightInPixels() const { return GetHeightInTiles() * Description.TileSize; }
	inline uint32 GetDepthInPixels() const { return DepthInTiles * Description.TileSize; }
	inline uint32 GetSpaceID() const { return SpaceID; }
	inline uint32 GetVirtualAddress() const { return VirtualAddress; }
	inline uint32 GetVirtualPageX() const { return VirtualPageX; }
	inline uint32 GetVirtualPageY() const { return VirtualPageY; }
	inline uint32 GetMaxLevel() const { return MaxLevel; }
	inline uint32 GetAllocatedAdaptiveBias() const { return Description.AllocatedAdaptiveBias; }
	inline EVTPageTableFormat GetPageTableFormat() const { return PageTableFormat; }

protected:
	friend class FVirtualTextureSystem;
	virtual void Destroy(class FVirtualTextureSystem* InSystem) = 0;
	virtual bool TryMapLockedTiles(class FVirtualTextureSystem* InSystem) const = 0;
	virtual ~IAllocatedVirtualTexture() = default;

	FAllocatedVTDescription Description;
	uint32 BlockWidthInTiles;
	uint32 BlockHeightInTiles;
	uint32 WidthInBlocks;
	uint32 HeightInBlocks;
	uint32 DepthInTiles;
	uint32 FrameDeleted;
	int32 NumRefs;
	bool bIsWaitingToMap;

	// should be set explicitly by derived class constructor
	EVTPageTableFormat PageTableFormat;
	uint32 SpaceID;
	uint32 MaxLevel;
	uint32 VirtualAddress;
	uint32 VirtualPageX;
	uint32 VirtualPageY;
};

/** 
 * Interface for adaptive virtual textures. 
 * This manages multiple allocated virtual textures in a space to simulate a single larger virtual texture. 
 */
class IAdaptiveVirtualTexture
{
public:
	/** Get the persistent allocated virtual texture for low mips from the adaptive virtual texture. */
	virtual IAllocatedVirtualTexture* GetAllocatedVirtualTexture() = 0;

protected:
	friend class FVirtualTextureSystem;
	virtual ~IAdaptiveVirtualTexture() = default;
	virtual int32 GetSpaceID() const = 0;
	virtual void Destroy(class FVirtualTextureSystem* InSystem) = 0;
};

/** Describes an adaptive virtual texture. */
struct FAdaptiveVTDescription
{
	uint32 TileCountX;
	uint32 TileCountY;
	uint32 MaxAdaptiveLevel;
};

/**
 * Identifies a VT tile within a given producer
 */
union FVirtualTextureLocalTile
{
	inline FVirtualTextureLocalTile() {}
	explicit inline FVirtualTextureLocalTile(const FVirtualTextureProducerHandle& InProducerHandle, uint32 InLocal_vAddress, uint8 InLocal_vLevel)
		: PackedProducerHandle(InProducerHandle.PackedValue)
		, Local_vAddress(InLocal_vAddress)
		, Local_vLevel(InLocal_vLevel)
		, Pad(0)
	{
	}
	explicit inline FVirtualTextureLocalTile(uint64 InPackedValue)
		: PackedValue(InPackedValue)
	{
		Pad = 0;
	}

	inline FVirtualTextureProducerHandle GetProducerHandle() const { return FVirtualTextureProducerHandle(PackedProducerHandle); }

	uint64 PackedValue = 0;
	struct
	{
		// Core "identity" data that identify a given tile
		uint32 PackedProducerHandle;
		uint32 Local_vAddress : 24;
		uint32 Local_vLevel : 4;
		uint32 Pad : 4;  // Important : this has to be kept in sync with FVirtualTextureLocalTileRequest : both are packed in a uint64 and the latter stores additional info in these bits
	};

	friend inline bool operator==(const FVirtualTextureLocalTile& Lhs, const FVirtualTextureLocalTile& Rhs) { return Lhs.PackedValue == Rhs.PackedValue; }
	friend inline bool operator!=(const FVirtualTextureLocalTile& Lhs, const FVirtualTextureLocalTile& Rhs) { return Lhs.PackedValue != Rhs.PackedValue; }
	friend inline uint32 GetTypeHash(const FVirtualTextureLocalTile& T) { return GetTypeHash(T.PackedValue); } 
	inline uint16 GetMurmurHash() const { return static_cast<uint16>(MurmurFinalize64(PackedValue)); }
};
static_assert(sizeof(FVirtualTextureLocalTile) == sizeof(uint64), "Bad packing");

/**
 * Identifies a VT tile request : a VT tile, along with priority information for processing this request
 */
union FVirtualTextureLocalTileRequest
{
	explicit inline FVirtualTextureLocalTileRequest() {}
	explicit inline FVirtualTextureLocalTileRequest(const FVirtualTextureLocalTile& InTile, EVTProducerPriority InProducerPriority, EVTInvalidatePriority InInvalidatePriority)
		: TilePackedValue(InTile.PackedValue)
		, ProducerPriority(static_cast<uint64>(InProducerPriority))
		, InvalidatePriority(static_cast<uint64>(InInvalidatePriority))
	{
		checkSlow(InTile == GetTile());
		ValidatePriorities();
	}

	inline FVirtualTextureLocalTile GetTile() const { return FVirtualTextureLocalTile(TilePackedValue); }
	inline EVTProducerPriority GetProducerPriority() const { return static_cast<EVTProducerPriority>(ProducerPriority); }
	inline EVTInvalidatePriority GetInvalidatePriority() const { return static_cast<EVTInvalidatePriority>(InvalidatePriority); }

	inline FVirtualTextureLocalTileRequest& MergeWith(const FVirtualTextureLocalTileRequest& InOther) 
	{	
		// Keep the highest priorities of the 2 :
		ProducerPriority = FMath::Max(ProducerPriority, InOther.ProducerPriority);
		InvalidatePriority = FMath::Max(InvalidatePriority, InOther.InvalidatePriority);

		checkSlow(*this == InOther);
		checkSlow(GetTile() == InOther.GetTile());
		ValidatePriorities();
		return *this;
	}

	uint64 PackedValue;
	struct
	{
		// Core "identity" data that identifies a given tile : 
		uint64 TilePackedValue : 60;
		// Additional data about the request :
		uint64 ProducerPriority : 3; // See EVTProducerPriority 
		uint64 InvalidatePriority : 1; // Indicates whether the tile should get prioritized over others
	};

	friend inline bool operator==(const FVirtualTextureLocalTileRequest& Lhs, const FVirtualTextureLocalTileRequest& Rhs) { return Lhs.GetIdentity() == Rhs.GetIdentity(); }
	friend inline bool operator!=(const FVirtualTextureLocalTileRequest& Lhs, const FVirtualTextureLocalTileRequest& Rhs) { return Lhs.GetIdentity() != Rhs.GetIdentity(); }
	friend inline uint32 GetTypeHash(const FVirtualTextureLocalTileRequest& T) { return GetTypeHash(T.GetIdentity()); }
	inline uint64 GetIdentity() const { return TilePackedValue; }  // 2 requests from the same tile are considered identical (it's up to the user to "merge" them if necessary)
	inline uint16 GetMurmurHash() const { return static_cast<uint16>(MurmurFinalize64(GetIdentity())); } 

private:
	RENDERCORE_API void ValidatePriorities() const;
};
static_assert(sizeof(FVirtualTextureLocalTileRequest) == sizeof(uint64), "Bad packing");

/**
 * Helper function that tries to add a new FVirtualTextureLocalTileRequest to a TSet but, in case this tile is already in the set, merges it with the existing request :
 */
inline FVirtualTextureLocalTileRequest& AddOrMergeTileRequest(const FVirtualTextureLocalTileRequest& InTileRequest, TSet<FVirtualTextureLocalTileRequest>& TileRequests)
{
	bool bAlreadyInSet = false;
	FVirtualTextureLocalTileRequest& Result = TileRequests.FindOrAdd(InTileRequest, &bAlreadyInSet);
	if (bAlreadyInSet)
	{
		// If the request already exists, merge with this new request, to keep the highest priority :
		Result.MergeWith(InTileRequest);
	}
	return Result;
}

RENDERCORE_API DECLARE_LOG_CATEGORY_EXTERN(LogVirtualTexturing, Log, All);

DECLARE_STATS_GROUP(TEXT("Virtual Texturing"), STATGROUP_VirtualTexturing, STATCAT_Advanced);
DECLARE_STATS_GROUP(TEXT("Virtual Texture Memory"), STATGROUP_VirtualTextureMemory, STATCAT_Advanced);
