// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/** Maximum number of virtual texture spaces that can support feedback */
#define VIRTUALTEXTURE_MAX_FEEDBACK_SPACES 16

/** Maximum number of layers that can be allocated in a single VT page table */
#define VIRTUALTEXTURE_SPACE_MAXLAYERS 8

/** Maximum dimension of VT page table texture */
#define VIRTUALTEXTURE_LOG2_MAX_PAGETABLE_SIZE 12u
#define VIRTUALTEXTURE_MAX_PAGETABLE_SIZE (1u << VIRTUALTEXTURE_LOG2_MAX_PAGETABLE_SIZE)
#define VIRTUALTEXTURE_MIN_PAGETABLE_SIZE 32u

class IAllocatedVirtualTexture;
class IAdaptiveVirtualTexture;
struct FAllocatedVTDescription;
struct FAdaptiveVTDescription;
union FVirtualTextureProducerHandle;
struct FVTProducerDescription;
class IVirtualTexture;
struct FVTProducerDescription;

typedef void (FVTProducerDestroyedFunction)(const FVirtualTextureProducerHandle& InHandle, void* Baton);
