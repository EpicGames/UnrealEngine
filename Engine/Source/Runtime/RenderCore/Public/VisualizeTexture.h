// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "Misc/EnumClassFlags.h"
#include "Misc/Optional.h"
#include "Misc/WildcardString.h"
#include "RenderGraphDefinitions.h"
#include "RenderResource.h"
#include "RendererInterface.h"
#include "Templates/RefCounting.h"


class FOutputDevice;
class FRDGBuilder;
class FRHICommandListImmediate;
class FWildcardString;

class FVisualizeTexture : public FRenderResource
{
public:
	FVisualizeTexture() = default;

	RENDERCORE_API void ParseCommands(const TCHAR* Cmd, FOutputDevice &Ar);

	RENDERCORE_API void DebugLogOnCrash();

	RENDERCORE_API void GetTextureInfos_GameThread(TArray<FString>& Infos) const;

#if SUPPORTS_VISUALIZE_TEXTURE
	RENDERCORE_API void BeginFrameRenderThread();
	RENDERCORE_API void BeginViewRenderThread(ERHIFeatureLevel::Type InFeatureLevel, int32 UniqueId, const TCHAR* Description, bool bIsSceneCapture, bool bIsReflectionCapture = false);
	RENDERCORE_API void SetSceneTextures(const TArray<FRDGTextureRef>& InSceneTextures, FIntPoint InFamilySize, const TArray<FIntRect>& InFamilyViewRects);
	RENDERCORE_API void EndViewRenderThread();
	RENDERCORE_API void EndFrameRenderThread();

	/** Creates a new checkpoint (e.g. "SceneDepth@N") for the pooled render target. A null parameter is a no-op. */
	RENDERCORE_API void SetCheckPoint(FRDGBuilder& GraphBuilder, IPooledRenderTarget* PooledRenderTarget);
	RENDERCORE_API void SetCheckPoint(FRHICommandListImmediate& RHICmdList, IPooledRenderTarget* PooledRenderTarget);
#else
	inline void BeginFrameRenderThread() {}
	inline void EndFrameRenderThread() {}

	inline void SetCheckPoint(FRDGBuilder& GraphBuilder, IPooledRenderTarget* PooledRenderTarget) {}
	inline void SetCheckPoint(FRHICommandListImmediate& RHICmdList, IPooledRenderTarget* PooledRenderTarget) {}
#endif

	inline bool IsActive() const
	{
#if SUPPORTS_VISUALIZE_TEXTURE
		return State != EState::Inactive;
#else
		return false;
#endif
	}

	inline bool IsRequestedView() const
	{
#if SUPPORTS_VISUALIZE_TEXTURE
		return bIsRequestedView;
#else
		return false;
#endif
	}

	// Returns whether the given actor Name (from GetActorNameOrLabel()) is currently being visualized.
	RENDERCORE_API bool IsNamedView(const TCHAR* Name) const;

	// bUseConfig specifies that the active configuration settings should be used to visualize the texture.  This includes checking to
	// make sure the texture is active in the config, and pulling settings (such as mip level) from the config.
	static RENDERCORE_API FRDGTextureRef AddVisualizeTexturePass(
		FRDGBuilder& GraphBuilder,
		class FGlobalShaderMap* ShaderMap,
		const FRDGTextureRef InputTexture,
		bool bUseConfig = false);

	static RENDERCORE_API FRDGTextureRef AddVisualizeTextureAlphaPass(
		FRDGBuilder& GraphBuilder,
		class FGlobalShaderMap* ShaderMap,
		const FRDGTextureRef InputTexture);

private:
	enum class EFlags
	{
		None				= 0,
		SaveBitmap			= 1 << 0,
		SaveBitmapAsStencil = 1 << 1, // stencil normally displays in the alpha channel of depth buffer visualization. This option is just for BMP writeout to get a stencil only BMP.
	};
	FRIEND_ENUM_CLASS_FLAGS(EFlags);

	enum class EState
	{
		Inactive,				// Default initial state, negligible overhead
		DisplayViews,			// Display views next render frame -- state activated on DisplayViewListToLog call if Inactive
		DisplayResources,		// Display resources next render frame -- state activated on DisplayResourceListToLog call if Inactive
		TrackResources,			// Track resources every frame, adding overhead -- state activated after visualize texture related command is issued
	};

	enum class ECommand
	{
		Unknown,
		DisableVisualization,
		VisualizeResource,
		DisplayHelp,
		DisplayPoolResourceList,
		DisplayResourceList,
		DisplayViewList,
		SetViewId
	};

	enum class EInputUVMapping
	{
		LeftTop,
		Whole,
		PixelPerfectCenter,
		PictureInPicture
	};

	enum class EInputValueMapping
	{
		Color,
		Depth,
		Shadow
	};

	enum class EDisplayMode
	{
		MultiColomn,
		Detailed,
	};

	enum class ESortBy
	{
		Index,
		Name,
		Size
	};

	enum class EShaderOp
	{
		Frac,
		Saturate
	};

#if SUPPORTS_VISUALIZE_TEXTURE
	static RENDERCORE_API void DisplayHelp(FOutputDevice &Ar);
	RENDERCORE_API void DisplayPoolResourceListToLog(ESortBy SortBy);
	RENDERCORE_API void DisplayResourceListToLog(const TOptional<FWildcardString>& Wildcard);
	RENDERCORE_API void DisplayViewListToLog();

	/** Determine whether a texture should be captured for debugging purposes and return the capture id if needed. */
	RENDERCORE_API TOptional<uint32> ShouldCapture(const TCHAR* DebugName, uint32 MipIndex);

	struct FConfig
	{
		float RGBMul = 1.0f;
		float AMul = 0.0f;

		// -1=off, 0=R, 1=G, 2=B, 3=A
		int32 SingleChannel = -1;
		float SingleChannelMul = 0.0f;

		EFlags Flags = EFlags::None;
		EInputUVMapping InputUVMapping = EInputUVMapping::PictureInPicture;
		EShaderOp ShaderOp = EShaderOp::Frac;
		uint32 MipIndex = 0;
		uint32 ArrayIndex = 0;
	};

	/** Adds a pass to visualize a texture. */
	static RENDERCORE_API FRDGTextureRef AddVisualizeTexturePass(
		FRDGBuilder& GraphBuilder,
		class FGlobalShaderMap* ShaderMap,
		const FRDGTextureRef InputTexture,
		const FConfig& Config,
		EInputValueMapping InputValueMapping,
		uint32 CaptureId);

	/** Create a pass capturing a texture. */
	RENDERCORE_API void CreateContentCapturePass(FRDGBuilder& GraphBuilder, FRDGTextureRef Texture, uint32 CaptureId);

	RENDERCORE_API void ReleaseRHI() override;

	RENDERCORE_API void Visualize(const FString& InName, TOptional<uint32> InVersion = {});

	RENDERCORE_API uint32 GetVersionCount(const TCHAR* InName) const;

	FConfig Config;

	EState State = EState::Inactive;
	TOptional<FWildcardString> DisplayResourcesParam;		// Cached parameter for EState::DisplayResources

	bool bAnyViewRendered = false;			// Track when any view is rendered in the current frame, so we can ignore frames where no views render
	bool bIsRequestedView = false;			// Set when this is a requested view, and we should capture visualizations from it
	bool bFoundRequestedView = false;		// Set so we can stop considering other views, after we found the specific view that was requested

	// Initialized in SetSceneTextures, tracks viewports from whichever scene renderer contains the view being visualized
	TArray<FIntRect> FamilyViewRects;

	struct FRequested
	{
		uint32 ViewUniqueId = 0;				// View requested to be visualized -- zero visualizes the last non-scene-capture view
		FString ViewName;						// Alternately, string name of view to visualize
		FString Name;
		TOptional<uint32> Version;
	} Requested;

	struct FCaptured
	{
		FCaptured()
		{
			Desc.DebugName = TEXT("VisualizeTexture");
		}

		TRefCountPtr<IPooledRenderTarget> PooledRenderTarget;
		FRDGTextureRef Texture = nullptr;
		FPooledRenderTargetDesc Desc;
		EInputValueMapping InputValueMapping = EInputValueMapping::Color;
		int32 ViewUniqueId = 0;					// View actually visualized
		FIntPoint OutputExtent;					// Viewport extent for visualized scene renderer
		TArray<FIntRect> ViewRects;				// Viewports from scene renderer being visualized
	} Captured;

	ERHIFeatureLevel::Type FeatureLevel = ERHIFeatureLevel::SM5;

	// Map of unique view ID to description, updated when views get rendered.
	TMap<int32, FString> ViewDescriptionMap;

	// Maps a texture name to its checkpoint version.
	TMap<FString, uint32> VersionCountMap;
#endif

	friend class FRDGBuilder;
	friend class FVisualizeTexturePresent;
};

ENUM_CLASS_FLAGS(FVisualizeTexture::EFlags);

/** The global render targets for easy shading. */
extern RENDERCORE_API TGlobalResource<FVisualizeTexture> GVisualizeTexture;

#if SUPPORTS_VISUALIZE_TEXTURE

// We use a macro to compile out calls to BeginViewRenderThread, because generating the arguments to the call may involve utility function calls
// that the compiler can't optimize out, even if the function itself was an empty inline.  This commonly includes a call to the "GetViewKey"
// function to fetch UniqueId, which involves two function calls (one virtual), and any string formatting used to generate the Description.
// For symmetry, a macro is also provided for EndViewRenderThread (even though for that case, an empty inline would compile out fine).
#define VISUALIZE_TEXTURE_BEGIN_VIEW(FeatureLevel, UniqueId, Description, bIsSceneCapture, bIsReflectionCapture) GVisualizeTexture.BeginViewRenderThread(FeatureLevel, UniqueId, Description, bIsSceneCapture, bIsReflectionCapture)
#define VISUALIZE_TEXTURE_END_VIEW() GVisualizeTexture.EndViewRenderThread()

#else
#define VISUALIZE_TEXTURE_BEGIN_VIEW(FeatureLevel, UniqueId, Description, bIsSceneCapture, bIsReflectionCapture) (void)0
#define VISUALIZE_TEXTURE_END_VIEW() (void)0
#endif
