// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;
using UnrealBuildBase;
using UnrealBuildTool;
using System;

public class RenderCore : ModuleRules
{
	public RenderCore(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(new string[] { "RHI", "CoreUObject" });

		PrivateIncludePathModuleNames.AddRange(new string[] { "Shaders", "TargetPlatform" });

		// JSON is used for the asset info in the shader library and dumping out frames.
		PrivateDependencyModuleNames.Add("Json");

		PrivateDependencyModuleNames.Add("BuildSettings");

		if (Target.bBuildEditor == true)
		{
			DynamicallyLoadedModuleNames.Add("TargetPlatform");
			PrivateIncludePathModuleNames.Add("IoStoreUtilities");
		}
		// shader runtime usage visualization requires ability to create images - it is only used in non-editor desktop development targets
		// UE_BUILD_DEVELOPMENT is also defined for DebugGame
		else if (Target.Type != TargetType.Program && (Target.Configuration == UnrealTargetConfiguration.Development || Target.Configuration == UnrealTargetConfiguration.DebugGame) && Array.IndexOf(Utils.GetPlatformsInClass(UnrealPlatformClass.Desktop), Target.Platform) >= 0)
		{
			PrivateDependencyModuleNames.Add("ImageWrapper");
		}

		// Copy the GPUDumpViewer's source code for the r.DumpGPU command.
		if (Target.Configuration != UnrealTargetConfiguration.Shipping)
		{
			RuntimeDependencies.Add(Path.Combine(Unreal.EngineDirectory.ToString(), "Extras/GPUDumpViewer/..."), StagedFileType.DebugNonUFS);
		}

		PrivateDependencyModuleNames.AddRange(new string[]
		{
			"Core",
			"Projects",
			"ApplicationCore",
			"TraceLog",
			"StateStream",
		});
		
		PublicIncludePathModuleNames.AddRange(new string[] { "RHI" });

		if (Target.bBuildEditor == true)
		{
			PrivateDependencyModuleNames.Add("DerivedDataCache");
		}
		else
		{
			PrivateIncludePathModuleNames.Add("DerivedDataCache");
		}

		if (Target.IsInPlatformGroup(UnrealPlatformGroup.Windows)
			|| Target.IsInPlatformGroup(UnrealPlatformGroup.Android)
			|| Target.IsInPlatformGroup(UnrealPlatformGroup.Linux))
		{
			// For IOpenGLDynamicRHI::RHIGenerateMips
			PublicIncludePathModuleNames.Add("OpenGLDrv");
		}

		if (Target.Configuration != UnrealTargetConfiguration.Shipping)
		{
			PrivateDefinitions.Add("ALLOW_SHADERMAP_TRACKING=1");
		}
    }
}
