<?xml version="1.0" encoding="utf-8"?>
<AutoVisualizer xmlns="http://schemas.microsoft.com/vstudio/debugger/natvis/2010">

<!--
*
* RDG Visualizers
*
-->

<Type Name="TRDGHandle&lt;*,*&gt;">
  <DisplayString>{Index}</DisplayString>
</Type>

<Type Name="FRDGEventName">
  <DisplayString>{FormatedEventName}</DisplayString>
</Type>

<Type Name="FRDGEventName" Priority="MediumLow">
  <DisplayString>{EventFormat}</DisplayString>
</Type>

<Type Name="FRDGEventName" Priority="Low">
  <DisplayString>Empty</DisplayString>
</Type>

<Type Name="FRDGPass">
  <DisplayString Condition="FullPathIfDebug.Data.ArrayNum != 0">{FullPathIfDebug}</DisplayString>
  <DisplayString Condition="FullPathIfDebug.Data.ArrayNum == 0">{Name}</DisplayString>
</Type>

<Type Name="FRDGPass" Priority="MediumLow">
  <DisplayString>{Name}</DisplayString>
</Type>

<Type Name="FRDGResource">
  <DisplayString>{Name}</DisplayString>
  <Expand>
    <Item Name="Name">Name</Item>
    <Item Name="ResourceRHI">ResourceRHI</Item>
  </Expand>
</Type>

<Type Name="FRDGParentResource">
  <DisplayString>{Name}</DisplayString>
  <Expand>
    <Item Name="FRDGResource">(FRDGResource*)this,nd</Item>
    <Item Name="External">bExternal == 1</Item>
    <Item Name="Extracted">bExtracted == 1</Item>
    <Item Name="Produced">bProduced == 1</Item>
    <Item Name="Transient">bTransient == 1</Item>
    <Item Name="AccessInitial">AccessInitial</Item>
    <Item Name="AccessFinal">AccessFinal</Item>
  </Expand>
</Type>

<Type Name="FRDGView">
  <DisplayString>{Name}</DisplayString>
  <Expand>
    <Item Name="FRDGResource">(FRDGResource*)this,nd</Item>
    <Item Name="Type">Type</Item>
  </Expand>
</Type>

<Type Name="FRDGShaderResourceView">
  <DisplayString>{Name}</DisplayString>
  <Expand>
    <Item Name="FRDGView">(FRDGView*)this,nd</Item>
  </Expand>
</Type>

<Type Name="FRDGUnorderedAccessView">
  <DisplayString>{Name}</DisplayString>
  <Expand>
    <Item Name="FRDGView">(FRDGView*)this,nd</Item>
  </Expand>
</Type>

<Type Name="FRDGTextureSRV">
  <DisplayString>{Name}</DisplayString>
  <Expand>
    <Item Name="FRDGShaderResourceView">(FRDGShaderResourceView*)this,nd</Item>
    <Item Name="Desc">Desc</Item>
    <Item Name="Texture">Desc.Texture</Item>
  </Expand>
</Type>

<Type Name="FRDGTextureUAV">
  <DisplayString>{Name}</DisplayString>
  <Expand>
    <Item Name="FRDGUnorderedAccessView">(FRDGUnorderedAccessView*)this,nd</Item>
    <Item Name="Desc">Desc</Item>
    <Item Name="Texture">Desc.Texture</Item>
  </Expand>
</Type>

<Type Name="FRDGBufferSRV">
  <DisplayString>{Name}</DisplayString>
  <Expand>
    <Item Name="FRDGShaderResourceView">(FRDGShaderResourceView*)this,nd</Item>
    <Item Name="Desc">Desc</Item>
    <Item Name="Buffer">Desc.Buffer</Item>
  </Expand>
</Type>

<Type Name="FRDGBufferUAV">
  <DisplayString>{Name}</DisplayString>
  <Expand>
    <Item Name="FRDGUnorderedAccessView">(FRDGUnorderedAccessView*)this,nd</Item>
    <Item Name="Desc">Desc</Item>
    <Item Name="Buffer">Desc.Buffer</Item>
  </Expand>
</Type>
  
<Type Name="FRDGBuffer">
  <DisplayString>{Name}</DisplayString>
  <Expand>
    <Item Name="FRDGParentResource">(FRDGParentResource*)this,nd</Item>
    <Item Name="Desc">Desc</Item>
    <Item Name="Flags">Flags</Item>
    <Item Name="ResourceRHI">(FRHIBuffer*)ResourceRHI</Item>
  </Expand>
</Type>

<Type Name="FRDGTexture">
  <DisplayString>{Name}</DisplayString>
  <Expand>
    <Item Name="FRDGParentResource">(FRDGParentResource*)this,nd</Item>
    <Item Name="Desc">Desc</Item>
    <Item Name="Flags">Flags</Item>
    <Item Name="ResourceRHI">(FRHITexture*)ResourceRHI</Item>
  </Expand>
</Type>

<Type Name="FRDGUniformBuffer">
  <DisplayString>{Name}</DisplayString>
  <Expand>
    <Item Name="FRDGResource">(FRDGResource*)this,nd</Item>
    <Item Name="ResourceRHI">(FRHIUniformBuffer*)ResourceRHI</Item>
    <Item Name="ParameterStruct">ParameterStruct</Item>
  </Expand>
</Type>

<Type Name="TRDGUniformBuffer&lt;*&gt;">
  <DisplayString>{Name}</DisplayString>
  <Expand>
    <Item Name="FRDGUniformBuffer">(FRDGUniformBuffer*)this,nd</Item>
    <Item Name="ResourceRHI">(FRHIUniformBuffer*)ResourceRHI</Item>
    <Item Name="Parameters">Parameters</Item>
  </Expand>
</Type>

<Type Name="FRDGParameterStruct">
  <DisplayString Condition="Layout != 0">{Layout->Name}</DisplayString>
  <DisplayString Condition="Layout == 0">Empty</DisplayString>
</Type>

</AutoVisualizer>